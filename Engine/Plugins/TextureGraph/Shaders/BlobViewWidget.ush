// Copyright Epic Games, Inc. All Rights Reserved.

// This shader file is included in the BlobView.uasset widget material, in a custom node.
//

// Fetch the blob texture no matter what
float4 sample = blobTex.SampleLevel(blobTexSampler, texcoord.xy, 0);
float opacity = sample.w;

// make a checker grid over the viewport
float checker = floor(texcoord.x * 10.0) + floor(texcoord.y * 10.0);
checker = frac(checker * 0.5) * 2.0;
float3 finalColor = checker; // black and white checker by default

// Draw texture sample RGB if valid
finalColor = lerp(finalColor, sample.xyz, blobTexIsValid);

return finalColor;
