// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

Texture2D				BackgroundTexture;
Texture2D       		ForegroundTexture;
Texture2D       		MaskTexture;
float           		Opacity;

float GetMask(Texture2D MaskTexture, float2 uv, float Opacity)
{
	float4 Mask = MaskTexture.Sample(SamplerStates_Linear_Clamp, uv);

	// Mask should be grey scale or single channel.
	// For mask we are going to keep the value between 0-1 range
	float MaskingValue = saturate(Mask.r) * Opacity;
	
	return MaskingValue;
}

float4 FSH_BlendNormal(in float2 uv : TEXCOORD0) : SV_Target0
{
	float4 BackgroundTex = BackgroundTexture.Sample(SamplerStates_Linear_Clamp, uv);
	float4 ForegroundTex = ForegroundTexture.Sample(SamplerStates_Linear_Clamp, uv);

	float MaskingValue = GetMask(MaskTexture, uv, Opacity);

#if IGNORE_ALPHA
	float3 FinalRGB = lerp(BackgroundTex.rgb, ForegroundTex.rgb, MaskingValue);
	float4 FinalColor = float4(FinalRGB, BackgroundTex.a);
#else
	float4 FinalColor = lerp(BackgroundTex, ForegroundTex, MaskingValue);
#endif

#if CLAMP
	FinalColor = saturate(FinalColor);
#endif
	
    return FinalColor;
}

float4 FSH_BlendAdd(in float2 uv : TEXCOORD0) : SV_Target0
{
	float4 BackgroundTex = BackgroundTexture.Sample(SamplerStates_Linear_Clamp, uv);
	float4 ForegroundTex = ForegroundTexture.Sample(SamplerStates_Linear_Clamp, uv);
	
	float MaskingValue = GetMask(MaskTexture, uv, Opacity);
#if IGNORE_ALPHA
	float3 FinalRGB = lerp(BackgroundTex.rgb, BackgroundTex.rgb + ForegroundTex.rgb, MaskingValue);
	float4 FinalColor = float4(FinalRGB, BackgroundTex.a);
#else
	float4 FinalColor = lerp(BackgroundTex, BackgroundTex + ForegroundTex, MaskingValue);
#endif

#if CLAMP
	FinalColor = saturate(FinalColor);
#endif
	
	return FinalColor;
}

float4 FSH_BlendSubtract(in float2 uv : TEXCOORD0) : SV_Target0
{
	float4 BackgroundTex = BackgroundTexture.Sample(SamplerStates_Linear_Clamp, uv);
	float4 ForegroundTex = ForegroundTexture.Sample(SamplerStates_Linear_Clamp, uv);

	float MaskingValue = GetMask(MaskTexture, uv, Opacity);

#if IGNORE_ALPHA
	float3 FinalRGB = lerp(BackgroundTex.rgb, BackgroundTex.rgb - ForegroundTex.rgb, MaskingValue);
	float4 FinalColor = float4(FinalRGB, BackgroundTex.a);
#else
	float4 FinalColor = lerp(BackgroundTex, BackgroundTex - ForegroundTex, MaskingValue);
#endif

#if CLAMP
	FinalColor = saturate(FinalColor);
#endif
	
	return FinalColor;
}

float4 FSH_BlendMultiply(in float2 uv : TEXCOORD0) : SV_Target0
{
	float4 BackgroundTex = BackgroundTexture.Sample(SamplerStates_Linear_Clamp, uv);
	float4 ForegroundTex = ForegroundTexture.Sample(SamplerStates_Linear_Clamp, uv);

	float MaskingValue = GetMask(MaskTexture, uv, Opacity);

#if IGNORE_ALPHA
	float3 FinalRGB = lerp(BackgroundTex.rgb, ForegroundTex.rgb * BackgroundTex.rgb, MaskingValue);
	float4 FinalColor = float4(FinalRGB, BackgroundTex.a);
#else
	float4 FinalColor = lerp(BackgroundTex, ForegroundTex * BackgroundTex, MaskingValue);
#endif

#if CLAMP
	FinalColor = saturate(FinalColor);
#endif
	
	return FinalColor;
}

float4 FSH_BlendDivide(in float2 uv : TEXCOORD0) : SV_Target0
{
	float4 BackgroundTex = BackgroundTexture.Sample(SamplerStates_Linear_Clamp, uv);
	float4 ForegroundTex = ForegroundTexture.Sample(SamplerStates_Linear_Clamp, uv);
	
	float MaskingValue = GetMask(MaskTexture, uv, Opacity);
	
	float4 DivisionResult = saturate(BackgroundTex/ForegroundTex);

#if IGNORE_ALPHA
	float3 FinalRGB = lerp(BackgroundTex.rgb, DivisionResult.rgb, MaskingValue);
	float4 FinalColor = float4(FinalRGB, BackgroundTex.a);
#else
	float4 FinalColor = lerp(BackgroundTex, DivisionResult, MaskingValue);
#endif

#if CLAMP
	FinalColor = saturate(FinalColor);
#endif
	
	return FinalColor;
}

float4 FSH_BlendDifference(in float2 uv : TEXCOORD0) : SV_Target0
{
	float4 BackgroundTex = BackgroundTexture.Sample(SamplerStates_Linear_Clamp, uv);
	float4 ForegroundTex = ForegroundTexture.Sample(SamplerStates_Linear_Clamp, uv);

	float MaskingValue = GetMask(MaskTexture, uv, Opacity);

#if IGNORE_ALPHA
	float3 FinalRGB = lerp(BackgroundTex.rgb, abs(BackgroundTex.rgb - ForegroundTex.rgb), MaskingValue);
	float4 FinalColor = float4(FinalRGB, BackgroundTex.a);
#else
	float4 FinalColor = lerp(BackgroundTex, abs(BackgroundTex - ForegroundTex), MaskingValue);
#endif

#if CLAMP
	FinalColor = saturate(FinalColor);
#endif
	
	return FinalColor;
}

float4 FSH_BlendMax(in float2 uv : TEXCOORD0) : SV_Target0
{
	float4 BackgroundTex = BackgroundTexture.Sample(SamplerStates_Linear_Clamp, uv);
	float4 ForegroundTex = ForegroundTexture.Sample(SamplerStates_Linear_Clamp, uv);

	float MaskingValue = GetMask(MaskTexture, uv, Opacity);

#if IGNORE_ALPHA
	float3 FinalRGB = lerp(BackgroundTex.rgb, max(BackgroundTex.rgb, ForegroundTex.rgb), MaskingValue);
	float4 FinalColor = float4(FinalRGB, BackgroundTex.a);
#else
	float4 FinalColor = lerp(BackgroundTex, max(BackgroundTex, ForegroundTex), MaskingValue);
#endif

#if CLAMP
	FinalColor = saturate(FinalColor);
#endif
	
	return FinalColor;
}

float4 FSH_BlendMin(in float2 uv : TEXCOORD0) : SV_Target0
{
	float4 BackgroundTex = BackgroundTexture.Sample(SamplerStates_Linear_Clamp, uv);
	float4 ForegroundTex = ForegroundTexture.Sample(SamplerStates_Linear_Clamp, uv);

	float MaskingValue = GetMask(MaskTexture, uv, Opacity);

#if IGNORE_ALPHA
	float3 FinalRGB = lerp(BackgroundTex.rgb, min(BackgroundTex.rgb, ForegroundTex.rgb), MaskingValue);
	float4 FinalColor = float4(FinalRGB, BackgroundTex.a);
#else
	float4 FinalColor = lerp(BackgroundTex, min(BackgroundTex, ForegroundTex), MaskingValue);
#endif

#if CLAMP
	FinalColor = saturate(FinalColor);
#endif
	
	return FinalColor;
}

float4 FSH_BlendStep(in float2 uv : TEXCOORD0) : SV_Target0
{
	float4 BackgroundTex = BackgroundTexture.Sample(SamplerStates_Linear_Clamp, uv);
	float4 ForegroundTex = ForegroundTexture.Sample(SamplerStates_Linear_Clamp, uv);

	float MaskingValue = GetMask(MaskTexture, uv, Opacity);
	float4 StepValue = 1 - step(ForegroundTex, BackgroundTex);

#if IGNORE_ALPHA
	float3 FinalRGB = lerp(BackgroundTex.rgb, StepValue.rgb, MaskingValue);
	float4 FinalColor = float4(FinalRGB, BackgroundTex.a);
#else
	float4 FinalColor = lerp(BackgroundTex, StepValue, MaskingValue);
#endif

#if CLAMP
	FinalColor = saturate(FinalColor);
#endif
	
	return FinalColor;
}

float4 OverlayBlend( float4 base , float4 overlay )
{
	return lerp(2 * base * overlay, 1 - 2 * (1 - base) * (1 - overlay), 1 - step(base, 0.5));
}

float3 OverlayBlend( float3 base , float3 overlay )
{
	return lerp(2 * base * overlay, 1 - 2 * (1 - base) * (1 - overlay), 1 - step(base, 0.5));
}

float4 FSH_BlendOverlay(in float2 uv : TEXCOORD0) : SV_Target0
{
	float4 BackgroundTex = BackgroundTexture.Sample(SamplerStates_Linear_Clamp, uv);
	float4 ForegroundTex = ForegroundTexture.Sample(SamplerStates_Linear_Clamp, uv);

	float MaskingValue = GetMask(MaskTexture, uv, Opacity);

#if IGNORE_ALPHA
	float3 FinalRGB = lerp(BackgroundTex.rgb, OverlayBlend(BackgroundTex.rgb, ForegroundTex.rgb), MaskingValue);
	float4 FinalColor = float4(FinalRGB, BackgroundTex.a);
#else
	float4 FinalColor = lerp(BackgroundTex, OverlayBlend(BackgroundTex, ForegroundTex), MaskingValue);
#endif

#if CLAMP
	FinalColor = saturate(FinalColor);
#endif
	
	return FinalColor;
}

float4 FSH_BlendDistort(in float2 uv : TEXCOORD0) : SV_Target0
{
	return float4(0,0,0,0);
}