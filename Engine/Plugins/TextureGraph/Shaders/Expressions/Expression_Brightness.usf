// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/ShaderUtil.ush"

Texture2D SourceTexture;
float Brightness;
float Contrast;

float4 FSH_Brightness(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Color = SourceTexture.Sample(SamplerStates_NoBorder, uv);
	return AdjustBrightnessAndContrast(Color, Brightness, Contrast);
}
