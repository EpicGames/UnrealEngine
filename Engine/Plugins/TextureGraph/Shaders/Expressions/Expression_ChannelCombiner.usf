// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

Texture2D SourceRed;
Texture2D SourceGreen;
Texture2D SourceBlue;
Texture2D SourceAlpha;

float4 FSH_ChannelCombiner(float2 uv : TEXCOORD0) : SV_Target0
{
	float Red = SourceRed.Sample(SamplerStates_NoBorder, uv).r;
	float Green = SourceGreen.Sample(SamplerStates_NoBorder, uv).r;
	float Blue = SourceBlue.Sample(SamplerStates_NoBorder, uv).r;
	float Alpha = SourceAlpha.Sample(SamplerStates_NoBorder, uv).r;
	
	return float4(Red, Green, Blue, Alpha);
}
