// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

Texture2D SourceTexture;

float4 FSH_ChannelSplitter_Red(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 SourceClr = SourceTexture.Sample(SamplerStates_NoBorder, uv);
	return float4(SourceClr.r, 0, 0, 1);
}

float4 FSH_ChannelSplitter_Green(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 SourceClr = SourceTexture.Sample(SamplerStates_NoBorder, uv);
	return float4(SourceClr.g, 0, 0, 1);
}

float4 FSH_ChannelSplitter_Blue(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 SourceClr = SourceTexture.Sample(SamplerStates_NoBorder, uv);
	return float4(SourceClr.b, 0, 0, 1);
}

float4 FSH_ChannelSplitter_Alpha(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 SourceClr = SourceTexture.Sample(SamplerStates_NoBorder, uv);
	return float4(SourceClr.a, 0, 0, 1);
}
