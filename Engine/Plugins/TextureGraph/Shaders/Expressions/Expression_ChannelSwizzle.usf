// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

Texture2D SourceTexture;

#define LINEAR 0
#define EXP 1
#define LOG 1

#ifndef DST_CHANNEL_RED
#define DST_CHANNEL_RED 0
#endif

#ifndef DST_CHANNEL_GREEN
#define DST_CHANNEL_GREEN 1
#endif

#ifndef DST_CHANNEL_BLUE
#define DST_CHANNEL_BLUE 2
#endif

#ifndef DST_CHANNEL_ALPHA
#define DST_CHANNEL_ALPHA 3
#endif

float4 FSH_ChannelSwizzle(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 SrcClr = SourceTexture.Sample(SamplerStates_NoBorder, uv);
	float4 DstClr = SrcClr;

#if DST_CHANNEL_RED == 0
	DstClr.r = SrcClr.r;
#elif DST_CHANNEL_RED == 1
	DstClr.r = SrcClr.g;
#elif DST_CHANNEL_RED == 2
	DstClr.r = SrcClr.b;
#elif DST_CHANNEL_RED == 3
	DstClr.r = SrcClr.a;
#endif 
	
#if DST_CHANNEL_GREEN == 0
	DstClr.g = SrcClr.r;
#elif DST_CHANNEL_GREEN == 1
	DstClr.g = SrcClr.g;
#elif DST_CHANNEL_GREEN == 2
	DstClr.g = SrcClr.b;
#elif DST_CHANNEL_GREEN == 3
	DstClr.g = SrcClr.a;
#endif 
	
#if DST_CHANNEL_BLUE == 0
	DstClr.b = SrcClr.r;
#elif DST_CHANNEL_BLUE == 1
	DstClr.b = SrcClr.g;
#elif DST_CHANNEL_BLUE == 2
	DstClr.b = SrcClr.b;
#elif DST_CHANNEL_BLUE == 3
	DstClr.b = SrcClr.a;
#endif 
	
#if DST_CHANNEL_ALPHA == 0
	DstClr.a = SrcClr.r;
#elif DST_CHANNEL_ALPHA == 1
	DstClr.a = SrcClr.g;
#elif DST_CHANNEL_ALPHA == 2
	DstClr.a = SrcClr.b;
#elif DST_CHANNEL_ALPHA == 3
	DstClr.a = SrcClr.a;
#endif 
	
	return DstClr;
}
