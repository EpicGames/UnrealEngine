// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

Texture2D SourceTexture;
Texture2D MinValue;
Texture2D MaxValue;

float4 FSH_Clamp(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Color = SourceTexture.Sample(SamplerStates_NoBorder, uv);
	float4 MinColor = MinValue.Sample(SamplerStates_NoBorder, uv);
	float4 MaxColor = MaxValue.Sample(SamplerStates_NoBorder, uv);
	
	return clamp(Color, MinColor, MaxColor);
}

float4 FSH_SmoothStep(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Color = SourceTexture.Sample(SamplerStates_NoBorder, uv);
	float4 MinColor = MinValue.Sample(SamplerStates_NoBorder, uv);
	float4 MaxColor = MaxValue.Sample(SamplerStates_NoBorder, uv);
	
	return smoothstep(MinColor, MaxColor, Color);
}
