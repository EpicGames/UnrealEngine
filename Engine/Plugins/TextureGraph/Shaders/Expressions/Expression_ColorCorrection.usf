// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/ShaderUtil.ush"

Texture2D Input;
float Brightness;
float Contrast;
float Gain;
float Lift;
float Gamma;
float Saturation;
float4 TemperatureRGB;
float TemperatureStrength;
float TemperatureBrightnessNormalization;

float4 FSH_ColorCorrection(float2 uv : TEXCOORD0) : SV_Target0
{
	const float3 HALF = float3(0.5, 0.5, 0.5);

	float4 Clr = Input.Sample(SamplerStates_NoBorder, uv);
	float4 ClrA = AdjustBrightnessAndContrast_NoAlpha(Clr, Brightness, Contrast);
	
	float OriginalBrightness = LumaFromRGB(Clr.rgb);
	float3 ClrTemp = lerp(ClrA.rgb, ClrA.rgb * TemperatureRGB.rgb, TemperatureStrength);
	float NewBrightness = LumaFromRGB(ClrTemp.rgb);
	float3 ClrNew = ClrTemp.rgb * lerp(1.0, (NewBrightness > 1e-6) ? (OriginalBrightness / NewBrightness) : 1.0, TemperatureBrightnessNormalization);
	float TemperatureBrightness = LumaFromRGB(ClrNew);
	float3 TemperatureBrightnessVec = float3(TemperatureBrightness, TemperatureBrightness, TemperatureBrightness);
	ClrNew = lerp(TemperatureBrightnessVec, ClrNew, Saturation);
	
	float3 ClrFinal = pow(ClrNew, HALF / Gamma); 
	return float4(ClrFinal.rgb, Clr.a);
}
