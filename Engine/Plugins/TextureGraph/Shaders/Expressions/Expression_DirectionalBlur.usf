// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/TileInfo.ush"

Texture2D SourceTexture;
float Angle;
float Strength;
float Steps;
float Sigma;
float StrengthMultiplier;

float2 GetTexelSize(Texture2D Tex)
{
	// Get the dimensions of the texture in pixels
	int Width, Height;
	Tex.GetDimensions(Width, Height);

	// Calculate the texel size in screen space
	float2 TexelSize = 1.0 / float2(Width, Height);

	return TexelSize;
}

float4 FSH_DirectionalBlur(float2 UV : TEXCOORD0) : SV_Target0
{
	const float PI = 3.141592653;
	
	const float2 TexelSize = GetTexelSize(SourceTexture);
	
	float FinalStrength = Strength * StrengthMultiplier;
	
	float2 FullUV = TileInfo_fromCurrentTileToLayer(UV);
	
	float2 dx = ddx(FullUV);
	float2 dy = ddy(FullUV);

	float Radians = radians(Angle); 
	float2 e = float2(cos(Radians), -sin(Radians)) *  TexelSize;

	float3 Result = float3(0, 0, 0);
	
	float WeightSum = 0.0;
	float Sig = Sigma *  FinalStrength;

	for( int i = -Steps; i < Steps; i++ )
	{
		float Weight = exp( -0.5 * (pow(float(i)/Sig, 2) )) / ( (2 * PI) * Sig * Sig );
		Result += Weight * SourceTexture.SampleGrad(SamplerStates_Linear_Wrap, FullUV + float(i) * e, dx, dy).xyz;
		WeightSum += Weight;
	}
	
	return float4(Result / WeightSum, 1);
}
