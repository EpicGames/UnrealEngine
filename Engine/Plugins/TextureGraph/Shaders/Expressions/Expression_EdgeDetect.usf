// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/TileInfo.ush"

Texture2D SourceTexture;
float Thickness;

float Intensity(float4 Color)
{
	return sqrt((Color.x * Color.x) + (Color.y * Color.y) + (Color.z * Color.z));
}

float3 Sobel(float StepX, float StepY, float2 Center)
{
	// get samples around pixel
	float TopLeft = Intensity(SourceTexture.Sample(SamplerStates_Linear_Clamp, Center + float2(-StepX, StepY)));
	float Left = Intensity(SourceTexture.Sample(SamplerStates_Linear_Clamp, Center + float2(-StepX, 0)));
	float BottomLeft = Intensity(SourceTexture.Sample(SamplerStates_Linear_Clamp, Center + float2(-StepX, -StepY)));
	float Top = Intensity(SourceTexture.Sample(SamplerStates_Linear_Clamp, Center + float2(0, StepY)));
	float Bottom = Intensity(SourceTexture.Sample(SamplerStates_Linear_Clamp, Center + float2(0, -StepY)));
	float TopRight = Intensity(SourceTexture.Sample(SamplerStates_Linear_Clamp, Center + float2(StepX, StepY)));
	float Right = Intensity(SourceTexture.Sample(SamplerStates_Linear_Clamp, Center + float2(StepX, 0)));
	float BottomRight = Intensity(SourceTexture.Sample(SamplerStates_Linear_Clamp, Center + float2(StepX, -StepY)));
 
	// Sobel masks (see http://en.wikipedia.org/wiki/Sobel_operator)
	//        1 0 -1     -1 -2 -1
	//    X = 2 0 -2  Y = 0  0  0
	//        1 0 -1      1  2  1
	
	// You could also use Scharr operator:
	//        3 0 -3        3 10   3
	//    X = 10 0 -10  Y = 0  0   0
	//        3 0 -3        -3 -10 -3
 
	float X = TopLeft + 2.0 * Left + BottomLeft - TopRight - 2.0 * Right - BottomRight;
	float Y = -TopLeft - 2.0 * Top - TopRight + BottomLeft + 2.0 * Bottom + BottomRight;
	float Color = sqrt((X * X) + (Y * Y));
	return float3(Color, Color, Color);
}

float4 FSH_EdgeDetect(float2 InUV : TEXCOORD0) : SV_Target0
{
	int Width = TileInfo_TileCountX * TileInfo_TileWidth;
	int Height = TileInfo_TileCountY * TileInfo_TileHeight;
	float WidthInv = rcp(float(Width));
	float HeightInv = rcp(float(Height));
	
	float StepX = Thickness * WidthInv;
	float StepY = Thickness * HeightInv;
	float2 UV = TileInfo_fromCurrentTileToLayer(InUV);
	
	float4 Center = SourceTexture.Sample(SamplerStates_Linear_Clamp, UV);
	float3 Result = Sobel(StepX, StepY, UV) * Center.xyz; 
	return float4(Result.x, Result.y, Result.z, Center.a);
}
