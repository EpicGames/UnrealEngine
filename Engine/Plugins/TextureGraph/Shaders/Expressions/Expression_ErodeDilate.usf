// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/TileInfo.ush"
#include "/Plugin/TextureGraph/ShaderUtil.ush"

#ifndef ED_KERNEL_BOX
#define ED_KERNEL_BOX 0
#endif

#ifndef ED_KERNEL_CIRCULAR
#define ED_KERNEL_CIRCULAR 1
#endif

#ifndef ED_KERNEL_DIAMOND
#define ED_KERNEL_DIAMOND 2
#endif

#ifndef ED_ERODE
#define ED_ERODE 0
#endif 

#ifndef ED_DILATE
#define ED_DILATE 1
#endif 

Texture2D Input;
int Size;

float4 FSH_ErodeDilate(float2 InUV : TEXCOORD0) : SV_Target0
{
	int Width = TileInfo_TileCountX * TileInfo_TileWidth;
	int Height = TileInfo_TileCountY * TileInfo_TileHeight;
	float WidthInv = rcp(float(Width));
	float HeightInv = rcp(float(Height));
	float2 UV = TileInfo_fromCurrentTileToLayer(InUV);
	
	float4 OrgClr = Input.Sample(SamplerStates_Linear_Clamp, UV);
	
#if ED_SINGLECHANNEL 
	float3 Clr = OrgClr.rrr;
#else
	float3 Clr = OrgClr.rgb;
#endif 
	
	float3 TmpClr = Clr;
	float Brightness = LumaFromRGB(Clr);

	for (int X = -Size; X <= Size; ++X)
	{
		float Xf = float(X) * WidthInv;
		for (int Y = -Size; Y <= Size; ++Y)
		{
			float Yf = float(Y) * HeightInv;
        // default kernel is box shaped anyway, so we don't need to do anything
        
        // diamond shaped kernel (45° rotated square)
#if ED_KERNEL == ED_KERNEL_DIAMOND
        if (abs(X) > Size - abs(Y))
            continue;
#elif ED_KERNEL == ED_KERNEL_CIRCULAR
        if (distance(float2(X, Y), float2(0, 0)) > float(Size))
            continue;
#endif

			float2 OffsetUV = UV + float2(Xf, Yf);
#if ED_SINGLECHANNEL 
			float3 NClr = Input.Sample(SamplerStates_Linear_Clamp, OffsetUV).rrr;
#else
			float3 NClr = Input.Sample(SamplerStates_Linear_Clamp, OffsetUV).rgb;
#endif 
			float NBrightness = LumaFromRGB(NClr);
        
#if ED_TYPE == ED_DILATE
			if (Brightness < NBrightness)
#else
			if (Brightness > NBrightness)
#endif 
			{
				Brightness = NBrightness;
				TmpClr = NClr;
			}
		}
	}
	
#if ED_TYPE == ED_DILATE
	float3 FinalClr = lerp(Clr, TmpClr, smoothstep(0, 1, Brightness));
#else
	float3 FinalClr = lerp(TmpClr, Clr, smoothstep(0, 1, Brightness));
#endif 
	return float4(FinalClr.rgb, OrgClr.a);
}
