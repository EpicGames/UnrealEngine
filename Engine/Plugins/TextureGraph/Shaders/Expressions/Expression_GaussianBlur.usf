// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/TileInfo.ush"


Texture2D SourceTexture;
int Radius;

//Algorithm from https://www.pixelstech.net/article/1353768112-Gaussian-Blur-Algorithm
float NormalDistribution(float x, float y)
{
	const float sigma = 30;

	float sigmaSquare = pow(sigma, 2);

	const float expConstant = 2.71828;
	const float PI = 3.141592653;
	
	float exponent = (-(pow(x, 2) + pow(y, 2))) / (2 * sigmaSquare);

	float gau = pow(expConstant, exponent) / (2 * PI * sigmaSquare);
	return gau;
}

float NormalDistribution1D(float x)
{
	return exp(-0.5 * pow(3.141 * (x), 2));
}


float4 GaussianBlur(float2 texelUV, float2 texelPos, int radius)
{
	float output = 0;
	int diameter = radius * 2;
	float4 pixelSum = float4(0, 0, 0, 0);
	float weightSum = 0;
	
	int width = TileInfo_TileCountX * TileInfo_TileWidth;
	int height = TileInfo_TileCountY * TileInfo_TileHeight;
	float widthInv = rcp(float(width));
	float heightInv = rcp(float(height));
	
	float2 centerSourceUV = texelUV;
	
	// (-r, -r)			(r, -r)
	//	
	//			(0, 0)
	//
	// (-r, r)			(r, r)
	// 1) This iteratres over every pixel from (-r, -r) to (r, r)
	// 2) and for each pixel, it finds the Weight through NormalDistrubution formula for that particular pixel location
	// 3) it then multiplies the Pixel Value of that location with the Weight of that location
	// 4) then at the end, the calculated gaussian value for pixel (0, 0) is the average of all pixel values by all their weights. 
	// 5) Through this we find weighted average of a pixel with a particular kernel

	for (int px = -radius; px <= radius; px++)
	{
		for (int py = -radius; py <= radius; py++)
		{
			int2 neighborTexel = int2(px + texelPos.x, py + texelPos.y); //Given the radius, find the texel for which it needs to calculate weight

			float2 uv = centerSourceUV + float2(px * widthInv, py * heightInv);
			
			float4 pixelValue = SourceTexture.Sample(SamplerStates_Linear_Clamp, uv);
			
			float weight = NormalDistribution(px, py); // Calculate the weight for given location
			pixelSum += pixelValue * weight; // Aggregate pixel value
			weightSum += weight; // Aggregate weight
		}
	}
	
	return pixelSum / weightSum;
}

float4 FSH_GaussianBlur(float2 uv : TEXCOORD0) : SV_Target0
{
	float2 SourceTexelUV = TileInfo_fromCurrentTileToLayer(uv);

	int Width = TileInfo_TileCountX * TileInfo_TileWidth;
	int Height = TileInfo_TileCountY * TileInfo_TileHeight;
	float2 SourceTexelPos = SourceTexelUV * float2(Width, Height);

	float4 MaskVal = GaussianBlur(SourceTexelUV, SourceTexelPos, Radius);
	
	return MaskVal; ///float4(maskVal, maskVal, maskVal, 1.0);
}
