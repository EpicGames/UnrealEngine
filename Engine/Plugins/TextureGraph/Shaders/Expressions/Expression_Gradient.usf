// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/TileInfo.ush"
#include "/Plugin/TextureGraph/Paint.ush"

#define LINEAR 0
#define EXP 1
#define LOG 1

#define ROT_0 0
#define ROT_90 1
#define ROT_180 2
#define ROT_270 3

#ifndef GRADIENT_INTERPOLATION
#define GRADIENT_INTERPOLATION LINEAR
#endif

#ifndef GRADIENT_ROTATION
#define GRADIENT_ROTATION ROT_0
#endif

float4 Center = float4(0, 0, 0, 0);
float4 Line = float4(0, 0, 0, 0);
float4 LineDir = float4(0, 0, 0, 0);

float CalcExp(float Input)
{
	const float eMax = exp(1);
	const float eMaxInverse = 1.0 / (eMax - 1.0);

	return (exp(Input) - 1) * eMaxInverse;
}

float4 FSH_GradientLinear_1(float2 InUV : TEXCOORD0) : SV_Target0
{
	float Input = 0, Gray = 0;
	float2 UV = TileInfo_fromCurrentTileToLayer(InUV);
	
#if GRADIENT_ROTATION == ROT_0
	Input = UV.x;
#elif GRADIENT_ROTATION == ROT_90
	Input = UV.y;
#elif GRADIENT_ROTATION == ROT_180
	Input = 1.0 - UV.x;
#elif GRADIENT_ROTATION == ROT_270
	Input = 1.0 - UV.y;
#endif 
	
#if GRADIENT_INTERPOLATION == LINEAR
	Gray = Input;
#elif GRADIENT_INTERPOLATION == EXP
	Gray = CalcExp(Input);
#endif 
	
	return float4(Gray, Gray, Gray, 1);
}

float4 FSH_GradientLinear_2(float2 InUV : TEXCOORD0) : SV_Target0
{
	float2 UV = TileInfo_fromCurrentTileToLayer(InUV);
	float Input = 0;
	
#if GRADIENT_ROTATION == ROT_0
	Input = UV.x;
#elif GRADIENT_ROTATION == ROT_90
	Input = UV.y;
#elif GRADIENT_ROTATION == ROT_180
	Input = 1.0 - UV.x;
#elif GRADIENT_ROTATION == ROT_270
	Input = 1.0 - UV.y;
#endif 
	
	float Gray = 1 - abs(Input * 2 - 1);
	
#if GRADIENT_INTERPOLATION == LINEAR
#elif GRADIENT_INTERPOLATION == EXP
	Gray = CalcExp(Gray);
#endif 
	
	return float4(Gray, Gray, Gray, 1);
}

float4 FSH_GradientRadial(float2 InUV : TEXCOORD0) : SV_Target0
{
	float2 UV = TileInfo_fromCurrentTileToLayer(InUV);
	float Input = distance(UV.xy, Center.xy);
	float Radius = Center.z;
	float Gray = min(Input / Radius, 1);
	
#if GRADIENT_INTERPOLATION == LINEAR
#elif GRADIENT_INTERPOLATION == EXP
	Gray = CalcExp(Gray);
#endif 
	
	/// Because center is white going onto gray
	Gray = 1 - Gray;
	
	return float4(Gray, Gray, Gray, 1);
}

float GetAxialValue(float2 UV)
{
	float2 P0 = Line.xy;
	float2 RayDir = Line.zw;
	float RayLenSq = LineDir.z;
	float RayLen = LineDir.w;
	float2 Segment = UV.xy - P0;
	float DP = dot(Segment, RayDir);
	float2 PtOnSegment = (DP / RayLenSq) * RayDir;
	float Input = DP <= 0 ? 0 : clamp(length(PtOnSegment) / RayLen, 0, 1);
	
	return Input;
}

float4 FSH_GradientAxial1(float2 InUV : TEXCOORD0) : SV_Target0
{
	float2 UV = TileInfo_fromCurrentTileToLayer(InUV);
	float Input = GetAxialValue(UV);
	float Gray = Input; 
	
#if GRADIENT_INTERPOLATION == LINEAR
#elif GRADIENT_INTERPOLATION == EXP
	Gray = CalcExp(Input);
#endif 
	
	return float4(Gray, Gray, Gray, 1);
}

float4 FSH_GradientAxial2(float2 InUV : TEXCOORD0) : SV_Target0
{
	float2 UV = TileInfo_fromCurrentTileToLayer(InUV);
	float Input = GetAxialValue(UV);
	float Gray = 1 - abs(Input * 2 - 1);
	
#if GRADIENT_INTERPOLATION == LINEAR
#elif GRADIENT_INTERPOLATION == EXP
	Gray = CalcExp(Input);
#endif 
	
	return float4(Gray, Gray, Gray, 1);
}
