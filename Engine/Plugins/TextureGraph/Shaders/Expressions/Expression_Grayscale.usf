// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/ShaderUtil.ush"

Texture2D SourceTexture;

float4 FSH_Grayscale(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 SourceClr = SourceTexture.Sample(SamplerStates_NoBorder, uv);
	float Gray = Grayscale(SourceClr.rgb);
	return float4(Gray, Gray, Gray, SourceClr.a);
}
