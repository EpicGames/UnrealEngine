// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/ShaderUtil.ush"

Texture2D Input;
float Hue = 1;
float Saturation = 1;
float Value = 1;

float4 FSH_HSV(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_NoBorder, uv);
	float3 ClrHSV = RGBToHSV(Clr.rgb);
	ClrHSV *= float3(Hue, Saturation, Value);
	return float4(HSVToRGB(ClrHSV), Clr.a);
}

float4 FSH_RGB2HSV(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_NoBorder, uv);
	float3 ClrHSV = RGBToHSV(Clr.rgb);
	return float4(ClrHSV, Clr.a);
}

float4 FSH_HSV2RGB(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_NoBorder, uv);
	float3 ClrRGB = HSVToRGB(Clr.rgb);
	return float4(ClrRGB, Clr.a);
}
