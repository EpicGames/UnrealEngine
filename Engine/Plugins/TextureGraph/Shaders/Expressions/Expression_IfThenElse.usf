// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/ShaderUtil.ush"

Texture2D LHS;
Texture2D RHS;
Texture2D Then;
Texture2D Else;

//////////////////////////////////////////////////////////////////////////////////////
/// GT
//////////////////////////////////////////////////////////////////////////////////////
float4 FSH_IfThenElse_GT_Component(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float4 Result;
	Result.r = LHSValue.r > RHSValue.r ? ThenValue.r : ElseValue.r;
	Result.g = LHSValue.g > RHSValue.g ? ThenValue.g : ElseValue.g;
	Result.b = LHSValue.b > RHSValue.b ? ThenValue.b : ElseValue.b;
	Result.a = LHSValue.a > RHSValue.a ? ThenValue.a : ElseValue.a;

	return Result;
}

float4 FSH_IfThenElse_GT_All(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float4 Result;

	if (LHSValue.r > RHSValue.r && LHSValue.g > RHSValue.g && LHSValue.b > RHSValue.b && LHSValue.a > RHSValue.a)
		Result = ThenValue;
	else
		Result = ElseValue;

	return Result;
}

float4 FSH_IfThenElse_GT_Grayscale(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float LHSGray = Grayscale(LHSValue.rgb);
	float RHSGray = Grayscale(RHSValue.rgb);

	float4 Result = LHSGray > RHSGray ? ThenValue : ElseValue;
	return Result;
}

//////////////////////////////////////////////////////////////////////////////////////
/// GT
//////////////////////////////////////////////////////////////////////////////////////
float4 FSH_IfThenElse_GTE_Component(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float4 Result;
	Result.r = LHSValue.r >= RHSValue.r ? ThenValue.r : ElseValue.r;
	Result.g = LHSValue.g >= RHSValue.g ? ThenValue.g : ElseValue.g;
	Result.b = LHSValue.b >= RHSValue.b ? ThenValue.b : ElseValue.b;
	Result.a = LHSValue.a >= RHSValue.a ? ThenValue.a : ElseValue.a;

	return Result;
}

float4 FSH_IfThenElse_GTE_All(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float4 Result;

	if (LHSValue.r >= RHSValue.r && 
		LHSValue.g >= RHSValue.g && 
		LHSValue.b >= RHSValue.b && 
		LHSValue.a >= RHSValue.a)
		Result = ThenValue;
	else
		Result = ElseValue;

	return Result;
}

float4 FSH_IfThenElse_GTE_Grayscale(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float LHSGray = Grayscale(LHSValue.rgb);
	float RHSGray = Grayscale(RHSValue.rgb);

	float4 Result = LHSGray >= RHSGray ? ThenValue : ElseValue;
	return Result;
}


//////////////////////////////////////////////////////////////////////////////////////
/// LT
//////////////////////////////////////////////////////////////////////////////////////
float4 FSH_IfThenElse_LT_Component(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float4 Result;
	Result.r = LHSValue.r < RHSValue.r ? ThenValue.r : ElseValue.r;
	Result.g = LHSValue.g < RHSValue.g ? ThenValue.g : ElseValue.g;
	Result.b = LHSValue.b < RHSValue.b ? ThenValue.b : ElseValue.b;
	Result.a = LHSValue.a < RHSValue.a ? ThenValue.a : ElseValue.a;

	return Result;
}

float4 FSH_IfThenElse_LT_All(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float4 Result;

	if (LHSValue.r < RHSValue.r && LHSValue.g < RHSValue.g && LHSValue.b < RHSValue.b && LHSValue.a < RHSValue.a)
		Result = ThenValue;
	else
		Result = ElseValue;

	return Result;
}

float4 FSH_IfThenElse_LT_Grayscale(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float LHSGray = Grayscale(LHSValue.rgb);
	float RHSGray = Grayscale(RHSValue.rgb);

	float4 Result = LHSGray < RHSGray ? ThenValue : ElseValue;
	return Result;
}

//////////////////////////////////////////////////////////////////////////////////////
/// LT
//////////////////////////////////////////////////////////////////////////////////////
float4 FSH_IfThenElse_LTE_Component(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float4 Result;
	Result.r = LHSValue.r <= RHSValue.r ? ThenValue.r : ElseValue.r;
	Result.g = LHSValue.g <= RHSValue.g ? ThenValue.g : ElseValue.g;
	Result.b = LHSValue.b <= RHSValue.b ? ThenValue.b : ElseValue.b;
	Result.a = LHSValue.a <= RHSValue.a ? ThenValue.a : ElseValue.a;

	return Result;
}

float4 FSH_IfThenElse_LTE_All(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float4 Result;

	if (LHSValue.r <= RHSValue.r && 
		LHSValue.g <= RHSValue.g && 
		LHSValue.b <= RHSValue.b && 
		LHSValue.a <= RHSValue.a)
		Result = ThenValue;
	else
		Result = ElseValue;

	return Result;
}

float4 FSH_IfThenElse_LTE_Grayscale(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float LHSGray = Grayscale(LHSValue.rgb);
	float RHSGray = Grayscale(RHSValue.rgb);

	float4 Result = LHSGray <= RHSGray ? ThenValue : ElseValue;
	return Result;
} 

//////////////////////////////////////////////////////////////////////////////////////
/// EQ
//////////////////////////////////////////////////////////////////////////////////////
float4 FSH_IfThenElse_EQ_Component(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float4 Result;
	Result.r = abs(LHSValue.r - RHSValue.r) < 0.00001 ? ThenValue.r : ElseValue.r;
	Result.g = abs(LHSValue.g - RHSValue.g) < 0.00001 ? ThenValue.g : ElseValue.g;
	Result.b = abs(LHSValue.b - RHSValue.b) < 0.00001 ? ThenValue.b : ElseValue.b;
	Result.a = abs(LHSValue.a - RHSValue.a) < 0.00001 ? ThenValue.a : ElseValue.a;

	return Result;
}

float4 FSH_IfThenElse_EQ_All(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float4 Result;

	if (abs(LHSValue.r - RHSValue.r) < 0.00001 && abs(LHSValue.g - RHSValue.g) < 0.00001 && abs(LHSValue.b - RHSValue.b) < 0.00001 && abs(LHSValue.a - RHSValue.a) < 0.00001)
		Result = ThenValue;
	else
		Result = ElseValue;

	return Result;
}

float4 FSH_IfThenElse_EQ_Grayscale(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float LHSGray = Grayscale(LHSValue.rgb);
	float RHSGray = Grayscale(RHSValue.rgb);

	float4 Result = abs(LHSGray - RHSGray) < 0.00001 ? ThenValue : ElseValue;
	return Result;
}

//////////////////////////////////////////////////////////////////////////////////////
/// NEQ
//////////////////////////////////////////////////////////////////////////////////////
float4 FSH_IfThenElse_NEQ_Component(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float4 Result;
	Result.r = abs(LHSValue.r - RHSValue.r) > 0.00001 ? ThenValue.r : ElseValue.r;
	Result.g = abs(LHSValue.g - RHSValue.g) > 0.00001 ? ThenValue.g : ElseValue.g;
	Result.b = abs(LHSValue.b - RHSValue.b) > 0.00001 ? ThenValue.b : ElseValue.b;
	Result.a = abs(LHSValue.a - RHSValue.a) > 0.00001 ? ThenValue.a : ElseValue.a;

	return Result;
}

float4 FSH_IfThenElse_NEQ_All(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float4 Result;

	if (abs(LHSValue.r - RHSValue.r) > 0.00001 && abs(LHSValue.g - RHSValue.g) > 0.00001 && abs(LHSValue.b - RHSValue.b) > 0.00001 && abs(LHSValue.a - RHSValue.a) > 0.00001)
		Result = ThenValue;
	else
		Result = ElseValue;

	return Result;
}

float4 FSH_IfThenElse_NEQ_Grayscale(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 LHSValue = LHS.Sample(SamplerStates_NoBorder, uv);
	float4 RHSValue = RHS.Sample(SamplerStates_NoBorder, uv);
	float4 ThenValue = Then.Sample(SamplerStates_NoBorder, uv);
	float4 ElseValue = Else.Sample(SamplerStates_NoBorder, uv);

	float LHSGray = Grayscale(LHSValue.rgb);
	float RHSGray = Grayscale(RHSValue.rgb);

	float4 Result = abs(LHSGray - RHSGray) > 0.00001 ? ThenValue : ElseValue;
	return Result;
}
