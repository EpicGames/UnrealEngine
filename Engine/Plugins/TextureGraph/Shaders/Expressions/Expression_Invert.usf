// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

Texture2D SourceTexture;
float MaxValue;

#ifndef INVERT_INCLUDE_ALPHA
#define INVERT_INCLUDE_ALPHA 0
#endif

#ifndef INVERT_CLAMP
#define INVERT_CLAMP 0
#endif

float4 FSH_Invert(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Color = SourceTexture.Sample(SamplerStates_NoBorder, uv);
	float4 InvertedColor;
	
#if INVERT_INCLUDE_ALPHA 
	InvertedColor = float4(MaxValue, MaxValue, MaxValue, MaxValue) - Color;
#else
	float3 InvertedColorXYZ = float3(MaxValue, MaxValue, MaxValue) - Color.xyz;
	InvertedColor = float4(InvertedColorXYZ.xyz, Color.a);
#endif
	
#if INVERT_CLAMP 
	InvertedColor = saturate(InvertedColor);
#endif 
	
	return InvertedColor;
}
