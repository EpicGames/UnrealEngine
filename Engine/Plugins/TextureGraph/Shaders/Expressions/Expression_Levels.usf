// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/ShaderUtil.ush"


#ifndef CONVERT_TO_GRAYSCALE
#define CONVERT_TO_GRAYSCALE 0
#endif

#ifndef AUTO_LEVELS
#define AUTO_LEVELS 0
#endif

Texture2D SourceTexture;
float LowValue;
float HighValue;
float MidValue;

float OutLow;
float OutHigh;
float OutputRange;

float DoAutoLevel;
float MidPercentage;

Texture2D Histogram;


float LevelsControlInputCurve(float Value, float LowInput, float HighInput, float MidInput)
{
	// Map incoming value to input range
	float ValueRanged = saturate((Value - LowInput) / (HighInput - LowInput));

	// Map mid to iput range within an epsilon
	float MidRanged = clamp((MidInput - LowInput) / (HighInput - LowInput), 0.001, 0.9999);
	//And evaluate the MidExponent:
	// 0.5 = EvalRange(Mid) ^ MidExponent
	// thus
	float MidExponent = log(0.5) / log(MidRanged);
	
	return pow(ValueRanged, MidExponent);
}

float4 FSH_Levels(float2 uv : TEXCOORD0) : SV_Target0
{
	// fetch the source texture value
	float4 Color = SourceTexture.Sample(SamplerStates_NoBorder, uv);

	// isolate the scalar value
#if CONVERT_TO_GRAYSCALE
	float SourceValue = Grayscale(Color.rgb);
#else
	float SourceValue = Color.x;
#endif

	// Standard path, Low and High come from specified uniforms
	float LowInput = LowValue;
	float HighInput = HighValue;
	float MidInput = MidValue;

#if AUTO_LEVELS

	float4 HistoMinCount	= Histogram.Load(int3(0, 1, 0));
	float4 HistoMaxCount	= Histogram.Load(int3(1, 1, 0));
	float4 HistoMin			= Histogram.Load(int3(2, 1, 0));
	float4 HistoMax			= Histogram.Load(int3(3, 1, 0));
			
	LowInput = 	HistoMin.x;
	HighInput = HistoMax.x;
	MidInput = MidPercentage * (HighInput - LowInput) + LowInput;
#endif /// AUTO_LEVELS

	float LeveledValue = LevelsControlInputCurve(SourceValue, LowInput, HighInput, MidInput);

	// Apply out levels
	LeveledValue = OutLow + (LeveledValue) * OutputRange;
	
#if CONVERT_TO_GRAYSCALE
	// The new  Grayscale level is used to scale the color darker or brighter
	float Ratio = (SourceValue == 0 ? 0.0 : LeveledValue / SourceValue); 	
	float3 LeveledColor = Color.rgb * Ratio;
#else
	float3 LeveledColor = LeveledValue;
#endif

	return float4(LeveledColor.rgb, 1.0);
}

