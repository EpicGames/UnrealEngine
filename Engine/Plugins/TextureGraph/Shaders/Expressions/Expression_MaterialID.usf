// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Engine/Private/GammaCorrectionCommon.ush"

Texture2D				MaterialIDTexture;
float4       			Buckets[375];
float4      			ActiveColors[128];
int         			ActiveColorsCount;

// DECLARE_SCALAR_ARRAY(float, Buckets, 375);

float3 SampleMaterialIDTex(float2 UV)
{
	float3 RGB = MaterialIDTexture.Sample(SamplerStates_Clamp, UV).rgb;
	RGB = LinearToSrgb(RGB);
	
	return RGB;
}
 float3 Rgb2Hsv(float3 RgbColor) 
 {
 	float4 K = float4(0.0, -1.0 / 3.0, 2.0 / 3.0, -1.0);
 	float4 P = lerp(float4(RgbColor.bg, K.wz), float4(RgbColor.gb, K.xy), step(RgbColor.b, RgbColor.g));
 	float4 Q = lerp(float4(P.xyw, RgbColor.r), float4(RgbColor.r, P.yzx), step(P.x, RgbColor.r));

 	float D = Q.x - min(Q.w, Q.y);
 	float E = 1.0e-10;
 	return float3(abs(Q.z + (Q.w - Q.y) / (6.0 * D + E)), D / (Q.x + E), Q.x);
 }

 float3 Hsv2Rgb(float3 HsvColor) 
 {
 	HsvColor = float3(HsvColor.x, clamp(HsvColor.yz, 0.0, 1.0));
 	float4 K = float4(1.0, 2.0 / 3.0, 1.0 / 3.0, 3.0);
 	float3 P = abs(frac(HsvColor.xxx + K.xyz) * 6.0 - K.www);
 	return HsvColor.z * lerp(K.xxx, clamp(P - K.xxx, 0.0, 1.0), HsvColor.y);
 }

 float3 GetBucket(float3 HSV, float3 Offset)
 {
 	const float3 Div = float3(24.0, 20.0, 20.0);
 	const float3 Mul = float3(359.0, 99.0, 99.0);

 	return clamp(HSV * (Mul + Offset), float3(0, 0, 0), Mul) / Div;
 }

 int GetBucketId(float3 HSV, float3 Offset)
 {
     float3 Bucket = GetBucket(HSV, Offset);
     int BucketId = int(Bucket.x) + int(Bucket.y) * 15 + int(Bucket.z) * 15 * 5;

     return BucketId;
 }

 float GetMaskAtOffset(float3 HSV, float3 Offset)
 {
     int bucketId = GetBucketId(HSV, Offset);

	return Buckets[bucketId].r;
 }

 float GetMaskAtPoint(float2 UV, float Off)
 {
 	/// Get the main colour
 	float3 RGB = SampleMaterialIDTex(UV);

 	float3 HSV = Rgb2Hsv(RGB);

 	/// This is essentially doing a bitwise OR (along with the min in the end)
 	/// We give a bit of offset to H, S and V values to counter for rounding 
 	/// off errors
 	float Mask = GetMaskAtOffset(HSV, float3(0, 0, 0)) +
 					GetMaskAtOffset(HSV, float3(1, 0, 0)) +
 					GetMaskAtOffset(HSV, float3(-1, 0, 0)) +
 					GetMaskAtOffset(HSV, float3(1, Off, Off)) +
 					GetMaskAtOffset(HSV, float3(-1, -Off, -Off)) +
 					GetMaskAtOffset(HSV, float3(0, Off, Off)) +
 					GetMaskAtOffset(HSV, float3(0, -Off, -Off));


 	return min(Mask, 1);
 }

 float MatchAtPoint(float2 UV, out float3 RGB)
 {
     /// Get the main colour
     // rgb = (tex2Dlod(_MaterialID, float4(uv, 0, 0)).rgb);
 	RGB = SampleMaterialIDTex(UV);
 	
 	/// Do the simple RGB distance check (we use dot, which is the squared distance)
     for (int i = 0; i < min(128, ActiveColorsCount); i++)
     {
     	//float3 dv = rgb - Colors[i].rgb;
         //float d = dot(dv, dv);

         ////if (d < 0.0025)
         //if (d < 0.01296)
         //    return 1;

     	float3 ActiveColor = ActiveColors[i].rgb;

     	//check if hue is the exact colour and a small allowance for variance in hsv.y (sat) and hsv.z (val)
 		float3 CurrentColourHSV = Rgb2Hsv(RGB);
 		float3 UniqueColorHSV = Rgb2Hsv(ActiveColor);
 		float3 CurrentBucketHSV = floor(GetBucket(CurrentColourHSV, 0));
 		float3 UniqueBucketHSV = floor(GetBucket(UniqueColorHSV, 0));
 	
 		if (CurrentBucketHSV.x == UniqueBucketHSV.x && abs(CurrentColourHSV.y - UniqueColorHSV.y) < 0.05 && abs(CurrentColourHSV.z - UniqueColorHSV.z) < 0.05)
 		//if (currentBucketHSV.x == uniqueBucketHSV.x && abs(currentBucketHSV.y*20 - uniqueBucketHSV.y*20) < 5 && abs(currentBucketHSV.z*20 - uniqueBucketHSV.z*20) < 5)
 		{
 			//return float4(1, 0, 0, 1);
 			return 1;
 		}

 		//if value is greater than 90 and saturation < 10 then colour is almost white
 		if (CurrentColourHSV.z > 0.89 && CurrentColourHSV.y < 0.10)
 		{
 			if (UniqueColorHSV.z > 0.89 && UniqueColorHSV.y < 0.10)
 			{
 				//return float4(1, 0, 0, 1);
 				return 1;
 			}
 		}

 		//if value is lesser than 5 then colour is almost black
 		if (CurrentColourHSV.z < 0.05)
 		{
 			if (UniqueColorHSV.z < 0.05)
 			{
 				//return float4(1, 0, 0, 1);
 				return 1;
 			}
 		}
     }


     /// if the RGB check fails then try to find the HSV bucket for this pixel 
     /// Figure out the HSV bucket for this pixel
     return GetMaskAtPoint(UV, 1);

 }

float4 FSH_MaterialIDMask(in float2 UvSource : TEXCOORD0) : SV_Target0
{
	float3 RGB;
	float MaskValue = MatchAtPoint(UvSource, RGB);
	float4 Mask = float4(MaskValue, MaskValue, MaskValue, 1.0);
	
	return Mask;

}