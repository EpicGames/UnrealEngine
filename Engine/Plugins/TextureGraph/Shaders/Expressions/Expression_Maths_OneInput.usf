// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

Texture2D Input;

float4 FSH_Sin(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return sin(Clr);
}

float4 FSH_Cos(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return cos(Clr);
}

float4 FSH_Tan(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return tan(Clr);
}

float4 FSH_ASin(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return asin(Clr);
}

float4 FSH_ACos(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return acos(Clr);
}

float4 FSH_ATan(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return atan(Clr);
}

float4 FSH_ToRadians(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return radians(Clr);
}

float4 FSH_ToDegrees(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return degrees(Clr);
}

float4 FSH_Abs(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return abs(Clr);
}

float4 FSH_Sqrt(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return sqrt(Clr);
}

float4 FSH_Square(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return Clr * Clr;
}

float4 FSH_Cube(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return Clr * Clr * Clr;
}

float4 FSH_Cbrt(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	const float Exp = 1 / 3;
	return pow(Clr, float4(Exp, Exp, Exp, Exp));
}

float4 FSH_Exp(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return exp(Clr);
}

float4 FSH_Log2(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return log2(Clr);
}

float4 FSH_Log10(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return log10(Clr);
}

float4 FSH_Log(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return log(Clr);
}

float4 FSH_Floor(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return floor(Clr);
}

float4 FSH_Ceil(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return ceil(Clr);
}

float4 FSH_Round(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_Clamp, uv);
	return round(Clr);
}

