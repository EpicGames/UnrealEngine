// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

Texture2D Operand1;
Texture2D Operand2;
Texture2D Operand3;

float4 FSH_Mad(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 op1 = Operand1.Sample(SamplerStates_Clamp, uv);
	float4 op2 = Operand2.Sample(SamplerStates_Clamp, uv);
	float4 op3 = Operand3.Sample(SamplerStates_Clamp, uv);
	return mad(op1, op2, op3);
}

float4 FSH_Lerp(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 op1 = Operand1.Sample(SamplerStates_Clamp, uv);
	float4 op2 = Operand2.Sample(SamplerStates_Clamp, uv);
	float4 op3 = Operand3.Sample(SamplerStates_Clamp, uv);
	return lerp(op1, op2, op3);
}
