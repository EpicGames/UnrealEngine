// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

Texture2D Operand1;
Texture2D Operand2;

float4 FSH_Multiply(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 op1 = Operand1.Sample(SamplerStates_Wrap, uv);
	float4 op2 = Operand2.Sample(SamplerStates_Wrap, uv);
	return op1 * op2;
}

float4 FSH_Divide(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 op1 = Operand1.Sample(SamplerStates_Wrap, uv);
	float4 op2 = Operand2.Sample(SamplerStates_Wrap, uv);
	return op1 / (op2 + float4(0.0001, 0.0001, 0.0001, 0.0001));
}

float4 FSH_Add(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 op1 = Operand1.Sample(SamplerStates_Wrap, uv);
	float4 op2 = Operand2.Sample(SamplerStates_Wrap, uv);
	return op1 + op2;
}

float4 FSH_Subtract(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 op1 = Operand1.Sample(SamplerStates_Wrap, uv);
	float4 op2 = Operand2.Sample(SamplerStates_Wrap, uv);
	return op1 - op2;
}

float4 FSH_Dot(float2 uv : TEXCOORD0) : SV_Target0
{
	float3 op1 = Operand1.Sample(SamplerStates_Wrap, uv).rgb;
	float3 op2 = Operand2.Sample(SamplerStates_Wrap, uv).rgb;
	float dp = dot(op1, op2);
	return float4(dp, dp, dp, 1);
}

float4 FSH_Cross(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 op1 = Operand1.Sample(SamplerStates_Wrap, uv);
	float4 op2 = Operand2.Sample(SamplerStates_Wrap, uv);
	float3 cp = cross(op1.xyz, op2.xyz);
	return float4(cp, 1);
}

float4 FSH_Pow(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 op1 = Operand1.Sample(SamplerStates_Wrap, uv);
	float4 op2 = Operand2.Sample(SamplerStates_Wrap, uv);
	return pow(op1, op2);
}

float4 FSH_Step(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 op1 = Operand1.Sample(SamplerStates_Wrap, uv);
	float4 op2 = Operand2.Sample(SamplerStates_Wrap, uv);
	return step(op1, op2);
}

