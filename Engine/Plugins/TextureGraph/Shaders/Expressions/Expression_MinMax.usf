// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

Texture2D Input1;
Texture2D Input2;

float4 FSH_Min(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Value1 = Input1.Sample(SamplerStates_NoBorder, uv);
	float4 Value2 = Input2.Sample(SamplerStates_NoBorder, uv);
	
	return min(Value1, Value2);
}

float4 FSH_Max(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Value1 = Input1.Sample(SamplerStates_NoBorder, uv);
	float4 Value2 = Input2.Sample(SamplerStates_NoBorder, uv);
	
	return max(Value1, Value2);
}

