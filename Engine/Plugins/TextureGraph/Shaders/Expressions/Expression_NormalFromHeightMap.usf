// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/ShaderUtil.ush"
#include "/Plugin/TextureGraph/TileInfo.ush"

Texture2D SourceTexture;
float Strength;
float Offset;

float4 FSH_NormalFromHeightMap(float2 UV : TEXCOORD0) : SV_Target0
{
	float2 FullTexPos = TileInfo_fromCurrentTileToLayer(UV);

	float2 UVLeft  = FullTexPos + (Offset * float2(-1,0));
	float2 UVRight = FullTexPos + (Offset * float2(1,0));
	float2 UVUp    = FullTexPos + (Offset * float2(0,-1));
	float2 UVDown  = FullTexPos + (Offset * float2(0,1));
	
	float SampleLeft  = SourceTexture.Sample(SamplerStates_Wrap, UVLeft).r;
	float SampleRight = SourceTexture.Sample(SamplerStates_Wrap, UVRight).r;
	float SampleUp    = SourceTexture.Sample(SamplerStates_Wrap, UVUp).r;
	float SampleDown  = SourceTexture.Sample(SamplerStates_Wrap, UVDown).r;
	
	float3 Horizontal = float3(float2(2,0),((SampleRight-SampleLeft) * Strength));
	float3 Vertical = float3(float2(0,2),((SampleUp-SampleDown) * Strength));
	
	float3 NormalRGB = (normalize(cross(Horizontal,Vertical))* 0.5+ 0.5);
	
	return float4(NormalRGB,1);
}
