// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/ShaderUtil.ush"

Texture2D Input;

float4 FSH_Premult(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Clr = Input.Sample(SamplerStates_NoBorder, uv);
	return float4(Clr.rgb * float3(Clr.a, Clr.a, Clr.a), Clr.a);
}
