// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/TileInfo.ush"


Texture2D SourceTexture;
int Radius;
float Strength;
float CenterX;
float CenterY;
float Samples;

float4 FSH_RadialBlur(float2 UV : TEXCOORD0) : SV_Target0
{
	const float2 Center = float2(CenterX, CenterY);
	
	float2 FullUV = TileInfo_fromCurrentTileToLayer(UV);
	float4 Col = float4(0,0,0,0);
	float2 Dist = FullUV - Center;
	
	for(int j = 0; j < Samples; j++) 
	{
		float Scale = 1 - Strength * (j / Samples) * (saturate(length(Dist) / Radius));
		Col += SourceTexture.Sample(SamplerStates_Linear_Clamp, Dist * (Scale) + Center);
	}

	Col /= Samples;                
	return Col;
}
