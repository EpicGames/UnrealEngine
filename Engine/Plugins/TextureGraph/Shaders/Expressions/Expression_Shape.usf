// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/TileInfo.ush"
#include "/Plugin/TextureGraph/ShapeMask.ush"


#define SHAPE_CIRCLE 0
#define SHAPE_SEGMENT 1
#define SHAPE_RECT 2
#define SHAPE_TRIANGLE 3
#define SHAPE_ELLIPSE 4
#define SHAPE_PENTAGON 5
#define SHAPE_HEXAGON 6
#define SHAPE_REGULARPOLYGON_7 7
#define SHAPE_REGULARPOLYGON_8 8
#define SHAPE_REGULARPOLYGON_9 9
#define SHAPE_REGULARPOLYGON_10 10
#define SHAPE_REGULARPOLYGON_11 11
#define SHAPE_REGULARPOLYGON_12 12
#define SHAPE_REGULARPOLYGON_13 13
#define SHAPE_REGULARPOLYGON_14 14
#define SHAPE_REGULARPOLYGON_15 15
#define SHAPE_REGULARPOLYGON_16 16
#define SHAPE_REGULARPOLYGON_17 17
#define SHAPE_REGULARPOLYGON_18 18
#define SHAPE_REGULARPOLYGON_19 19
#define SHAPE_REGULARPOLYGON_20 20
#define SHAPE_REGULARPOLYGON_21 21
#define SHAPE_REGULARPOLYGON_22 22
#define SHAPE_REGULARPOLYGON_23 23
#define SHAPE_REGULARPOLYGON_24 24
#define SHAPE_REGULARPOLYGON_25 25
#define SHAPE_REGULARPOLYGON_26 26
#define SHAPE_REGULARPOLYGON_27 27
#define SHAPE_REGULARPOLYGON_28 28
#define SHAPE_REGULARPOLYGON_29 29
#define SHAPE_REGULARPOLYGON_30 30
#define SHAPE_REGULARPOLYGON_31 31
#define SHAPE_REGULARPOLYGON_32 32

#ifndef SHAPE_TYPE
#define SHAPE_TYPE SHAPE_CIRCLE
#endif

float RotateX;
float RotateY;

float ParamX;
float ParamY;

float Rounding;

float BevelWidth;
float BevelCurve;

float BlendSDF;


float4 FSH_DrawShapeMask(float2 uv : TEXCOORD0) : SV_Target0
{
	float2 layerPos = TileInfo_fromCurrentTileToLayer(uv);
	
	// Map UV to the centered normal space[-1,1]
	float2 centeredPos = layerPos * 2.0 - 1.0;
	
	// Apply orientation rotation (flip the rotation so Y is up)
	centeredPos = float2(centeredPos.x * RotateX + centeredPos.y * (-RotateY),
						 centeredPos.x * -(-RotateY) + centeredPos.y * RotateX);
	
	// Compute the shape mask value for the shape in the uv domain
	float4 shapeMask = 0;
	
#if (SHAPE_TYPE == SHAPE_CIRCLE)
	shapeMask = ShapeMask_Circle(centeredPos, ParamX, Rounding, float2(BevelWidth, BevelCurve));
	
#elif (SHAPE_TYPE == SHAPE_ELLIPSE) // ParamX & ParamY must be different, use the circle if so
	shapeMask = ShapeMask_Ellipse(centeredPos, float2(ParamX, ParamY), Rounding, float2(BevelWidth, BevelCurve));
	
#elif (SHAPE_TYPE == SHAPE_SEGMENT)
	shapeMask = ShapeMask_Segment(centeredPos, ParamX, Rounding, float2(BevelWidth, BevelCurve));

#elif (SHAPE_TYPE == SHAPE_RECT)
	shapeMask = ShapeMask_Rect(centeredPos, float2(ParamX, ParamY), Rounding, float2(BevelWidth, BevelCurve));	
	
#elif (SHAPE_TYPE == SHAPE_TRIANGLE)
	shapeMask = ShapeMask_Triangle(centeredPos, ParamX, Rounding, float2(BevelWidth, BevelCurve));	
	
#elif (SHAPE_TYPE == SHAPE_PENTAGON)
	shapeMask = ShapeMask_Pentagon(centeredPos, ParamX, Rounding, float2(BevelWidth, BevelCurve));	

#elif (SHAPE_TYPE == SHAPE_HEXAGON)
	shapeMask = ShapeMask_Hexagon(centeredPos, ParamX, Rounding, float2(BevelWidth, BevelCurve));	
		
#else // (SHAPE_TYPE >= SHAPE_REGULARPOLYGON_7) 
	shapeMask = ShapeMask_Polygon(centeredPos, ParamX, SHAPE_TYPE, Rounding, float2(BevelWidth, BevelCurve));	

#endif
	
	// Blend with SDF
	float mask = lerp(shapeMask.x, shapeMask.y, BlendSDF);

	return float4(mask, mask, mask, 1.0);
}
