// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/ShaderUtil.ush"

Texture2D Input;
float Threshold;

float4 FSH_Threshold(float2 uv : TEXCOORD0) : SV_Target0
{
	float4 Signal = Input.Sample(SamplerStates_NoBorder, uv);
	
	float Luminance = Grayscale(Signal.rgb);
		
	float Result = step(Threshold, Luminance);
	return float4(Result, Result, Result, 1.0);
}

