// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/TileInfo.ush"
#include "/Plugin/TextureGraph/Paint.ush"

Texture2D SourceTexture;
float4 FillColor;

float CoverageX;
float CoverageY;

float TranslationX;
float TranslationY;

float PivotX;
float PivotY;

float RotationX;
float RotationY;

float ScaleX;
float ScaleY;

float FilterMode;
float MirrorX;
float MirrorY;

float StaggerX;
float StaggerY;

float StrideX;
float StrideY;

float Zoom;
float StretchToFit;
float SpacingX;
float SpacingY;

float CellScaling;


float BlendDebugGrid;

float MaskBox(in float2 uv, in float2 minValue, in float2 maxValue)
{
    float2 masked = step(minValue, uv) * step(uv, maxValue);
    return min(masked.x, masked.y);
}
float MaskBox(in float2 uv, in float minValue, in float maxValue)
{
    return MaskBox(uv, float2(minValue, minValue), float2(maxValue, maxValue));
}

float4 FSH_TransformBlit(float2 uv : TEXCOORD0) : SV_Target0
{	
	float2 layerPos = TileInfo_fromCurrentTileToLayer(uv);
	float2 layerResolution = TileInfo_layerResolution();
	float2 pixelSize = rcp(layerResolution);
	float LayerAspectRatio = layerResolution.x/layerResolution.y;
	
	float2 Coverage = float2(CoverageX, CoverageY);
	float2 Translation = float2(TranslationX, TranslationY);
	float2 Pivot = float2(PivotX, PivotY) * Coverage; // Pivot is applied in the normalized Coverage rectangle
	float2 Rotation = float2(RotationX, RotationY);
	float2 Scale = float2(ScaleX, ScaleY);
	
	float2 Stagger = float2(StaggerX, StaggerY);
	int2 Stride = int2(StrideX, StrideY);
	
	// TilePos is the going to be the uv of the pixel on which we are going to apply the transformation steps
	// Frst of all flip the y to work with Y up
	float2 tilePos = float2(layerPos.x, 1.0 - layerPos.y);
		
	// Apply translation	
	tilePos -= Translation;
			
	// Apply the pivot and rotation	
	tilePos -= Pivot;
	tilePos = float2(dot(tilePos, Rotation), dot(tilePos, float2(-Rotation.y, Rotation.x)));
	tilePos += Pivot;
	
	// eval the tile grid mask for debug
	float tileGridMask = Paint_Grid(tilePos, 1.0, length(pixelSize) * 4);
	
	// Eval the coverage mask, wrap it on the neighboor tiles or not
	float2 filteredTilePos = (FilterMode > 0.0 ? frac(tilePos) : tilePos);
	float CoverageMask = MaskBox(filteredTilePos, 0, Coverage);
	
	// focus on the cell domain inside of the tile
	float2 cellPos = tilePos - floor(tilePos);
	
	// eval debug mask for coverage box and the Pivot point
	float coverageGridMask = CoverageMask * Paint_BoxEdge(cellPos, 0, Coverage, 0, pixelSize * 3);
    float pivotMask = Paint_BoxEdge(tilePos, Pivot, Pivot, pixelSize * 6, -pixelSize * 3);
	
	// Aspect ratio of the uv space is going follow scaling transformations
	float UVAspectRatio = 1.0;
	
	// apply coverage scaling
	cellPos /= Coverage;
	UVAspectRatio *= (Coverage.x / Coverage.y);
	pixelSize /= Coverage;
	
	// apply scaling
	cellPos *= Scale; 
	UVAspectRatio *= (Scale.y / Scale.x);
	pixelSize *= Scale;
	
	// apply stagger
	cellPos -= floor(cellPos).yx * Stagger;
	
	// Eval the staggered grid for debug
	float stagGridMask = CoverageMask * Paint_BoxEdge(cellPos, 0, 1.0, 0.0, pixelSize * 2.5 );
	
	// apply stride effect as a mask
    int2 cellTilePos = int2(floor(cellPos)); // this is the cell tile coordinate
    int2 StrideCell = (cellTilePos % (1 + Stride));
    float StrideMask = (float((StrideCell.x == 0) && (StrideCell.y == 0)));
     
	// Eval spacing as a stretching to respect the aspect ratio or not
	float2 spacing = lerp(float2(0, 1 - (UVAspectRatio)), float2(1 - rcp(UVAspectRatio), 0), (UVAspectRatio > 1.0));
	spacing = lerp(spacing, float2(SpacingX, SpacingY), StretchToFit);
	
	// Eval the spaced cell pos
	// we do a simple remap of the cellPos from [half_spacing, 1 - half_spacing] to [0, 1]
	float2 spacedCellPos = ( (cellPos - cellTilePos) - spacing * 0.5) / (1 - spacing);

	// and a last scaling to apply the zoom control
	float scaling = rcp(max(length(pixelSize), Zoom)); 
	spacedCellPos = ((spacedCellPos * 2 - 1) * scaling) * 0.5 + 0.5;

	// Eval the cell grid masked  for debug
	float cellGridMask = CoverageMask * Paint_BoxEdge(spacedCellPos, 0, 1.0, -pixelSize * 0.5, pixelSize * 1.25);
	
		
	// also compute the mask for spacedCell and exclude anything out of the [0,1] box
	float CellMask = MaskBox(spacedCellPos, 0, 1);
	CoverageMask *= CellMask;
	
	// Finally fetch the source texture
	float2 fetchPos = spacedCellPos; 
	
	// Apply mirroring to uvs
	fetchPos.x = lerp(fetchPos.x, 1.0 - fetchPos.x, MirrorX);	
	fetchPos.y = lerp(1.0 - fetchPos.y, fetchPos.y, MirrorY);
	
	float4 SourceColor = float4(0, 0, 0, 1);
	SourceColor = SourceTexture.Sample(SamplerStates_Linear_Wrap, fetchPos);
		
	float4 Color = lerp(FillColor, SourceColor, (CoverageMask * StrideMask * SourceColor.a));
	
	Color = lerp(Color, float4(1, 1, 0, 1), BlendDebugGrid * tileGridMask);
	Color = lerp(Color, float4(0, 1, 1, 1), BlendDebugGrid * coverageGridMask);
	//Color = lerp(Color, float4(1, 0.5, 0, 1), BlendDebugGrid * stagGridMask);
	Color = lerp(Color, float4(1, 0, 1, 1), BlendDebugGrid * cellGridMask);
	Color = lerp(Color, float4(0, 1, 1, 1), BlendDebugGrid * pivotMask);
	
	return Color;
}
