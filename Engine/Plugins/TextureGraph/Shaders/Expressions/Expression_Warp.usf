// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/TileInfo.ush"

Texture2D SourceTexture;
Texture2D Mask;
float Intensity;
float AngleRad;
float PhaseU;
float PhaseV;

#define HALF_PI 1.5707963267948966
#define QUARTER_PI HALF_PI * 0.5

float2 SineWave(float2 p, float tx, float ty)
{
	//float tx = 0.3477;
	//float ty = 0.7812;
    // wave distortion
	float x = sin(25.0 * p.y + 30.0 * p.x + 6.28 * tx) * 0.05;
	float y = sin(25.0 * p.y + 30.0 * p.x + 6.28 * ty) * 0.05;
	return float2(p.x + x, p.y + y);
}

float4 FSH_DirectionalWarp(float2 InUV_ : TEXCOORD0) : SV_Target0
{
	float2 InUV = TileInfo_fromCurrentTileToLayer(InUV_);
	float MaskValue = Mask.Sample(SamplerStates_Linear_Clamp, InUV).r;
	
	/// We premultiply the intensity with mask. Whites will get maximum intensity
	/// and blacks will get none essentially
	float IntensityValue = Intensity * MaskValue * 0.1;
	
	/// We need to calculate the UV that will get warped to the current UV. So we need
	/// to move in the opposite direction of the angle to find the UV that will end
	/// replacing the current pixel. Hence we add 180 degrees to it (or half PI in radians)
	float AngleValue = AngleRad; // * HALF_PI;
	float C = cos(AngleValue);
	float S = sin(AngleValue);
	float2 Dir = float2(C, S);
	
	float2 UVOffset = Dir * float2(IntensityValue, IntensityValue) + InUV;
	return SourceTexture.Sample(SamplerStates_Linear_Clamp, UVOffset);
}

float4 FSH_NormalWarp(float2 InUV_ : TEXCOORD0) : SV_Target0
{
	float2 InUV = TileInfo_fromCurrentTileToLayer(InUV_);
	float MaskValue = Mask.Sample(SamplerStates_Linear_Clamp, InUV).r;
	
	/// We premultiply the intensity with mask. Whites will get maximum intensity
	/// and blacks will get none essentially
	float IntensityValue = Intensity * MaskValue * 0.1;
	
	/// We need to calculate the UV that will get warped to the current UV. So we need
	/// to move in the opposite direction of the angle to find the UV that will end
	/// replacing the current pixel. Hence we add 180 degrees to it (or half PI in radians)
	float AngleValue = MaskValue * HALF_PI;
	float C = cos(AngleValue);
	float S = sin(AngleValue);
	float2 Dir = float2(C, S);
	
	float2 UVOffset = Dir * float2(IntensityValue, IntensityValue) + InUV;
	return SourceTexture.Sample(SamplerStates_Linear_Clamp, UVOffset);
}

float4 FSH_SineWarp(float2 InUV_ : TEXCOORD0) : SV_Target0
{
	float2 InUV = TileInfo_fromCurrentTileToLayer(InUV_);
	float MaskValue = Mask.Sample(SamplerStates_Linear_Clamp, InUV).r;
	
	/// We premultiply the intensity with mask. Whites will get maximum intensity
	/// and blacks will get none essentially
	float IntensityValue = Intensity * MaskValue * 0.1;
	float2 SineUV = SineWave(InUV, PhaseU, PhaseV);
	float2 UVOffset = float2(IntensityValue, IntensityValue) * SineUV;
	return SourceTexture.Sample(SamplerStates_Linear_Clamp, SineUV);
}
