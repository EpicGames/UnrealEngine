// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

#ifndef CLEAR_COLOR
#define CLEAR_COLOR 0
#endif
#ifndef SHIFT_ALPHA
#define SHIFT_ALPHA 0
#endif
#ifndef INVERT
#define INVERT 0
#endif
float4 ColorOffset;
float Contrast;
float4 AverageColor;
Texture2D SourceTex;

float4 FSH_AdjustColor(in float2 uvSource : TEXCOORD0) : SV_Target0
{
	
#if CLEAR_COLOR
	return AverageColor;
#endif
	
	float4 sourceColor = SourceTex.Sample(SamplerStates_NoBorder, uvSource);
	
	#if SHIFT_ALPHA
		#if INVERT
			sourceColor.a = 1 - sourceColor.a;
		#endif
		float shiftedAlpha = saturate(ColorOffset.r + sourceColor.a);
		float finalAlpha = lerp(AverageColor.a, shiftedAlpha, Contrast);
		return float4(sourceColor.rgb, finalAlpha);	
	#else
		#if INVERT
			sourceColor.rgb = 1 - sourceColor.rgb;
		#endif
		float3 finalColor = saturate(((sourceColor.rgb) + ColorOffset.rgb));
		finalColor = lerp((AverageColor.rgb), finalColor, Contrast);
		return float4(finalColor, sourceColor.a);
	#endif


}