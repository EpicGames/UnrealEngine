// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdjustUVCommon.ush"

float HeightMultiplier;
float HeightMidPoint;
float IsGreyscale;
Texture2D SourceTex;

float4 FSH_AdjustDisplacement(in float2 uv : TEXCOORD0) : SV_Target0
{
	// Fetch the displacement at uv
	float4 finalDisplacement = SourceTex.Sample(SamplerStates_NoBorder, uv);
	return AdjustDisplacement(finalDisplacement, HeightMidPoint, HeightMultiplier, IsGreyscale);
}
