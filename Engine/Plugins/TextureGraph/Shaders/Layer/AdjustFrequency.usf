// Copyright Epic Games, Inc. All Rights Reserved.
#ifndef SOLID
#define SOLID 0
#endif

#include "/Engine/Public/Platform.ush"


Texture2D MainTex;

// Use the TiledFetch for blurredTex
#include "../TiledFetch.ush"
Declare_Tiles_And_FetchTiled(BlurredTex);

float FreqLow;
float Threshold;
float FreqHigh;


float4 FSH_AdjustFrequency(in float2 uv : TEXCOORD0) : SV_Target0
{

	uint width = 0;
	uint height = 0;
	MainTex.GetDimensions(width, height);
	int2 texelPos = int2(uv.x * width, uv.y * height);

	float4 mainTexVar = MainTex.Load(int3(texelPos, 0));

	float4 blurredTexVar = FetchTiled_BlurredTex(uv); // Blurry

	float range = ((mainTexVar.r - blurredTexVar.r) * FreqHigh);

	float displacement = range + lerp(0.5, blurredTexVar.r, FreqLow);

	float3 result = float3(float2(displacement, range), 0.0);

	return float4(result, 1);

}
