// Copyright Epic Games, Inc. All Rights Reserved.
#ifndef RENORMALIZE
#define RENORMALIZE 0
#endif

#include "/Engine/Public/Platform.ush"


Texture2D MainTex;

// Use the TiledFetch for blurredTex
#include "../TiledFetch.ush"
Declare_Tiles_And_FetchTiled(BlurredTex);

float FreqLow;
float Threshold;
float FreqHigh;

float3 PartialDerivatesBlend(float3 n1, float3 n2)
{
	return normalize(float3(n1.xy * n2.z + n2.xy * n1.z, n1.z * n2.z));
}

float3 BlendNormals(float3 n1, float3 n2)
{
	return PartialDerivatesBlend(n1, n2);
}
            
float3 ReconstructNormalZ(float2 rg)
{
	float b = sqrt(1 - dot(rg, rg));
	return float3(rg, b);
}

float3 ReconstructNormalZ(float3 rgb)
{
	return ReconstructNormalZ(rgb.xy);
}


float3 ConformNormals(float3 inputNormals)
{
	//Takes inputNormals in 0 to 1 range and outputs them in -1 to 1 range with clamp, normalization and blue reconstruction.
	//Useful for when using mips of normal maps with the partial derivative blending, or when reading normals with inverted blue channel texels or ranges that are off.
	float2 clampedNormalized = normalize(saturate(inputNormals) * 2 - 1).xy;
	return ReconstructNormalZ(clampedNormalized);
}

float3 ConformBlurry(float3 A)
{
#if RENORMALIZE
    return ConformNormals(A);
#else
	return A * 2 - 1;
#endif
}
            
float3 ConformCrisp(float3 A)
{
#if RENORMALIZE
    return ConformNormals(A);
#else
	return A * 2 - 1;
#endif
}

float4 FSH_AdjustFrequencyNormals(in float2 uv : TEXCOORD0) : SV_Target0
{
	uint width = 0;
	uint height = 0;
	MainTex.GetDimensions(width, height);
	int2 texelPos = int2(uv.x * width, uv.y * height);

	float4 mainTexVar = MainTex.Load(int3(texelPos, 0));

	float4 _BlurredTex_var = FetchTiled_BlurredTex(uv); // Blurry

	float3 confromed = ConformBlurry(_BlurredTex_var.rgb);
	float3 blendedNormals = BlendNormals((float3(FreqHigh, FreqHigh, 1.0) * ((ConformCrisp(mainTexVar.rgb) - confromed) + float3(0, 0, 1))), (confromed * float3(FreqLow, FreqLow, 1.0)));
	float3 emissive = (blendedNormals * 0.5 + 0.5);
	float3 finalColor = emissive;
	return float4(finalColor, 1);
}
