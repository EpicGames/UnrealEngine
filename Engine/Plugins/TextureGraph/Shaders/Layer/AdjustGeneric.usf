// Copyright Epic Games, Inc. All Rights Reserved.
#include "AdjustUVCommon.ush"

float ClampMaskX;
float ClampMaskY;
Texture2D SourceTex;

float4 FSH_AdjustGeneric(in float2 uv : TEXCOORD0) : SV_Target0
{
	float4 blob = SourceTex.Sample(SamplerStates_NoBorder, uv);
    
	float3 emissive = AdjustGeneric(blob, uv, ClampMaskX, ClampMaskY, 1, 1);
	float3 finalColor = emissive;
  
	return float4(clamp(finalColor, 0, 1), blob.a);
}
