// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

#include "../Placement_Projection.ush"

#ifndef DEBUG_TILES
#define DEBUG_TILES 0
#endif

Texture2DArray CombinedBlob;
Texture2DArray DisplacementLayer;

float2 VerticalAxisRotation;
float2 LateralAxisRotation;
float2 UVScaling;

float Radius;
float HeightInfluence;
float HeightMultiplier; // Not used but keep for now
float HeightMidPoint;

float4 FSH_AdjustHexaplanarGeneric
	(in
	float2 tile_uv : TEXCOORD0) :
	SV_Target0
{
	// Fetch the mesh info at raster uv
	float2 raster_uv = TileInfo_fromCurrentTileToLayer(tile_uv);
	
	// Fetch the mesh surface info at the texel
	float2x3 mesh_surf = Fetch_MeshSurface(raster_uv);
	float3 mesh_pos = mesh_surf[0];
	float3 mesh_nor = mesh_surf[1];

	mesh_pos = Transform_Pos_RotateAroundZX(VerticalAxisRotation, LateralAxisRotation, mesh_pos);
	mesh_nor = Transform_Dir_RotateAroundZX(VerticalAxisRotation, LateralAxisRotation, mesh_nor);

	float3x2 triplanar_uvs = EvalTriplanar_uvs(mesh_pos, mesh_nor, UVScaling);
	
	// Fetch the source map triplanar samples
	float3x4 triplanarSamples = Fetch_Source_Triplanar(CombinedBlob, triplanar_uvs);
	
	//height mask
	float3 heightMask = HeightMaskFromDisplacement(DisplacementLayer, triplanar_uvs, HeightMidPoint, HeightInfluence);
	
    //Soften the normal
	float3 projNormal = ProjectionNormals(mesh_nor, 1.0 - Radius, heightMask);

	// Blending tri samples
	float4 rawDisplacement = triplanarSamples[0] * projNormal.x 
						+ triplanarSamples[1] * projNormal.y
						+ triplanarSamples[2] * projNormal.z;
    
	return clamp(rawDisplacement, 0, 1);
}