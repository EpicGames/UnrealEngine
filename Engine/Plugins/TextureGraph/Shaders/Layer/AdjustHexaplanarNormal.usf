// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

#include "/Plugin/TextureGraph/NormalsCommon.ush"
/*
Texture2DArray CombinedBlob;
float RotateUV;
float Center;
float4 ScaleOffset;
float Renormalize;


float4 FSH_AdjustUVNormals(in float2 uvSource : TEXCOORD0) : SV_Target0
{
    float2 start = float2(0, 0);
    float2 center = float2(0.5, 0.5);
    
    ToPivotInBlob(start);
    ToPivotInBlob(center);
    
    float speed = 1.0;
	float tAng = (-1*RotateUV);
    float angle = RotateUV;
    float cosine = cos(speed * angle);
    float sine = sin(speed * angle);
    float2 scale = ScaleOffset.rg;
    float2 offset = ScaleOffset.ba;

    float2 uvDestination = uvSource + offset;
    float2 rotatedOffsetUV = mul((uvDestination) - center, float2x2(cosine, -sine, sine, cosine)) + center;
    
#if FREEFORM
    start = center;
#endif
    
    float2 scaledUVStart = mul((rotatedOffsetUV) - start, float2x2(scale.r, 0, 0, scale.g)) + start;
    uvDestination = scaledUVStart;
	
	float4 blob = SampleBlob(CombinedBlob, uvDestination);
    
    
	float3 conformed = Conform(blob.rgb, Renormalize).rgb;
    float2 rotatedConform = mul(float2(conformed.r, conformed.g), float2x2(cosine, -sine, sine, cosine));
   
    float3 emissive = (float3(rotatedConform, conformed.b) * 0.5+0.5);
    float3 finalColor = emissive;
	return fixed4(finalColor.rgb, blob.a);
}
*/
#include "../Placement_Projection.ush"

#ifndef DEBUG_TILES
#define DEBUG_TILES 0
#endif

Texture2DArray CombinedBlob;
Texture2DArray DisplacementLayer;

float2 VerticalAxisRotation;
float2 LateralAxisRotation;
float2 UVScaling;

float Radius;
float HeightInfluence;
float HeightMultiplier; // Not used but keep for now
float HeightMidPoint;

float4 FSH_AdjustHexaplanarNormals
	(in
	float2 tile_uv : TEXCOORD0) :
	SV_Target0
{
	// Fetch the mesh info at raster uv
	float2 raster_uv = TileInfo_fromCurrentTileToLayer(tile_uv);
	
	// Fetch the mesh surface info at the texel
	float2x3 mesh_surf = Fetch_MeshSurface(raster_uv);
	float3 mesh_pos = mesh_surf[0];
	float3 mesh_nor = mesh_surf[1];
	
	mesh_pos = Transform_Pos_RotateAroundZX(VerticalAxisRotation, LateralAxisRotation, mesh_pos);
	mesh_nor = Transform_Dir_RotateAroundZX(VerticalAxisRotation, LateralAxisRotation, mesh_nor);

	float3x2 triplanar_uvs = EvalTriplanar_uvs(mesh_pos, mesh_nor, UVScaling);
	
	// Fetch the source map triplanar samples
	float3x4 triSamples = Fetch_Source_Triplanar(CombinedBlob, triplanar_uvs);

	float3x3 normalSamples;
	normalSamples[0] = (triSamples[0].xyz) ;
	normalSamples[1] = (triSamples[1].xyz) ;
	normalSamples[2] = (triSamples[2].xyz) ;

	// height mask
	float3 heightMask = HeightMaskFromDisplacement(DisplacementLayer, triplanar_uvs, HeightMidPoint, HeightInfluence);
	
    //Soften the normal
	float3 projNormal = ProjectionNormals(mesh_nor, 1.0 - Radius, heightMask);

	// Blending tri samples
	float3 absObjSpace = (normalSamples[0] * projNormal.x +
						normalSamples[1] * projNormal.y +
						normalSamples[2] * projNormal.z);
	
	return float4(absObjSpace, 1);

	/// TODO: Do the correct tangent space reprojection of the normal
	//convert back to tangent space
	/*
	float3 bitangent = normalize(cross(worldNormal, worldTangent) * worldTangent.w);
	float3x3 worldToTangent = float3x3( worldTangent.rgb, bitangent, worldNormal );
	float3 tangentSpaceNormal = mul( worldToTangent, absObjSpace.rgb );

	tangentSpaceNormal.z = sqrt(1 - tangentSpaceNormal.x*tangentSpaceNormal.x - tangentSpaceNormal.y*tangentSpaceNormal.y); //reconstruct z
	tangentSpaceNormal = saturate(tangentSpaceNormal*0.5+0.5); //convert back to map range

	return float4(tangentSpaceNormal, absObjSpace.w); //preserve alpha from input map, we have gloss packed there
	*/
	
}