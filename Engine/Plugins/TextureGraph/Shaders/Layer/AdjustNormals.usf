// Copyright Epic Games, Inc. All Rights Reserved.
#include "AdjustUVCommon.ush"
#include "/Plugin/TextureGraph/NormalsCommon.ush"

Texture2D SourceTex;
float Renormalize;

float4 FSH_AdjustNormals(in float2 uv : TEXCOORD0) : SV_Target0
{
	float4 blob = SourceTex.Sample(SamplerStates_NoBorder, uv);
    
	float3 conformed = Conform(blob.rgb, Renormalize).rgb;
	float3 emissive = (conformed * 0.5 + 0.5);
	return fixed4(emissive.rgb, blob.a);
}