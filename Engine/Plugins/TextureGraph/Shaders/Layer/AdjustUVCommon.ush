// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

// ==== Defining terms ====

#ifndef FREEFORM
#define FREEFORM 0
#endif

#ifndef TILING
#define TILING 0
#endif

#ifndef CLAMP
#define CLAMP 0
#endif

#ifndef WRAP
#define WRAP 0
#endif

//===== Permutation based defines ===== //

#if FREEFORM
#undef CLAMP
#define CLAMP 1
#endif

#if TILING
#undef WRAP
#define WRAP 1
#endif

//CLAMP will always overright if both the above are 1


float4 AdjustDisplacement(float4 FinalDisplacement, float HeightMidPoint, float HeightMultiplier, float IsGreyscale)
{
	// Apply the displacement scaling
	float displacement = (((FinalDisplacement.r - HeightMidPoint) * HeightMultiplier) + HeightMidPoint);

	//FinalDisplacement = fixed4(displacement, lerp(displacement, FinalDisplacement.g, IsGreyscale), lerp(displacement, FinalDisplacement.b, IsGreyscale), lerp(1, FinalDisplacement.a, IsGreyscale));
	FinalDisplacement = float4(displacement,
		lerp(displacement, FinalDisplacement.g, IsGreyscale),
		lerp(displacement, FinalDisplacement.b, IsGreyscale),
		lerp(1, FinalDisplacement.a, IsGreyscale)
	);

	return FinalDisplacement;
}

float3 AdjustGeneric(float4 blob, float2 uv, float ClampMaskX, float ClampMaskY, float scaleX, float scaleY)
{
	float2 uvFrac = frac(uv);
	float3 emissive = ((lerp(1.0, (1.0 - saturate(distance(uvFrac.r, scaleX))), ClampMaskX) * lerp(1.0, (1.0 - saturate(distance(uvFrac.g, scaleY))), ClampMaskY)) * blob.rgb);
	return emissive;
}

#include "/Plugin/TextureGraph/TiledFetch_Combined.ush"

