// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdjustUVCommon.ush"

Texture2DArray CombinedBlob;
float RotateUV;
float Center;
float4 ScaleOffset;
float HeightMultiplier;
float HeightMidPoint;
float IsGreyscale;

float4 FSH_AdjustUVDisplacement(in float2 uvSource : TEXCOORD0) : SV_Target0
{
    float2 center = float2(0.5f, 0.5f);
    float2 start = float2(0, 0);

    ToPivotInBlob(start);
    ToPivotInBlob(center);

    float angle = RotateUV;
    float speed = 1.0;
    float cosine = cos(speed * angle);
    float sine = sin(speed * angle);
    
    float2 scale = ScaleOffset.rg;
    float2 offset = ScaleOffset.ba;
    float2 uvDestination = uvSource + offset;
    float2 rotatedOffsetUV = mul((uvDestination) - center, float2x2(cosine, -sine, sine, cosine)) + center;
#if FREEFORM
        start = center;
#endif
    float2 scaledUVStart = mul((rotatedOffsetUV) - start, float2x2(scale.r, 0, 0, scale.g)) + start;
    uvDestination = scaledUVStart;
    
	// Fetch the displacement at uv
	float4 finalDisplacement = SampleBlob(CombinedBlob, uvDestination);
	
	return AdjustDisplacement(finalDisplacement, HeightMidPoint, HeightMultiplier, IsGreyscale);

	//// Apply the displacement scaling
	//float displacement = (((finalDisplacement.r - HeightMidPoint) * HeightMultiplier) + HeightMidPoint);

	////finalDisplacement = fixed4(displacement, lerp(displacement, finalDisplacement.g, IsGreyscale), lerp(displacement, finalDisplacement.b, IsGreyscale), lerp(1, finalDisplacement.a, IsGreyscale));
	//finalDisplacement = float4(displacement, 
	//	lerp(displacement, finalDisplacement.g, IsGreyscale), 
	//	lerp(displacement, finalDisplacement.b, IsGreyscale), 
	//	lerp(1, finalDisplacement.a, IsGreyscale)
	//);

 //   return finalDisplacement;

}
