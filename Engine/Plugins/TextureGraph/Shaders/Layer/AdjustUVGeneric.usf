// Copyright Epic Games, Inc. All Rights Reserved.
#include "AdjustUVCommon.ush"
#include "/Plugin/TextureGraph/TiledFetch_Combined.ush"

#ifndef DEBUG_TILES
#define DEBUG_TILES 0
#endif

float RotateUV;
float Center;
float ClampMaskX;
float ClampMaskY;
float4 ScaleOffset;
Texture2DArray CombinedBlob;

float4 FSH_AdjustUVGeneric(in float2 uvSource : TEXCOORD0) : SV_Target0
{

    
    float2 center = float2(0.5f, 0.5f);
    float2 start = float2(0, 0);
   
    ToPivotInBlob(start);
    ToPivotInBlob(center);

    float angle = RotateUV;
    float speed = 1.0;
    float cosine = cos(speed * angle);
    float sine = sin(speed * angle);
    
    float2 scale = ScaleOffset.rg;
    float2 offset = ScaleOffset.ba;
    float2 uvDestination = uvSource + offset;
     
    float2 rotatedOffsetUV = mul((uvDestination) - center, float2x2(cosine, -sine, sine, cosine)) + center;
// #if FREEFORM
	start = center;
// #endif
    
    float2 scaledUVStart = mul((rotatedOffsetUV) - start, float2x2(scale.r, 0, 0, scale.g)) + start;
    uvDestination = scaledUVStart;
    

	float2 uv = uvDestination;
	
	uint width = 0;
	uint height = 0;
	uint dimension = 0;
	
	CombinedBlob.GetDimensions(width, height, dimension);
	int2 texelPos = int2(uv.x * width, uv.y * height);
	int index = TileInfo_tileIndexFromLayerTexel(texelPos);
	int2 fixedTexel = TileInfo_fromLayerToTile(texelPos); // Wrapping texel for blob
	float2 uvLookup = fixedTexel / float2(TileInfo_TileWidth, TileInfo_TileHeight);						// UV from texel

	float4 blob = CombinedBlob.Sample(SamplerStates_NoBorder, float3(uvLookup, index)); 

	float3 emissive = AdjustGeneric(blob, uvDestination, ClampMaskX, ClampMaskY, scale.r, scale.g);
    float3 finalColor = emissive;
  
#if DEBUG_TILES
    if (uvSource.r > 0.0f && uvSource.r < 0.01f)
        return float4(1, 0, 0, 1);
    if (uvSource.g > 0.0f && uvSource.g < 0.01f)
        return float4(1, 0, 0, 1);
    
    if (uvSource.r > 0.99f && uvSource.r <= 1.0f)
        return float4(1, 0, 0, 1);
    if (uvSource.g > 0.99f && uvSource.g <= 1.0f)
        return float4(1, 0, 0, 1);
#endif

    
	return float4(clamp(finalColor, 0, 1), blob.a);

}