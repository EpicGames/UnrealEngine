// Copyright Epic Games, Inc. All Rights Reserved.
#include "AdjustUVCommon.ush"
#include "/Plugin/TextureGraph/NormalsCommon.ush"

Texture2DArray CombinedBlob;
float RotateUV;
float Center;
float4 ScaleOffset;
float Renormalize;

float4 FSH_AdjustUVNormals(in float2 uvSource : TEXCOORD0) : SV_Target0
{
    float2 start = float2(0, 0);
    float2 center = float2(0.5, 0.5);
    
    ToPivotInBlob(start);
    ToPivotInBlob(center);
    
    float speed = 1.0;
	float tAng = (-1*RotateUV);
    float angle = RotateUV;
    float cosine = cos(speed * angle);
    float sine = sin(speed * angle);
    float2 scale = ScaleOffset.rg;
    float2 offset = ScaleOffset.ba;

    float2 uvDestination = uvSource + offset;
    float2 rotatedOffsetUV = mul((uvDestination) - center, float2x2(cosine, -sine, sine, cosine)) + center;
    
#if FREEFORM
    start = center;
#endif
    
    float2 scaledUVStart = mul((rotatedOffsetUV) - start, float2x2(scale.r, 0, 0, scale.g)) + start;
    uvDestination = scaledUVStart;
	
	float4 blob = SampleBlob(CombinedBlob, uvDestination);
    
    
	float3 conformed = Conform(blob.rgb, Renormalize).rgb;
    float2 rotatedConform = mul(float2(conformed.r, conformed.g), float2x2(cosine, -sine, sine, cosine));
   
    float3 emissive = (float3(rotatedConform, conformed.b) * 0.5+0.5);
    float3 finalColor = emissive;
	return fixed4(finalColor.rgb, blob.a);
}