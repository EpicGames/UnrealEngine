// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

sampler Source;
float OffsetX;
float OffsetY;

float4 FSH_BlitSourceOffset(in float2 uv : TEXCOORD0) : SV_Target0
{
    float2 offsetUV = float2(uv.x + OffsetX, uv.y + OffsetY);
    return tex2D(Source, offsetUV);
}