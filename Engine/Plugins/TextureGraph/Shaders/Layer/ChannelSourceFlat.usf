// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

float4 Color;

float4 FSH_FlatColorTexture(in float2 uvSource : TEXCOORD0) : SV_Target0
{
	return Color;
}