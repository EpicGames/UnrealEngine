// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

Texture2D 	SourceTexture;
Texture2D 	DestinationTexture;
Texture2D 	Mask; 
Texture2D 	BlurredBase;

float 		BorderThreshold;
float 		ReplaceHeight;
float 		ReplaceHeightMip;
float 		Opacity;
float 		ChannelOpacity;

float 		BorderFade;
float 		FromAbove;

float LowFrequencyBlend(float mask, float map1, float map2, float radius, float flip)
{
    //This is used to resolve halo issues when lerping heightmaps.
    
    float radius2 = radius / 2;
    float dist = distance(mask, (1.00 - mask));
    float softStripe = 1 - dist * dist;
    float halfMix = (map1+map2) / 2 + radius2 * flip;
    float bowl = lerp(map1, map2, mask);
    float lerpedMap1 = lerp(map1, halfMix, mask);
    float lerpedMap2 = lerp(map2, halfMix, 1 - mask);
    float blend1 = lerp(lerpedMap1, lerpedMap2, mask);
    float blend2 = lerp(blend1, bowl, softStripe * radius2);
    
    return blend2;

//float simpleLerp = lerp(map1,map2,mask);
//return lerp(blend2,simpleLerp,wrap);
}

float4 heightBlend(float4 input1, float4 height1, float4 input2, float4 height2)
{
    float4 topPoint = max(height1,height2) - 0.05;
    float4 level1 = max(height1 - topPoint,0);
    float4 level2 = max(height2 - topPoint,0);
    return ((input1 * level1) + (input2 * level2))/ (level1 + level2);
}

float4 heightlerp(float4 input1, float4 height1, float4 input2, float4 height2, float t)
{
    t = clamp(t, 0, 1);
    return heightBlend(input1, height1 * (1 - t), input2, height2 * t);
}

float MakeSoftMask(float newHeight, float oldHeight, float radius)
{
	float mask = saturate(((newHeight - (oldHeight - radius)) / ((oldHeight + radius) - (oldHeight - radius))));

    if (FromAbove>1)
    {
        
	  //mask = 1 - mask; //invert
    }

    return mask;
}


float4 FSH_Displacement(in float2 uv : TEXCOORD0) : SV_Target0
{
    float2 pivot = float2(0.5, 0.5);
    
	float4 _LerpMask = Mask.Sample(SamplerStates_Linear_Clamp, uv);
	float mask = (Opacity * _LerpMask.r * ChannelOpacity);
	float4 old = DestinationTexture.Sample(SamplerStates_Linear_Clamp, uv);

    float oldMap = old.r;
	float4 _MainTextt = SourceTexture.Sample(SamplerStates_Linear_Clamp, uv);

    float newLowFreq = (_MainTextt.r - _MainTextt.g);
    float newHeight = (BorderThreshold + (newLowFreq));
				
    float flipMask = (FromAbove * -2.0 + 1.0);
    
	float4 _BlurredBase_var = BlurredBase.SampleLevel(SamplerStates_NoBorder, uv, ReplaceHeightMip);
	float oldMapBlurred = _BlurredBase_var.r;
	float wrappedMap = ((newLowFreq - 0.5) + oldMapBlurred);

	float mask2 = (Opacity * _LerpMask.g);
	float newHighFreq = (_MainTextt.g);
	float3 emissive = float3((LowFrequencyBlend(mask, oldMap, newHeight, BorderFade, flipMask) + lerp((mask2 * max(0.0, newHighFreq)), newHighFreq, saturate((mask * 2.0 + -1.0)))), 0.0, 0.0);

    float3 finalColor = emissive;
    finalColor.b = _LerpMask.b;
    return float4(finalColor, 1);
}
