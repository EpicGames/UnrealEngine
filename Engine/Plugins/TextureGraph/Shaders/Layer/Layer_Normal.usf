// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

Texture2D Destination;
Texture2D SourceTexture;
Texture2D Mask;
Texture2D BlurredBase;
Texture2D NormalDiff;

float Opacity;
float InvertX;
float InvertY;
float Strength;
float ReplaceHeightMip;
float ReplaceHeight;


float3 PartialDerivatesBlend(float3 n1, float3 n2)
{
	return normalize(float3(n1.xy * n2.z + n2.xy * n1.z, n1.z * n2.z));
}

float3 PartialDerivatesTransition(float3 a, float3 b, float t)
{
	float3 n1 = a * float3(1 - t, 1 - t, 1);
	float3 n2 = b * float3(t, t, 1);
	return normalize(float3(n1.xy * n2.z + n2.xy * n1.z, n1.z * n2.z));
}

float3 ReconstructNormalZ(float2 rg)
{
	float b = sqrt(1 - dot(rg, rg));
	return float3(rg, b);
}

float3 ReconstructNormalZ(float3 rgb)
{
	return ReconstructNormalZ(rgb.xy);
}


float3 ConformNormals(float3 inputNormals)
{
	//Takes inputNormals in 0 to 1 range and outputs them in -1 to 1 range with clamp, normalization and blue reconstruction.
	//Useful for when using mips of normal maps with the partial derivative blending, or when reading normals with inverted blue channel texels or ranges that are off.
	float2 clampedNormalized = normalize(saturate(inputNormals) * 2 - 1).xy;
	return ReconstructNormalZ(clampedNormalized);
}

//if nothing is in red or green channel, return flat normal. Works on [0,1] space normals
float3 FillBlackPixels01Range(float3 nm)
{
	const float3 flat = float3(.5, .5, 1);
	nm = saturate(nm);
	float2 c = ceil(nm.rg);
	float mask = max(c.x, c.y);
	return lerp(flat, nm, mask);
}

float4 FSH_Normal(in float2 uv : TEXCOORD0) : SV_Target0
{	
	//sample, remap flip and scale layer normals
	float4 normalSample = SourceTexture.Sample(SamplerStates_NoBorder, uv);
	//normalSample.x = lerp(normalSample.x, 1 - normalSample.x, step(0.5, _InvertX));
	//normalSample.y = lerp(normalSample.y, 1 - normalSample.y, step(0.5, _InvertY));
	float3 layerNormals = (normalSample.xyz * 2.0 -1.0);
	//layerNormals = normalize(lerp(float3(0, 0, 1), layerNormals.xyz, _Strength));
	//base normals
	float4 destinationSample = Destination.SampleLevel(SamplerStates_NoBorder, uv, 0.0); // old normals
	float3 baseNormals = (destinationSample.rgb * 2.0 - 1.0);
	//blurred base normals
	float4 blurredBaseSample = BlurredBase.SampleLevel(SamplerStates_NoBorder, uv, ReplaceHeightMip);
	float3 blurredBase = ConformNormals(blurredBaseSample.rgb);
	//wrap to base
	float3 wrappedNormals = PartialDerivatesBlend(blurredBase , layerNormals);
	float3 wrappedOrNot = PartialDerivatesTransition(layerNormals , wrappedNormals , ReplaceHeight);
				
	//lerp base to new normals
	float2 mask = Mask.Sample(SamplerStates_NoBorder, uv).rg * Opacity;
	float3 origVsNew = PartialDerivatesTransition(baseNormals , wrappedOrNot , mask.g);
	//add normal discrepancy fix
	float3 normalDiff = NormalDiff.Sample(SamplerStates_NoBorder, uv).xyz * 2.0 - 1.0;
	float3 addNormalDiff = saturate(PartialDerivatesBlend(origVsNew , normalDiff)*0.5 + 0.5);
	//preserve gloss
	//float gloss = lerp(destinationSample.a, normalSample.a, detailMask);
	            //the lerp here is to make sure that no normals that are masked away get affected.
	//the normal blending happening above, especially in cases where normalization was being run could affect normals where the layer mask is black.
	float3 blendedNormal = lerp(destinationSample.rgb, addNormalDiff,ceil(mask.r));
	return float4 (blendedNormal, destinationSample.a);
}
