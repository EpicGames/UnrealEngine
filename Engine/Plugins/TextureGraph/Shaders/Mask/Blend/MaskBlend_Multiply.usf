// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

float		MaskFactor0;
Texture2D	Mask0;

float		MaskFactor1;
Texture2D	Mask1;
//float		InvRange;


float4 FSH_MaskBlend_Multiply(in float2 uv : TEXCOORD0) : SV_Target0
{
	float m0 = MaskFactor0 * Mask0.Sample(SamplerStates_Linear_Clamp, uv).r;
	float m1 = MaskFactor1 * Mask1.Sample(SamplerStates_Linear_Clamp, uv).r;
	float m = saturate(m0 * m1);
	
    return float4(m, m, m, 1);

}