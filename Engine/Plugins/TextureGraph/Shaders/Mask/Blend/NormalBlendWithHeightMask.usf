// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

Texture2D       		DestinationTexture;
Texture2D       		SourceTexture;
Texture2D       		Mask;
float           		Opacity;
float           		LerpRGB;      /// 0 or 1 depending on whether we want the RGB lerp
float           		LerpAlpha;    /// 0 or 1 depending on whether we want the RGB lerp

float4 FSH_NormalBlend(in float2 uv : TEXCOORD0) : SV_Target0
{
	float4 src          = SourceTexture.Sample(SamplerStates_Linear_Clamp, uv);	
	float4 dst          = DestinationTexture.Sample(SamplerStates_Linear_Clamp, uv);
	float4 mask         = Mask.Sample(SamplerStates_Linear_Clamp, uv);
	float strength		= (mask.g * Opacity);

	float4 finalColor   = dst;
	
	finalColor.rgb      = lerp(dst.rgb, src.rgb, strength * LerpRGB);
	finalColor.a        = lerp(dst.a, src.a, strength * LerpAlpha);
	
    return finalColor;
}
