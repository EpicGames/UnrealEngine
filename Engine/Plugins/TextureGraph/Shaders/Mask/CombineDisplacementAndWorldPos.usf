// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "../ShaderUtil.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

Texture2D SourceDisplacement;
Texture2D WorldPos;
Texture2D WorldNormals;
Texture2D WorldUVMask;
Texture2D MinMaxTex;
float MidPoint;
float StandardHeight;
float4 MeshBoundsMin;
float4 MeshBoundsSize;


float4 FSH_CombineDisplacementAndWorldPos(float2 uv : TEXCOORD0) : SV_Target0
{
	half3 on = WorldNormals.Sample(SamplerStates_Linear_Clamp, uv).rgb;
	on = (on * 2) - 1;
	
	float addedDisplacement = SourceDisplacement.Sample(SamplerStates_Linear_Clamp, uv).r - MidPoint;
	
	//Converting displacement back to meters
	addedDisplacement /= StandardHeight;
	
	float3 pt = WorldPos.Sample(SamplerStates_Linear_Clamp, uv).rgb;
	pt = (pt * MeshBoundsSize.xyz) + MeshBoundsMin.xyz;
	pt = pt + (addedDisplacement * on);

	return float4(pt, 1);
}
