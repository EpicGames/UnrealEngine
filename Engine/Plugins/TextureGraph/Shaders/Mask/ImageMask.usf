// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#ifndef NORMALIZE
#define NORMALIZE 0
#endif

#ifndef INVERT
#define INVERT 0
#endif



Texture2D	SourceTexture;
Texture2D	MinMax;
float		RangeMin;
float		RangeMax;

float4 FSH_ImageMask(in float2 uv : TEXCOORD0) : SV_Target0
{
	const float g_black = 0.0;
	const float g_gray = 0.5;
	
	/// It's a 1x1 texture, so doesn't matter what we sample
	float3 minMax = MinMax.Sample(SamplerStates_Linear_Clamp, float2(0, 0)).rgb;
	float4 srcTex = SourceTexture.Sample(SamplerStates_Linear_Clamp, uv);
	float disp = srcTex.r;
                
	#if NORMALIZE
		float darkPoint = minMax.r;
		float brightPoint = minMax.g;
		float range = abs(brightPoint - darkPoint);
		float baseVal = disp;
		float rangeVal = ((((g_black + ((disp - darkPoint) * (1.0 - g_black)) / (brightPoint - darkPoint)) - g_gray)) + g_gray);

		disp = range <= 0.0001 ? baseVal : rangeVal;
	#endif

	#if INVERT
	disp = (1 - disp);
	#endif

    float remap2 = (disp - RangeMin) / (RangeMax - RangeMin + 0.0001);
    remap2 = saturate(remap2);

    return float4(remap2, remap2, remap2, 1);
}
