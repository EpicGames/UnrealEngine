// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"


// Permutations
#ifndef FROMABOVE
#define FROMABOVE 0
#endif
#ifndef HEIGHTMASK
#define HEIGHTMASK 0
#endif
#ifndef DETAILS
#define DETAILS 0
#endif
#ifndef MATERIALID
#define MATERIALID 0
#endif
#ifndef PAINTMASK
#define PAINTMASK 0
#endif
#ifndef UVMASK
#define UVMASK 0
#endif
#ifndef OPACITY
#define OPACITY 0
#endif
#ifndef MASKSTACK
#define MASKSTACK 0
#endif

float MakeSoftMask(float newHeight, float oldHeight, float radius)
{
	float mask = saturate(((newHeight - (oldHeight - radius)) / ((oldHeight + radius) - (oldHeight - radius))));

#if FROMABOVE
	mask = 1 - mask; //invert
#endif
	 
	return mask;
}
            
float MakeDetailMask(float softMask, float layerAO, float oldAO, float detailAmount) //This function will likely be based on something other than ao when we skip ao channels per layer
{
	float aoMask = saturate(((layerAO - 0.5) + (1.0 - oldAO)));
	float blended = saturate((softMask > 0.5 ? (aoMask / ((1.0 - softMask) * 2.0)) : (1.0 - (((1.0 - aoMask) * 0.5) / softMask)))); //vivid light
	float detailMask = saturate((floor(softMask) + saturate(lerp(softMask, blended, detailAmount))));
	return detailMask;
}
            
Texture2D Displacement;		/// Current layer adjusted displacement map 
Texture2D OldDisplacement;	/// Previous layer displacement map
Texture2D Occlusion;		/// Current layer adjusted occlusion map
Texture2D OldOcclusion;		/// Previous layer occlusion map
Texture2D OpacityChannel;	/// Current layer adjusted opacity channel map

Texture2D MaskStack;		/// Mask stack map from current layer
Texture2D UVMask;			/// UVMask map for target set
Texture2D PaintMask;		/// Paint mask map from layer
Texture2D MaterialID;		/// Material ID map for target set

float PreserveDetails;
float BorderFade;
float BorderThreshold;
float Opacity;

float4 FSH_LayerMask(in float2 uv : TEXCOORD0) : SV_Target0
{

	float2 output = float2(1, 1); //white starting point

#if HEIGHTMASK //calculate height mask
	// We fetch the current layer displacement D and the previous layer displacement pD
	// D is offset by BorderThreshold which give us the new displacement nD
	// Then the soft mask equation:
	// The prev displacement is the center of a range of length 2*BorderFade
	// Which give us a lower bound L = pD - BorderFade
	// and a upper bound U = pD + BorderFade
	// The soft mask (SM) value is the saturated ratio: sm = saturate((D - L) / (U - L))
	// So if D is above pD + BorderFade, SM is 1
	// So if D is below pD - BorderFade, SM is 0
	// And SM is in the range [0,1] otherwise

	float layerSample = Displacement.Sample(SamplerStates_Wrap, uv).r;
	float newHeight = (layerSample + BorderThreshold);
	float oldHeight = OldDisplacement.Sample(SamplerStates_Wrap, uv).r; // previous layers height
	float softMask = MakeSoftMask(newHeight, oldHeight, BorderFade);

	// The detailed mask (DM) is a more refined version of SM introducing the occlusion contribution (if available)
	float detailMask = softMask;

#if DETAILS // apply occclusion map
	// Fetch the current layer occlusion O and the previous layer occlusion pO
	// 
	float layerOcclusion = Occlusion.Sample(SamplerStates_Wrap, uv).r;
	float oldOcclusion = OldOcclusion.Sample(SamplerStates_Wrap, uv).r;
	detailMask = MakeDetailMask(softMask, layerOcclusion, oldOcclusion, PreserveDetails);
#endif

	// With HeighMask enabled, the output is initialized as (SM, DM)
	output = float2(softMask, detailMask);
#endif

#if MASKSTACK //apply mask stack
	output *= saturate(MaskStack.Sample(SamplerStates_Wrap, uv).r);
#endif
				
#if MATERIALID // apply material id
	output *= saturate(MaterialID.Sample(SamplerStates_Wrap, uv).r);
#endif

#if PAINTMASK//apply paint mask
		float4 pSample = PaintMask.Sample(SamplerStates_Wrap, uv);
		float removePremul = saturate(pSample.r / pSample.a);
		output = lerp(output, removePremul, pSample.a);
#endif

#if UVMASK //apply uv mask
		output *= UVMask.Sample(SamplerStates_Wrap, uv).r;
#endif

#if OPACITY //apply opacity channel
		output *= OpacityChannel.Sample(SamplerStates_Wrap, uv).r;
#endif
	output = lerp(0, output, Opacity);
	return float4(output, 0, 1);
}
