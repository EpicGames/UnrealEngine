// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

#ifndef BRIGHTNESS_MODIFIER
#define BRIGHTNESS_MODIFIER 0
#endif

#ifndef CLAMP_MODIFIER
#define CLAMP_MODIFIER 0
#endif

#ifndef INVERT_MODIFIER
#define INVERT_MODIFIER 0
#endif

#ifndef NORMALIZE_MODIFIER
#define NORMALIZE_MODIFIER 0
#endif

#ifndef GRADIENT_REMAP_MODIFIER
#define GRADIENT_REMAP_MODIFIER 0
#endif

#ifndef POSTERIZE_MODIFIER
#define POSTERIZE_MODIFIER 0
#endif

#include "/Plugin/TextureGraph/TiledFetch_Combined.ush"

Texture2D SourceTexture;

#if BRIGHTNESS_MODIFIER
float Brightness;
float Contrast;
Texture2D MinMaxTexture;
#endif

#if CLAMP_MODIFIER
float ClampMin;
float ClampMax;
#endif

#if NORMALIZE_MODIFIER
Texture2D MinMaxTexture;
#endif

#if GRADIENT_REMAP_MODIFIER
float RangeMin;
float RangeMax;
int	  Repeat;
float Curve;
int   Mirror;

float SimpleCurve(float input, float curve) 
{
	float absCurve = abs(curve);
	float stepCurve = step(0.0, curve);
	float combine = ((input*(1.0 - absCurve)) + (saturate(sin(acos((1.0 - input))))*absCurve*stepCurve) + (saturate((1.0 - sin(acos(input))))*absCurve*(1.0 - stepCurve)));
	return combine;
}
#endif

#if POSTERIZE_MODIFIER
int	  StepCount;
#endif


float4 FSH_MaskModifier(float2 uv : TEXCOORD0) : SV_Target0
{
	uint width = 0;
	uint height = 0;
	SourceTexture.GetDimensions(width, height);
	int2 texelPos = int2(uv.x * width, uv.y * height);

	float maskValSource = SourceTexture.Load(int3(texelPos, 0)).x;

#if   BRIGHTNESS_MODIFIER
	float2 minMax = MinMaxTexture.Load(int3(0, 0, 0)).xy;
	minMax = clamp(minMax, 0, 1);
	float midPoint = (minMax.x + minMax.y) * 0.5 + Brightness;

	float maskVal = lerp(midPoint, maskValSource + Brightness, Contrast);

#elif CLAMP_MODIFIER
	float maskVal = clamp(maskValSource, ClampMin, ClampMax);

#elif INVERT_MODIFIER
	float maskVal = 1.0 - maskValSource;

#elif NORMALIZE_MODIFIER
	float2 minMax = MinMaxTexture.Load(int3(0,0,0)).xy;
	float minVal = minMax.x;
	float maxVal = minMax.y;
	float range = abs(maxVal - minVal);
	float baseVal = maskValSource;
	float maskVal = (baseVal - minVal) / (range);

#elif GRADIENT_REMAP_MODIFIER
	float maskVal = maskValSource;
	if (Repeat > 0) 
	{
		float i = maskValSource * Repeat;
		maskVal= frac(i);
		if (maskValSource == 1)
			maskVal = 1;
	}
	if (Mirror)
		maskVal = 1.0 - abs((maskVal - 0.5) * 2.0);

	maskVal = (maskVal - RangeMin) / (RangeMax - RangeMin + 0.0001);
	maskVal = saturate(maskVal);		
	maskVal = SimpleCurve(maskVal, Curve);

#elif POSTERIZE_MODIFIER
	float maskVal = maskValSource;
	maskVal = (floor(maskValSource * StepCount)) / (StepCount - 1);
	
#endif
	
	

	return float4(maskVal, maskVal, maskVal, 1.0);
}
