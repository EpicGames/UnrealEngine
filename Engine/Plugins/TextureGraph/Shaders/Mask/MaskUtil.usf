// Copyright Epic Games, Inc. All Rights Reserved.
float MakeSoftMask(float newHeight, float oldHeight, float radius)
{
	float mask = saturate(((newHeight - (oldHeight - radius)) / ((oldHeight + radius) - (oldHeight - radius))));

	#ifdef FROMABOVE
		mask = 1 - mask; //invert
	#endif

	return mask;
}

float MakeDetailMask(float softMask, float layerAO, float oldAO, float detailAmount) //This function will likely be based on something other than ao when we skip ao channels per layer
{
    float aoMask = saturate(((layerAO - 0.5) + (1.0 - oldAO)));
    float blended = saturate((softMask > 0.5 ? (aoMask / ((1.0 - softMask) * 2.0)) : (1.0 - (((1.0 - aoMask) * 0.5) / softMask)))); //vivid light
    float detailMask = saturate((floor(softMask) + saturate(lerp(softMask, blended, detailAmount))));
    return detailMask;
}
            
