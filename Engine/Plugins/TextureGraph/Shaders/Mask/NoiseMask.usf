// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

#define NOISE_SIMPLEX	0
#define NOISE_PERLIN	1
#define NOISE_WORLEY1	2
#define NOISE_WORLEY2	3
#define NOISE_WORLEY3	4

#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/TileInfo.ush"

#ifndef NOISE_TYPE
#define NOISE_TYPE NOISE_SIMPLEX
#endif

#if	NOISE_TYPE == NOISE_SIMPLEX
#include "/Plugin/TextureGraph/Noise/Noise_Simplex.ush"
#elif NOISE_TYPE == NOISE_PERLIN
#include "/Plugin/TextureGraph/Noise/Noise_Perlin.ush"
#else
#include "/Plugin/TextureGraph/Noise/Noise_Voronoise.ush"
#endif

int	  NoiseType;

float Seed;
float Amplitude;
float Frequency;
int   Octaves;
float Lacunarity;
float Persistance;
float Invert;

float4 FSH_NoiseMask(float2 uv : TEXCOORD0) : SV_Target0
{
	float2 layer_uv = TileInfo_fromCurrentTileToLayer(uv);
	
	layer_uv.y = 1.0 - layer_uv.y; /// TODO: Check the need for mirrored vertically


	FBMDesc fbm;
	fbm.Seed = Seed;
	fbm.Amplitude = Amplitude;
	fbm.Frequency = Frequency;
	fbm.Octaves = Octaves;
	fbm.Lacunarity = Lacunarity;
	fbm.Persistance = Persistance;

	float maskVal =
#if	  NOISE_TYPE == NOISE_SIMPLEX
					FBM_Simplex4D(layer_uv, fbm);
#elif NOISE_TYPE == NOISE_PERLIN
					FBM_Perlin4D(layer_uv, fbm);
#elif NOISE_TYPE == NOISE_WORLEY1
					FBM_Voronoise4D_F1_F0_Squared(layer_uv, fbm);
#elif NOISE_TYPE == NOISE_WORLEY2
					FBM_Voronoise4D_F0_Squared(layer_uv, fbm);
#elif NOISE_TYPE == NOISE_WORLEY3
					FBM_Voronoise4D_F1_Squared(layer_uv, fbm);
#endif

	maskVal += 0.5; // we offset the midpoint of noise to 0.5 for all types of noises

	maskVal = lerp(maskVal, 1 - maskVal, Invert); // Apply invert eventually
	
	return float4(maskVal, maskVal, maskVal, 1.0);
}
