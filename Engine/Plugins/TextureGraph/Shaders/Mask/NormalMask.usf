// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "../ShaderUtil.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

// Permutations
#ifndef THREE_DIMESIONS
#define THREE_DIMESIONS 0
#endif

Texture2D SourceTex;
Texture2D MeshNormals;
Texture2D MeshTangents;
float Range;
float Radius;
float DirectionX;
float DirectionY;
float DirectionZ;
int ObjectSpace;
float4x4 LocalToWorldMatrix;

float4 FSH_NormalMask(float2 uv : TEXCOORD0) : SV_Target0
{   
	//Step 1 unpack _MainTex that is layer normals or underlaying normals,
	// and unpack normal that is mesh normals or loaded backed mesh normals
	float3 sourceNormals = SourceTex.Sample(SamplerStates_Linear_Clamp, uv).yzx;
	float3 normalTS = sourceNormals * 2.0 - 1.0;
    float3 normalsForMask = normalTS;

	float3 direction = float3(DirectionY, DirectionZ, DirectionX);
	
    #if THREE_DIMESIONS                  //(this is converted to arithmetic below)
    
	float3 normalOS = MeshNormals.Sample(SamplerStates_Linear_Clamp, uv).yzx * 2.0 - 1.0;	
	
	float4 tangent = MeshTangents.Sample(SamplerStates_Linear_Clamp, uv);
    float4 tangentOS = float4(tangent.y, tangent.z, tangent.x, tangent.w) * 2.0 - 1.0;	

    float3x3 tangentToWorld = GetTransform_TS_to_WS((float3x3)LocalToWorldMatrix, normalOS, tangentOS);

    normalsForMask = (ObjectSpace * mul(tangentToWorld, normalTS)) + ((1 - ObjectSpace) * normalOS);
    
	#endif
  
    float clampedDot = saturate((dot(direction, normalsForMask) * 0.5 + 0.5));
  
    float max = 1 - Range;
    float min = 1 - Radius;
  
    float maskVal = Remap(clampedDot, min, max, 0, 1);
    maskVal = saturate(maskVal);

    return float4(maskVal, maskVal, maskVal, 1);
}
