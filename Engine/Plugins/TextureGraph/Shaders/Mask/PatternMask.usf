// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"


#include "/Plugin/TextureGraph/TileInfo.ush"
#include "/Plugin/TextureGraph/ShaderUtil.ush"

#define PATTERN_SQUARE 0
#define PATTERN_CIRCLE 1
#define PATTERN_CHECKER 2
#define PATTERN_GRADIENT 3

#ifndef PATTERN_TYPE
#define PATTERN_TYPE PATTERN_SQUARE
#endif

float2 Repeat;
float2 Spacing;
float2 Offset;
float Invert;
int	PatternType;

// Bevel
float Bevel;
float BevelCurve;

//Size Jitter Variables
float JS_Amount;
float JS_Threshold;
int JS_Seed;

//Brightness Jitter Variables
float JB_Amount;
float JB_Threshold;
int JB_Seed;

//Tilt Jitter Variables
float2 JT_Amount;
float JT_Threshold;
int JT_Seed;
	
//Angle Jitter Variables
float2 JA_Amount;
int JA_Seed;

//Cut off
float CO_Threshold;
int CO_Seed;

//Gradient 
float2 GradientDir;

float2 GetAddedOffset(float2 uv, float2 tiles, float2 offsets)
{
	float u = Posterize(uv.x, tiles.x) * (tiles.x - 1) * offsets.y;
	float v = Posterize(uv.y, tiles.y) * (tiles.y - 1) * offsets.x;
	return float2(v, u);
}


float ApplyBevel(float value, float bevel, float curve)
{
	float bevelled = saturate(Remap(value, 0, bevel, 0, 1));
	bevelled = saturate(SimpleCurve(bevelled, curve));
	return bevelled;
}

float GetSizeJitter(float2 jitterUVs)
{
	float sizeJitter = SFNoise(jitterUVs, JS_Seed);
	sizeJitter = Remap(sizeJitter, 0, 1, JS_Amount, 0);
	float jsThreshold = 1 - SFNoise(jitterUVs, JS_Seed + 0.1);
	float sizeJitterThreshold = step(jsThreshold, JS_Threshold);
	sizeJitter = 1 - (sizeJitter * sizeJitterThreshold);
	return sizeJitter;
}

float GetBrightnessJitter(float2 jitterUVs)
{
	float jbValue = 1 - SFNoise(jitterUVs, JB_Seed);
	float jbThreshold = 1 - SFNoise(jitterUVs, JB_Seed + JB_Seed + 10);
	jbThreshold = step(jbThreshold, JB_Threshold);
	jbValue = Remap(jbValue, 0, 1, 0, JB_Amount) * jbThreshold;
	return jbValue;
}

float GetAngleTiltJitter(float2 jitterUVs, float2 pattern_cell_uv)
{
	float jaVariance = SFNoise(jitterUVs, JA_Seed + 0.2);
	jaVariance = (lerp(JA_Amount.x, JA_Amount.y, jaVariance) - 0.5f) * 2 * PI;
				//jaVariance = lerp(((_JA_Amount.x + _JA_Amount.y) / 2.0 - 0.5f) * 2 * PI, jaVariance, jaThreshold);

	float angle = atan2(pattern_cell_uv.x, pattern_cell_uv.y) + jaVariance;
	float distance = length(pattern_cell_uv) / (abs(cos(jaVariance)) + abs(sin(jaVariance)));
	float2 angledUV = float2(cos(angle) * distance, sin(angle) * distance);
	angledUV = (angledUV + 1.0) / 2.0;

				// Tilt Jitter
	float jtValue = SFNoise(jitterUVs, JA_Seed);
	float jtThreshold = 1 - SFNoise(jitterUVs, JA_Seed + 0.1);
	jtThreshold = step(jtThreshold, JT_Threshold);
	jtValue = Remap(jtValue, 0, 1, JT_Amount.x, JT_Amount.y) * jtThreshold * angledUV.x;

	return jtValue;
}
float2 StretchCompensation(float tx, float ty, float sx, float sy, float2 spacingApplied)
{
	float xR = tx / (1.0 - (tx * sx));
	float yR = ty / (1.0 - (ty * sy));

	float x = xR / yR;
	float y = yR / xR;

	float2 xy = max(1 - float2(x, y), 0);

	xy = saturate(Remap(spacingApplied, xy, 1, 0, 1));
				
	return xy;
}

float4 FSH_PatternMask(float2 uv : TEXCOORD0) : SV_Target0
{

	float2 layer_uv = TileInfo_fromCurrentTileToLayer(uv);

	layer_uv.y = 1.0 - layer_uv.y; /// TODO: Check the need for mirrored vertically
	
	/*
	Newer code
		float2 pattern_cell_uv = layer_uv * Repeat.xy;
		float2 pattern_cell_coord = floor(pattern_cell_uv);
		pattern_cell_uv += Offset * pattern_cell_coord.yx; // Apply Offset
		pattern_cell_coord = floor(pattern_cell_uv);
		pattern_cell_uv = frac(pattern_cell_uv); // Full cell uv normalized
		pattern_cell_uv = (pattern_cell_uv * 2.0 - float2(1.0, 1.0)); // Full cell uv in [-1,1]

		// Apply Scaling if square or circle type
	#if (PATTERN_TYPE == PATTERN_SQUARE) || (PATTERN_TYPE == PATTERN_CIRCLE)
		float2 cell_space = (float2(1.0, 1.0) - min(Spacing * Repeat, 1));
		pattern_cell_uv /= cell_space; // Reduce the cell uv removing the spacing ratio
	#endif

		float maskVal = 1.0;

	#if PATTERN_TYPE == PATTERN_SQUARE
			float2 maskXY = (step(float2(-1, -1), pattern_cell_uv) * step(pattern_cell_uv, float2(1, 1)));
			maskVal = maskXY.x * maskXY.y;
	#elif PATTERN_TYPE == PATTERN_CIRCLE
			maskVal = step(dot(pattern_cell_uv,pattern_cell_uv), 1);
	#elif PATTERN_TYPE == PATTERN_CHECKER
			float2 maskXY = (step(float2(-1, -1), pattern_cell_uv) * step(pattern_cell_uv, float2(1, 1)));
			float pattern_cell_coordI = int(pattern_cell_coord.x + pattern_cell_coord.y) % 2;
			maskVal = maskXY.x * maskXY.y * (pattern_cell_coordI);
	#endif

		// Invert maybe...
		maskVal = (Invert  > 0.0 ? 1.0 - maskVal : maskVal);
		return float4(maskVal, maskVal, maskVal, 1.0);
	*/
	
#if (PATTERN_TYPE == PATTERN_CHECKER)
	float2 blackWhite = floor(layer_uv * float2(Repeat));
	float checkerA = (blackWhite.x - blackWhite.y) % 2;
	float checkerB = (blackWhite.y - blackWhite.x) % 2;
	float checker = clamp(checkerA, 0, 1) + clamp(checkerB, 0, 1);
	float Final = (Invert  > 0.0 ? 1.0 - checker : checker);
	return float4(Final, Final, Final, 1);
	
#endif
#if (PATTERN_TYPE == PATTERN_GRADIENT)

    float2 center = float2(0.5, 0.5);
	float angleGrad = radians(GradientDir.y);
	float tiltGrad = (1 - (GradientDir.x / 90)) / 2;
    float cosine = cos(angleGrad);
    float sine = sin(angleGrad);

	
	float2 rotatedOffsetUV = mul((layer_uv ) - center, float2x2(cosine, -sine, sine, cosine)) + center;
	float gradValue = rotatedOffsetUV.x + tiltGrad;
	float Final = (Invert  > 0.0 ? 1.0 - gradValue : gradValue);
	return float4(Final, Final, Final, 1);
#endif

	// BASE VALUES
	float2 rawUV = layer_uv;
	float2 tiles = Repeat;
	float2 offsets = Offset;
	float2 addedOffset = GetAddedOffset(rawUV, tiles, offsets); 


	// UV
	float2 repeatedUVnoFrac = rawUV * tiles + addedOffset;
	float2 repeatedUV = frac(repeatedUVnoFrac);
	float2 repeatedUVCentralized = (repeatedUV - 0.5) * 2;
	float2 absoluteUV = abs(repeatedUVCentralized);

	float2 jitterUVs = frac(floor(repeatedUVnoFrac) / tiles);
				
	// Size Jitter
	float sizeJitter = SFNoise(jitterUVs, JS_Seed);
	
	sizeJitter = Remap(sizeJitter, 0, 1, JS_Amount, 0);
	float jsThreshold = 1 - SFNoise(jitterUVs, JS_Seed + 0.1);
	float sizeJitterThreshold = step(jsThreshold, JS_Threshold);
	sizeJitter = 1 - (sizeJitter * sizeJitterThreshold);
	
	// SPACING
	float2 rawSpaces = Spacing;
	float2 spaces = 1 - min(rawSpaces * tiles,1);
	float2 spacingApplied = min(Remap(absoluteUV / sizeJitter, 0, spaces, 0, 1), 1);

#if (PATTERN_TYPE == PATTERN_CIRCLE)
	//Circle
	float2 offsetUV =  repeatedUVCentralized * 0.5;
	float circleFill = step(sqrt((pow(offsetUV.x, 2) / pow(spaces.x, 2)) + (pow(offsetUV.y, 2) / pow(spaces.y, 2))), lerp(0.5, fmod(sizeJitter, 0.5), any(JS_Threshold)));
	
	float Final = (Invert  > 0.0 ? 1.0 - circleFill : circleFill);
	return float4(Final, Final, Final, 1);
#endif
	
	float2 stretchCompensation = StretchCompensation(tiles.x, tiles.y, Spacing.x, Spacing.y, spacingApplied);
	float grayScale = 1 - max(stretchCompensation.x, stretchCompensation.y);
	float bevelFactor = saturate(max(Repeat.x, Repeat.y) * Bevel) / sizeJitter;

	float withBevel = ApplyBevel(grayScale, max(bevelFactor, 0.0001), BevelCurve); //Final Tile Value

	// Brightness Jitter
	float jbValue = 1 - SFNoise(jitterUVs, JB_Seed);
	float jbThreshold = 1 - SFNoise(jitterUVs, JB_Seed  + JB_Seed + 10);
	jbThreshold = step(jbThreshold, JB_Threshold);
	jbValue = Remap(jbValue, 0, 1, 0, JB_Amount) * jbThreshold;;
				
	// Angle Jitter
	//float jaThreshold = SFNoise(jitterUVs, _JA_Seed + 0.1);
	//jaThreshold = step(jaThreshold, _JA_Threshold);

	float jaVariance = SFNoise(jitterUVs, JA_Seed + 0.2);
	jaVariance = (lerp(JA_Amount.x, JA_Amount.y, jaVariance) - 0.5f) * 2 * PI;
	//jaVariance = lerp(((_JA_Start + _JA_End) / 2.0 - 0.5f) * 2 * PI, jaVariance, jaThreshold);

	float angle = atan2(repeatedUVCentralized.x, repeatedUVCentralized.y) + jaVariance;
	float distance = length(repeatedUVCentralized) / (abs(cos(jaVariance)) + abs(sin(jaVariance)));
	float2 angledUV = float2(cos(angle) * distance, sin(angle) * distance);
	angledUV = (angledUV + 1.0) / 2.0;

	// Tilt Jitter
	float jtValue = SFNoise(jitterUVs, JT_Seed);
	float jtThreshold = 1 - SFNoise(jitterUVs, JT_Seed + 0.1);
	jtThreshold = step(jtThreshold, JT_Threshold);
	jtValue = Remap(jtValue, 0, 1, JT_Amount.x, JT_Amount.y) * jtThreshold  * angledUV.x;

	// Cut Off
	float coValue = 1 - step(CO_Threshold, SFNoise(jitterUVs, CO_Seed));

	// FINAL COLOR
	float final = saturate(withBevel - jtValue - jbValue - coValue);
	final = (Invert > 0.0 ? 1.0 - final : final);
	return float4(final, final, final, 1);

}
