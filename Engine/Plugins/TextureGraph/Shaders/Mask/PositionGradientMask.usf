// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/TileInfo.ush"
#include "/Plugin/TextureGraph/ShaderUtil.ush"

// Permutations
#ifndef THREE_DIMESIONS
#define THREE_DIMESIONS 0
#endif

Texture2D SourceTex;
Texture2D MinMaxTex;
float Falloff;
float StartingOffset;
float DirectionX; // X is forward
float DirectionY; // Y is right
float DirectionZ; // Z is up
float4 RelativeBoundsMin;
float4 RelativeBoundsMax;
								
float4 FSH_PositionGradientMask(float2 uv : TEXCOORD0) : SV_Target0
{
	float directionX = DirectionX;
	float directionY = DirectionY;
	float directionZ = DirectionZ;
	float total = abs(DirectionX) + abs(DirectionY) + abs(DirectionZ);
	float percentage = 0;
	float3 pt = float3(0, 0, 0);
	
	float3 direction;
#if THREE_DIMESIONS

	// TODO: This is a bug, we need to somehow calculate the min and max for all the textures sets selected and send it here.
	float3 minValue = RelativeBoundsMin.xyz; 
	float3 maxValue = RelativeBoundsMax.xyz;
	
	pt = (SourceTex.Sample(SamplerStates_Linear_Clamp, uv).yzx - minValue) / abs(maxValue - minValue);

	if (DirectionY < 0)
	{
		pt.x = 1 - pt.x;
	}	

	if (DirectionZ < 0)
	{
		pt.y = 1 - pt.y;
	}
					
	if (DirectionX < 0)
	{
		pt.z = 1 - pt.z;
	}

	direction = float3(abs(DirectionX), abs(DirectionY), abs(DirectionZ));

#else
	
	float2 layer_uv = TileInfo_fromCurrentTileToLayer(uv);

	// TWO DIMENSIONS	
	pt = float3(layer_uv.x, layer_uv.y, lerp(0, SourceTex.Sample(SamplerStates_Linear_Clamp, uv).r, step(0, DirectionX)));
	
	if (DirectionY < 0)
	{
		pt.x = 1 - pt.x;
	}

	if (DirectionZ < 0)
	{
		pt.y = 1 - pt.y;
	}
	
	directionY = abs(DirectionY);
	directionZ = abs(DirectionZ);
	

	float2 minMax = MinMaxTex.Sample(SamplerStates_Linear_Clamp, float2(0, 0)).rg;
	
	float displacementRange = abs(minMax.g - minMax.r);
	if (abs(displacementRange) < 0.0001)
	{
		displacementRange = minMax.g;
						
		if (abs(displacementRange) < 0.0001)
			displacementRange = 1.0;
	}
	pt.z = (pt.z - minMax.r) / displacementRange;

#endif
	
	pt.x *= directionY;
	pt.y *= directionZ;
	pt.z *= directionX;
					
	percentage = (pt.x + pt.y + pt.z) / total;
	
	//remap the percentage
	float col = (percentage - StartingOffset) * 1 / (Falloff - StartingOffset);
	col = saturate(col);
	return float4(col, col, col, 1);
}
