// Copyright Epic Games, Inc. All Rights Reserved.

//
// Mesh Vertex Shader utils functions and vertex input / output structs
//
#ifndef MESH_VERTEX_SHADER_USH
#define MESH_VERTEX_SHADER_USH

struct CoreMeshInput
{
	float4 pos : ATTRIBUTE0;
	float2 uv : ATTRIBUTE1;
	float3 normal : ATTRIBUTE2;
	float4 tangent : ATTRIBUTE3;
	float4 color : ATTRIBUTE4;
};

struct CoreMeshOutput
{
	float4 pos : SV_POSITION;
	float2 uv : TEXCOORD0;
	float3 normal : NORMAL0;
	float4 tangent : TANGENT0;
	float4 color : COLOR0;
};

CoreMeshOutput VSH_MeshUV(CoreMeshInput v)
{
	CoreMeshOutput o;
	o.uv = float2(v.uv.x, 1 - v.uv.y);
	//v.vertex.xyz = float3(((o.uv * 2.0 + -1.0) * float2(1.0, _ProjectionParams.r)), 0.0);
	o.pos = float4(((o.uv * 2.0 + -1.0) * float2(1.0, 1.0)), 0.0, v.pos.w);
	o.normal = v.normal;
	o.tangent = v.tangent;
	o.color = v.color;
	
	return o;
}

#endif

