// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"

// This shader was writtern using following reference
// https://shaderbits.com/blog/uv-dilation

float		InvSourceWidth;
float		InvSourceHeight;
float		Steps;
Texture2D	SourceTexture;

float4 DilateFunction(Texture2D sourceTex, float2 uv, float maxSteps)
{
	float2 texelSize = float2(InvSourceWidth, InvSourceHeight);
	float minDist = 10000000;
	float2 offsets[8] = { float2(-1, 0), float2(1, 0), float2(0, 1), float2(0, -1), float2(-1, 1), float2(1, 1), float2(1, -1), float2(-1, -1) };

	float4 sample = sourceTex.SampleLevel(SamplerStates_NoBorder, uv, 0);
	float4 curminsample = sample;

	if (sample.r == -2)
	{
		int i = 0;
		while (i < maxSteps)
		{
			i++;
			int j = 0;
			while (j < 8)
			{
				float2 curUV = uv + offsets[j] * texelSize * i;
				float4 offsetsample = sourceTex.SampleLevel(SamplerStates_NoBorder, curUV, 0);

				if (offsetsample.r >= -1)
				{
					float curdist = length(uv - curUV);

					if (curdist < minDist)
					{
						minDist = curdist;
						float2 projectUV = curUV + offsets[j] * texelSize * i;
						float4 direction = sourceTex.SampleLevel(SamplerStates_NoBorder, projectUV, 0);

						if (direction.r >= -1)
						{
							float4 delta = offsetsample - direction;
							curminsample = offsetsample + delta;

						}
						else
						{
							curminsample = offsetsample;
						}
					}
				}
				j++;
			}
		}
	}

	return curminsample;
}

float4 FSH_Main(in float2 uv : TEXCOORD0) : SV_Target0
{
	return DilateFunction(SourceTexture, uv, Steps);
}
