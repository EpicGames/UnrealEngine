// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "./MeshVertexShader.ush"

float4 FSH_WorldPos(CoreMeshOutput input) : SV_Target0
{
	return float4(input.color.rgb, 1);
}

float4 FSH_WorldNormals(CoreMeshOutput input) : SV_Target0
{
	return float4(normalize(input.normal.xyz) * 0.5 + 0.5, 0.0);
}

float4 FSH_WorldTangents(CoreMeshOutput input) : SV_Target0
{
	float4 tangent = input.tangent;
	return float4(normalize(tangent.xyz) * 0.5 + 0.5, tangent.w * 0.5 + 0.5);
}

float4 FSH_WorldUVMask(CoreMeshOutput input) : SV_Target0
{
	return float4(1, 0, 0, 1);
}
