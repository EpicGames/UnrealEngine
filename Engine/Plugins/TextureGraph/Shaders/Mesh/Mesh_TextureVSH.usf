// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "./MeshVertexShader.ush"

////////////////////////////////////////////////////////////////////////////
CoreMeshOutput VSH_MeshTexture(CoreMeshInput input)
{
	CoreMeshOutput output = VSH_MeshUV(input);
    return output;
}

////////////////////////////////////////////////////////////////////////////

float3 BoundsMin;
float3 InvBoundsDiameter;

CoreMeshOutput VSH_MeshTexture_WorldPos(CoreMeshInput input)
{
	CoreMeshOutput output = VSH_MeshUV(input);
	
	//float3 bmin = float3(-32, -32, 0);
	//float3 bdia = float3(1.0 / 64.0, 1.0 / 64.0, 1);

	output.color = float4((input.pos.xyz - BoundsMin.xyz) * InvBoundsDiameter.xyz, 1.0);
	return output;
}
