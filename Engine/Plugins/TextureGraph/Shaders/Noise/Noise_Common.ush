// Copyright Epic Games, Inc. All Rights Reserved.



static const float TWO_PI = 2.0 * 3.14159265358979323846f;
static const float TWO_PI_INV = 1.0f / TWO_PI;

/// Make a 4D coordinate from a 2d uv coordinate
float4 Make4DNoiseCoord(float2 uv)
{
	return float4(
		cos(uv.x * TWO_PI) * TWO_PI_INV,
		cos(uv.y * TWO_PI) * TWO_PI_INV,
		sin(uv.x * TWO_PI) * TWO_PI_INV,
		sin(uv.y * TWO_PI) * TWO_PI_INV
	);
}

/// Standard struct of properties to configure a FBM (fractal brownian motion) Noise sum
// regardless of the noise function used
struct FBMDesc
{
	float Seed;
	float Amplitude;
	float Frequency;
	int   Octaves;
	float Lacunarity;
	float Persistance;
};