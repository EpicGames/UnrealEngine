// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Plugin/TextureGraph/Noise/Noise_Common.ush"

#define FADE(t) ( t * t * t * ( t * ( t * 6 - 15 ) + 10 ) )

#define FASTFLOOR(x) ( ((x)>0) ? ((int)x) : ((int)x-1 ) )
#define LERP(t, a, b) ((a) + (t)*((b)-(a)))

static const uint PermutationTable[64] = {
1535746199, 226692954, 895508425, 3775392194, 510076044, 1661505093, 169209893, 2483469847,
1273657591, 1053104640, 3420191838, 537600885, 1478603065, 1463326189, 2289898670, 2940512427,
2252842314, 2786799755, 3885929037, 2061856595, 3867530044, 693922268, 687156791, 915367668,
2705267009, 1230034945, 3146009809, 2836552144, 2189935816, 1453309044, 3329057956, 1073986221,
4209170740, 3389356924, 2121700134, 3562361599, 3812347599, 289017903, 706526646, 3584735199,
43579511, 1185126956, 2607127005, 162278311, 4247197313, 1852596755, 3907023183, 1752218034,
3831625434, 3253871355, 210817774, 4053971903, 3952161617, 1810829049, 534167601, 2641020853,
2966181048, 758282611, 4271244415, 1573776522, 490959582, 2381531160, 1112458112, 3030138327 };

uint permF(uint x)
{
	int index = x & 255;
	int base4 = index / 4;
	int mod4 = index % 4;
	return (PermutationTable[base4] >> (mod4 * 8)) & 0xFF;
}

float grad4(int hash, float x, float y, float z, float t) {
	int h = hash & 31;      // Convert low 5 bits of hash code into 32 simple
	float u = h < 24 ? x : y; // gradient directions, and compute dot product.
	float v = h < 16 ? y : z;
	float w = h < 8 ? z : t;
	return ((h & 1) ? -u : u) + ((h & 2) ? -v : v) + ((h & 4) ? -w : w);
}

float Perlin4D(float4 coord)
{
	float x = coord.x;
	float y = coord.y;
	float z = coord.z;
	float w = coord.w;

	int ix0, iy0, iz0, iw0, ix1, iy1, iz1, iw1;
	float fx0, fy0, fz0, fw0, fx1, fy1, fz1, fw1;
	float s, t, r, q;
	float nxyz0, nxyz1, nxy0, nxy1, nx0, nx1, n0, n1;

	ix0 = FASTFLOOR(x); // Integer part of x
	iy0 = FASTFLOOR(y); // Integer part of y
	iz0 = FASTFLOOR(z); // Integer part of y
	iw0 = FASTFLOOR(w); // Integer part of w
	fx0 = x - ix0;        // Fractional part of x
	fy0 = y - iy0;        // Fractional part of y
	fz0 = z - iz0;        // Fractional part of z
	fw0 = w - iw0;        // Fractional part of w
	fx1 = fx0 - 1.0f;
	fy1 = fy0 - 1.0f;
	fz1 = fz0 - 1.0f;
	fw1 = fw0 - 1.0f;
	ix1 = (ix0 + 1) & 0xff;  // Wrap to 0..255
	iy1 = (iy0 + 1) & 0xff;
	iz1 = (iz0 + 1) & 0xff;
	iw1 = (iw0 + 1) & 0xff;
	ix0 = ix0 & 0xff;
	iy0 = iy0 & 0xff;
	iz0 = iz0 & 0xff;
	iw0 = iw0 & 0xff;

	q = FADE(fw0);
	r = FADE(fz0);
	t = FADE(fy0);
	s = FADE(fx0);

	nxyz0 = grad4(permF(ix0 + permF(iy0 + permF(iz0 + permF(iw0)))), fx0, fy0, fz0, fw0);
	nxyz1 = grad4(permF(ix0 + permF(iy0 + permF(iz0 + permF(iw1)))), fx0, fy0, fz0, fw1);
	nxy0 = LERP(q, nxyz0, nxyz1);

	nxyz0 = grad4(permF(ix0 + permF(iy0 + permF(iz1 + permF(iw0)))), fx0, fy0, fz1, fw0);
	nxyz1 = grad4(permF(ix0 + permF(iy0 + permF(iz1 + permF(iw1)))), fx0, fy0, fz1, fw1);
	nxy1 = LERP(q, nxyz0, nxyz1);

	nx0 = LERP(r, nxy0, nxy1);

	nxyz0 = grad4(permF(ix0 + permF(iy1 + permF(iz0 + permF(iw0)))), fx0, fy1, fz0, fw0);
	nxyz1 = grad4(permF(ix0 + permF(iy1 + permF(iz0 + permF(iw1)))), fx0, fy1, fz0, fw1);
	nxy0 = LERP(q, nxyz0, nxyz1);

	nxyz0 = grad4(permF(ix0 + permF(iy1 + permF(iz1 + permF(iw0)))), fx0, fy1, fz1, fw0);
	nxyz1 = grad4(permF(ix0 + permF(iy1 + permF(iz1 + permF(iw1)))), fx0, fy1, fz1, fw1);
	nxy1 = LERP(q, nxyz0, nxyz1);

	nx1 = LERP(r, nxy0, nxy1);

	n0 = LERP(t, nx0, nx1);

	nxyz0 = grad4(permF(ix1 + permF(iy0 + permF(iz0 + permF(iw0)))), fx1, fy0, fz0, fw0);
	nxyz1 = grad4(permF(ix1 + permF(iy0 + permF(iz0 + permF(iw1)))), fx1, fy0, fz0, fw1);
	nxy0 = LERP(q, nxyz0, nxyz1);

	nxyz0 = grad4(permF(ix1 + permF(iy0 + permF(iz1 + permF(iw0)))), fx1, fy0, fz1, fw0);
	nxyz1 = grad4(permF(ix1 + permF(iy0 + permF(iz1 + permF(iw1)))), fx1, fy0, fz1, fw1);
	nxy1 = LERP(q, nxyz0, nxyz1);

	nx0 = LERP(r, nxy0, nxy1);

	nxyz0 = grad4(permF(ix1 + permF(iy1 + permF(iz0 + permF(iw0)))), fx1, fy1, fz0, fw0);
	nxyz1 = grad4(permF(ix1 + permF(iy1 + permF(iz0 + permF(iw1)))), fx1, fy1, fz0, fw1);
	nxy0 = LERP(q, nxyz0, nxyz1);

	nxyz0 = grad4(permF(ix1 + permF(iy1 + permF(iz1 + permF(iw0)))), fx1, fy1, fz1, fw0);
	nxyz1 = grad4(permF(ix1 + permF(iy1 + permF(iz1 + permF(iw1)))), fx1, fy1, fz1, fw1);
	nxy1 = LERP(q, nxyz0, nxyz1);

	nx1 = LERP(r, nxy0, nxy1);

	n1 = LERP(t, nx0, nx1);

	return  (LERP(s, n0, n1));
}


float FBM_Perlin4D(float2 uv, FBMDesc fbm)
{
	float4 pos = Make4DNoiseCoord(uv);

	int octaves = fbm.Octaves;
	float frequency = fbm.Frequency * 2;
	float amplitude = fbm.Amplitude / 2;

	float noise = 0.0;
	for (int i = 0; i < octaves; ++i)
	{
		float4 coordOffset = fbm.Seed + i;
		float4 coord = coordOffset + pos * frequency;

		// noise += Noise_Perlin(coord) * amplitude;
		noise += Perlin4D(coord) * amplitude;

		frequency *= fbm.Lacunarity;
		amplitude *= fbm.Persistance;
	}

	return noise;
}