// Copyright Epic Games, Inc. All Rights Reserved.

// This simplex noise imlementation is adapted from
// http://weber.itn.liu.se/~stegu/simplexnoise/SimplexNoise.java
/*
 * A speed-improved simplex noise algorithm for 2D, 3D and 4D in Java.
 *
 * Based on example code by Stefan Gustavson (stegu@itn.liu.se).
 * Optimisations by Peter Eastman (peastman@drizzle.stanford.edu).
 * Better rank ordering method for 4D by Stefan Gustavson in 2012.
 *
 * This could be speeded up even further, but it's useful as it is.
 *
 * Version 2012-03-09
 *
 * This code was placed in the public domain by its original author,
 * Stefan Gustavson. You may use it as you see fit, but
 * attribution is appreciated.
 *
 */

#include "/Plugin/TextureGraph/Noise/Noise_Common.ush"

// ----------------------------------- 4D -------------------------------------

/*
static uint perm[256] = {
        151,160,137,91,90,15,
        131,13,201,95,96,53,194,233,7,225,140,36,103,30,69,142,8,99,37,240,21,10,23,
        190, 6,148,247,120,234,75,0,26,197,62,94,252,219,203,117,35,11,32,57,177,33,
        88,237,149,56,87,174,20,125,136,171,168, 68,175,74,165,71,134,139,48,27,166,
        77,146,158,231,83,111,229,122,60,211,133,230,220,105,92,41,55,46,245,40,244,
        102,143,54, 65,25,63,161, 1,216,80,73,209,76,132,187,208, 89,18,169,200,196,
        135,130,116,188,159,86,164,100,109,198,173,186, 3,64,52,217,226,250,124,123,
        5,202,38,147,118,126,255,82,85,212,207,206,59,227,47,16,58,17,182,189,28,42,
        223,183,170,213,119,248,152, 2,44,154,163, 70,221,153,101,155,167, 43,172,9,
        129,22,39,253, 19,98,108,110,79,113,224,232,178,185, 112,104,218,246,97,228,
        251,34,242,193,238,210,144,12,191,179,162,241, 81,51,145,235,249,14,239,107,
        49,192,214, 31,181,199,106,157,184, 84,204,176,115,121,50,45,127, 4,150,254,
        138,236,205,93,222,114,67,29,24,72,243,141,128,195,78,66,215,61,156,180};*/


const static uint PermutationTable[64] = {
1535746199, 226692954, 895508425, 3775392194, 510076044, 1661505093, 169209893, 2483469847,
1273657591, 1053104640, 3420191838, 537600885, 1478603065, 1463326189, 2289898670, 2940512427,
2252842314, 2786799755, 3885929037, 2061856595, 3867530044, 693922268, 687156791, 915367668,
2705267009, 1230034945, 3146009809, 2836552144, 2189935816, 1453309044, 3329057956, 1073986221,
4209170740, 3389356924, 2121700134, 3562361599, 3812347599, 289017903, 706526646, 3584735199,
43579511, 1185126956, 2607127005, 162278311, 4247197313, 1852596755, 3907023183, 1752218034,
3831625434, 3253871355, 210817774, 4053971903, 3952161617, 1810829049, 534167601, 2641020853,
2966181048, 758282611, 4271244415, 1573776522, 490959582, 2381531160, 1112458112, 3030138327 };

uint permF(uint x)
{
	int index = x & 255;
	int base4 = index / 4;
	int mod4 = index % 4;
	return (PermutationTable[base4] >> (mod4 * 8)) & 0xFF;
}


float dot(int g[4], float x, float y, float z, float w)
{
	return g[0] * x + g[1] * y + g[2] * z + g[3] * w;
}

float dotF(float4 g, float x, float y, float z, float w)
{
	return g.x * x + g.y * y + g.z * z + g.w * w;
}

static float4 grad4F2[32] =
{
	float4(0, 1, 1, 1),
	float4(0, 1, 1, -1),
	float4(0, 1, -1, 1),
	float4(0, 1, -1, -1),
	float4(0, -1, 1, 1),
	float4(0, -1, 1, -1),
	float4(0, -1, -1, 1),
	float4(0, -1, -1, -1),
	float4(1, 0, 1, 1),
	float4(1, 0, 1, -1),
	float4(1, 0, -1, 1),
	float4(1, 0, -1, -1),
	float4(-1, 0, 1, 1),
	float4(-1, 0, 1, -1),
	float4(-1, 0, -1, 1),
	float4(-1, 0, -1, -1),
	float4(1, 1, 0, 1),
	float4(1, 1, 0, -1),
	float4(1, -1, 0, 1),
	float4(1, -1, 0, -1),
	float4(-1, 1, 0, 1),
	float4(-1, 1, 0, -1),
	float4(-1, -1, 0, 1),
	float4(-1, -1, 0, -1),
	float4(1, 1, 1, 0),
	float4(1, 1, -1, 0),
	float4(1, -1, 1, 0),
	float4(1, -1, -1, 0),
	float4(-1, 1, 1, 0),
	float4(-1, 1, -1, 0),
	float4(-1, -1, 1, 0),
	float4(-1, -1, -1, 0)
};

float4 grad4F(int index)
{
	return grad4F2[index];
}

// The skewing and unskewing factors are hairy again for the 4D case
//static const float F4 = (sqrt(5.0f) - 1.0f) / 4.0f;
//static const float G4 = (5.0f - sqrt(5.0f)) / 20.0f;

float Simplex4D(float4 coord)
{
	float x = coord.x;
	float y = coord.y;
	float z = coord.z;
	float w = coord.w;


	float F4 = (sqrt(5.0f) - 1.0f) / 4.0f;
	float G4 = (5.0f - sqrt(5.0f)) / 20.0f;
	float n0, n1, n2, n3, n4; // Noise contributions from the five corners
							  // Skew the (x,y,z,w) space to determine which cell of 24 simplices we're in
	float s = (x + y + z + w) * F4; // Factor for 4D skewing
	int i = floor(x + s);
	int j = floor(y + s);
	int k = floor(z + s);
	int l = floor(w + s);
	float t = (i + j + k + l) * G4; // Factor for 4D unskewing
	float X0 = i - t; // Unskew the cell origin back to (x,y,z,w) space
	float Y0 = j - t;
	float Z0 = k - t;
	float W0 = l - t;
	float x0 = x - X0; // The x,y,z,w distances from the cell origin
	float y0 = y - Y0;
	float z0 = z - Z0;
	float w0 = w - W0;
	// For the 4D case, the simplex is a 4D shape I won't even try to describe.
    // To find out which of the 24 possible simplices we're in, we need to
    // determine the magnitude ordering of x0, y0, z0 and w0.
    // Six pair-wise comparisons are performed between each possible pair
    // of the four coordinates, and the results are used to rank the numbers.
	int rankx = 0;
	int ranky = 0;
	int rankz = 0;
	int rankw = 0;
	if (x0 > y0)
		rankx++;
	else
		ranky++;
	if (x0 > z0)
		rankx++;
	else
		rankz++;
	if (x0 > w0)
		rankx++;
	else
		rankw++;
	if (y0 > z0)
		ranky++;
	else
		rankz++;
	if (y0 > w0)
		ranky++;
	else
		rankw++;
	if (z0 > w0)
		rankz++;
	else
		rankw++;
	int i1, j1, k1, l1; // The integer offsets for the second simplex corner
	int i2, j2, k2, l2; // The integer offsets for the third simplex corner
	int i3, j3, k3, l3; // The integer offsets for the fourth simplex corner
    // [rankx, ranky, rankz, rankw] is a 4-vector with the numbers 0, 1, 2 and 3
    // in some order. We use a thresholding to set the coordinates in turn.
	// Rank 3 denotes the largest coordinate.
	i1 = rankx >= 3 ? 1 : 0;
	j1 = ranky >= 3 ? 1 : 0;
	k1 = rankz >= 3 ? 1 : 0;
	l1 = rankw >= 3 ? 1 : 0;
    // Rank 2 denotes the second largest coordinate.
	i2 = rankx >= 2 ? 1 : 0;
	j2 = ranky >= 2 ? 1 : 0;
	k2 = rankz >= 2 ? 1 : 0;
	l2 = rankw >= 2 ? 1 : 0;
    // Rank 1 denotes the second smallest coordinate.
	i3 = rankx >= 1 ? 1 : 0;
	j3 = ranky >= 1 ? 1 : 0;
	k3 = rankz >= 1 ? 1 : 0;
	l3 = rankw >= 1 ? 1 : 0;
    // The fifth corner has all coordinate offsets = 1, so no need to compute that.
	float x1 = x0 - i1 + G4; // Offsets for second corner in (x,y,z,w) coords
	float y1 = y0 - j1 + G4;
	float z1 = z0 - k1 + G4;
	float w1 = w0 - l1 + G4;
	float x2 = x0 - i2 + 2.0 * G4; // Offsets for third corner in (x,y,z,w) coords
	float y2 = y0 - j2 + 2.0 * G4;
	float z2 = z0 - k2 + 2.0 * G4;
	float w2 = w0 - l2 + 2.0 * G4;
	float x3 = x0 - i3 + 3.0 * G4; // Offsets for fourth corner in (x,y,z,w) coords
	float y3 = y0 - j3 + 3.0 * G4;
	float z3 = z0 - k3 + 3.0 * G4;
	float w3 = w0 - l3 + 3.0 * G4;
	float x4 = x0 - 1.0 + 4.0 * G4; // Offsets for last corner in (x,y,z,w) coords
	float y4 = y0 - 1.0 + 4.0 * G4;
	float z4 = z0 - 1.0 + 4.0 * G4;
	float w4 = w0 - 1.0 + 4.0 * G4;
    // Work out the hashed gradient indices of the five simplex corners
	int ii = i & 255;
	int jj = j & 255;
	int kk = k & 255;
	int ll = l & 255;

    /*
    int gi0 = perm[ii+perm[jj+perm[kk+perm[ll]]]] % 32;
    int gi1 = perm[ii+i1+perm[jj+j1+perm[kk+k1+perm[ll+l1]]]] % 32;
    int gi2 = perm[ii+i2+perm[jj+j2+perm[kk+k2+perm[ll+l2]]]] % 32;
    int gi3 = perm[ii+i3+perm[jj+j3+perm[kk+k3+perm[ll+l3]]]] % 32;
    int gi4 = perm[ii+1+perm[jj+1+perm[kk+1+perm[ll+1]]]] % 32;
    */

	int gi0 = permF(ii + permF(jj + permF(kk + permF(ll)))) % 32;
	int gi1 = permF(ii + i1 + permF(jj + j1 + permF(kk + k1 + permF(ll + l1)))) % 32;
	int gi2 = permF(ii + i2 + permF(jj + j2 + permF(kk + k2 + permF(ll + l2)))) % 32;
	int gi3 = permF(ii + i3 + permF(jj + j3 + permF(kk + k3 + permF(ll + l3)))) % 32;
	int gi4 = permF(ii + 1 + permF(jj + 1 + permF(kk + 1 + permF(ll + 1)))) % 32;

	float t0 = 0.5 - x0 * x0 - y0 * y0 - z0 * z0 - w0 * w0;
	if (t0 < 0)
		n0 = 0.0;
	else
	{
		t0 *= t0;
		n0 = t0 * t0 * dotF(grad4F(gi0), x0, y0, z0, w0);
	}

	float t1 = 0.5 - x1 * x1 - y1 * y1 - z1 * z1 - w1 * w1;
	if (t1 < 0)
		n1 = 0.0;
	else
	{
		t1 *= t1;
		n1 = t1 * t1 * dotF(grad4F(gi1), x1, y1, z1, w1);
	}

	float t2 = 0.5 - x2 * x2 - y2 * y2 - z2 * z2 - w2 * w2;
	if (t2 < 0)
		n2 = 0.0;
	else
	{
		t2 *= t2;
		n2 = t2 * t2 * dotF(grad4F(gi2), x2, y2, z2, w2);
	}

	float t3 = 0.5 - x3 * x3 - y3 * y3 - z3 * z3 - w3 * w3;
	if (t3 < 0)
		n3 = 0.0;
	else
	{
		t3 *= t3;
		n3 = t3 * t3 * dotF(grad4F(gi3), x3, y3, z3, w3);
	}

	float t4 = 0.5 - x4 * x4 - y4 * y4 - z4 * z4 - w4 * w4;
	if (t4 < 0)
		n4 = 0.0;
	else
	{
		t4 *= t4;
		n4 = t4 * t4 * dotF(grad4F(gi4), x4, y4, z4, w4);
	}

    // Sum up and scale the result to cover the range [-1,1]
	return 27.0 * (n0 + n1 + n2 + n3 + n4);


}


float FBM_Simplex4D(float2 uv, FBMDesc fbm)
{
	// Simplex noise swap x and y
	float4 pos = Make4DNoiseCoord(uv.yx);

	int octaves = fbm.Octaves;
	float frequency = fbm.Frequency;
	float amplitude = fbm.Amplitude;

	float noise = 0.0;
	for (int i = 0; i < octaves; ++i)
	{
		float4 coordOffset = fbm.Seed + i;
		float4 coord = coordOffset + pos * frequency;

		noise += Simplex4D(coord) * amplitude;

		frequency *= fbm.Lacunarity;
		amplitude *= fbm.Persistance;
	}

	return noise;
}

