// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Plugin/TextureGraph/Noise/Noise_Common.ush"

//1/7
#define K 0.142857142857
//3/7
#define Ko 0.428571428571


float4 mod(float4 x, float y) { float4 _y = y; return x - y * floor(x/_y); }
float3 mod(float3 x, float y) { float3 _y = y; return x - y * floor(x/_y); }

// Permutation polynomial: (34x^2 + x) mod 289
float3 Permutation(float3 x) 
{
  return mod((34.0 * x + float3(1.0, 1, 1)) * x, 289.0);
}


float2 Voronoise4D(float4 P, float jitter)
{			
	float4 Pi = mod(floor(P), 289.0);
 	float4 Pf = frac(P); 
	float3 oi = float3(-1.0, 0.0, 1.0); 
	float3 of = float3(-0.5, 0.5, 1.5); 
	float3 px = Permutation(Pi.xxx + oi); 
	float3 py = Permutation(Pi.yyy + oi); 
	float3 pz = Permutation(Pi.zzz + oi); 

	float3 p, ox, oy, oz, ow, dx, dy, dz, dw, d;
	float2 F = 1e6; // This will contain lowest distance so initialized with a very high value
	int i, j, k, n;

	for(i = 0; i < 3; i++)
	{
		for(j = 0; j < 3; j++)
		{
			for(k = 0; k < 3; k++)
			{
				p = Permutation(px[i] + py[j] + pz[k] + Pi.w + oi); // pijk1, pijk2, pijk3
	
				ox = frac(p*K) - Ko;
				oy = mod(floor(p*K),7.0 )*K - Ko;
				
				p = Permutation(p);
				
				oz = frac(p*K) - Ko;
				ow = mod(floor(p*K),7.0)*K - Ko;
			
				dx = Pf.x - of[i] + jitter*ox ;
				dy = Pf.y - of[j] + jitter*oy ;
				dz = Pf.z - of[k] + jitter*oz;
				dw = Pf.w - of + jitter*ow ;
				
				d = dx * dx + dy * dy + dz * dz + dw * dw; // dijk1, dijk2 and dijk3, squared
				//Find the lowest and second lowest distances
				for(n = 0; n < 3; n++)
				{
					if(d[n] < F[0])
					{
						F[1] = F[0];
						F[0] = d[n];
					}
					else if(d[n] < F[1])
					{
						F[1] = d[n];
					}
				}
			}
		}
	}
	
	return F;
}

float FrequencyRemap(float freq)
{
	float _Frequency = freq;
	float _TextureArea = 1.0;
	float freqRemap = lerp(1, 6.5 * (_TextureArea / 2), (_Frequency / (20 * _TextureArea)));
	return freq / freqRemap;
}

float FBM_Voronoise4D_F0_Squared(float2 uv, FBMDesc fbm)
{
	float4 pos = Make4DNoiseCoord(uv);

	int octaves = fbm.Octaves;
	float frequency = FrequencyRemap(fbm.Frequency);
	float amplitude = 1.0;

	float noise = 0.0;
	for (int i = 0; i < octaves; ++i)
	{
		float4 coordOffset = fbm.Seed + i;
		float4 coord = coordOffset + pos * frequency;
		coord *= frequency;

		float2 F = Voronoise4D(coord, 1.0);

		float n = F.x;
		n -= 0.37;
		n *= amplitude;
		noise += n;

		frequency *= fbm.Lacunarity * 0.7;
		amplitude *= fbm.Persistance;
	}

	noise *= 0.85;
	noise *= fbm.Amplitude;

	return noise;
}

float FBM_Voronoise4D_F1_Squared(float2 uv, FBMDesc fbm)
{
	float4 pos = Make4DNoiseCoord(uv);

	int octaves = fbm.Octaves;
	float frequency = FrequencyRemap(fbm.Frequency);
	float amplitude = 1.0;

	float noise = 0.0;
	for (int i = 0; i < octaves; ++i)
	{
		float4 coordOffset = fbm.Seed + i;
		float4 coord = coordOffset + pos * frequency;
		coord *= frequency;

		float2 F = Voronoise4D(coord, 1.0);
		float n = F.y;
		n -= 0.58;
		n *= amplitude;
		noise += n;

		frequency *= fbm.Lacunarity * 0.7;
		amplitude *= fbm.Persistance;
	}

	noise *= 0.75; // Ad hoc amplitude adjustement for Worley2s
	noise *= fbm.Amplitude;

	return noise;
}

float FBM_Voronoise4D_F1_F0_Squared(float2 uv, FBMDesc fbm)
{
	float4 pos = Make4DNoiseCoord(uv);

	int octaves = fbm.Octaves;
	float frequency = FrequencyRemap(fbm.Frequency);
	float amplitude = 1.0;

	float noise = 0.0;
	for (int i = 0; i < octaves; ++i)
	{
		float4 coordOffset = fbm.Seed + i;
		float4 coord = coordOffset + pos * frequency;
		coord *= frequency;

		float2 F = Voronoise4D(coord, 1.0);

		float n = F.y - F.x;
		n -= 0.2;
		n *= amplitude;
		noise += n;

		frequency *= fbm.Lacunarity * 0.7;
		amplitude *= fbm.Persistance;
	}

	noise *= 0.85; // Ad hoc amplitude adjustement for Worley3
	noise *= fbm.Amplitude;

	return noise;
}