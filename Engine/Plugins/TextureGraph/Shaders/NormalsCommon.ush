// Copyright Epic Games, Inc. All Rights Reserved.

// 
// Declare the Normal  helper functions api
//

#ifndef NORMALS_COMMON_USH
#define NORMALS_COMMON_USH

//Some extra functions for dealing with normal maps. Partial derivates are used a lot in Mixer since they correspond well with manipulating displacement maps by pushing things up or down.
//http://blog.selfshadow.com/publications/blending-in-detail/

float3 PartialDerivatesBlend(float3 n1, float3 n2)
{
	return normalize(float3(n1.xy*n2.z + n2.xy*n1.z, n1.z*n2.z));
}

float3 PartialDerivatesTransition(float3 a, float3 b, float t)
{
	float3 n1 = a*float3(1 - t, 1 - t, 1);
	float3 n2 = b*float3(t, t, 1);
	return normalize(float3(n1.xy*n2.z + n2.xy*n1.z, n1.z*n2.z));
}

float3 ReconstructNormalZ(float2 rg)
{
	float b =  sqrt(1-dot(rg, rg));
	return float3 (rg, b);
}

float3 ReconstructNormalZ(float3 rgb)
{
	return ReconstructNormalZ(rgb.xy);
}


float3 ConformNormals(float3 inputNormals)
{
	//Takes inputNormals in 0 to 1 range and outputs them in -1 to 1 range with clamp, normalization and blue reconstruction.
	//Useful for when using mips of normal maps with the partial derivative blending, or when reading normals with inverted blue channel texels or ranges that are off.
	float2 clampedNormalized = normalize(saturate(inputNormals)*2-1).xy;
	return ReconstructNormalZ(clampedNormalized);
}

float3 Conform(float3 A, float Renormalize)
{
	return lerp(A * 2 - 1, ConformNormals(A), Renormalize);
}

//if nothing is in red or green channel, return flat normal. Works on [0,1] space normals
float3 FillBlackPixels01Range(float3 nm)
{
	const float3 flat = float3(.5, .5, 1);
	nm = saturate(nm);
	float2 c = ceil(nm.rg);
	float mask = max(c.x, c.y);
	return lerp (flat, nm, mask);
}

#endif