// Copyright Epic Games, Inc. All Rights Reserved.

// 
// Declare the transform structs and api
//

#ifndef PAINT_USH
#define PAINT_USH

// API Paint: Functions painting an antialised mask value in the range [0,1] 
// FRAGMENT Shader only, using fwidth() for fragment footprint integration of signal

float ddLength(float a)
{
 // return length(float2(ddx(a), ddy(a)));
  return fwidth(a) * 0.5;
}

// Evaluate (Coord <= Edge)
float Paint_EdgeUnder(float Coord, float Edge)
{
	float width = ddLength(Coord);
	float low = Coord - width;
	float high = Coord + width;
	return saturate((Edge - Coord) / (high - low));
}
// Evaluate (Coord >= Edge)
float Paint_EdgeAbove(float Coord, float Edge)
{
	return 1.0 - Paint_EdgeUnder(Coord, Edge);
}

// Evaluate (Coord >= LeftEdge) * (Coord <= RightEdge)
float Paint_Interval(float Coord, float LeftEdge, float RightEdge)
{
	float width = ddLength(Coord);
	float low = Coord - width;
	float high = Coord + width;
	float invWidth = rcp(high - low);
	return saturate((Coord - LeftEdge) * invWidth) * saturate((RightEdge - Coord) * invWidth);
}

// Evaluate Coord in the Box [(Left, Bottom), (Right, Top)]
float Paint_Box(float2 Coord, float Left, float Bottom, float Right, float Top)
{
	float maskX = Paint_Interval(Coord.x, Left, Right);
	float maskY = Paint_Interval(Coord.y, Bottom, Top);
	return min(maskX, maskY);	
}
float Paint_Box(float2 Coord, float2 LeftBottom, float2 RightTop) { return Paint_Box(Coord, LeftBottom.x, LeftBottom.y, RightTop.x, RightTop.y); }
float Paint_Box(float2 Coord, float4 LeftBottomRightTop) { return Paint_Box(Coord, LeftBottomRightTop.x, LeftBottomRightTop.y, LeftBottomRightTop.z, LeftBottomRightTop.w); }


// Evaluate Coord in the Box [(Left, Bottom), (Right, Top)]
float Paint_BoxEdge(float2 Coord, float2 LeftBottom, float2 RightTop, float2 OutEdge, float2 InEdge)
{
	return (Paint_Box(Coord, LeftBottom - OutEdge, RightTop + OutEdge) * (1.0 -	Paint_Box(Coord, LeftBottom + InEdge, RightTop - InEdge)));
}

float Paint_Stripe(float value, float period, float stripe)
{
	float normalizedWidth = fwidth(value);
	normalizedWidth /= (period);
	float half_stripe_width = 0.5 * stripe;
	float offset = half_stripe_width;
	float stripe_over_period = stripe / period;
	float edge = stripe_over_period;
	float x0 = (value - offset) / (period) - normalizedWidth * 0.5;
	float x1 = x0 + normalizedWidth;
	float balance = 1.0 - edge;
	float i0 = edge * floor(x0) + max(0, frac(x0) - balance);
	float i1 = edge * floor(x1) + max(0, frac(x1) - balance);
	float strip = (i1 - i0) / normalizedWidth;
	return clamp(strip, 0.0, 1.0);
}

float Paint_Grid(float2 Coord, float period, float stripe)
{
	float2 normalizedWidth = fwidth(Coord);
	normalizedWidth /= (period);
	float half_stripe_width = 0.5 * stripe;
	float2 offset = half_stripe_width;
	float stripe_over_period = stripe / period;
	float2 edge = stripe_over_period;
	float2 x0 = (Coord - offset) / (period) - normalizedWidth * 0.5;
	float2 x1 = x0 + normalizedWidth;
	float2 balance = float2(1.0, 1.0) - edge;
	float2 i0 = edge * floor(x0) + max(float2(0.0, 0.0), frac(x0) - balance);
	float2 i1 = edge * floor(x1) + max(float2(0.0, 0.0), frac(x1) - balance);
	float2 strip =clamp( (i1 - i0) / normalizedWidth, 0.0, 1.0);
	return length(strip);
}
	
#endif // PAINT_USH