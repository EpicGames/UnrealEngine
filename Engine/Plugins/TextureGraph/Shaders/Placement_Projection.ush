// Copyright Epic Games, Inc. All Rights Reserved.

// 
// Declare the placement projection struct and api
//
#include "SamplerStates.ush"
#include "TiledFetch_Combined.ush"

#ifndef PLACEMENT_PROJECTION_USH
#define PLACEMENT_PROJECTION_USH

 
// Convert the mesh map position to world coordinate
float3 MeshBoxMin;
float3 MeshBoxSize;
float3 MeshViewScale;

Texture2D MeshmapPos;
Texture2D MeshmapNor;

float2x3 Fetch_MeshSurface(float2 layer_uv)
{
	float4 mesh_pos = MeshmapPos.SampleLevel(SamplerStates_NoBorder, layer_uv, 0);
	float4 mesh_nor = MeshmapNor.SampleLevel(SamplerStates_NoBorder, layer_uv, 0);

	// Center the mesh pos on the center of the Mesh bounding box
	// the projection origin is centered on the pos origin
	// Convert from cm to meter
	//mesh_pos.xyz = (mesh_pos.xyz * MeshBoxExtent + MeshBoxOffset) * MeshViewScale;
	mesh_pos.xyz = ((mesh_pos.xyz - 0.5) * MeshBoxSize) * MeshViewScale * 0.01;

	// Unpack normal
	mesh_nor.xyz = (mesh_nor.xyz * 2.0 - 1.0);
	
	return float2x3( mesh_pos.xyz, mesh_nor.xyz );
}

float3 Transform_Pos_RotateAroundZX(float2 Zcs, float2 Xcs, float3 pos)
{
	float3x3 rot = float3x3(
		float3(Zcs.x, Zcs.y, 0),
		float3(Xcs.x * -Zcs.y, Xcs.x * Zcs.x, Xcs.y),
		float3(-Xcs.y * -Zcs.y, -Xcs.y * Zcs.x, Xcs.x)
	);

	return mul(pos, rot);
}
float3 Transform_Dir_RotateAroundZX(float2 Zcs, float2 Xcs, float3 dir)
{
	float3x3 rot = float3x3(
		float3(Zcs.x, Zcs.y, 0),
		float3(Xcs.x * -Zcs.y, Xcs.x * Zcs.x, Xcs.y),
		float3(-Xcs.y * -Zcs.y, -Xcs.y * Zcs.x, Xcs.x)
	);

	return mul(dir, rot);
}

float3x2 EvalTriplanar_uvs(float3 mesh_pos, float3 mesh_nor, float2 uv_scaling)
{
	//offset for pushing projection planes sideways until centered
	float2 offsetPivot = float2(0.5, 0.5);

	float3 nsign = sign(mesh_nor.xyz);
	//UNITY version for ref:
	// float2 xuv = uv_scaling * (mesh_pos.zy + offsetPivot) * float2(nsign.x, 1.0);
	//float2 yuv = uv_scaling * (mesh_pos.xz + offsetPivot) * float2(nsign.y, 1.0);
	//float2 zuv = uv_scaling * (mesh_pos.xy + offsetPivot) * float2(-nsign.z, 1.0);
	float2 xuv = (uv_scaling * mesh_pos.yz + offsetPivot) * float2(-nsign.x, 1.0);
	float2 yuv = (uv_scaling * mesh_pos.xz + offsetPivot) * float2(nsign.y, 1.0);
	float2 zuv = (uv_scaling * mesh_pos.xy + offsetPivot) * float2(nsign.z, -1.0);
	
	return float3x2(xuv, yuv, zuv);
}

float3x4 Fetch_Source_Triplanar (Texture2DArray source, float3 mesh_pos, float3 mesh_nor, float2 uv_scaling)
{
	float3x2 triplanar_uvs = EvalTriplanar_uvs(mesh_pos, mesh_nor, uv_scaling);
	
	float4 sample_x = SampleLayerRaster(source, triplanar_uvs[0]);
	float4 sample_y = SampleLayerRaster(source, triplanar_uvs[1]);
	float4 sample_z = SampleLayerRaster(source, triplanar_uvs[2]);
	
	return float3x4(sample_x, sample_y, sample_z);
}

float3x4 Fetch_Source_Triplanar(Texture2DArray source, float3x2 triplanar_uvs)
{
	return float3x4(
		SampleLayerRaster(source, triplanar_uvs[0]),
		SampleLayerRaster(source, triplanar_uvs[1]),
		SampleLayerRaster(source, triplanar_uvs[2]) 
	);
}

// Convert height to mask value
float3 HeightMask(float3 triplanarHeightSamples, float midpoint, float strength)
{
	return saturate((triplanarHeightSamples - midpoint) * strength);
}

// Fetch height from displacement map at triplanar coordinate
float3 HeightMaskFromDisplacement(Texture2DArray displacement, float3x2 triplanar_uvs, float midpoint, float strength)
{
	if (strength < 0) /// BRANCH! do no go fetch anything if strength is 0
		return float3(0, 0 , 0);

	float3 triplanar_height;
	triplanar_height.x = SampleLayerRaster(displacement, triplanar_uvs[0]).x;
	triplanar_height.y = SampleLayerRaster(displacement, triplanar_uvs[1]).x;
	triplanar_height.z = SampleLayerRaster(displacement, triplanar_uvs[2]).x;

	return HeightMask(triplanar_height, midpoint, strength);
}

float EaseIn(float t)
{
	return t == 0.0 ? t : pow(2.0, 10.0 * (t - 1.0));
}

float3 ProjectionNormals(float3 normals, float sharpenEdges, float3 heightMask)
{
	//ease sharpen value for a more linear feel when dragging sliders
	float eased = EaseIn(sharpenEdges);

	//tighten projection borders, using multiple normalized pow to get sharp edges with no black corners on the planes
	float remappedSharpen = 1.0 + eased * 9; //remap to 1-10 range
	float3 maskedAbsNormals = max(0, (abs(normals) + heightMask));
	//float3 maskedAbsNormals = saturate(abs(normals)+heightMask );
	float3 pow1 = normalize(pow(maskedAbsNormals, remappedSharpen));
	float3 pow2 = normalize(pow(pow1, remappedSharpen));
	float3 pow3 = normalize(pow(pow2, remappedSharpen));
	pow3 /= (pow3.x + pow3.y + pow3.z); // + 0.00001;

	return saturate(pow3);
}


#endif