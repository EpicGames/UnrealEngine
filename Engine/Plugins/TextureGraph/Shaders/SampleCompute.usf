// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

#ifndef THREADGROUPSIZE_X
	#define THREADGROUPSIZE_X 1
	#define THREADGROUPSIZE_Y 1
	#define THREADGROUPSIZE_Z 1
#endif

RWTexture2D<uint4>	Result;
Texture2D<uint4>	Albedo;

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, THREADGROUPSIZE_Z)]
void CmpSH_Test(uint3 ThreadId : SV_DispatchThreadID)
{
	//uint4 clr = Tile[ThreadId.xy];
	//Result[ThreadId.xy] = float4(0, 1, 0, 1);
	#ifdef DO_THIS
		Result[ThreadId.xy] = uint4(255, 255, 0, 255);
	#endif
	
	#ifdef DO_THAT
		Result[ThreadId.xy] = uint4(0, 255, 255, 255);
	#endif
}
