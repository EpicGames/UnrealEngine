// Copyright Epic Games, Inc. All Rights Reserved.

// 
// Declare the standard SamplerStates
//

#ifndef SAMPLER_STATES_USH
#define SAMPLER_STATES_USH

SamplerState SamplerStates_Clamp;
SamplerState SamplerStates_Wrap;
SamplerState SamplerStates_NoBorder;
SamplerState SamplerStates_Linear_Clamp;
SamplerState SamplerStates_Linear_Wrap;
SamplerState SamplerStates_Linear_Border;

#endif