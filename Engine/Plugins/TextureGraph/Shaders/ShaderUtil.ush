// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
//#define PI 3.14159265358979323846

float SimpleCurve(float input, float curve)
{
	float absCurve = abs(curve);
	float stepCurve = step(0.0, curve);
	float combine = ((input * (1.0 - absCurve)) + (saturate(sin(acos((1.0 - input)))) * absCurve * stepCurve) + (saturate((1.0 - sin(acos(input)))) * absCurve * (1.0 - stepCurve)));
	return combine;
}

float Remap( float value, float from1, float to1, float from2, float to2) 
{
	return (value - from1) / (to1 - from1) * (to2 - from2) + from2;
}
float2 Remap(float2 value, float2 iMin, float2 iMax, float2 oMin, float2 oMax)
{
	return (oMin + ((value - iMin) * (oMax - oMin)) / (iMax - iMin));
}
float Posterize(float value, float steps)
{
	return floor(value * steps) / (steps - 1);

}
float SFNoise(float2 uv, float seed)
{
	uv = uv + seed;
	float2 skew = uv + 0.2127 + uv.x * 0.3713 * uv.y;
	float2 rnd = 4.789 * sin(489.123 * (skew));
	return frac(rnd.x * rnd.y * (1 + skew.x));
}

float3x3 GetTransform_TS_to_WS(float3x3 _ObjectToWorld, float3 worldNormal, float4 worldTangent)
{

	float3 normal = mul(_ObjectToWorld, worldNormal);
	float3 tangent = mul(_ObjectToWorld, worldTangent.xyz);
	float3 bitangent = normalize(cross(normal, tangent) * worldTangent.w);

	/// For orthogonal matrices, transpose = inverse
	float3x3 tangentToWorld = transpose(float3x3(tangent, normal, bitangent));

	return tangentToWorld;
}

float4 AdjustBrightnessAndContrast(float4 Color, float Brightness, float Contrast)
{
	float MidPoint = 0.5 + Brightness;

	// TODO: Support HDR values
	return saturate(lerp(MidPoint, Color + float4(Brightness, Brightness, Brightness, Brightness), float4(Contrast, Contrast, Contrast, Contrast)));	
}

float4 AdjustBrightnessAndContrast_NoAlpha(float4 Color, float Brightness, float Contrast)
{
	float4 Result = AdjustBrightnessAndContrast(Color, Brightness, Contrast);
	return float4(Result.rgb, Color.a);
}

float LumaFromRGB(float3 Color)
{
	// ITU-R BT.709 colour luminance gradients
	const float3 BrightnessFactor = float3(0.2126, 0.7152, 0.0722);
	return dot(Color, BrightnessFactor);
	//0.2126 * Color.r + 0.7152 * Color.g + 0.0722 * Color.b;
}


float Grayscale(float3 Color)
{
	// ITU-R BT.709 colour luminance gradients
	return LumaFromRGB(Color);
}

float3 RGBToHSV(float3 c)
{
	float4 K = float4(0.0, -1.0 / 3.0, 2.0 / 3.0, -1.0);
	float4 p = lerp( float4( c.bg, K.wz ), float4( c.gb, K.xy ), step( c.b, c.g ) );
	float4 q = lerp( float4( p.xyw, c.r ), float4( c.r, p.yzx ), step( p.x, c.r ) );
	float d = q.x - min( q.w, q.y );
	float e = 1.0e-10;
	return float3( abs(q.z + (q.w - q.y) / (6.0 * d + e)), d / (q.x + e), q.x);
}

float3 HSVToRGB( float3 c )
{
	float4 K = float4( 1.0, 2.0 / 3.0, 1.0 / 3.0, 3.0 );
	float3 p = abs( frac( c.xxx + K.xyz ) * 6.0 - K.www );
	return c.z * lerp( K.xxx, saturate( p - K.xxx ), c.y );
}
