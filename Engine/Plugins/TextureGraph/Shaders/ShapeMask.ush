// Copyright Epic Games, Inc. All Rights Reserved.

// 
// Declare the ShapeMask api kit
//

#include "SignedDistanceFunction.ush"
#include "ShaderUtil.ush"

#ifndef SHAPE_MASK_USH
#define SHAPE_MASK_USH

// Eval the beveled value to the input distance from the edge
// Clamp the result to the range [0,1]
float RemapDistanceBevelled(float InDistance, float InBevelWidth)
{
	return saturate(Remap(InDistance, 0, InBevelWidth, 0, 1));
}

// Eval the bevel curve to the (eventually bevelled) input distance from the edge
// Return the corresponding
float EvalBevelCurve(float input, float curve)
{
	float absCurve = abs(curve);
	float stepCurve = step(0.0, curve);
	return  (input * (1.0 - absCurve)) + absCurve * (stepCurve ? sqrt(input*(2-input)) : (1.0 - sqrt(1-input*input)));
}

float EvalMaskFromSDF(float SDF, float BevelWidth, float BevelCurve)
{
	float Mask = RemapDistanceBevelled(SDF, BevelWidth);
	return EvalBevelCurve(Mask, BevelCurve);
}

// Eval the shape mask for a particular Shape Type
// Functions signature are all following the similar pattern:
// float4 ShapeMask_XXX(float2 Pos, shape_specific_arguments, float Rounding = 0, float2 Bevel = 0)
//
// Pos is the coordinate in the 2d space where we evaluate the function
// The space is expected to be centered and the X and Y coordinates in the range [-1,1]
// in order to take advantage of the symetries of the coordinates in the shape.
// Thus the shape_specific_arguments dimensions specified for a shape are always the half dimensions.
// 
// Rounding is applied as a modifier and is expressed as a percentage of the shape dimension
//
// Bevel is applied as a modifier to compute the profil of the final mask value 
// 
// Return in a float4:
// In XY,  the sdf value,
// the mask value in the range [0,1],
// the rounding width
// and bevel width


// A circle centered on the origin of radius R
// Rounding is not used, it will be returned to the radius value
float4 ShapeMask_Circle(float2 Pos, float Radius, float Rounding = 0, float2 Bevel = 0)
{
	// Rounding and Bevel width depend on shape and actual size
	float RoundingWidth = Radius;
	float BevelWidth = Bevel.x * Radius;

	// Eval sdf, negative is outside, and adjust for rounding (not in the case of circle)
	float sdf = - SDF_Circle(Pos, Radius);
		
	return float4(EvalMaskFromSDF(sdf, BevelWidth, Bevel.y), sdf, RoundingWidth, BevelWidth);
}

// An Ellipse centered on the origin of radius Rx = Size.x, Ry = Size.y
// Rounding is not used, it will be returned to the radius value
float4 ShapeMask_Ellipse(float2 Pos, float2 Size, float Rounding = 0, float2 Bevel = 0)
{
	// Rounding and Bevel width depend on shape and actual size
	float RoundingWidth = Rounding * min(Size.x, Size.y);
	float BevelWidth = Bevel.x * max(Size.x, Size.y);

	// Eval sdf, negative is outside, and adjust for rounding (not in the case of circle)
	float sdf = - SDF_Ellipse(Pos, Size.x - RoundingWidth, Size.y - RoundingWidth);
		
	return float4(EvalMaskFromSDF(sdf, BevelWidth, Bevel.y), sdf, RoundingWidth, BevelWidth);
}

// A rectangle centered on the origin of width 2*Size.x and height 2*Size.y
float4 ShapeMask_Rect(float2 Pos, float2 Size, float Rounding = 0, float2 Bevel = 0)
{
	// Rounding and Bevel width depend on shape and actual size
	float RoundingWidth = Rounding * min(Size.x, Size.y);
	float BevelWidth = Bevel.x * max(Size.x, Size.y);

	// Eval sdf, negative is outside, and adjust for rounding
	float sdf = RoundingWidth - SDF_Rect(Pos, Size.x - RoundingWidth, Size.y - RoundingWidth);
		
	return float4(EvalMaskFromSDF(sdf, BevelWidth, Bevel.y), sdf, RoundingWidth, BevelWidth);
}	

float4 ShapeMask_Segment(float2 Pos, float SizeX, float Rounding = 0, float2 Bevel = 0)
{
	// Rounding and Bevel width depend on shape and actual size
	float RoundingWidth = Rounding;
	float BevelWidth = Bevel.x * RoundingWidth;
	
	// Eval sdf, negative is outside, and adjust for rounding
	float sdf = RoundingWidth - SDF_Segment(Pos, SizeX);
	
	return float4(EvalMaskFromSDF(sdf, BevelWidth, Bevel.y), sdf, RoundingWidth, BevelWidth);
}

float4 ShapeMask_Triangle(float2 Pos, float Radius, float Rounding = 0, float2 Bevel = 0)
{
	// Rounding and Bevel width depend on shape and actual size
	const float Cos60 = 0.5; // cos(PI/3)
	float RoundingWidth = Rounding * Radius * Cos60;
	float BevelWidth = Bevel.x * Radius * Cos60;
	
	// Eval sdf, negative is outside, and adjust for rounding
	float sdf = RoundingWidth - SDF_EquilateralTriangle(Pos, Radius * (1 - Rounding));
	
	return float4(EvalMaskFromSDF(sdf, BevelWidth, Bevel.y), sdf, RoundingWidth, BevelWidth);
}

float4 ShapeMask_Pentagon(float2 Pos, float Radius, float Rounding = 0, float2 Bevel = 0)
{
	// Rounding and Bevel width depend on shape and actual size
    const float Cos36 = 0.809016994; // cos(PI/5)
	float RoundingWidth = Rounding * Radius * Cos36;
	float BevelWidth = Bevel.x * Radius * Cos36;
	
	// Eval sdf, negative is outside, and adjust for rounding
	float sdf = RoundingWidth - SDF_Pentagon(Pos, Radius * (1 - Rounding));
	
	return float4(EvalMaskFromSDF(sdf, BevelWidth, Bevel.y), sdf, RoundingWidth, BevelWidth);
}

float4 ShapeMask_Hexagon(float2 Pos, float Radius, float Rounding = 0, float2 Bevel = 0)
{
	// Rounding and Bevel width depend on shape and actual size
    const float Cos30 = 0.5 * sqrt(3.0); // cos(PI/6)
	float RoundingWidth = Rounding * Radius * Cos30;
	float BevelWidth = Bevel.x * Radius * Cos30;
	
	// Eval sdf, negative is outside, and adjust for rounding
	float sdf = RoundingWidth - SDF_Hexagon(Pos, Radius * (1 - Rounding));
	
	return float4(EvalMaskFromSDF(sdf, BevelWidth, Bevel.y), sdf, RoundingWidth, BevelWidth);
}

float4 ShapeMask_Polygon(float2 Pos, float Radius, const int NumSides, float Rounding = 0, float2 Bevel = 0)
{
	// Rounding and Bevel width depend on shape and actual size
    const float CosN = cos(acos(-1.0) / float(NumSides));
	float RoundingWidth = Rounding * Radius * CosN;
	float BevelWidth = Bevel.x * Radius * CosN;
	
	// Eval sdf, negative is outside, and adjust for rounding
	float sdf = RoundingWidth - SDF_Polygon(Pos, Radius * (1 - Rounding), NumSides);
	
	return float4(EvalMaskFromSDF(sdf, BevelWidth, Bevel.y), sdf, RoundingWidth, BevelWidth);
}

#endif // SHAPE_MASK_USH