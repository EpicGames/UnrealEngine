// Copyright Epic Games, Inc. All Rights Reserved.

// 
// Declare the Signed Distance Function (SDF) api
// For a given shape, a SDF returns the signed distance to the edge of the shape
// The sign of the distance is:
//  - negative if inside the shape
//  - 0 at the edge
//  - positive if outside the shape
//
// Acknowledgment:
// Inigo Quilez built a huge corpus of reference implementation for signed distance functions and
// most of the SDF api here is inspired by his amazing work (https://iquilezles.org/articles/distfunctions2d/) 
//

#ifndef SDF_USH
#define SDF_USH

// 2D API 
// Left handed XY plane, X point right, Y point down
// Similar to the typical UV space of texcoord in a texture
//
// Functions are always of the form SDF_ShapeName(float2 P, ...)
// Where P is the position of query
// Most of the shapes are centered on the origin (0,0) to optimize for symetries
//

// SDF to the circle centered on origin of radius R
float SDF_Circle(float2 P, float Radius)
{
	return length(P) -  Radius;
}

// SDF to the ellipse centered on origin of radius R
// Technique taken from IQ's shadertoy "Ellipse - distance 2D II No Trig"
// Using a Newton solver over 8 passes
float SDF_Ellipse(float2 P, float RadiusX, float RadiusY)
{
	float2 AB = float2(RadiusX, RadiusY);

	// symmetry
    P = abs( P );
    
    // initial value
    float2 q = AB*(P-AB);
    float2 cs = normalize( (q.x<q.y) ? float2(0.01,1) : float2(1,0.01) );
    
    // find root with Newton solver
    for( int i=0; i<8; i++ )
    {
        float2 u = AB*float2( cs.x,cs.y);
        float2 v = AB*float2(-cs.y,cs.x);
        float a = dot(P-u,v);
        float c = dot(P-u,u) + dot(v,v);
        float b = sqrt(c*c-a*a);
        cs = float2( cs.x*b-cs.y*a, cs.y*b+cs.x*a )/c;
    }
    
    // compute final point and distance
    float d = length(P-AB*cs);
    
    // return signed distance
    return (dot(P/AB,P/AB)>1.0) ? d : -d;

}

// SDF to the rectangle centered on origin of size [2 * HalfSizeX, 2 * HalfSizeY]
float SDF_Rect(float2 P, float HalfSizeX, float HalfSizeY)
{
	P = abs(P);
	float2 D = P - float2(HalfSizeX, HalfSizeY);
	return length(max(D, 0.0)) + min(max(D.x, D.y), 0.0);
}


// SDF to the segment along X axis centered on origin of size (2 * HalfSize), the SDF is always >= 0
float SDF_Segment(float2 P, float HalfSize)
{
	float X = abs(P.x) - HalfSize;
	return length(float2((X < 0 ? 0 : X), P.y));
}

// SDF to the segment AB, the SDF is always >= 0
float SDF_Segment(float2 P, float2 A, float2 B)
{
	float2 AP = P - A, Tangent = B - A;
	float PTanProj = saturate(dot(AP, Tangent) / dot(Tangent, Tangent));
	return length(AP - Tangent * PTanProj);
}

// SDF to the oriented segment AB
// the function returns the pair (SqDistance, Region)
// - SqDistance is the square of SDF_Segment()
// - Region express the region of space relative to the segment AB:
//		From the segment tangent vector T, we define the normal vector N (-Ty, Tx).
//		The strip of space [A,B] contained between the 2 lines (A + tN) and (B + tN) which are
//		perpendicular to the segment AB passing by A and B.
//		+1 is the region of space in the strip [A,B] on the Right of the segment 
//		-1 is the region of space in the strip [A,B] on the Left of the segment 
//		0  is the points of the segment
//		0  is the half space defined by (A + tN) NOT containing the segment
//		0  is the half space defined by (B + tN) NOT containing the segment
//
float2 SDF_OrientedSegment_SquareRegion(float2 P, float2 A, float2 B)
{
	float2 AP = P - A, Tangent = B - A;
	float2 Normal = float2(-Tangent.y, Tangent.x);
	float PTanProj = (dot(AP, Tangent) / dot(Tangent, Tangent));
	float2 PNormalProj = AP - Tangent * saturate(PTanProj);

	float Region = ((dot(PNormalProj, Normal) >= 0) * 2 - 1.0) * (abs(PTanProj - 0.5) <= 0.5);

	return float2(dot(PNormalProj, PNormalProj), Region);
}

// SDF to the general triangle ABC
float SDF_Triangle(float2 P, float2 V0, float2 V1, float2 V2)
{
	float2 SD0 = SDF_OrientedSegment_SquareRegion(P, V1, V2);
	float closestSqDistance = SD0.x;
	float closestSign = SD0.y;
	
	float2 SD1 = SDF_OrientedSegment_SquareRegion(P, V2, V0);
	if (SD1.x <= closestSqDistance)
	{
		closestSqDistance = SD1.x;
		closestSign *= SD1.y;
	}
		
	float2 SD2 = SDF_OrientedSegment_SquareRegion(P, V0, V1);
	if (SD2.x <= closestSqDistance)
	{
		closestSqDistance = SD2.x;
		closestSign *= SD2.y;
	}
	
	float signedDistance = -sqrt(closestSqDistance) * (closestSign ? closestSign : -1.0);
	return signedDistance;
}

// SDF to the equilateral triangle centered on origin O, of radius R
// Pointing upward -Y axis
// Base is symetric along horizontal X axis
float SDF_EquilateralTriangle(float2 P, float Radius)
{
	// Symmetry angle (half sector angle) is 60deg
	const float Cos60 = 0.5;
	const float Sin60 = 0.5 * sqrt(3.0);

	// Y axial symmetry, fold space along Y axis
	P.x = abs(P.x);
	
	// Fold space along the C-V0 line
    P -= 2.0*min(dot(float2(-Cos60, Sin60), P), 0.0)*float2(-Cos60, Sin60);
	
	// Compress the sector 0 (between V0 and V2) on the Y axis
	// brings V0 and V2 to the origin
	// Now if P inside is on the line in neg Y or if P is outside it s in the pos Y half space 
	P -= float2(clamp( P.x, -Radius*Sin60, Radius*Sin60), Radius*Cos60);

	return length(P) * sign(P.y);
}

// SDF to the pentagon centered on origin C, of radius R
// First side is horizontal
// Pointing upward -Y axis
// Pentagon has 5 vertices Vn
// The center C to 2 consecutive vertices define a sector S 
// There are 5 sectors: (C-V0 C-V1) ... (C-V4 C-V0)
float SDF_Pentagon(float2 P, float Radius)
{
	// Symmetry angle (half sector angle) is 36deg
    const float Cos36 = 0.809016994;
	const float Sin36 = 0.587785252;
	
	// Y axial symmetry, fold space along Y axis
    P.x = abs(P.x);
	
	// Fold space along the C-V0 line
    P -= 2.0*min(dot(float2(-Cos36,Sin36), P), 0.0)*float2(-Cos36, Sin36);
	
	// Fold space along the C-V4 (n-1) line
    P -= 2.0*min(dot(float2( Cos36,Sin36), P), 0.0)*float2( Cos36, Sin36);
	
	// Compress the sector 0 (between V0 and V4) on the Y axis
	// brings V0 and V4 to the origin
	// Now if P inside is on the line in neg Y or if P is outside it s in the pos Y half space 
    P -= float2(clamp(P.x,  -Radius*Sin36, Radius*Sin36), Radius*Cos36);
	
    return length(P)*sign(P.y);
}

// SDF to the hexagon centered on origin C, of radius R
// First side is horizontal
// Hexagon has 6 vertices Vn
// The center C to 2 consecutive vertices define a sector S 
// There are 6 sectors: (C-V0 C-V1) ... (C-V5 C-V0)

float SDF_Hexagon(float2 P, float Radius)
{
	// Symmetry angle (half sector angle) is 30deg
    const float Cos30 = 0.5 * sqrt(3.0);
	const float Sin30 = 0.5;
    
	// Point symmetry, fold space along X and Y axes 
	P = abs(P);
	
	// Fold space once along the C-V0 line
    P -= 2.0*min(dot(float2(-Cos30, Sin30), P), 0.0)*float2(-Cos30, Sin30);
	
	// Compress the sector 0 (between V0 and V5) on the Y axis
	// brings V0 and V5 to the origin
	// Now if P inside is on the line in neg Y or if P is outside it s in the pos Y half space 	
    P -= float2(clamp(P.x, -Radius*Sin30, Radius*Sin30), Radius*Cos30);
	
    return length(P)*sign(P.y);
}



// SDF to the Regular Polygon with N sides
// Polygon is centered on origin C, of radius R
// First side is horizontal
// Polygon has N vertices Vn
// The center C to 2 consecutive vertices define a sector S 
// There are N sectors: (C-V0 C-V1) ... (C-Vn-1 C-V0)
float SDF_Polygon(float2 P, float Radius, const int NumSides)
{
	// Symmetry angle (half sector angle) is 30deg
    const float CosSectorHalf = cos(acos(-1.0) / float(NumSides));
	const float SinSectorHalf = sin(acos(-1.0) / float(NumSides));
	const bool IsNumSidesOdd = bool(NumSides % 2);
	const int NumFolds = ((NumSides - 1) >> 1);
    
	if (IsNumSidesOdd)
	{
		// Y axial symmetry, fold space along Y axis
		P.x = abs(P.x);
	}
	else
	{
		// Point symmetry, fold space along X and Y axes 
		P = abs(P);
	}
	
	for ( int i = 0; i < NumFolds; ++i)
	{	
		// Fold space along the C-V line
		// switch between C-V0 and C-Vn depending on loop oddity
		float2 CV_N = float2((2 * (i%2) - 1) * CosSectorHalf, SinSectorHalf);
		P -= 2.0*min(dot(CV_N, P), 0.0)*CV_N;
	}
	
	// Compress the sector 0 (between V0 and VN) on the Y axis
	// brings V0 and VN to the origin
	// Now if P inside is on the line in neg Y or if P is outside it s in the pos Y half space 	
    P -= float2(clamp(P.x, -Radius*SinSectorHalf, Radius*SinSectorHalf), Radius*CosSectorHalf);
	
    return length(P)*sign(P.y);
}

#endif // SDF_USH