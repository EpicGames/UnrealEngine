// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

#include "/Plugin/TextureGraph/SamplerStates.ush"

/// Simple vertex shader: Pass through pos and uv attributes
void VSH_Simple(float4 ipos : ATTRIBUTE0, float2 iuv : ATTRIBUTE1, out float2 ouv : TEXCOORD0, out float4 opos : SV_POSITION)
{
	opos = ipos;
	ouv = iuv;
}

Texture2D SourceTexture;

/// Simple fragment shader: Fetch texture at uv and blit out
float4 FSH_Simple(in float2 uv : TEXCOORD0) : SV_Target0
{
	return SourceTexture.Sample(SamplerStates_NoBorder, uv); ///tex2D(SourceTexture, uv);
}
