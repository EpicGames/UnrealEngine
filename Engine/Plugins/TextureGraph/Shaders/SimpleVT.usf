// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

#include "/Plugin/TextureGraph/SamplerStates.ush"
#define NUM_VIRTUALTEXTURE_SAMPLES 0
// Workaround for unbound View uniform buffer
#define VT_DISABLE_VIEW_UNIFORM_BUFFER 1

#include "/Engine/Private/VirtualTextureCommon.ush"

Texture2D<uint4> InPageTableTexture0;
Texture2D<uint4> InPageTableTexture1;
Texture2D InPhysicalTexture;

uint4 VTPackedPageTableUniform[2];
uint4 VTPackedUniform;

SamplerState InTextureSampler;

/// Simple fragment shader: Fetch virtual texture at uv and blit out
float4 FSH_SimpleVT(in float2 uv : TEXCOORD0,in float4 InPosition : SV_POSITION) : SV_Target0
{
	uint LayerIndex = 0;
	float4 Sample;
	VTPageTableResult PageTableResult = TextureLoadVirtualPageTable(InPageTableTexture0, InPageTableTexture1, VTPageTableUniform_Unpack(VTPackedPageTableUniform[0], VTPackedPageTableUniform[1]), uv, VTADDRESSMODE_WRAP, VTADDRESSMODE_WRAP, 0.0, InPosition.xy);
	Sample = TextureVirtualSample( InPhysicalTexture, InTextureSampler, PageTableResult, LayerIndex, VTUniform_Unpack(VTPackedUniform));

	return Sample;
}
