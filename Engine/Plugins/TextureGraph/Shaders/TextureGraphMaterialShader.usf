// Copyright Epic Games, Inc. All Rights Reserved.
#undef SCENE_TEXTURES_DISABLED
#define SCENE_TEXTURES_DISABLED 1
#define SUBSTRATE_INLINE_SHADING 1
#define SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL 1


#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Private/BasePassCommonFunctions.ush"

#define SHADER_MATERIAL_USES_SUBSTRATE	(SUBSTRATE_ENABLED && MATERIAL_IS_SUBSTRATE)
#if SHADER_MATERIAL_USES_SUBSTRATE
#define MATERIAL_SUBSTRATE_OPAQUE_PRECOMPUTED_LIGHTING 0
#define SUBSTRATE_MATERIAL_EXPORT_REQUESTED 1
#include "/Engine/Private/Substrate/SubstrateExport.ush"
#endif // SHADER_MATERIAL_USES_SUBSTRATE

float4 PSControl;

#include "TileInfo.ush"

void TextureGraphMaterialShaderVS(
	in float4 InPosition : ATTRIBUTE0,
	in float2 InTexCoord : ATTRIBUTE1,
	out float2 OutUV : TEXCOORD0,
	out float4 OutPosition : SV_POSITION
	)
{
	OutPosition = InPosition;
	OutUV = InTexCoord;
}

void TextureGraphMaterialShaderPS(
	in float2 InUV : TEXCOORD0,
	in float4 SvPosition : SV_Position, // after all interpolators
	out float4 OutColor : SV_Target0
	)
{
	// Transform the uv injected coming from [0,1] domain to target the region of the current tile beeing rendered
	InUV = (InUV + float2(TileInfo_TileX, TileInfo_TileY)) / float2(TileInfo_TileCountX, TileInfo_TileCountY);

	ResolvedView = ResolveView();
	FMaterialPixelParameters MaterialParameters = MakeInitializedMaterialPixelParameters();

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	InitializeVirtualTextureFeedback(MaterialParameters.VirtualTextureFeedback);
#endif

	float2 ViewportUV = InUV;

#if NUM_MATERIAL_TEXCOORDS
	for(int CoordinateIndex = 0;CoordinateIndex < NUM_MATERIAL_TEXCOORDS;CoordinateIndex++)
	{
		MaterialParameters.TexCoords[0] = ViewportUV;
	}
#endif // NUM_MATERIAL_TEXCOORDS

	SvPosition.z = LookupDeviceZ(ViewportUVToBufferUV(ViewportUV));
	SvPosition.z = max(SvPosition.z, 1e-18);

	// fill out other related material parameters
	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParametersPost(MaterialParameters, PixelMaterialInputs, SvPosition, ViewportUV, true);
	
	// Extract the value we want to represent
	half3 MatBaseColor = GetMaterialBaseColor(PixelMaterialInputs);
	half3 MatMetallic = GetMaterialMetallic(PixelMaterialInputs);
	half3 MatSpecular = GetMaterialSpecular(PixelMaterialInputs);
	half3 MatRoughness = GetMaterialRoughness(PixelMaterialInputs);
	half3 MatAnisotropy = GetMaterialAnisotropy(PixelMaterialInputs);
	half3 MatEmissive = GetMaterialEmissive(PixelMaterialInputs);
	half3 MatOpacity = GetMaterialOpacity(PixelMaterialInputs);
	
	half3 MatOpacityMask = float3(0,0,0);
#if MATERIALBLENDING_MASKED
	MatOpacityMask = GetMaterialOpacityMaskClipValue();
#endif // MATERIALBLENDING_TRANSLUCENT

	half3 MatNormal = float3(0,0,0);
	half3 MatTangent = float3(0,0,0);
#if MATERIAL_TANGENTSPACENORMAL
	MatNormal = GetMaterialNormal(MaterialParameters, PixelMaterialInputs) * 0.5 + 0.5;
	MatTangent = GetMaterialTangent(PixelMaterialInputs) * 0.5 + 0.5;
#endif // MATERIAL_TANGENTSPACENORMAL

#if SHADER_MATERIAL_USES_SUBSTRATE
	// Initialise a Substrate header with normal in registers
	FSubstrateData SubstrateData = PixelMaterialInputs.GetFrontSubstrateData();
	FSubstratePixelHeader SubstratePixelHeader = MaterialParameters.GetFrontSubstrateHeader();
	FSubstrateIntegrationSettings Settings = InitSubstrateIntegrationSettings();

	const float3 SurfaceWorldNormal = MaterialParameters.TangentToWorld[2].xyz;
	const float3 V = MaterialParameters.CameraVector;
	FExportResult Export = SubstrateMaterialExportOut(
		Settings,
		SubstratePixelHeader,
		SubstrateData,
		V,
		SurfaceWorldNormal,
		MaterialParameters.WorldPosition_CamRelative,
		0.0f // Curvature
	);

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	FinalizeVirtualTextureFeedback(
		MaterialParameters.VirtualTextureFeedback,
		MaterialParameters.SvPosition,
		View.VTFeedbackBuffer
	);
#endif
	
	MatBaseColor = Export.BaseColor;
	MatMetallic = Export.Metallic;
	MatSpecular = Export.Specular;
	MatRoughness = Export.Roughness;
	MatEmissive = Export.EmissiveLuminance;
	MatNormal = Export.WorldNormal * 0.5 + 0.5;
	MatOpacity = Export.Opacity;

#if MATERIAL_USES_ANISOTROPY
	MatTangent = Export.WorldTangent * 0.5 + 0.5;
	MatAnisotropy = Export.Anisotropy;
#endif
#endif // SHADER_MATERIAL_USES_SUBSTRATE

	half3 UV = half3(ViewportUV, 0);
	half3 Tile = half3(float2(TileInfo_TileX, TileInfo_TileY) / float2(TileInfo_TileCountX, TileInfo_TileCountY), 0);

	float Attribute = floor(clamp(PSControl.x, 0, PSControl.y));

	half3 Color = (Attribute <= 0.0) * MatBaseColor;
	Color += (Attribute == 1.0) * MatMetallic;
	Color += (Attribute == 2.0) * MatSpecular;
	Color += (Attribute == 3.0) * MatRoughness;
	Color += (Attribute == 4.0) * MatAnisotropy;
	Color += (Attribute == 5.0) * MatEmissive;
	Color += (Attribute == 6.0) * MatOpacity;
	Color += (Attribute == 7.0) * MatOpacityMask;
	Color += (Attribute == 8.0) * MatNormal;
	Color += (Attribute >= 9.0) * MatTangent;
	
	Color = lerp(Color, UV, PSControl.z);
	Color = lerp(Color, Tile, PSControl.w);

	OutColor = float4(Color, 1.0);
}
