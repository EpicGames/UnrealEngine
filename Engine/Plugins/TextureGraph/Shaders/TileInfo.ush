// Copyright Epic Games, Inc. All Rights Reserved.

// 
// Declare the tile info struct and api
//

#ifndef TILE_INFO_USH
#define TILE_INFO_USH

// 
// TileInfo uniform parameters and api are provided in a shader by including this header "TileInfo.ush"
//
// In the Shader c++ class just declare the parameter:
// ...
// SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
// ...
//
// The value of TileInfo is updated automatically for each invocations of a Job / Transform
// by using a JobArg_TileInfo argument. 
//
//
// When executing a shader invocation of a Transform for a particular Tile,
// the api provides the details to:
// - know which tile is being processed
// - know the dimension of the grid of tiles in the layer (see schema below)
// - convert a coordinate between the "Tile space" or the "Layer space"
// 
//               Layer texcoord X
//               0 --------+---------+---------+-------- 1 --> 
//                                                                
//            TX :    0    :    1    :         :  CX-1   :  CX
//          TY                                                    
//     L 0 - - - +---------+---------+ - - - - +---------+        
//     a |       |         |         |         |         |        
//     y |   0   |   0,0   |    1,0  |  ...,0  | CX-1,0  |        
//     e |       |         |         |         |         |        
//     r + - - - +---------+---------+ - - - - +---------+        
//       |       |         |         |         |         |        
//     t |   1   |   0,1   |    1,1  |  ...,1  | CX-1,1  |        
//     e |       |         |         |         |         |        
//     x + - - - +---------+---------+ - - - - +---------+        
//     c |       :         :         :         :         :        
//     o |       :  0,...  :   1,... :   ...   : CX-1,...:        
//     o |       :         :         :         :         :        
//     r + - - - +---------+---------+ - - - - +---------+        
//     d |       |         |         |         |         |        
//       |  CY-1 |  0,CY-1 |  1,CY-1 | ...,CY-1|CX-1,CY-1|        
//     Y |       |         |         |         |         |        
//       1 - - - +---------+---------+ - - - - +---------+        
//       |                                                        
//       |   CY                                                   
//       V
//           
//       TileX       <=> TX
//       TileY       <=> TY
//       TileCountX  <=> CX
//       TileCountY  <=> CY
//  
//  A tile resolution in pixels is:       TileWidth * TileHeight
//  The layer resolution in pixels is:    (TileCountX * TileWidth) * (TileCountY * TileHeight)
// 
//           


struct TileInfo
{
    float TileX;
    float TotalCountX;
    float TileWidth;
    float TileY;
    float TotalCountY;
    float TileHeight;
};
float TileInfo_TileX;
float TileInfo_TileCountX;
float TileInfo_TileWidth;
float TileInfo_TileY;
float TileInfo_TileCountY;
float TileInfo_TileHeight;

    
/// Access the full tile grid dimension (numCols, numRows) (xy)
float2 TileInfo_tileGridDim()
{
	return float2(TileInfo_TileCountX, TileInfo_TileCountY);
}
/// Access the total resolution of the layer as a single image
float2 TileInfo_layerResolution()
{
	return float2(TileInfo_TileCountX * TileInfo_TileWidth, TileInfo_TileCountY * TileInfo_TileHeight);
}

/// Access the current job invocation tile position (xy) within the full tile grid
float2 TileInfo_tilePos()
{
	return float2(TileInfo_TileX, TileInfo_TileY);
}

/// Convert from the CURRENT tile coordinate to the full layer space uv coordinate
float2 TileInfo_fromCurrentTileToLayer(float2 tileCoord)
{
    float2 tileRatio = float2(1.0 / TileInfo_TileCountX, 1.0 / TileInfo_TileCountY);
    float2 tileOffset = float2(TileInfo_TileX, TileInfo_TileY) * tileRatio;
    return tileOffset + tileCoord * tileRatio;
}

/// TODO find a better name
int2 TileInfo_fromLayerToTile(int2 texelCoord)
{
	int2 limit = int2(TileInfo_TileWidth, TileInfo_TileHeight);
	return (texelCoord % limit + limit) % limit;
}

/// Convert from the layer space coordinate uv to the tile space uv
/// return the normalized uv coord in the tile (in xy) and the tile coordinate in the grid (zw)
float4 TileInfo_fromLayerToTile(float2 layerCoord) 
{
	float2 tileRatio = 1.0 / TileInfo_tileGridDim(); // tile ratio is 1/gridDim so inverse of ratio is gridDIm
	float2 tileRatioInv = TileInfo_tileGridDim(); // tile ratio is 1/gridDim so inverse of ratio is gridDIm
	float4 tileCoord = float4(layerCoord * tileRatioInv, 0, 0);
	tileCoord.zw = floor(tileCoord.xy);
	tileCoord.xy -= tileCoord.zw;
	
	return tileCoord;
}

/// Given texel in layer, find the LINEAR index of the tile (used in a TextureArray) that contains the blob for that texel
int TileInfo_tileIndexFromLayerTexel(int2 texel)
{
	int texelX = texel.x + (TileInfo_TileX * TileInfo_TileWidth);
	int texelY = texel.y + (TileInfo_TileY * TileInfo_TileHeight);

	int indexX = texelX / TileInfo_TileWidth;
	int indexY = texelY / TileInfo_TileHeight;

	int index = (indexX * TileInfo_TileCountX) + indexY;
	return index;
}



#endif
