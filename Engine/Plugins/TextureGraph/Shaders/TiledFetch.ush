// Copyright Epic Games, Inc. All Rights Reserved.

//
// Shader header file providing helper functions to fetch texels in a tiled raster
//
#ifndef TILED_FETCH_USH
#define TILED_FETCH_USH

#include "SamplerStates.ush"

// Declare an array of textures which are the 9 tiles defining...
// a center 'main' tile, and its 8 possible neighbor tiles:
//
// Texcoord.x   +--- 0 ----- 1/0 ---- 1/0 ----- 1 -->
// Texcoord.y   |    :        :        :        :
//              |    :        :        :        :         + TileY
//              0 -- +--------+--------+--------+         |
//                   |        |        |        |          
//              |    |   NW   |  North |   NE   |        -1
//                   |        |        |        |          
//             1/0 - +--------+--------+--------+         |
//                   |        |        |        |          
//              |    |  West  |  Main  |  East  |         0
//                   |        |        |        |          
//             1/0 - +--------+--------+--------+         |
//                   |        |        |        |          
//              |    |   SW   |  South |   SE   |        +1
//                   |        |        |        |         
//              1 -- +--------+--------+--------+         |
//              |                                         V TileY
//              V
//                   --- -1 ------ 0 ----- +1 -->  TileX
//
//  Initialized from cpu with an array of textures in the following layout:
//  Texture2D tiles[9] = { NW, W, SW, N, Main, S, NE, E, SE }
//


#define FetchLinear(tex, _uv) \
	tex.Sample(SamplerStates_Linear_Clamp, _uv)

#define Fetch(tex, _uv) \
	tex.Sample(SamplerStates_Clamp, _uv)

#define Tile(T, I) T##_##I

#define Declare_TiledArray(T) \
Texture2D	Tile(T, 0); \
Texture2D	Tile(T, 1); \
Texture2D	Tile(T, 2); \
Texture2D	Tile(T, 3); \
Texture2D	Tile(T, 4); \
Texture2D	Tile(T, 5); \
Texture2D	Tile(T, 6); \
Texture2D	Tile(T, 7); \
Texture2D	Tile(T, 8);


#define Declare_FetchTile(T) \
float4 FetchTile_##T(float2 tile, float2 uv) \
{ \
	int tileIdx = (tile.x + 1) * 3 + (tile.y + 1); \
	\
	if (tileIdx == 1) return  Fetch(Tile(T, 1), uv);  \
	if (tileIdx == 7) return  Fetch(Tile(T, 7), uv); ; \
	if (tileIdx == 3) return  Fetch(Tile(T, 3), uv);  \
	if (tileIdx == 5) return  Fetch(Tile(T, 5), uv);  \
	\
	if (tileIdx == 0) return  Fetch(Tile(T, 0), uv);  \
	if (tileIdx == 2) return  Fetch(Tile(T, 2), uv);  \
	if (tileIdx == 6) return  Fetch(Tile(T, 6), uv);  \
	if (tileIdx == 8) return  Fetch(Tile(T, 8), uv);  \
	\
	return  Fetch(Tile(T, 4), uv); \
}
				

/* This should be the version we use, but we cannot include the comments in the macro...
#define Declare_FetchTiled(T) \
float4 FetchTiled_##T(float2 uv) \
{ \
	// Evaluate the tiles and quad of samples required for this fetch \
	float2 tileSize; \
	Tile(T,4).GetDimensions(tileSize.x, tileSize.y); \
	float2 invTileSize = 1.0 / tileSize; \
	float2 tileQuad = 0; \
	float tileQuadCase = EvalTileQuadAt(uv, invTileSize, tileQuad); \
	\
	float2 texelPos = uv * tileSize; \
	float2 texelPointPos = floor(texelPos); \
	float2 texelPointUV = (texelPointPos + 0.5) * invTileSize; // transform the texel int pos to the actual uv coord \
	\
	// if only need the main tile, then we are done, just return the fetched value \
	if (tileQuadCase == 0) \
		return FetchLinear(Tile(T,4), uv); \
	\
	// else, need to gather texels from different tiles and filter \
	\
	// First the texel at the point uv in main tile \
	float4 texel = Fetch(Tile(T,4), texelPointUV); \
	\
	// we ll need the position of the fetch in the quad of the 4 texels to do proper filtering \
	float2 quadPos = (texelPos - texelPointPos) - 0.5; \
	\
	// Quad pos is signed properly in the uv space here so we can compute the uv offset for 3 other samples \
	float2 uvOffsetXY = sign(quadPos) * invTileSize; \
	float2 uvOffsetX = float2(uvOffsetXY.x, 0); \
	float2 uvOffsetY = float2(0, uvOffsetXY.y); \
	\
	float2 tileX = float2(tileQuad.x, 0); \
	float2 tileY = float2(0, tileQuad.y); \
	float2 tileXY = tileQuad; \
	\
	float4 texelX; \
	float4 texelY; \
	float4 texelXY; \
	\
	float2 texelXuv = float2(1.0 - texelPointUV.x, texelPointUV.y); \
	float2 texelYuv = float2(texelPointUV.x, 1.0 - texelPointUV.y); \
	\
	// Fetch 2th texel in X corner \
	if (tileQuad.x != 0) \
		texelX = FetchTile_##T(tileX, texelXuv); \
	else \
		texelX = Fetch(Tile(T,4), texelPointUV + uvOffsetX); \
		\
	// Fetch 3th texel in Y corner \
	if (tileQuad.y != 0) \
		texelY = FetchTile_##T(tileY, texelYuv); \
	else \
		texelY = Fetch(Tile(T,4), texelPointUV + uvOffsetY); \
		\
	// Fetch 4th texel in XY corner \
	if (tileQuadCase == 1) \
		texelXY = FetchTile_##T(tileX, texelXuv + uvOffsetY); \
	else if (tileQuadCase == 2) \
		texelXY = FetchTile_##T(tileY, texelYuv + uvOffsetX); \
	else \
		texelXY = FetchTile_##T(tileQuad, float2(1.0 - texelPointUV.x, 1.0 - texelPointUV.y));	\
		\
	// now remove the sign in quadPos and use it as the weighting of the filter \
	quadPos = abs(quadPos); \
	\
	// 4 texels bilinear filter \
	return lerp(lerp(texel, texelX, quadPos.x), lerp(texelY, texelXY, quadPos.x), quadPos.y); \
}*/


// Evaluate the tiles and quad of samples required for this fetch
// if only need the main tile, then we are done, just return the fetched value
// else, need to gather texels from different tiles and filter
// First the texel at the point uv in main tile
// we ll need the position of the fetch in the quad of the 4 texels to do proper filtering
// Quad pos is signed properly in the uv space here so we can compute the uv offset for 3 other samples
// Fetch 2th texel in X corner
// Fetch 3th texel in Y corner
// Fetch 4th texel in XY corner
// now remove the sign in quadPos and use it as the weighting of the filter
// 4 texels bilinear filter
#define Declare_FetchTiled(T) \
float4 FetchTiled_##T(float2 uv) \
{ \
	float2 tileSize; \
	Tile(T,4).GetDimensions(tileSize.x, tileSize.y); \
	float2 invTileSize = 1.0 / tileSize; \
	float2 tileQuad = 0; \
	float tileQuadCase = EvalTileQuadAt(uv, invTileSize, tileQuad); \
	\
	float2 texelPos = uv * tileSize; \
	float2 texelPointPos = floor(texelPos); \
	float2 texelPointUV = (texelPointPos + 0.5) * invTileSize; \
	\
	if (tileQuadCase == 0) \
		return FetchLinear(Tile(T,4), uv); \
	\
	float4 texel = Fetch(Tile(T,4), texelPointUV); \
	\
	float2 quadPos = (texelPos - texelPointPos) - 0.5; \
	\
	float2 uvOffsetXY = sign(quadPos) * invTileSize; \
	float2 uvOffsetX = float2(uvOffsetXY.x, 0); \
	float2 uvOffsetY = float2(0, uvOffsetXY.y); \
	\
	float2 tileX = float2(tileQuad.x, 0); \
	float2 tileY = float2(0, tileQuad.y); \
	float2 tileXY = tileQuad; \
	\
	float4 texelX; \
	float4 texelY; \
	float4 texelXY; \
	\
	float2 texelXuv = float2(1.0 - texelPointUV.x, texelPointUV.y); \
	float2 texelYuv = float2(texelPointUV.x, 1.0 - texelPointUV.y); \
	\
	if (tileQuad.x != 0) \
		texelX = FetchTile_##T(tileX, texelXuv); \
	else \
		texelX = Fetch(Tile(T,4), texelPointUV + uvOffsetX); \
		\
	if (tileQuad.y != 0) \
		texelY = FetchTile_##T(tileY, texelYuv); \
	else \
		texelY = Fetch(Tile(T,4), texelPointUV + uvOffsetY); \
		\
	if (tileQuadCase == 1) \
		texelXY = FetchTile_##T(tileX, texelXuv + uvOffsetY); \
	else if (tileQuadCase == 2) \
		texelXY = FetchTile_##T(tileY, texelYuv + uvOffsetX); \
	else \
		texelXY = FetchTile_##T(tileQuad, float2(1.0 - texelPointUV.x, 1.0 - texelPointUV.y));	\
		\
	quadPos = abs(quadPos); \
	\
	return lerp(lerp(texel, texelX, quadPos.x), lerp(texelY, texelXY, quadPos.x), quadPos.y); \
}


// EvalTileQuadAt()
// Evaluate the tile quad fetching situation at the specified 'texcoord' uv in the range [0,1]
// in a tile containg 'size' texels, the function uses the inverse of size 'invTileSize'
// the function compute the tileQuad which indicate the neighbor tiles touched by the fetching quad along X and Y.
// 
// The function returns a CODE indicating the case:
// 0 <=> tileQuad == (   0,    0) : the quad is fully in the main tile, no need for other tiles 
// 1 <=> tileQuad == (+/-1,    0) : the quad is crossing a West or East border into another tile (but no North/South)
// 2 <=> tileQuad == (   0, +/-1) : the quad is crossing a North or South border into another tile (but not West/East)
// 3 <=> tileQuad == (+/-1, +/-1) : the quad is crossing 2 borders in a corner, 4 tiles touched
//
float EvalTileQuadAt(float2 texcoord, float2 invTileSize, in out float2 tileQuad)
{
	float tileQuadCase = 1; // initiate with 1 if the x axis catch left or right, cancel if x is in the main tile
	tileQuad = 0;
	float2 halfTexelUV = 0.5 * invTileSize;

	// detect the X axis cases
	if (texcoord.x < halfTexelUV.x) 
		tileQuad.x = -1;
	else if (texcoord.x > (1.0 - halfTexelUV.x))
		tileQuad.x = +1;
	else
		tileQuadCase = 0; // cancel the first neighbor tile if x is in the main tile

	// detect the Y axis cases
	if (texcoord.y < halfTexelUV.y)
	{
		tileQuad.y = -1;
		tileQuadCase += 2;
	}
	else if (texcoord.y > (1.0 - halfTexelUV.y))
	{
		tileQuad.y = +1;
		tileQuadCase += 2;
	}

	return tileQuadCase;
}

#define Declare_Tiles_And_FetchTiled(T) \
Declare_TiledArray(T) \
Declare_FetchTile(T) \
Declare_FetchTiled(T)


#endif