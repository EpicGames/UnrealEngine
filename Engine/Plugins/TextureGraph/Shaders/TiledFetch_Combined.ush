// Copyright Epic Games, Inc. All Rights Reserved.

// 
// Declare the tiled fetch api for combined blob raster
//

#ifndef TILED_FETCH_COMBINED_USH
#define TILED_FETCH_COMBINED_USH

#include "SamplerStates.ush"
#include "TileInfo.ush"

void ToPivotInBlob(inout float2 pivot)
{
    float tileWidth = TileInfo_TileWidth;
    float tileHeight = TileInfo_TileHeight;
    float startTileX =	TileInfo_TileX * tileWidth;
    float startTileY = TileInfo_TileY * tileHeight;
    float middleX = (TileInfo_TileCountX * tileWidth) * pivot.x;
    float middleY = (TileInfo_TileCountY * tileHeight) * pivot.y;
    float pivotx = ((middleX - startTileX) / tileWidth);
    float pivoty = ((middleY - startTileY) / tileHeight);
    float2 scaleFactor = float2(1.0f / TileInfo_TileCountX, 1.0f /TileInfo_TileCountY);

    pivot = float2(pivotx, pivoty);
}

float2 WrapUV(float2 uv, float2 repeatAfter)
{
	return (uv%repeatAfter + repeatAfter)%repeatAfter;
}

float2 ClampUV(float2 uv, float2 min, float2 max)
{
    return clamp(uv, min, max);
}

bool inRange(int low, int high, int x)
{
    return ((x-high)*(x-low) <= 0);
}


float4 SampleBlob(in Texture2DArray CombinedBlobSRV, in float2 layer_uv)
{
	float4 blob;
	float rows = TileInfo_TileCountX;
	float cols = TileInfo_TileCountY;
	float totalTiles = rows * cols;

	float tileX = TileInfo_TileX;
	float tileY = TileInfo_TileY;
  
	int index = 0;
	float2 rowCol = float2(rows, cols);
	for (int x = 0; x < rows; x++)
	{
		for (int y = 0; y < cols; y++)
		{

			float2 tileUV = layer_uv - float2(x, y) + float2(tileX, tileY);

#if CLAMP
            tileUV = ClampUV(tileUV, float2(-1 * x, -1 * y), float2(rows - x, cols - y));
            tileUV = WrapUV(tileUV, float2(rows,cols));
#elif WRAP
            tileUV = WrapUV(tileUV, float2(rows,cols)); //do not move this after condition; there can be cases with NO repeat modes
#endif

			blob += CombinedBlobSRV.Sample(SamplerStates_NoBorder, float3(tileUV, index++));

		}
	}

	return blob;
}

// Wrap the uv in the [0, 1] range
float2 wrap_uv(float2 uv)
{
	return uv - floor(uv);
}

float4 SampleLayerRaster(in Texture2DArray CombinedBlobSRV, in float2 layer_uv)
{
	// Wrap the uv in the [0, 1] range
	layer_uv = wrap_uv(layer_uv);
	
	// find the tile and inner uv at layer_uv
	float4 tile_uv_id = TileInfo_fromLayerToTile(layer_uv);

	// knowing the tile id pick the correct slice 
	float tile_index = (tile_uv_id.z * TileInfo_TileCountX) + tile_uv_id.w;
	
	// fetch!
	return  CombinedBlobSRV.Sample(SamplerStates_NoBorder, float3(tile_uv_id.xy, tile_index));
}

#endif
