// Copyright Epic Games, Inc. All Rights Reserved.

// 
// Declare the transform structs and api
//

#ifndef TRANSFORM_USH
#define TRANSFORM_USH

float2 Transform_Rotate(float2 InCoord, float2 Rotation)
{
	return float2(dot(InCoord, Rotation), dot(InCoord, float2(-Rotation.y, Rotation.x)));
}

float2 Transform_PivotRotate(float2 InCoord, float2 Pivot, float2 Rotation)
{
	InCoord -= Pivot;
	InCoord = Transform_Rotate(InCoord, Rotation);
	InCoord += Pivot;
	
	return InCoord;
}

#endif // TRANSFORM_USH