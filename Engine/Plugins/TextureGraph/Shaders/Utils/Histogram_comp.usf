// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/ShaderUtil.ush"

#ifndef MAX_flt
#define MAX_flt			(3.402823466e+38F)
#endif

Texture2D SourceTiles;

RWStructuredBuffer<uint4> TilesHistogramUAV;
StructuredBuffer<uint4> TilesHistogramSRV;

RWTexture2D<float4> Result;

int4 InvocationDim;

// Shared mem Histogram is 4x256, one element per bin.channel
#define HistogramSize 256
groupshared uint4 _SharedHistogram[HistogramSize];

#ifndef THREADGROUPSIZE_X
#define THREADGROUPSIZE_X 16
#define THREADGROUPSIZE_Y 16
#define THREADGROUPSIZE_Z 1
#endif

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, THREADGROUPSIZE_Z)]
void CSH_HistogramPerTile(
	uint3 GroupId : SV_GroupID,
	uint3 ThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint GroupIndex: SV_GroupIndex
	)
{
	// Grab one pixel of the source
	float4 Fetched = SourceTiles.Load(uint3(ThreadId.xy, 0));
	
	// Evaluate Luma for 4th component and find the bin index for each channel
	float Luma = Grayscale(Fetched.rgb);
	// The considered value is in the range [0, 1];
	float4 Value = saturate(float4(Fetched.rgb, Luma));
	// From the float value to the matching histogram bin [0,255]
	float4 BinValue = floor(Value * 256.0);
	uint4 Bin = clamp(uint4(BinValue), 0, 255);
	
	// Offset where to write the histogram for this tile in the destination buffer,
	// using the region allocated for this tile
	int TileOffset = InvocationDim.y * 256; 
	uint4 offsetbins = Bin + TileOffset;
	
	InterlockedAdd(TilesHistogramUAV[offsetbins.x].x, 1);
	InterlockedAdd(TilesHistogramUAV[offsetbins.y].y, 1);
	InterlockedAdd(TilesHistogramUAV[offsetbins.z].z, 1);
	InterlockedAdd(TilesHistogramUAV[offsetbins.w].w, 1);
}


[numthreads(256, 1, 1)]
void CSH_Histogram(
	uint3 GroupId : SV_GroupID,
	uint3 ThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint GroupIndex: SV_GroupIndex
	)
{
	uint ThreadIndex = ThreadId.x;
	
	// Accumulate all the histogram bin values for all the tiles in THIS thread bin
	uint4 HistogramAccum = 0;
	for (uint i = 0; i < InvocationDim.x; ++i)
	{
		uint TileHistogramBinOffset = (i * 256 + ThreadIndex);
		HistogramAccum += TilesHistogramSRV.Load(TileHistogramBinOffset);
	}
	
	// And store in the histogram texture and also in shared meme histo
	Result[uint2(ThreadIndex,0)] = float4(HistogramAccum);
	_SharedHistogram[ThreadIndex] = HistogramAccum;
	
	
	GroupMemoryBarrierWithGroupSync();

	// Now only in first group, let's go through the shared mem histogram bins SEQUENTIALLY and extract the min and max count and min and max values
	if (ThreadId.x == 0)
	{
		float4 MinCount = float4(MAX_flt,MAX_flt,MAX_flt,MAX_flt);
		float4 MaxCount = float4(0,0,0,0);
		
		float4 MinBin = float4(MAX_flt,MAX_flt,MAX_flt,MAX_flt);
		float4 MaxBin = float4(0, 0, 0, 0);
			
		float4 TotalCount = float4(0,0,0,0);
		
		LOOP for(uint y = 0; y < 256 ; y++)
		{		
			
			float4 HistogramBin = float4(_SharedHistogram[y]);
	
			MinCount = min(HistogramBin, MinCount);
			MaxCount = max(HistogramBin, MaxCount);
			TotalCount += HistogramBin;
			
			if (any(HistogramBin))
			{
				// Assigned min bin for the first HistogramBin found, never after
				MinBin.x = (MinBin.x != MAX_flt ? MinBin.x : (HistogramBin.x == 0 ? MAX_flt : y) );
				MinBin.y = (MinBin.y != MAX_flt ? MinBin.y : (HistogramBin.y == 0 ? MAX_flt : y) );
				MinBin.z = (MinBin.z != MAX_flt ? MinBin.z : (HistogramBin.z == 0 ? MAX_flt : y) );
				MinBin.w = (MinBin.w != MAX_flt ? MinBin.w : (HistogramBin.w == 0 ? MAX_flt : y) );
				
				// Assigned max bin, always the current Y of the loop if the Histogram is not 0.
				MaxBin.x = (HistogramBin.x == 0 ? MaxBin.x : y + 1); 
				MaxBin.y = (HistogramBin.y == 0 ? MaxBin.y : y + 1); 
				MaxBin.z = (HistogramBin.z == 0 ? MaxBin.z : y + 1); 
				MaxBin.w = (HistogramBin.w == 0 ? MaxBin.w : y + 1); 
			}			
		}
		
		MaxBin = max(MaxBin, MinBin); // Make sure Max was assigned and coherent with min
		
		// Normalize the min / max bin index
		MinBin /= 256.0;
		MaxBin /= 256.0;
			
		Result[uint2(0,1)] = MinCount;//2nd row first pixel is the min count in a bin (usually 0)
		Result[uint2(1,1)] = MaxCount;//2nd row second pixel is the max count in a bin
		Result[uint2(2,1)] = MinBin;//2nd row third pixel is the min bin index normalized 
		Result[uint2(3,1)] = MaxBin;//2nd row fourth pixel is the max bin index noramlized
		Result[uint2(4,1)] = TotalCount;//2nd row fifth pixel is the total count
	}

}