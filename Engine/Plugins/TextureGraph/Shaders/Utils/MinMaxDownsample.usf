// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/SamplerStates.ush"
#include "/Plugin/TextureGraph/ShaderUtil.ush"

// Permutations
#ifndef CHECK_UVMASK
#define CHECK_UVMASK 0
#endif

#ifndef FIRST_PASS
#define FIRST_PASS 0
#endif

Texture2D						SourceTexture;
Texture2D						WorldUVMask;

float							DX; 
float							DY; 
			
float2 tex2Dlod_UVMask(Texture2D tex, Texture2D uvMask, float2 uv)
{
#if CHECK_UVMASK
                const float inf = 3.402823466e+38F;
                float4 s = Texture2DSample(tex, SamplerStates_Linear_Clamp, uv);
                float mask = Texture2DSample(uvMask, SamplerStates_Linear_Clamp, uv).r;

                /// If the mask is anything other than 0, then we return the pixel value as the min
                /// and the max. If it is then our (min, max) = (+inf, -inf) so that they fail every
                /// subsequent min and max test
                return mask > 0 ? s.rr : float2(+inf, -inf);
#else
                /// Both min and max are the same
	return Texture2DSampleLevel(tex, SamplerStates_NoBorder, uv, 0).rr;
#endif
}

float2 CalcMinMaxWithUVMask(Texture2D tex, float2 uv, float2 pixelSize)
{
	float2 s1 = tex2Dlod_UVMask(tex, WorldUVMask, uv + float2(-1.0, 1.0) * pixelSize).r;
	float2 s2 = tex2Dlod_UVMask(tex, WorldUVMask, uv + float2(1.0, 1.0) * pixelSize).r;
	float2 s3 = tex2Dlod_UVMask(tex, WorldUVMask, uv + float2(-1.0, -1.0) * pixelSize).r;
	float2 s4 = tex2Dlod_UVMask(tex, WorldUVMask, uv + float2(1.0, -1.0) * pixelSize).r;

	float r = min(min(s1.r, s2.r), min(s3.r, s4.r));
	float g = max(max(s1.g, s2.g), max(s3.g, s4.g));

	return float2(r, g);
}

float2 CalcMinMax(Texture2D tex, float2 uv, float2 pixelSize)
{
	float2 s1 = Texture2DSampleLevel(tex, SamplerStates_NoBorder, uv + float2(-1.0, 1.0) * pixelSize, 0).rg;
	float2 s2 = Texture2DSampleLevel(tex, SamplerStates_NoBorder, uv + float2(1.0, 1.0) * pixelSize, 0).rg;
	float2 s3 = Texture2DSampleLevel(tex, SamplerStates_NoBorder, uv + float2(-1.0, -1.0) * pixelSize, 0).rg;
	float2 s4 = Texture2DSampleLevel(tex, SamplerStates_NoBorder, uv + float2(1.0, -1.0) * pixelSize, 0).rg;

	float r = min(min(s1.r, s2.r), min(s3.r, s4.r));
	float g = max(max(s1.g, s2.g), max(s3.g, s4.g));

	return float2(r, g);
}

float4 FSH_MinMaxDownsampleFirstPass(float2 uv : TEXCOORD0) : SV_Target0
{
	return float4(CalcMinMaxWithUVMask(SourceTexture, uv, float2(DX, DY)), 0.0, 0.0);
}

float4 FSH_MinMaxDownsampleSecondPass(float2 uv : TEXCOORD0) : SV_Target0
{
	return float4(CalcMinMax(SourceTexture, uv, float2(DX, DY)), 0.0, 0.0);
}

float4 FSH_MinMaxDownsample(float2 uv : TEXCOORD0) : SV_Target0
{
	#if FIRST_PASS	
	return FSH_MinMaxDownsampleFirstPass(uv);
	#else
	return FSH_MinMaxDownsampleSecondPass(uv);
	#endif
}
