// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

// Permutations
#ifndef SOURCE_TO_MIN_MAX_PASS
#define SOURCE_TO_MIN_MAX_PASS 0
#endif

int4							TilingDimensions; /// The width and height of each dst tiles (xy) and the number of tiles in the source (zw)

Texture2D						SourceTiles_0;
Texture2D						SourceTiles_1;
Texture2D						SourceTiles_2;
Texture2D						SourceTiles_3;

RWTexture2D<float2>				Result;


#define Fetch4(tex)	float4( \
	tex.Load(uint3(tile_pos, 0)).x, \
	tex.Load(uint3(tile_pos.x + 1, tile_pos.y, 0)).x, \
	tex.Load(uint3(tile_pos.x, tile_pos.y + 1, 0)).x, \
	tex.Load(uint3(tile_pos.x + 1, tile_pos.y + 1, 0)).x)

float4 fetch4At(int2 tile_pos)
{
	uint2 halfTileSize = TilingDimensions.xy >> 1;
	uint2 tile = min(tile_pos / halfTileSize, TilingDimensions.zw - 1);

	tile_pos -= tile * halfTileSize;

	tile_pos *= 2;

	int tileIdx = tile.x * 2 + tile.y;
	if (tileIdx == 1) return  Fetch4(SourceTiles_1);
	if (tileIdx == 2) return  Fetch4(SourceTiles_2);
	if (tileIdx == 3) return  Fetch4(SourceTiles_3);

	return Fetch4(SourceTiles_0);
}

#define Fetch4XY(tex)	float4x2( \
	tex.Load(uint3(tile_pos, 0)).xy, \
	tex.Load(uint3(tile_pos.x + 1, tile_pos.y, 0)).xy, \
	tex.Load(uint3(tile_pos.x, tile_pos.y + 1, 0)).xy, \
	tex.Load(uint3(tile_pos.x + 1, tile_pos.y + 1, 0)).xy )

float4x2 fetch4XYAt(int2 tile_pos)
{
	uint2 halfTileSize = TilingDimensions.xy >> 1;
	uint2 tile = min(tile_pos / halfTileSize, TilingDimensions.zw - 1);

	tile_pos -= tile * halfTileSize;

	tile_pos *= 2;

	int tileIdx = tile.x * 2 + tile.y;
	if (tileIdx == 1) return  Fetch4XY(SourceTiles_1);
	if (tileIdx == 2) return  Fetch4XY(SourceTiles_2);
	if (tileIdx == 3) return  Fetch4XY(SourceTiles_3);

	return Fetch4XY(SourceTiles_0);
}


#ifndef THREADGROUPSIZE_X
#define THREADGROUPSIZE_X 4
#define THREADGROUPSIZE_Y 4
#define THREADGROUPSIZE_Z 1
#endif

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, THREADGROUPSIZE_Z)]
void CSH_MinMaxDownsample(uint3 ThreadId : SV_DispatchThreadID)
{
#if SOURCE_TO_MIN_MAX_PASS
	float4 values = fetch4At(ThreadId.xy);

	float vmin = min(min(values.x, values.y), min(values.z, values.w));
	float vmax = max(max(values.x, values.y), max(values.z, values.w));
#else
	float4x2 values = fetch4XYAt(ThreadId.xy);

	float vmin = min(min(values[0].x, values[1].x), min(values[2].x, values[3].x));
	float vmax = max(max(values[0].y, values[1].y), max(values[2].y, values[3].y));
#endif

	Result[ThreadId.xy] = float2(vmin, vmax);
}
