// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"

Texture2D						SourceTiles;

RWTexture2D<float4>				Result;


#define Fetch4XYZW(tex)	float4x4( \
	tex.Load(uint3(tile_pos, 0)), \
	tex.Load(uint3(tile_pos.x + 1, tile_pos.y, 0)), \
	tex.Load(uint3(tile_pos.x, tile_pos.y + 1, 0)), \
	tex.Load(uint3(tile_pos.x + 1, tile_pos.y + 1, 0)) )

float4x4 fetch4XYZWAt(int2 tile_pos)
{
	tile_pos *= 2;
	return Fetch4XYZW(SourceTiles);
}


#ifndef THREADGROUPSIZE_X
#define THREADGROUPSIZE_X 4
#define THREADGROUPSIZE_Y 4
#define THREADGROUPSIZE_Z 1
#endif

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, THREADGROUPSIZE_Z)]
void CSH_MipMapDownsample(uint3 ThreadId : SV_DispatchThreadID)
{
	// Fetch the 4 texels from source texture
	float4x4 values = fetch4XYZWAt(ThreadId.xy);

	// Apply the mip map filter
	float4 r = values[0];
	r += values[1];
	r += values[2];
	r += values[3];
	r *= 0.25;

	// Output the downsampled texel
	Result[ThreadId.xy] = r;
}
