// Copyright Epic Games, Inc. All Rights Reserved.
#include "/Engine/Public/Platform.ush"
#include "/Plugin/TextureGraph/TileInfo.ush"

Texture2D						SourceTexture;

RWTexture2D<float4>				Result;

#ifndef THREADGROUPSIZE_X
#define THREADGROUPSIZE_X 4
#define THREADGROUPSIZE_Y 4
#define THREADGROUPSIZE_Z 1
#endif

[numthreads(THREADGROUPSIZE_X, THREADGROUPSIZE_Y, THREADGROUPSIZE_Z)]
void CSH_SplitToTiles(uint3 ThreadId : SV_DispatchThreadID)
{
	uint2 pos = ThreadId.xy;

	uint2 fullResPos = uint2(pos.x + (TileInfo_TileWidth * TileInfo_TileX), 
							   pos.y + (TileInfo_TileHeight * TileInfo_TileY));

	float4 r = SourceTexture.Load(uint3(fullResPos, 0));

	Result[ThreadId.xy] = r;
}
