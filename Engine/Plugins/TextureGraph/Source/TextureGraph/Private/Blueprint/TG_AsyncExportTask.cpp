// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/TG_AsyncExportTask.h"
#include "Misc/Paths.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "TG_Graph.h"
#include "UObject/Package.h"
#include "Model/Mix/MixSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_AsyncExportTask)

UTG_AsyncExportTask::UTG_AsyncExportTask(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UTG_AsyncExportTask* UTG_AsyncExportTask::TG_AsyncExportTask( UTextureGraphBase* InTextureGraph, const bool OverwriteTextures, const bool bSave, const bool bExportAll, const bool bDisableCache)
{
	UTG_AsyncExportTask* Task = NewObject<UTG_AsyncExportTask>();
	Task->SetFlags(RF_Standalone);
	Task->bOverwriteTextures = OverwriteTextures;
	Task->bSave = bSave;
	Task->bExportAll = bExportAll;

	if (InTextureGraph != nullptr)
	{
		Task->bBlobberCachingState = TextureGraphEngine::GetBlobber()->IsCacheEnabled();
		if (bDisableCache)
			TextureGraphEngine::GetBlobber()->SetEnableCache(false);

		Task->OriginalTextureGraphPtr = InTextureGraph;
		Task->TextureGraphPtr = Cast<UTextureGraphBase>(StaticDuplicateObject(Task->OriginalTextureGraphPtr, GetTransientPackage(), NAME_None, RF_Standalone));
		Task->TextureGraphPtr->Initialize();
		FTG_HelperFunctions::InitTargets(Task->TextureGraphPtr);
		Task->RegisterWithTGAsyncTaskManger();
	}
	
	return Task;
}

const TArray<UTextureRenderTarget2D*>& UTG_AsyncExportTask::ActivateBlocking(JobBatchPtr Batch)
{
	if (!IsValid(TextureGraphPtr))
	{
		UE_LOGF(LogTextureGraph, Warning, "TG_AsyncExportTask::Cannot export Texture Graph not selected");
		return OutputRts;
	}
	TargetExportSettings = FExportSettings();
	TargetExportSettings.OnDone.BindUFunction(this, "OnExportDone");

	if (!Batch)
		Batch = FTG_HelperFunctions::InitExportBatch(TextureGraphPtr, "", "", TargetExportSettings, false, bOverwriteTextures, bExportAll, bSave);
	
	Super::ActivateBlocking(Batch);

	bool bExportDone = false;
	TextureExporter::ExportAsUAsset(TextureGraphPtr, MakeShared<FExportSettings>(TargetExportSettings), "").then([&bExportDone]()
		{
			bExportDone = true;
		});

	while (!bExportDone)
	{
		TextureGraphEngine::GetInstance()->Update(0);
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
	}

	return OutputRts;
}

void UTG_AsyncExportTask::Activate()
{
	// Start the async task on a new thread
	Super::Activate();
	UE_LOGF(LogTextureGraph, Log, "TG_AsyncExportTask:: Activate");

	if (!IsValid(TextureGraphPtr))
	{
		UE_LOGF(LogTextureGraph, Warning, "TG_AsyncExportTask::Cannot export Texture Graph not selected");
		return;
	}

	TargetExportSettings = FExportSettings();
	TargetExportSettings.OnDone.BindUFunction(this, "OnExportDone");

	FTG_HelperFunctions::ExportAsync(TextureGraphPtr, "", "", TargetExportSettings, false, bOverwriteTextures, bExportAll, bSave);
}

void UTG_AsyncExportTask::OnExportDone()
{
	TargetExportSettings.ExportPreset.clear();
	TargetExportSettings.OnDone.Unbind();

	if (OnDone.IsBound())
		OnDone.Broadcast();

	TextureGraphPtr->FlushInvalidations();
	ClearFlags(RF_Standalone);
	SetReadyToDestroy();

	TextureGraphEngine::GetBlobber()->SetEnableCache(bBlobberCachingState);
}

void UTG_AsyncExportTask::FinishDestroy()
{
	if (TextureGraphPtr != nullptr)
	{
		TextureGraphPtr->GetSettings()->FreeTargets();
		TextureGraphPtr->ClearFlags(RF_Standalone);
		TextureGraphPtr = nullptr;
		OriginalTextureGraphPtr = nullptr;
	}
	Super::FinishDestroy();
}
