// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/TG_AsyncRenderTask.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "TG_Graph.h"
#include "TG_Node.h"
#include "Expressions/TG_Expression.h"
#include "2D/Tex.h"
#include "Expressions/Output/TG_Expression_Output.h"
#include "UObject/Package.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include "Model/Mix/MixSettings.h"
#include "GameDelegates.h"
#include "Job/Scheduler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_AsyncRenderTask)

UTG_AsyncRenderTaskBase::UTG_AsyncRenderTaskBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

JobBatchPtr UTG_AsyncRenderTaskBase::PrepareActivate(JobBatchPtr Batch, bool bIsAsync)
{
	UE_LOGF(LogTextureGraph, Verbose, "UTG_AsyncRenderTaskBase::PrepareActivate");

	if (TextureGraphPtr == nullptr)
	{
		UE_LOGF(LogTextureGraph, Warning, "UTG_AsyncRenderTaskBase::Cannot render Texture Graph not selected");
		return nullptr;
	}

	OutputBlobs.Empty();
	OutputRts.Empty();
	TextureGraphPtr->FlushInvalidations();

	if (!Batch)
		Batch = FTG_HelperFunctions::InitRenderBatch(TextureGraphPtr, nullptr);

	Batch->SetAsync(bIsAsync);
	Batch->SetNoCache(true);

	bool bIsDisableIdle = TextureGraphEngine::GetScheduler()->IsDisableIdle();
	TextureGraphEngine::GetScheduler()->SetDisableIdle(true);

	Batch->OnDone([=](JobBatch*) mutable
	{
		TextureGraphEngine::GetScheduler()->SetDisableIdle(bIsDisableIdle);
	});

	TextureGraphEngine::GetScheduler()->AddBatch(Batch);

	return Batch;
}

const TArray<UTextureRenderTarget2D*>& UTG_AsyncRenderTaskBase::ActivateBlocking(JobBatchPtr Batch)
{
	UE_LOGF(LogTextureGraph, Verbose, "UTG_AsyncRenderTaskBase::ActivateBlocking");

	Batch = PrepareActivate(Batch, false);

	if (!Batch)
		return OutputRts;

	while (!Batch->IsFinished())
	{
		TextureGraphEngine::GetInstance()->Update(0);
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
	}

	GatherAllOutputBlobs();
	FinalizeAllOutputBlobs();

	bRenderComplete = false;
	GetRenderTextures();

	while (!bRenderComplete)
	{
		TextureGraphEngine::GetInstance()->Update(0);
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
	}

	return OutputRts;
}

void UTG_AsyncRenderTaskBase::GatherAllOutputBlobs()
{
	TextureGraphPtr->Graph()->ForEachNodes([this](const UTG_Node* node, uint32 index)
		{
			if (node && node->GetExpression()->IsA(UTG_Expression_Output::StaticClass()))
			{
				auto outputs = FTG_HelperFunctions::GetTexturedOutputs(node);

				if (outputs.Num() > 0)
				{
					OutputBlobs.Add(outputs[0]);
				}
			}
		});
}

AsyncBool UTG_AsyncRenderTaskBase::FinalizeAllOutputBlobs()
{
	std::vector<AsyncBlobResultPtr> promises;
	for (const auto& Blob : OutputBlobs)
	{
		auto TiledOutput = std::static_pointer_cast<TiledBlob>(Blob);
		promises.push_back(TiledOutput->OnFinalise());
	}

	return cti::when_all(promises.begin(), promises.end()).then([=](std::vector<const Blob*> results) mutable
	{
		return true;
	});
}

void UTG_AsyncRenderTaskBase::GatherAllRenderTargets()
{
	for (const auto& Blob : OutputBlobs)
	{
		auto TiledOutput = std::static_pointer_cast<TiledBlob>(Blob);
		auto FXBuffer = std::static_pointer_cast<DeviceBuffer_FX>(TiledOutput->GetBufferRef().GetPtr());
		auto Rt = FXBuffer->GetTexture()->GetRenderTarget();
		OutputRts.Add(Rt);
	}

	bRenderComplete = true;

	UE_LOGF(LogTextureGraph, Verbose, "UTG_AsyncRenderTaskBase:: OnDone : bShouldDestroyOnRenderComplete %i", bShouldDestroyOnRenderComplete ? 1 : 0);
	if (bShouldDestroyOnRenderComplete)
	{
		SetReadyToDestroy();
	}
}

AsyncBool UTG_AsyncRenderTaskBase::GetRenderTextures()
{
	std::vector<AsyncBufferResultPtr> promises;
	for (const auto& Blob : OutputBlobs)
	{
		auto TiledOutput = std::static_pointer_cast<TiledBlob>(Blob);
		promises.push_back(TiledOutput->CombineTiles(false, false));
	}

	return cti::when_all(promises.begin(), promises.end()).then([this](std::vector<BufferResultPtr> results) mutable
	{
		GatherAllRenderTargets();
		return true;
	});
}

void UTG_AsyncRenderTaskBase::OnWorldCleanup(UWorld* World, bool bSessionEnded, bool CleanupResources)
{
	FWorldDelegates::OnWorldCleanup.RemoveAll(this);
	bShouldDestroyOnRenderComplete = true;

	//Destroy now if rendering is already complete
	if (bRenderComplete)
	{
		SetReadyToDestroy();
	}

	UE_LOGF(LogTextureGraph, Verbose, "UTG_AsyncRenderTaskBase:: OnWorldCleanup");
}

void UTG_AsyncRenderTaskBase::SetReadyToDestroy()
{
	UE_LOGF(LogTextureGraph, Verbose, "UTG_AsyncRenderTaskBase:: SetReadyToDestroy");
	TextureGraphPtr->FlushInvalidations();
	FlushRenderingCommands();
	ClearFlags(RF_Standalone);
	UTG_AsyncTask::SetReadyToDestroy();
}

void UTG_AsyncRenderTaskBase::FinishDestroy()
{
	UE_LOGF(LogTextureGraph, Verbose, "UTG_AsyncRenderTaskBase:: FinishDestroy");
	if (TextureGraphPtr != nullptr)
	{
		TextureGraphPtr->GetSettings()->FreeTargets();
		TextureGraphPtr->ClearFlags(RF_Standalone);
		TextureGraphPtr = nullptr;
		OriginalTextureGraphPtr = nullptr;
	}
	Super::FinishDestroy();
}

/////////////////////////////////////////////////////////////////////
UTG_AsyncRenderTask::UTG_AsyncRenderTask(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UTG_AsyncRenderTask* UTG_AsyncRenderTask::TG_AsyncRenderTask(UTextureGraphBase* InTextureGraph)
{
	UTG_AsyncRenderTask* Task = NewObject<UTG_AsyncRenderTask>();
	Task->SetFlags(RF_Standalone);

	if (InTextureGraph != nullptr)
	{
		Task->OriginalTextureGraphPtr = InTextureGraph;
		Task->TextureGraphPtr = Cast<UTextureGraphBase>(StaticDuplicateObject(Task->OriginalTextureGraphPtr, GetTransientPackage(), NAME_None, RF_Standalone));
		Task->TextureGraphPtr->Initialize();
		FTG_HelperFunctions::InitTargets(Task->TextureGraphPtr);
		Task->RegisterWithTGAsyncTaskManger();
		FWorldDelegates::OnWorldCleanup.AddUObject(Task, &UTG_AsyncRenderTask::OnWorldCleanup);
	}

	return Task;
}


void UTG_AsyncRenderTask::Activate()
{
	// Start the async task on a new thread
	Super::Activate();
	UE_LOGF(LogTextureGraph, Verbose, "UTG_AsyncRenderTask::Activate");

	JobBatchPtr Batch = PrepareActivate(nullptr, true);

	if (!Batch)
		return;

	FTG_HelperFunctions::RenderAsync(TextureGraphPtr)
		.then([this](bool bRenderResult) mutable
			{
				GatherAllOutputBlobs();
				return FinalizeAllOutputBlobs();
			})
		.then([this](bool bFinalized)
			{
				return GetRenderTextures();
			})
		.then([this](bool bRtResult)
			{
				OnDone.Broadcast(OutputRts);
				return bRtResult;
			});
}

