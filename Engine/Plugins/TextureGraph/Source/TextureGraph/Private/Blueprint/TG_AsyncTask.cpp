// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/TG_AsyncTask.h"

#include "TextureGraphEngine.h"
#include "Blueprint/TG_AsyncTaskManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_AsyncTask)


UTG_AsyncTask::UTG_AsyncTask(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UTG_AsyncTask::RegisterWithTGAsyncTaskManger()
{
	FTG_AsyncTaskManager::GetInstance().RegisterTask(this);
}

void UTG_AsyncTask::SetReadyToDestroy()
{
	UBlueprintAsyncActionBase::SetReadyToDestroy();
	FTG_AsyncTaskManager::GetInstance().UnRegisterTask(this);
}

void UTG_AsyncTask::Activate()
{
	TextureGraphEngine::SetRunEngine();
}
