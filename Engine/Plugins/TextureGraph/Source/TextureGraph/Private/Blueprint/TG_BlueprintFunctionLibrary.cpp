// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/TG_BlueprintFunctionLibrary.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "TG_Graph.h"
#include "Expressions/TG_Expression.h"
#include "Expressions/Input/TG_Expression_Texture.h"
#include "Expressions/Input/TG_Expression_Scalar.h"
#include "Expressions/Input/TG_Expression_Vector.h"
#include "Expressions/Input/TG_Expression_Color.h"
#include "Expressions/Input/TG_Expression_Bool.h"
#include "Expressions/Input/TG_Expression_String.h"
#include "Expressions/Input/TG_Expression_OutputSettings.h"
#include "TG_HelperFunctions.h"
#include "Engine/Texture2D.h"
#include "Job/Scheduler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_BlueprintFunctionLibrary)

//////////////////////////////////////////////////////////////////////////
// UTG_BlueprintFunctionLibrary

#define LOCTEXT_NAMESPACE "TG_BlueprintFunctionLibrary"

UTG_Pin* GetParamPin(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		if (InTextureGraph)
		{
			return InTextureGraph->Graph()->FindParamPin(ParameterName);
		}
	}

	return nullptr;
}

template <typename T_Expr>
T_Expr* GetParamExpression(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName)
{
	UTG_Pin* PinParam = GetParamPin(WorldContextObject, InTextureGraph, ParameterName);
	if (PinParam)
	{
		return Cast<T_Expr>(PinParam->GetNodePtr()->GetExpression());
	}

	return nullptr;
}

template <typename T_Expr, typename T_Expr_Value>
void SetParameterValue_Generic(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName, T_Expr_Value ParameterValue, FString FunctionName)
{
	UTG_Pin* PinParam = GetParamPin(WorldContextObject, InTextureGraph, ParameterName);
	if (PinParam)
	{
		PinParam->SetValue(ParameterValue);
	}
	else
	{
		UTG_BlueprintFunctionLibrary::AddParamWarning(ParameterName, InTextureGraph, FunctionName);
	}
}

template <typename T_Expr_Value>
T_Expr_Value GetParameterValue_Generic(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName, FString FunctionName, T_Expr_Value& OutValue)
{
	UTG_Pin* PinParam = GetParamPin(WorldContextObject, InTextureGraph, ParameterName);

	if (PinParam)
	{
		PinParam->GetValue(OutValue);
	}
	else
	{
		UTG_BlueprintFunctionLibrary::AddParamWarning(ParameterName, InTextureGraph, FunctionName);
	}

	return OutValue;
}


void UTG_BlueprintFunctionLibrary::SetTextureParameterValue(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName, UTexture* ParameterValue)
{
	UTG_Expression_Texture* ExpressionPtr = GetParamExpression<UTG_Expression_Texture>(WorldContextObject, InTextureGraph, ParameterName);
	UTextureRenderTarget2D* RenderTarget2D = Cast<UTextureRenderTarget2D>(ParameterValue);

	if (ExpressionPtr && ExpressionPtr->CanHandleAsset(ParameterValue))
	{
		UTexture2D* Texture2D = Cast<UTexture2D>(ParameterValue);
		if (Texture2D)
		{
			UTexture2D* DupTexture = (UTexture2D*)StaticDuplicateObject(ParameterValue, GetTransientPackage(), NAME_None, RF_Transient, UTexture2D::StaticClass());
			ExpressionPtr->SetAsset(DupTexture);
		}
		
	}
#if WITH_EDITOR
	else if (ExpressionPtr && RenderTarget2D)
	{
		UTexture2D* Texture = NewObject<UTexture2D>(GetTransientPackage(), NAME_None, RF_Transient);
		UKismetRenderingLibrary::ConvertRenderTargetToTexture2DEditorOnly(WorldContextObject, RenderTarget2D, Texture);
		ExpressionPtr->SetAsset(Texture);
	}
#endif
	else
	{
		AddParamWarning(ParameterName, InTextureGraph, "SetTextureParameterValue");
	}
}

UTexture* UTG_BlueprintFunctionLibrary::GetTextureParameterValue(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName)
{
	UTG_Expression_Texture* ExpressionPtr = GetParamExpression<UTG_Expression_Texture>(WorldContextObject, InTextureGraph, ParameterName);

	if (ExpressionPtr)
	{
		return ExpressionPtr->Source;
	}
	else
	{
		AddParamWarning(ParameterName, InTextureGraph, "SetTextureParameterValue");
	}

	return nullptr;
}

void UTG_BlueprintFunctionLibrary::SetScalarParameterValue(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName, float ParameterValue)
{
	SetParameterValue_Generic<UTG_Expression_Scalar, float>(WorldContextObject, InTextureGraph, ParameterName, ParameterValue, "SetScalarParameterValue");
}

float UTG_BlueprintFunctionLibrary::GetScalarParameterValue(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName)
{
	float ParameterValue = 0.0f;
	return GetParameterValue_Generic<float>(WorldContextObject, InTextureGraph, ParameterName, "GetScalarParameterValue", ParameterValue);
}

void UTG_BlueprintFunctionLibrary::SetVectorParameterValue(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName, FVector4f ParameterValue)
{
	SetParameterValue_Generic<UTG_Expression_Vector, FVector4f>(WorldContextObject, InTextureGraph, ParameterName, ParameterValue, "SetVectorParameterValue");
}

FVector4f UTG_BlueprintFunctionLibrary::GetVectorParameterValue(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName)
{
	FVector4f ParameterValue = FVector4f::Zero();
	return GetParameterValue_Generic<FVector4f>(WorldContextObject, InTextureGraph, ParameterName, "GetVectorParameterValue", ParameterValue);
}

void UTG_BlueprintFunctionLibrary::SetColorParameterValue(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName, FLinearColor ParameterValue)
{
	SetParameterValue_Generic<UTG_Expression_Color, FLinearColor>(WorldContextObject, InTextureGraph, ParameterName, ParameterValue, "SetColorParameterValue");
}

FLinearColor UTG_BlueprintFunctionLibrary::GetColorParameterValue(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName)
{
	FLinearColor ParameterValue = FLinearColor::Black;
	return GetParameterValue_Generic<FLinearColor>(WorldContextObject, InTextureGraph, ParameterName, "GetColorParameterValue", ParameterValue);
}

void UTG_BlueprintFunctionLibrary::SetBoolParameterValue(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName, bool bParameterValue)
{
	SetParameterValue_Generic<UTG_Expression_Bool, bool>(WorldContextObject, InTextureGraph, ParameterName, bParameterValue, "SetBoolParameterValue");
}

bool UTG_BlueprintFunctionLibrary::GetBoolParameterValue(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName)
{
	bool bParameterValue = false;
	return GetParameterValue_Generic<bool>(WorldContextObject, InTextureGraph, ParameterName, "GetBoolParameterValue", bParameterValue);
}

void UTG_BlueprintFunctionLibrary::SetStringParameterValue(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName, FString ParameterValue)
{
	SetParameterValue_Generic<UTG_Expression_String, FString>(WorldContextObject, InTextureGraph, ParameterName, ParameterValue, "SetStringParameterValue");
}

FString UTG_BlueprintFunctionLibrary::GetStringParameterValue(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName)
{
	FString ParameterValue;
	return GetParameterValue_Generic<FString>(WorldContextObject, InTextureGraph, ParameterName, "GetStringParameterValue", ParameterValue);
}

void UTG_BlueprintFunctionLibrary::SetSettingsParameterValue(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName, int Width, int Height, 
	FName FileName /*= "None"*/, FName Path /*= "None"*/, ETG_TextureFormat Format /*= ETG_TextureFormat::BGRA8*/, ETG_TexturePresetType TextureType /*= ETG_TexturePresetType::None*/,
	TextureGroup LODTextureGroup /*= TextureGroup::TEXTUREGROUP_World*/, TextureCompressionSettings Compression /*= TextureCompressionSettings::TC_Default*/, bool SRGB /*= false*/)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		bool bFoundParameter = false;
		bool bSizeError = false;
		bool bPathError = false;
		bool bNameError = false;

		const FString FunctionName = "SetSettingsParameterValue";

		if (Path.ToString().IsEmpty() || Path == "None")
		{
			bPathError = true;
		}
		if (FileName.ToString().IsEmpty() || FileName == "None")
		{
			bNameError = true;
		}


		if (InTextureGraph && !bSizeError && !bPathError && !bNameError)
		{
			auto PinParam = InTextureGraph->Graph()->FindParamPin(ParameterName);
			if (PinParam)
			{
				auto ExpressionPtr = Cast<UTG_Expression_OutputSettings>(PinParam->GetNodePtr()->GetExpression());
				if (ExpressionPtr)
				{
					FTG_OutputSettings ParameterValue;
					ParameterValue.Set(Width, Height, FileName, Path, Format, TextureType, Compression, LODTextureGroup, SRGB);
					ExpressionPtr->Settings = ParameterValue;
					PinParam->EditSelfVar()->EditAs<FTG_OutputSettings>() = ParameterValue;
					bFoundParameter = true;
				}
			}

			if (!bFoundParameter)
			{
				AddParamWarning(ParameterName, InTextureGraph, FunctionName);
			}
		}

		if (bPathError)
		{
			AddError(InTextureGraph, FunctionName, "Invalid path try to set a valid path , path connot be empty or none");
		}

		if (bNameError)
		{
			AddError(InTextureGraph, FunctionName, "Invalid file name , file name cannot be empty or None");
		}
	}
}

FTG_OutputSettings UTG_BlueprintFunctionLibrary::GetSettingsParameterValue(UObject* WorldContextObject, UTextureGraph* InTextureGraph, FName ParameterName, int& Width, int& Height)
{
	FTG_OutputSettings ParameterValue;

	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		auto bFoundParameter = false;
		if (InTextureGraph)
		{
			auto PinParam = InTextureGraph->Graph()->FindParamPin(ParameterName);
			if (PinParam)
			{
				ParameterValue = PinParam->GetSelfVar()->GetAs<FTG_OutputSettings>();
				Width = (int)ParameterValue.Width;
				Height = (int)ParameterValue.Height;
				bFoundParameter = true;
			}
		}

		if (!bFoundParameter)
		{
			AddParamWarning(ParameterName, InTextureGraph, "GetSettingsParameterValue");
		}
	}

	return ParameterValue;
}

const TArray<UTextureRenderTarget2D*>& UTG_BlueprintFunctionLibrary::RenderTextureGraph(UObject* WorldContextObject, UTextureGraphBase* InTextureGraph)
{
	UTG_AsyncRenderTask* Task = UTG_AsyncRenderTask::TG_AsyncRenderTask(InTextureGraph);
	return Task->ActivateBlocking(nullptr);
}

void UTG_BlueprintFunctionLibrary::ExportTextureGraph(UObject* WorldContextObject, UTextureGraphBase* InTextureGraph, bool bOverwriteTextures, bool bSave, bool bExportAll)
{
	if (InTextureGraph != nullptr)
	{
		UTG_AsyncExportTask* Task = UTG_AsyncExportTask::TG_AsyncExportTask(InTextureGraph, bOverwriteTextures, bSave, bExportAll);
		Task->ActivateBlocking(nullptr);
	}
}

void UTG_BlueprintFunctionLibrary::AddParamWarning(FName ParamName, UObject* ObjectPtr, FString FunctionName)
{
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("ParamName"), FText::FromName(ParamName));
	FMessageLog("PIE").Warning()
		->AddToken(FTextToken::Create(FText::Format(LOCTEXT("{FunctionName}", "{FunctionName} called on"), FText::FromString(FunctionName))))
		->AddToken(FUObjectToken::Create(ObjectPtr))
		->AddToken(FTextToken::Create(FText::Format(LOCTEXT("WithInvalidParam", "with invalid ParameterName '{ParamName}'. This is likely due to a Blueprint error."), Arguments)));
}

void UTG_BlueprintFunctionLibrary::AddError( UObject* ObjectPtr, FString FunctionName , FString Error)
{
	FMessageLog("PIE").Error()
		->AddToken(FTextToken::Create(FText::Format(LOCTEXT("{FunctionName}", "{FunctionName} called on"), FText::FromString(FunctionName))))
		->AddToken(FUObjectToken::Create(ObjectPtr))
		->AddToken(FTextToken::Create(FText::FromString(Error)));
}
#undef LOCTEXT_NAMESPACE

