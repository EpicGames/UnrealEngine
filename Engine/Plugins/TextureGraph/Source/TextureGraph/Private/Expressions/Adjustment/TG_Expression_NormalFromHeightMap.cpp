// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Adjustment/TG_Expression_NormalFromHeightMap.h"
#include "Transform/Expressions/T_Conversions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_NormalFromHeightMap)

void UTG_Expression_NormalFromHeightMap::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if (!Input)
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	auto DesiredDesc = Output.Descriptor;

	// if it is set to auto, we force it to be of a 4-channel format
	if(Output.Descriptor.TextureFormat == ETG_TextureFormat::Auto)
	{
		DesiredDesc.TextureFormat = ETG_TextureFormat::BGRA8;
	}

	Output = T_Conversions::CreateNormalFromHeightMap(InContext->Cycle, Input, Offset, Strength, InContext->TargetId, DesiredDesc);
}

bool UTG_Expression_NormalFromHeightMap::Validate(MixUpdateCyclePtr Cycle)
{
	if(Output.Descriptor.TextureFormat != ETG_TextureFormat::Auto && TextureHelper::GetNumChannelsFromTGTextureFormat(Output.Descriptor.TextureFormat) < 3)
	{
		UMixInterface* ParentMix = Cast<UMixInterface>(GetOutermostObject());
		auto ErrorType = static_cast<int32>(ETextureGraphErrorType::NODE_WARNING);

		TextureGraphEngine::GetErrorReporter(ParentMix)->ReportWarning(ErrorType, FString::Printf(TEXT("Output requires minimum 3 channels to display normal correctly")), GetParentNode());
	}
	return Super::Validate(Cycle);
}
