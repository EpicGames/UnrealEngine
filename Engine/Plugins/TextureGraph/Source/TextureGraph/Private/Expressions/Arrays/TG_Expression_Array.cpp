// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Arrays/TG_Expression_Array.h"

#include "TG_Graph.h"
#include "2D/TextureHelper.h"
#include "Model/StaticImageResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Array)

void UTG_Expression_Array4::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	Output.SetNum(4);

	//Output[0] = Input1;
	//Output[1] = Input2;
	//Output[2] = Input3;
	//Output[3] = Input4;

	if (Input1)
		Output.Set(0, Input1);
	else 
		Output.Set(0, FTG_Texture::GetBlack());

	if (Input2)
		Output.Set(1, Input2);
	else 
		Output.Set(1, FTG_Texture::GetBlack());

	if (Input3)
		Output.Set(2, Input3);
	else 
		Output.Set(2, FTG_Texture::GetBlack());

	if (Input4)
		Output.Set(3, Input4);
	else 
		Output.Set(3, FTG_Texture::GetBlack());
}
