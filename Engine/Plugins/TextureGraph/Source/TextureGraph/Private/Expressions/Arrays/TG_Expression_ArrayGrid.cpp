// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Arrays/TG_Expression_ArrayGrid.h"

#include "TG_Graph.h"
#include "2D/TextureHelper.h"
#include "Model/StaticImageResource.h"
#include "Transform/Expressions/T_ArrayGrid.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_ArrayGrid)

void UTG_Expression_ArrayGrid::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	TArray<TiledBlobPtr> Inputs;
	Inputs.Reserve(Input.Num());

	/// Filter out the input textures, just in case someone tries to slip in a non-texture
	/// through the variant system
	for (int Index = 0; Index < Input.Num(); Index++)
	{
		FTG_Variant& InputItem = Input.Get(Index);
		if (InputItem.IsTexture())
		{
			FTG_Texture InputTexture = InputItem.GetTexture();
			Inputs.Add(InputTexture);
		}
	}

	/// If there are no inputs then we simply render a black texture
	if (Inputs.IsEmpty())
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	int32 OutRows = 0;
	int32 OutCols = 0;
	if (Rows <= 0 && Columns <= 0)
	{
		int32 Count = FMath::CeilToInt(FMath::Sqrt((float)Inputs.Num()));
		OutRows = OutCols = Count;
	}
	else if (Rows <= 0 && Columns > 0)
	{
		OutCols = std::min<int32>(Columns, Inputs.Num());
		OutRows = std::ceil((float)Inputs.Num() / (float)OutCols);
	}
	else if (Rows > 0 && Columns <= 0)
	{
		OutRows = std::min<int32>(Rows, Inputs.Num());
		OutCols = std::ceil((float)Inputs.Num() / (float)OutRows);
	}
	else
	{
		OutRows = Rows;
		OutCols = Columns;
	}

	/// Make sure there are no zeros
	OutRows = std::max(OutRows, 1);
	OutCols = std::max(OutCols, 1);

	if (OutRows * OutCols > Inputs.Num())
	{
		FTextureGraphErrorReporter* ErrorReporter = TextureGraphEngine::GetErrorReporter(InContext->Cycle->GetMix());
		if (ErrorReporter)
		{
			ErrorReporter->ReportWarning((int32)ETextureGraphErrorType::NODE_WARNING, FString::Printf(TEXT("Number of rows and columns for the tiling do not match the number of inputs. Grid: %d x %d [Max Inputs: %d]. Padding with transparent images (background color will blend through)."), 
				OutRows, OutCols, Inputs.Num()), GetParentNode());
		}

		int32 NumDesiredEntires = OutRows * OutCols;
		for (int32 Index = Inputs.Num(); Index < NumDesiredEntires; Index++)
		{
			Inputs.Add(TextureHelper::GetTransparent());
		}
	}

	BufferDescriptor DesiredDesc = Output.Descriptor;
	Output = T_ArrayGrid::Create(InContext->Cycle, DesiredDesc, Inputs, OutRows, OutCols, BackgroundColor);
}
