// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Arrays/TG_Expression_ArrayOps.h"

#include "TG_Graph.h"
#include "2D/TextureHelper.h"
#include "Model/StaticImageResource.h"

/////////////////////////////////////////////////////////////
/// Array concatenation
/////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_ArrayOps)
void UTG_Expression_ArrayConcat::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	uint32 Input1Count = (uint32)Input1.Num();
	uint32 Input2Count = (uint32)Input2.Num();
	uint32 MaxCount = Input1Count + Input2Count;
	Output.SetNum(MaxCount);

	uint32 ActualStartIndex = StartIndex >= 0 ? StartIndex : 0;

	if (ActualStartIndex >= MaxCount || ActualStartIndex >= Input1Count)
	{
		FTextureGraphErrorReporter* ErrorReporter = TextureGraphEngine::GetErrorReporter(InContext->Cycle->GetMix());

		if (ErrorReporter)
		{
			ErrorReporter->ReportWarning((int32)ETextureGraphErrorType::NODE_WARNING, FString::Printf(TEXT("Invalid starting index for Input-1 in the output array specified: %d [Input-1 Size: %d | Output Size: %d]."), 
				StartIndex, Input1Count, MaxCount), GetParentNode());
		}

		ActualStartIndex = 0;
	}

	for (uint32 i = 0; i < Input2Count && i < ActualStartIndex; i++)
		Output.Set(i, Input2.GetArray()[i]);

	for (uint32 i = 0; i < Input1Count; i++)
		Output.Set(i + ActualStartIndex, Input1.GetArray()[i]);

	for (uint32 i = ActualStartIndex; i < Input2Count; i++)
		Output.Set(i + Input1Count, Input2.GetArray()[i]);
}

/////////////////////////////////////////////////////////////
/// Array slicing/splicing
/////////////////////////////////////////////////////////////
void UTG_Expression_ArraySplit::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if (Input.GetArray().IsEmpty())
	{
		Sliced.SetNum(1);
		Spliced.SetNum(1);
		Sliced.Set(0, FTG_Texture::GetBlack());
		Spliced.Set(0, FTG_Texture::GetBlack());
		return;
	}

	if (EndIndex < 0 || EndIndex < StartIndex)
		EndIndex = Input.Num();
	if (StartIndex < 0)
		StartIndex = 0;

	check(StartIndex <= EndIndex);

	int32 InputCount = Input.Num();
	int32 SlicedCount = std::min(EndIndex - StartIndex, InputCount - 1);
	int32 SplicedCount = std::max(StartIndex + (InputCount - EndIndex), 0);

	if (SlicedCount > 0)
	{
		Sliced.SetNum(SlicedCount);
		for (int i = 0; i < SlicedCount; i++)
			Sliced.Set(i, Input.GetArray()[StartIndex + i]);
	}
	else
	{
		Sliced.SetNum(1);
		Sliced.Set(0, FTG_Texture::GetBlack());
	}

	if (SplicedCount > 0)
	{
		Spliced.SetNum(SplicedCount);

		for (int i = 0; i < StartIndex; i++)
			Spliced.Set(i, Input.GetArray()[i]);

		for (int i = EndIndex; i < Input.Num(); i++)
			Spliced.Set(i - EndIndex + StartIndex, Input.GetArray()[i]);
	}
	else
	{
		Spliced.SetNum(1);
		Spliced.Set(0, FTG_Texture::GetBlack());
	}
}
