// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Channel/TG_Expression_ChannelCombiner.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Expressions/T_Channel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_ChannelCombiner)

void UTG_Expression_ChannelCombiner::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_ChannelCombiner>(TEXT("T_ChannelCombiner"));

	check(RenderMaterial);
	TiledBlobPtr Source = nullptr;

	if (!Red)
		Red = FTG_Texture::GetBlack();
	else
		Source = Red;

	if (!Green)
		Green = FTG_Texture::GetBlack();
	else if (!Source)
		Source = Green;

	if (!Blue)
		Blue = FTG_Texture::GetBlack();
	else if (!Source)
		Source = Blue;

	if (!Alpha)
		Alpha = FTG_Texture::GetWhite();
	else if (!Source)
		Source = Alpha;

	/// Need at least one valid source to be able to do this
	if (!Source)
	{
		Output = TextureHelper::GetBlack();
		return;
	}

	JobUPtr RenderJob = std::make_unique<Job>(InContext->Cycle->GetMix(), InContext->TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));

	RenderJob
		->AddArg(ARG_BLOB(Red, "SourceRed"))
		->AddArg(ARG_BLOB(Green, "SourceGreen"))
		->AddArg(ARG_BLOB(Blue, "SourceBlue"))
		->AddArg(ARG_BLOB(Alpha, "SourceAlpha"))
		;

	const FString Name = TEXT("ChannelCombiner");

	BufferDescriptor Desc = Output.GetBufferDescriptor();
	Desc.ItemsPerPoint = 4;
	
	Output = RenderJob->InitResult(Name, &Desc);
	InContext->Cycle->AddJob(InContext->TargetId, std::move(RenderJob));
}
