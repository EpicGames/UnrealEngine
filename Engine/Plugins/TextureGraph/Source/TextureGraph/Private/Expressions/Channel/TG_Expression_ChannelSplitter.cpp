// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Channel/TG_Expression_ChannelSplitter.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include <Transform/Expressions/T_Color.h>

#include "Transform/Expressions/T_Channel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_ChannelSplitter)

template <typename FSH_PixelShaderType>
static void GenericEvaluate(FTG_EvaluationContext* InContext, FString ChannelName, FTG_Texture& Input, FTG_Texture& Output)
{
	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_PixelShaderType>(TEXT("T_ChannelSplitter_") + ChannelName);

	check(RenderMaterial);

	if (!Input)
	{
		Output = FTG_Texture::GetBlack();
		return;
	}
	
	FTG_Texture Default = FTG_Texture::GetWhite();
	TiledBlobRef Source = InContext->Inputs.GetVar("Input")->GetAsWithDefault<FTG_Texture>(Default).RasterBlob;

	JobUPtr RenderJob = std::make_unique<Job>(InContext->Cycle->GetMix(), InContext->TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));

	RenderJob
		->AddArg(ARG_BLOB(Source, "SourceTexture"))
		;

	const FString Name = TEXT("ChannelSplitter_") + ChannelName; 

	BufferDescriptor Desc = Output.GetBufferDescriptor();
	
	Output = RenderJob->InitResult(Name, &Desc);
	InContext->Cycle->AddJob(InContext->TargetId, std::move(RenderJob));
}

void UTG_Expression_ChannelSplitter::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if (!Input)
	{
		Red = FTG_Texture::GetBlack();
		Green = FTG_Texture::GetBlack();
		Blue = FTG_Texture::GetBlack();
		Alpha = FTG_Texture::GetBlack();
		return;
	}

	GenericEvaluate<FSH_ChannelSplitter_Red>(InContext, TEXT("Red"), Input, Red);
	GenericEvaluate<FSH_ChannelSplitter_Green>(InContext, TEXT("Green"), Input, Green);
	GenericEvaluate<FSH_ChannelSplitter_Blue>(InContext, TEXT("Blue"), Input, Blue);
	GenericEvaluate<FSH_ChannelSplitter_Alpha>(InContext, TEXT("Alpha"), Input, Alpha);
}
