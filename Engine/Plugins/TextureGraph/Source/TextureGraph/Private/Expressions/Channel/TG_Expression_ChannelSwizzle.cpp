// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Channel/TG_Expression_ChannelSwizzle.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include <Transform/Expressions/T_Color.h>

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_ChannelSwizzle)

void UTG_Expression_ChannelSwizzle::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if (!Input)
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	FTG_Texture Default = FTG_Texture::GetBlack();
	TiledBlobRef Source = InContext->Inputs.GetVar("Input")->GetAsWithDefault<FTG_Texture>(Default).RasterBlob;

	FSH_ChannelSwizzle::FPermutationDomain PermutationVector;
	PermutationVector.Set<FVar_SwizzleDstChannelRed>((int32)RedChannel);
	PermutationVector.Set<FVar_SwizzleDstChannelGreen>((int32)GreenChannel);
	PermutationVector.Set<FVar_SwizzleDstChannelBlue>((int32)BlueChannel);
	PermutationVector.Set<FVar_SwizzleDstChannelAlpha>((int32)AlphaChannel);

	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_ChannelSwizzle>(TEXT("T_ChannelSwizzle"), PermutationVector);
	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(InContext->Cycle->GetMix(), InContext->TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_BLOB(Source, "SourceTexture"))
		->AddArg(WithUnbounded(ARG_INT((int32)RedChannel, "RedChannel")))
		->AddArg(WithUnbounded(ARG_INT((int32)GreenChannel, "GreenChannel")))
		->AddArg(WithUnbounded(ARG_INT((int32)BlueChannel, "BlueChannel")))
		->AddArg(WithUnbounded(ARG_INT((int32)AlphaChannel, "AlphaChannel")))
		;

	BufferDescriptor Desc = Output.GetBufferDescriptor();
	Output = RenderJob->InitResult("ChannelSwizzle", &Desc);
	InContext->Cycle->AddJob(InContext->TargetId, std::move(RenderJob));
}
