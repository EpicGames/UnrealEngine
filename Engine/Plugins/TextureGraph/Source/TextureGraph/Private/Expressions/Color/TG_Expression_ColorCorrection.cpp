// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Color/TG_Expression_ColorCorrection.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Expressions/T_Color.h"
#include "TextureGraphEngine/Helper/ColorUtil.h"
#include "TextureGraphEngine/Helper/GraphicsUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_ColorCorrection)

static FLinearColor ColorTemperatureRGB(float Temperature)
{
	static const int NumElements = 3;
	static const float CoolConv[NumElements][NumElements] = {
		{ 0.0f, -2902.1955373783176f, -8257.7997278925690f },
		{ 0.0f, 1669.5803561666639f, 2575.2827530017594f },
		{ 1.0f, 1.3302673723350029f, 1.8993753891711275f }
	};

	static const float WarmConv[NumElements][NumElements] = {
		{ 1745.0425298314172f, 1216.6168361476490f, -8257.7997278925690f },
		{ -2666.3474220535695f, -2173.1012343082230f, 2575.2827530017594f },
		{ 0.55995389139931482f, 0.70381203140554553f, 1.8993753891711275f }
	};

	const auto Mat = (Temperature <= 6500.0f) ? CoolConv : WarmConv;
	float V0 = FMath::Clamp(Mat[0][0] / (Temperature + Mat[1][0]) + Mat[2][0], 0.0f, 1.0f);
	float V1 = FMath::Clamp(Mat[0][1] / (Temperature + Mat[1][1]) + Mat[2][1], 0.0f, 1.0f);
	float V2 = FMath::Clamp(Mat[0][2] / (Temperature + Mat[1][2]) + Mat[2][2], 0.0f, 1.0f);

	return FLinearColor(V0, V1, V2, 0.0f);
}

void UTG_Expression_ColorCorrection::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_ColorCorrection>(TEXT("T_ColorCorrection"));

	check(RenderMaterial);

	if (!Input)
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	FLinearColor TemperatureRGB = ColorTemperatureRGB(FMath::Clamp(Temperature, 1000.0f, 20000.0f));

	JobUPtr RenderJob = std::make_unique<Job>(InContext->Cycle->GetMix(), InContext->TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));

	RenderJob
		->AddArg(ARG_BLOB(Input, "Input"))
		->AddArg(ARG_FLOAT(Brightness, "Brightness"))
		->AddArg(ARG_FLOAT(Contrast, "Contrast"))
		->AddArg(ARG_FLOAT(Gamma, "Gamma"))
		->AddArg(ARG_FLOAT(Saturation, "Saturation"))
		->AddArg(ARG_LINEAR_COLOR(TemperatureRGB, "TemperatureRGB"))
		->AddArg(ARG_FLOAT(TemperatureStrength, "TemperatureStrength"))
		->AddArg(ARG_FLOAT(TemperatureBrightnessNormalization, "TemperatureBrightnessNormalization"))
		;

	const FString Name = TEXT("ColorCorrection");
	BufferDescriptor Desc = Output.GetBufferDescriptor();
	Output = RenderJob->InitResult(Name, &Desc);
	InContext->Cycle->AddJob(InContext->TargetId, std::move(RenderJob));
}
