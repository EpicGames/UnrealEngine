// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Color/TG_Expression_Grayscale.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Expressions/T_Color.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Grayscale)

void UTG_Expression_Grayscale::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_Grayscale>(TEXT("T_Grayscale"));

	check(RenderMaterial);

	if (!Input)
	{
		Output = FTG_Texture::GetBlack();
		return;
	}
	
	JobUPtr RenderJob = std::make_unique<Job>(InContext->Cycle->GetMix(), InContext->TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));

	RenderJob
		->AddArg(ARG_BLOB(Input, "SourceTexture"))
		;

	const FString Name = TEXT("Grayscale"); // FString::Printf(TEXT("Grayscale.[%s].[%d].[%llu]"), *gmask->ID(), InContext->TargetId, InContext->Cycle->Batch()->BatchId());
	BufferDescriptor Desc = Output.GetBufferDescriptor();
	Output = RenderJob->InitResult(Name, &Desc);
	InContext->Cycle->AddJob(InContext->TargetId, std::move(RenderJob));
}
