// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Color/TG_Expression_HSV.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Expressions/T_Color.h"
#include "TextureGraphEngine/Helper/ColorUtil.h"

//////////////////////////////////////////////////////////////////////////
/// RGB2HSV Correction
//////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_HSV)
FTG_Texture UTG_Expression_RGB2HSV::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_RGB2HSV>(TEXT("T_RGB2HSV"));

	check(RenderMaterial);

	if (!Input)
	{
		return FTG_Texture::GetBlack();
	}

	JobUPtr RenderJob = std::make_unique<Job>(InContext->Cycle->GetMix(), InContext->TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));

	RenderJob
		->AddArg(ARG_BLOB(Input.GetTexture(InContext), "Input"))
		;

	const FString Name = TEXT("RGB2HSV"); // FString::Printf(TEXT("Grayscale.[%s].[%D].[%llu]"), *gmask->ID(), InContext->TargetId, InContext->Cycle->Batch()->BatchId());
	BufferDescriptor Desc = Output.EditTexture().GetBufferDescriptor();

	TiledBlobPtr OutputResult = RenderJob->InitResult(Name, &Desc);
	InContext->Cycle->AddJob(InContext->TargetId, std::move(RenderJob));

	return OutputResult;
}

FVector4f UTG_Expression_RGB2HSV::EvaluateVector_WithValue(FTG_EvaluationContext* InContext, const FVector4f* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 1); 
	const FVector4f& C = ValuePtr[0];
	FLinearColor Result = ColorUtil::RGB2HSV(FLinearColor(C.X, C.Y, C.Z, 1.0f));
	return FVector4f(Result.R, Result.G, Result.B, 1.0f);
}

//////////////////////////////////////////////////////////////////////////
/// HSV2RGB Correction
//////////////////////////////////////////////////////////////////////////
FTG_Texture UTG_Expression_HSV2RGB::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_HSV2RGB>(TEXT("T_HSV2RGB"));

	check(RenderMaterial);

	if (!Input)
	{
		return FTG_Texture::GetBlack();
	}

	JobUPtr RenderJob = std::make_unique<Job>(InContext->Cycle->GetMix(), InContext->TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));

	RenderJob
		->AddArg(ARG_BLOB(Input.GetTexture(InContext), "Input"))
		;

	const FString Name = TEXT("HSV2RGB"); // FString::Printf(TEXT("Grayscale.[%s].[%D].[%llu]"), *gmask->ID(), InContext->TargetId, InContext->Cycle->Batch()->BatchId());
	BufferDescriptor Desc = Output.EditTexture().GetBufferDescriptor();
	TiledBlobPtr OutputResult = RenderJob->InitResult(Name, &Desc);
	InContext->Cycle->AddJob(InContext->TargetId, std::move(RenderJob));

	return OutputResult;
}

FVector4f UTG_Expression_HSV2RGB::EvaluateVector_WithValue(FTG_EvaluationContext* InContext, const FVector4f* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 1); 
	const FVector4f& C = ValuePtr[0];
	FLinearColor Result = ColorUtil::HSV2RGB(FLinearColor(C.X, C.Y, C.Z, 1.0f));
	return FVector4f(Result.R, Result.G, Result.B, 1.0f);
}

//////////////////////////////////////////////////////////////////////////
/// HSV Correction
//////////////////////////////////////////////////////////////////////////
FTG_Texture UTG_Expression_HSV::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_HSV>(TEXT("T_HSV"));

	check(RenderMaterial);

	if (!Input)
	{
		return FTG_Texture::GetBlack();
	}

	JobUPtr RenderJob = std::make_unique<Job>(InContext->Cycle->GetMix(), InContext->TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));

	RenderJob
		->AddArg(ARG_BLOB(Input.GetTexture(InContext), "Input"))
		->AddArg(ARG_FLOAT(Hue, "Hue"))
		->AddArg(ARG_FLOAT(Saturation, "Saturation"))
		->AddArg(ARG_FLOAT(Value, "Value"))
		;

	const FString Name = TEXT("HSV"); // FString::Printf(TEXT("Grayscale.[%s].[%D].[%llu]"), *gmask->ID(), InContext->TargetId, InContext->Cycle->Batch()->BatchId()); 
	BufferDescriptor Desc = Output.EditTexture().GetBufferDescriptor();

	TiledBlobPtr OutputResult = RenderJob->InitResult(Name, &Desc);
	InContext->Cycle->AddJob(InContext->TargetId, std::move(RenderJob));

	return OutputResult;
}

FVector4f UTG_Expression_HSV::EvaluateVector_WithValue(FTG_EvaluationContext* InContext, const FVector4f* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 1); 
	const FVector4f& C = ValuePtr[0];
	FLinearColor Result = ColorUtil::HSVTweak(FLinearColor(C.X, C.Y, C.Z, 1.0f), Hue, Saturation, Value);
	return FVector4f(Result.R, Result.G, Result.B, 1.0f);
}
