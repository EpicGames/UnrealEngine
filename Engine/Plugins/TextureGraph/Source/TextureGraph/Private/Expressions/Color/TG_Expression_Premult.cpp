// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Color/TG_Expression_Premult.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Expressions/T_Color.h"
#include "TextureGraphEngine/Helper/ColorUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Premult)

void UTG_Expression_Premult::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_Premult>(TEXT("T_Premult"));

	check(RenderMaterial);

	if (!Input)
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	JobUPtr RenderJob = std::make_unique<Job>(InContext->Cycle->GetMix(), InContext->TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_BLOB(Input, "Input"))
		;

	const FString Name = TEXT("Premult");
	BufferDescriptor Desc = Output.GetBufferDescriptor();
	Output = RenderJob->InitResult(Name, &Desc);
	InContext->Cycle->AddJob(InContext->TargetId, std::move(RenderJob));
}
