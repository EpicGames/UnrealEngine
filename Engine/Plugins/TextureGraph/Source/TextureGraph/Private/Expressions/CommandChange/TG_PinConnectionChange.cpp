// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/CommandChange/TG_PinConnectionChange.h"
#include "TG_Pin.h"
#include "TG_Node.h"
#include "TG_Graph.h"
#include "Misc/ITransaction.h"

void FTG_PinConnectionChange::StoreChange(UTG_Graph* InGraph, UTG_Node& NodeFrom, FTG_Name& PinFromName)
{
#if WITH_EDITOR
	if (!GIsTransacting && GUndo)
	{
		TUniquePtr<FTG_PinConnectionChange> Change = MakeUnique<FTG_PinConnectionChange>(InGraph, NodeFrom, PinFromName);
		GUndo->StoreUndo(InGraph, MoveTemp(Change));
	}
#endif
}

FTG_PinConnectionChange::FTG_PinConnectionChange(UTG_Graph* InGraph, UTG_Node& NodeTo, FTG_Name& PinToName)
	: FCommandChange()
	, Graph(InGraph)
{
	PinTo = NodeTo.GetPin(PinToName);
}

void FTG_PinConnectionChange::Revert(UObject* Object)
{ // undo
	check(PinTo);

	//If the pin is not conneccted we will copy its self var to the expression property
	if(!PinTo->IsConnected())
		PinTo->GetNodePtr()->OnPinConnectionUndo(PinTo->GetId());
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FTG_PinConnectionBreakChange::StoreChange(UTG_Graph* InGraph, UTG_Node& NodeFrom, FTG_Name& PinFromName)
{
#if WITH_EDITOR
	if (!GIsTransacting && GUndo)
	{
		TUniquePtr<FTG_PinConnectionBreakChange> Change = MakeUnique<FTG_PinConnectionBreakChange>(InGraph, NodeFrom, PinFromName);
		GUndo->StoreUndo(InGraph, MoveTemp(Change));
	}
#endif
}

FTG_PinConnectionBreakChange::FTG_PinConnectionBreakChange(UTG_Graph* InGraph, UTG_Node& NodeTo, FTG_Name& PinToName)
	: FCommandChange()
	, Graph(InGraph)
{
	PinTo = NodeTo.GetPin(PinToName);
}

void FTG_PinConnectionBreakChange::Apply(UObject* Object)
{ // undo
	check(PinTo);

	//we will rest its self var and copy to the expression property
	for (auto OtherPinId : PinTo->GetEdges())
	{
		auto OtherPin = Graph->GetPin(OtherPinId);
		if (OtherPin)
		{
			OtherPin->GetNodePtr()->OnPinConnectionUndo(OtherPin->GetId());
		}
	}

	PinTo->GetNodePtr()->OnPinConnectionUndo(PinTo->GetId());
}