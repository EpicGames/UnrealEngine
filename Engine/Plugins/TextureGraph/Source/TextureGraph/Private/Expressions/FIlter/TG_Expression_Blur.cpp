// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Filter/TG_Expression_Blur.h"

#include "Transform/Expressions/T_Blur.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Blur)
#if WITH_EDITOR
bool UTG_Expression_Blur::CanEditChange(const FProperty* InProperty) const
{
	bool bEditCondition = Super::CanEditChange(InProperty);

	// if already set to false Or InProperty not directly owned by us, early out
	if (!bEditCondition || this->GetClass() != InProperty->GetOwnerClass())
	{
		return bEditCondition;
	}

	const FName PropertyName = InProperty->GetFName();
	

	// Specific logic associated with Property
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Blur, Radius))
	{
		bEditCondition = BlurType == EBlurType::Gaussian || BlurType == EBlurType::Radial;
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Blur, Angle))
	{
		bEditCondition = BlurType == EBlurType::Directional;
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Blur, Strength))
	{
		bEditCondition = BlurType == EBlurType::Directional || BlurType == EBlurType::Radial;
	}

	// Default behaviour
	return bEditCondition;
}
#endif
void UTG_Expression_Blur::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if(Input)
	{
		if(BlurType == EBlurType::Gaussian)
		{
			Output = T_Blur::CreateGaussian(InContext->Cycle, Output.GetBufferDescriptor(), Input, Radius, InContext->TargetId);
		}
		else if(BlurType == EBlurType::Radial)
		{
			Output = T_Blur::CreateRadial(InContext->Cycle, Output.GetBufferDescriptor(), Input, Radius, Strength, InContext->TargetId);
		}
		else if(BlurType == EBlurType::Directional)
		{
			Output = T_Blur::CreateDirectional(InContext->Cycle, Output.GetBufferDescriptor(), Input, Angle, Strength, InContext->TargetId);
		}
	}
	else
	{
		Output = TextureHelper::GetBlack();
	}

}
