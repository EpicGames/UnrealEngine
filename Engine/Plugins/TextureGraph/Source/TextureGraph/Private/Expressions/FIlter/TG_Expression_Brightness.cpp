// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Filter/TG_Expression_Brightness.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Expressions/T_Brightness.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Brightness)

void UTG_Expression_Brightness::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_Brightness>(TEXT("T_Brightness"));

	check(RenderMaterial);

	if (!Input)
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	Output = T_Brightness::Create(InContext->Cycle, Output.GetBufferDescriptor(), Input, Brightness, Contrast, InContext->TargetId);
}
