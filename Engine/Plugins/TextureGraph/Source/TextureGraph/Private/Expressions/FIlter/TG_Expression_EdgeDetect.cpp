// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Filter/TG_Expression_EdgeDetect.h"
#include "Transform/Expressions/T_Filter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_EdgeDetect)

void UTG_Expression_EdgeDetect::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if (!Input)
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	Output = T_Filter::CreateEdgeDetect(InContext->Cycle, Output.GetBufferDescriptor(), Input, Thickness, InContext->TargetId);
}
