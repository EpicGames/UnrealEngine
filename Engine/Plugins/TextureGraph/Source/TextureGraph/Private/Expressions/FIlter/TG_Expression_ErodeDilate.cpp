// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Filter/TG_Expression_ErodeDilate.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Expressions/T_Filter.h"
#include "Transform/Utility/T_CombineTiledBlob.h"

//////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_ErodeDilate)
void UTG_Expression_ErodeDilate::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if (!Input)
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	if (Size == 0)
	{
		Output = Input;
		return;
	}

	FSH_ErodeDilate::FPermutationDomain PermutationVector;
	PermutationVector.Set<FVar_ErodeDilate_Type>((int32)Type);
	PermutationVector.Set<FVar_ErodeDilate_Kernel>((int32)Kernel);
	bool bIsSingleChannel = Input->GetDescriptor().ItemsPerPoint == 1;
	PermutationVector.Set<FVar_ErodeDilate_IsSingleChannel>(bIsSingleChannel);

	RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_ErodeDilate>(TEXT("T_ErodeDilate"), PermutationVector);
	check(RenderMaterial);

	// Combine the tiled texture
	TiledBlobPtr CombinedBlob = T_CombineTiledBlob::Create(InContext->Cycle, Input->GetDescriptor(), InContext->TargetId, Input);
	BufferDescriptor Desc = BufferDescriptor::Combine(Output.Descriptor, Input->GetDescriptor());

	FTileInfo TileInfo;
	JobUPtr RenderJob = std::make_unique<Job>(InContext->Cycle->GetMix(), InContext->TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_TILEINFO(TileInfo, "TileInfo"))
		->AddArg(ARG_BLOB(CombinedBlob, "Input"))
		->AddArg(ARG_INT(Size, "Size"))
		->AddArg(WithUnbounded(ARG_INT((int)Type, "Type")))
		->AddArg(WithUnbounded(ARG_INT((int)Kernel, "Kernel")))
		;

	const FString Name = FString::Printf(TEXT("T_ErodeDilate"));

	TiledBlobPtr Result = RenderJob->InitResult(Name, &Desc);
	InContext->Cycle->AddJob(InContext->TargetId, std::move(RenderJob));

	Output = Result;
}
