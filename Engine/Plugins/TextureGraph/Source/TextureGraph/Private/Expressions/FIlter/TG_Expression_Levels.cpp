// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Filter/TG_Expression_Levels.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Expressions/T_Levels.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Levels)

void FTG_LevelsSettings_VarPropertySerialize(FTG_Var::VarPropertySerialInfo& Info)
{
	FTG_Var::FGeneric_Struct_Serializer< FTG_LevelsSettings>(Info);
}

template <> FString TG_Var_LogValue(FTG_LevelsSettings& Value)
{
	FString LogMessage = Value.ToString();
	return LogMessage;
}

template <> void TG_Var_SetValueFromString(FTG_LevelsSettings& Value, const FString& StrVal)
{
	Value.InitFromString(StrVal);
}

bool FTG_LevelsSettings::SetHigh(float InValue)
{
	float NewValue = std::min(1.f, std::max(InValue, Low));
	if (NewValue != High)
	{
		float CurveExponent = EvalMidExponent();
		High = NewValue;
		return SetMidFromMidExponent(CurveExponent);
	}
	return false;
}

bool FTG_LevelsSettings::SetLow(float InValue)
{
	float NewValue = std::max(0.f, std::min(InValue, High));
	if (NewValue != Low)
	{
		float CurveExponent = EvalMidExponent();
		Low = NewValue;
		return SetMidFromMidExponent(CurveExponent);
	}
	return false;
}

bool FTG_LevelsSettings::SetMid(float InValue)
{
	float NewValue = std::max(Low, std::min(InValue, High));
	if (NewValue != Mid)
	{
		Mid = NewValue;
		return true;
	}
	return false;
}

float FTG_LevelsSettings::EvalRange(float Val) const
{
	return std::max(0.f, std::min((Val - Low)/(High - Low), 1.f));
}
float FTG_LevelsSettings::EvalRangeInv(float Val) const
{
	return Val * (High - Low) + Low;
}

float FTG_LevelsSettings::EvalMidExponent() const
{
	// 0.5 = EvalRange(Mid) ^ Exponent
	// thus
	float MidRanged = EvalRange(Mid);
	MidRanged = std::max(0.001f, std::min(9.999f, MidRanged));

	return log(0.5) / log(MidRanged);
}

bool FTG_LevelsSettings::SetMidFromMidExponent(float InExponent)
{
	// 0.5 = EvalRange(Mid) ^ Exponent
	// thus
	float NewValue = EvalRangeInv(pow(0.5, 1.0/InExponent));

	if (NewValue != Mid)
	{
		Mid = NewValue;
		return true;
	}
	return false;
}


void UTG_Expression_Levels::PostLoad()
{
	Super::PostLoad();

	// Restore LevelsSettings inner struct from saved values
	Levels.Low = (LowValue);
	Levels.High = (HighValue);
	Levels.Mid = (MidValue);
}

#if WITH_EDITOR

void UTG_Expression_Levels::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Catch if any of low / mid / high changes do the proper range check and feedback final values
	// Low
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTG_Expression_Levels, LowValue))
	{
		SetLowValue(LowValue);
	}
	// High
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTG_Expression_Levels, HighValue))
	{
		SetHighValue(HighValue);
	}
	// Mid
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTG_Expression_Levels, MidValue))
	{
		SetMidValue(MidValue);
	}
	// Out Low
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTG_Expression_Levels, OutLowValue))
	{
		SetOutLowValue(OutLowValue);
	}
	// Out High
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTG_Expression_Levels, OutHighValue))
	{
		SetOutHighValue(OutHighValue);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UTG_Expression_Levels::CanEditChange(const FProperty* InProperty) const
{
	bool bEditCondition = Super::CanEditChange(InProperty);
	// if already set to false Or InProperty not directly owned by us, early out
	if (!bEditCondition || this->GetClass() != InProperty->GetOwnerClass())
	{
		return bEditCondition;
	}

	const FName PropertyName = InProperty->GetFName();

	// Specific logic associated with Property
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Levels, LowValue))
	{
		bEditCondition = (!IsAutoLevel());
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Levels, MidValue))
	{
		bEditCondition = (!IsAutoLevel());
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Levels, HighValue))
	{
		bEditCondition = (!IsAutoLevel());
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Levels, MidAutoLevels))
	{
		bEditCondition = (IsAutoLevel());
	}

	return bEditCondition;
}

#endif

void UTG_Expression_Levels::SetLowValue(float InValue)
{
	if (Levels.SetLow(InValue))
	{
		LowValue = Levels.Low;
		MidValue = Levels.Mid;
		FeedbackPinValue(GET_MEMBER_NAME_CHECKED(UTG_Expression_Levels, MidValue), MidValue);
	}
	else
	{
		LowValue = Levels.Low;
	}
}

void UTG_Expression_Levels::SetMidValue(float InValue)
{
	Levels.SetMid(InValue);
	MidValue = Levels.Mid;
}

void UTG_Expression_Levels::SetHighValue(float InValue)
{
	if (Levels.SetHigh(InValue))
	{
		MidValue = Levels.Mid;
		HighValue = Levels.High;
		FeedbackPinValue(GET_MEMBER_NAME_CHECKED(UTG_Expression_Levels, MidValue), MidValue);
	}
	else
	{
		HighValue = Levels.High;
	}
}


void UTG_Expression_Levels::SetOutLowValue(float InValue)
{
	OutLowValue = FMath::Clamp(InValue, 0, OutHighValue);
}

void UTG_Expression_Levels::SetOutHighValue(float InValue)
{
	OutHighValue = FMath::Clamp(OutHighValue, OutLowValue, 1.0f);
}

void UTG_Expression_Levels::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if (!Input) // No Input, black Output
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	LevelsControl = MakeShared<FLevels>();

	switch (LevelsExpressionType)
	{
	case ELevelsExpressionType::LowMidHigh:
		LevelsControl->InitFromLowMidHigh(LowValue, MidValue, HighValue, OutLowValue, OutHighValue);
		break;
	case ELevelsExpressionType::AutoLowHigh:
		LevelsControl->InitFromAutoLevels(MidAutoLevels);
		break;
	}

	BufferDescriptor Desc = Output.GetBufferDescriptor();
	Output = T_Levels::Create(InContext->Cycle, Desc, Input.RasterBlob, LevelsControl, InContext->TargetId);
}

/////////////////////////////////////////////////////////////////
/// Histogram scan
/////////////////////////////////////////////////////////////////
void UTG_Expression_HistogramScan::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if (!Input) // No Input, black Output
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	if (Position < 0.00001f)
	{
		Output = FTG_Texture::GetBlack();
		return;
	}
	else if (Position >= 0.99999f)
	{
		Output = FTG_Texture::GetWhite();
		return;
	}

	LevelsControl = MakeShared<FLevels>();
	LevelsControl->InitFromHistogramScan(Position, Contrast);


	BufferDescriptor Desc = Output.GetBufferDescriptor();
	Output = T_Levels::Create(InContext->Cycle, Desc, Input.RasterBlob, LevelsControl, InContext->TargetId);
}

/////////////////////////////////////////////////////////////////
/// Histogram range
/////////////////////////////////////////////////////////////////
void UTG_Expression_HistogramRange::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if (!Input) // No Input, black Output
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	LevelsControl = MakeShared<FLevels>();
	LevelsControl->InitFromHistogramRange(Position, Range);

	BufferDescriptor Desc = Output.GetBufferDescriptor();
	Output = T_Levels::Create(InContext->Cycle, Desc, Input.RasterBlob, LevelsControl, InContext->TargetId);
}

/////////////////////////////////////////////////////////////////
/// Histogram select
/////////////////////////////////////////////////////////////////
#if 0 /// TODO
void UTG_Expression_HistogramSelect::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if (!Input) // No Input, black Output
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	LevelsControl = MakeShared<FLevels>();
	LevelsControl->InitFromPositionContrast(Position, Contrast);
	LevelsControl->InitFromRange(Range, Position);

	BufferDescriptor Desc = Output.GetBufferDescriptor();
	Output = T_Levels::Create(InContext->Cycle, Desc, Input.RasterBlob, LevelsControl, InContext->TargetId);
}

#endif 
