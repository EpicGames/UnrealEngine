// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Filter/TG_Expression_Warp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Warp)

void UTG_Expression_Warp::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if (!Input)
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	/// By default we just a flat white mask
	if (!Mask)
		Mask = FTG_Texture::GetWhite();

	if (Type == EWarp::Directional)
	{
		Output = T_Filter::CreateDirectionalWarp(InContext->Cycle, Output.GetBufferDescriptor(), Input, Mask, Intensity, (Angle * PI) / 180.0f, InContext->TargetId);
	}
	else if (Type == EWarp::Normal)
	{
		Output = T_Filter::CreateNormalWarp(InContext->Cycle, Output.GetBufferDescriptor(), Input, Mask, Intensity, InContext->TargetId);
	}
	else if (Type == EWarp::Sine)
	{
		Output = T_Filter::CreateSineWarp(InContext->Cycle, Output.GetBufferDescriptor(), Input, Mask, Intensity, PhaseU, PhaseV, InContext->TargetId);
	}
	else
	{
		Output = FTG_Texture::GetBlack();
	}
}

#if WITH_EDITOR
bool UTG_Expression_Warp::CanEditChange(const FProperty* InProperty) const
{
	bool bEditCondition = Super::CanEditChange(InProperty);

	const FName PropertyName = InProperty->GetFName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Warp, Angle))
	{
		bEditCondition = Type == EWarp::Directional;
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Warp, PhaseU))
	{
		bEditCondition = Type == EWarp::Sine;
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Warp, PhaseV))
	{
		bEditCondition = Type == EWarp::Sine;
	}

	// Default behaviour
	return bEditCondition;
}
#endif
