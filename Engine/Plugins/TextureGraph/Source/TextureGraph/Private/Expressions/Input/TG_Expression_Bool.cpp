// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Input/TG_Expression_Bool.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Bool)



void UTG_Expression_Bool::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	// The Value is updated either as an input or as a setting and then becomes the output for this expression
	// The pin out is named "ValueOut"
	ValueOut = Bool;
}

void UTG_Expression_Bool::SetTitleName(FName NewName)
{
	GetParentNode()->GetInputPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Bool, Bool))->SetAliasName(NewName);
}

FName UTG_Expression_Bool::GetTitleName() const
{
	return GetParentNode()->GetInputPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Bool, Bool))->GetAliasName();
}
