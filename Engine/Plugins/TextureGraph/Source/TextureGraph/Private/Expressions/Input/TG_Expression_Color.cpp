// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Input/TG_Expression_Color.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Color)

void UTG_Expression_Color::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);
	
	ValueOut = Color;
}

void UTG_Expression_Color::SetTitleName(FName NewName)
{
	GetParentNode()->GetInputPin("Color")->SetAliasName(NewName);
}

FName UTG_Expression_Color::GetTitleName() const
{
	return GetParentNode()->GetInputPin("Color")->GetAliasName();
}
