// Copyright Epic Games, Inc. All Rights Reserved.
#include "Expressions/Input/TG_Expression_Graph.h"

#include "TG_Graph.h"
#include "UObject/Package.h"
#include "TextureGraph.h"
#include "UObject/ObjectSaveContext.h"

//////////////////////////////////////////////////////////////////////////
//// Generic subgraph
//////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Graph)

void UTG_Expression_Graph::NotifyGraphChanged()
{
	UE_LOGF(LogTextureGraph, Log, "Graph Expression Reset Signature.");

	// Signature is reset, notify the owning node / graph to update itself
	NotifySignatureChanged();
}

FTG_SignaturePtr UTG_Expression_Graph::BuildSignatureDynamically() const
{
	// Initialize the signature from the expression class itself 
	FTG_Signature::FInit SignatureInit = GetSignatureInitArgsFromClass();

	// Then add the signature from the embedded Graph if exist
	if (GetGraph())
	{
		// Append the Graph params signature 
		FTG_Arguments GraphArgs;
		GetGraph()->AppendParamsSignature(GraphArgs, InParamIds, OutParamIds);

		for (auto& Arg : GraphArgs)
		{
			if (Arg.IsInput() || Arg.IsSetting())
			{
				Arg.SetPersistentSelfVar();
			}
			SignatureInit.Arguments.Add(Arg);
		}
	}
	return MakeShared<FTG_Signature>(SignatureInit);
}

void UTG_Expression_Graph::SetupAndEvaluate(FTG_EvaluationContext* InContext)
{
	//Super::SetupAndEvaluate(InContext);
	if (InContext->bDoLog)
	{
		LogEvaluation(InContext);
	}
	Evaluate(InContext);
}

UTG_Expression_TextureGraph::UTG_Expression_TextureGraph()
{
#if WITH_EDITOR
	// listener for UObject saves, so we can synchronise when linked TextureGraphs get updated 
	PreSaveHandle = FCoreUObjectDelegates::OnObjectPreSave.AddUObject(this, &UTG_Expression_TextureGraph::OnTextureGraphPreSave);
#endif
}

UTG_Expression_TextureGraph::~UTG_Expression_TextureGraph()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPreSave.Remove(PreSaveHandle);
#endif
}

void UTG_Expression_Graph::Evaluate(FTG_EvaluationContext* InContext)
{
	auto Graph = GetGraph();

	if (Graph)
	{
		InContext->GraphDepth++;
		if (Graph->Validate(InContext->Cycle))
		{
			Graph->Evaluate(InContext);
		}
		else
		{
			auto ErrorType = static_cast<int32>(ETextureGraphErrorType::SUBGRAPH_INTERNAL_ERROR);
			const FString ErrorMsg = FString::Format(TEXT("Internal error in {0} node."), {GetTitleName().ToString()} );
			TextureGraphEngine::GetErrorReporter(InContext->Cycle->GetMix())->ReportError(ErrorType, ErrorMsg, GetParentNode());
		}
		
		InContext->GraphDepth--;	
	}
}


//////////////////////////////////////////////////////////////////////////
// TextureTextureGraph Expression
//////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR
void UTG_Expression_TextureGraph::PreEditChange(FProperty* PropertyAboutToChange)
{
	// if Graph changes catch it first
	if (PropertyAboutToChange->GetName() == GET_MEMBER_NAME_CHECKED(UTG_Expression_TextureGraph, TextureGraph))
	{	
		bool bShouldMarkAsDirty;
		if (GIsTransacting)
		{
			// Don't mark the outer package as dirty during an undo/redo operation.
			bShouldMarkAsDirty = false;
		}
		else if (PropertyAboutToChange && PropertyAboutToChange->HasAnyPropertyFlags(CPF_SkipSerialization))
		{
			// Don't mark the outer package as dirty if we're about to change a non-serializable property.
			bShouldMarkAsDirty = false;
		}
		else
		{
			PreEditTextureGraph = TextureGraph;

			// Don't mark the outer package as dirty if annotated to be deferred (e.g. during propagation).
			bShouldMarkAsDirty = false;
		}

		Modify(bShouldMarkAsDirty);

	}
	else
	{
		Super::PreEditChange(PropertyAboutToChange);
	}
}


void UTG_Expression_TextureGraph::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// if Graph changes catch it first
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTG_Expression_TextureGraph, TextureGraph))
	{
		// In the case of TextureGraph changed from detail panel
		// Validate the TextureGraph assigned here
		// if it is not passing then fall back to the previous version which should have been captured in the PreEditChange()
		bool IsValidTextureGraph = CheckDependencies(TextureGraph);
		UTextureGraph* PreEditTextureGraphPtr = PreEditTextureGraph.Get();
		PreEditTextureGraph.Reset();
		if (!IsValidTextureGraph)
		{
			// Early exit without changing anything
			// Restore the TextureGRaph to its pre edit change values
			TextureGraph = PreEditTextureGraphPtr;
		}
		else
		{
			SetTextureGraphInternal(TextureGraph);
		}		
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UTG_Expression_TextureGraph::PostEditUndo()
{
	// Make sure the signature is in sync after undo in case we undo a TextureGRaph assignment:
	// So recreate it internally without notifying, normally, the node's pins should match
	DynSignature.Reset();
	GetSignature();

	Super::PostEditUndo();
}
#endif

void UTG_Expression_TextureGraph::OnTextureGraphPreSave(UObject* Object, FObjectPreSaveContext SaveContext)
{
	// every editor should check if your texture graph is dependent on the object being saved
	UTextureGraph* GraphBeingSaved = Cast<UTextureGraph>(Object);
	UTextureGraph* ParentGraph = GetTypedOuter<UTextureGraph>();

	// if object being saved is our linked TextureGraph, 
	// and this expression's graph is NOT the same as the graph being saved, we re-create
	if (GraphBeingSaved && TextureGraph && ParentGraph &&
		GraphBeingSaved != ParentGraph )
	{
		
		// if graph being saved is not ourself, and there is a cyclic dependency
		bool bSameGraphs = GraphBeingSaved->GetOuter() == TextureGraph->GetOuter();
		if (bSameGraphs || ParentGraph->IsDependentOn(GraphBeingSaved))
		{
			// Check if the graph saved has caused cyclic dependency on the parent graph of this expression
			if (!bSameGraphs && ParentGraph->HasCyclicDependency())
			{
				if (TextureGraphEngine::GetErrorReporter(ParentGraph) != nullptr)
				{
					auto ErrorType = static_cast<int32>(ETextureGraphErrorType::RECURSIVE_CALL);
	
					TextureGraphEngine::GetErrorReporter(ParentGraph)->ReportError(ErrorType, FString::Printf(TEXT("Can not use graph as a cyclic dependency is detected")));
				}
				Modify();
				//we reset graph
				TextureGraph = nullptr;
				RuntimeGraph = nullptr;
#if WITH_EDITOR
				// We need to find its property and trigger property change event manually.
				const auto SourcePin = GetParentNode()->GetInputPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_TextureGraph, TextureGraph));

				check(SourcePin)
	
				if(SourcePin)
				{
					FProperty* Property = SourcePin->GetExpressionProperty();
					NotifyExpressionChanged(FPropertyChangedEvent(Property));
				}
#endif
			}
			else
			{
				// Recreate the Runtime copy
				RuntimeGraph = DuplicateObject(TextureGraph->Graph(), this);	
#ifdef UE_BUILD_DEBUG
				RuntimeGraph->IsRuntime = 2;
#endif
			}
			
			// Graph is reset, notify the owning node / graph to update itself
			NotifyGraphChanged();
		}
	}
}
void UTG_Expression_TextureGraph::Initialize()
{
	if (TextureGraph && !RuntimeGraph)
	{
		RuntimeGraph = DuplicateObject(TextureGraph->Graph(), this);
#ifdef UE_BUILD_DEBUG
		RuntimeGraph->IsRuntime = 2;
#endif
	}

}

// CheckDependencies: Returns true if no unwanted dependencies found
bool UTG_Expression_TextureGraph::CheckDependencies(const UTextureGraph* InTextureGraph) const
{
	if (InTextureGraph == nullptr)
		return true;
		
	// check if InTextureGraph doesn't have "this" texture graph as a dependency
	UTextureGraph* ParentGraph = CastChecked<UTextureGraph>(GetOutermostObject());
	const UPackage* Package = this->GetPackage();  
	const UPackage* SecondPackage = InTextureGraph->GetPackage();  
	const bool bIsTransientPackage = Package->HasAnyFlags(RF_Transient) || Package == GetTransientPackage();
		
	if (ParentGraph == InTextureGraph ||
		(Package == SecondPackage && !bIsTransientPackage))	 // early out if we're trying to assign our own texture graph)
	{
		auto ErrorType = static_cast<int32>(ETextureGraphErrorType::UNSUPPORTED_TYPE);
			
		TextureGraphEngine::GetErrorReporter(ParentGraph)->ReportError(ErrorType, FString::Printf(TEXT("Can not assign graph to itself")));
		return false;
	}

	if (InTextureGraph->IsDependentOn(ParentGraph))
	{
		auto ErrorType = static_cast<int32>(ETextureGraphErrorType::RECURSIVE_CALL);
	
		TextureGraphEngine::GetErrorReporter(ParentGraph)->ReportError(ErrorType, FString::Printf(TEXT("Can not use graph as a cyclic dependency is detected")));
		return false;
	}

	return true;
}

void UTG_Expression_TextureGraph::SetTextureGraphInternal(UTextureGraph* InTextureGraph)
{
	// validate that we're allowed to use InTextureGraph as input here
	// if dependencies exist, we don't want to use the InTextureGraph so we ll treat it just like a nullptr.
	bool IsValidTextureGraph = CheckDependencies(InTextureGraph);

	if (!IsValidTextureGraph)
	{
		return;
	}

	Modify();
	if (!InTextureGraph)
	{
		TextureGraph = nullptr;
		RuntimeGraph = nullptr;
	}
	else
	{
		// This is a valid TextureGraph
		TextureGraph = InTextureGraph;
		RuntimeGraph = DuplicateObject(TextureGraph->Graph(), this);
#ifdef UE_BUILD_DEBUG
		RuntimeGraph->IsRuntime = 2;
#endif
	}

	// Graph is reset, notify the owning node / graph to update itself
	NotifyGraphChanged();
}

void UTG_Expression_TextureGraph::SetTextureGraph(UTextureGraph* InTextureGraph)
{
	// This is the public setter of the TextureGraph, 
	// This is NOT called if the TextureGraph is modified from the detail panel!!!
	// We catch that case in PostEditChangeProperty, which will call SetTextureGraphInternal 

	// If it is the same TextureGraph then avoid anymore work, we should be good to go
	if (InTextureGraph == TextureGraph)
	{
		return;
	}

	SetTextureGraphInternal(InTextureGraph);
}

UTG_Graph* UTG_Expression_TextureGraph::GetGraph() const
{
	return RuntimeGraph;
}

void UTG_Expression_TextureGraph::CopyVarGeneric(const FTG_Argument& Arg, FTG_Var* InVar, bool CopyVarToArg)
{
	// Look into the subgraph parameter pins for arguments of the TextureTextureGraph expression 
	if (GetGraph())
	{
		auto InnerParamId = GetGraph()->FindParamPinId(Arg.GetName());
		if (InnerParamId.IsValid())
		{
			auto InnerVar = GetGraph()->GetVar(InnerParamId);

			if (CopyVarToArg)
			{
				InVar->CopyTo(InnerVar);
			}
			else
			{
				InVar->CopyFrom(InnerVar);
			}
		}
	}
}


void UTG_Expression_TextureGraph::Evaluate(FTG_EvaluationContext* InContext)
{
	if(TextureGraph)
	{
		InContext->Cycle->PushMix(TextureGraph);
		Super::Evaluate(InContext);
		InContext->Cycle->PopMix();	
	}
}

bool UTG_Expression_TextureGraph::Validate(MixUpdateCyclePtr Cycle)
{
	// if TextureGraph is already in execution list then
	UTextureGraph* ParentGraph = Cast<UTextureGraph>(GetOutermostObject());

	if((TextureGraph &&  Cycle->ContainsMix(TextureGraph)) ||
		(ParentGraph && ParentGraph->HasCyclicDependency()))
	{
		auto ErrorType = static_cast<int32>(ETextureGraphErrorType::RECURSIVE_CALL);
		
		const FString ErrorMsg = FString::Format(TEXT("Recursive Texture Graph call found in '{0}' asset."), {TextureGraph->GetName()} );
		TextureGraphEngine::GetErrorReporter(Cycle->GetMix())->ReportError(ErrorType, ErrorMsg, GetParentNode());
		return false;
	}

	return true;
}

bool UTG_Expression_TextureGraph::CanHandleAsset(UObject* Asset)
{
	const UTextureGraph* TSTextureGraph = Cast<UTextureGraph>(Asset);
	
	return TSTextureGraph != nullptr && this->GetPackage() != Asset->GetPackage();
}

void UTG_Expression_TextureGraph::SetAsset(UObject* Asset)
{
	if(UTextureGraph* TSAsset = Cast<UTextureGraph>(Asset); TSAsset != nullptr)
	{
		SetTextureGraphInternal(TSAsset);
	}
}
