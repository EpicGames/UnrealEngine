// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Input/TG_Expression_InputParam.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_InputParam)


#if WITH_EDITOR
void UTG_Expression_InputParam::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// if Graph changes catch it first
	if (PropertyChangedEvent.GetPropertyName() == FName(TEXT("IsConstant")))
	{
		UE_LOGF(LogTextureGraph, VeryVerbose, "InputParam  Expression Parameter/Constant PostEditChangeProperty.");
		NotifySignatureChanged();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

FTG_SignaturePtr UTG_Expression_InputParam::BuildInputParameterSignature() const
{
	FTG_Signature::FInit SignatureInit = GetSignatureInitArgsFromClass();
	return MakeShared<FTG_Signature>(SignatureInit);
};

FTG_SignaturePtr UTG_Expression_InputParam::BuildInputConstantSignature() const
{
	FTG_Signature::FInit SignatureInit = GetSignatureInitArgsFromClass();
	for (auto& Arg : SignatureInit.Arguments)
	{
		if (Arg.IsInput() && Arg.IsParam())
		{
			Arg.ArgumentType = Arg.ArgumentType.Unparamed();
			Arg.ArgumentType.SetNotConnectable();
		}
	}
	return MakeShared<FTG_Signature>(SignatureInit);
};

void UTG_Expression_InputParam::SetbIsConstant(bool InIsConstant)
{
	if (bIsConstant != InIsConstant)
	{
		Modify();
		bIsConstant = InIsConstant;
		NotifySignatureChanged();
	}
}

void UTG_Expression_InputParam::ToggleIsConstant()
{
	Modify();
	if (bIsConstant)
	{
		bIsConstant = false;
	}
	else {
		bIsConstant = true;
	}
	NotifySignatureChanged();
}
