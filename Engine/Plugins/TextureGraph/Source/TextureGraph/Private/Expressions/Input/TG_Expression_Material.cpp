// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Input/TG_Expression_Material.h"
#include "UObject/ObjectSaveContext.h"
#include "Misc/TransactionObjectEvent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Material)

#if WITH_EDITOR
void UTG_Expression_Material::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// First catch if InputMaterial changes, specifically the <AssetPath> field
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTG_Expression_Material, InputMaterial)
		|| PropertyChangedEvent.GetPropertyName() == TEXT("AssetPath"))
	{
		UE_LOGF(LogTextureGraph, VeryVerbose, "Material Expression PostEditChangeProperty.");
		SetMaterialInternal(InputMaterial.GetMaterial());
		FeedbackPinValue(GET_MEMBER_NAME_CHECKED(UTG_Expression_Material, RenderedAttribute), RenderedAttribute);
	}
	// Second catch if AttributeName changes
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTG_Expression_Material, RenderedAttribute))
	{
		UE_LOGF(LogTextureGraph, VeryVerbose, "Material Expression PostEditChangeProperty.");
		SetRenderedAttribute(RenderedAttribute);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UTG_Expression_Material::PostEditUndo()
{
	// Make sure the signature is in sync after undo in case we undo a material assignment:
	// So recreate it internally without notifying, normally, the node's pins should match
	DynSignature.Reset();
	GetSignature();
	FeedbackPinValue(GET_MEMBER_NAME_CHECKED(UTG_Expression_Material, RenderedAttribute), RenderedAttribute);

	Super::PostEditUndo();
}

void UTG_Expression_Material::OnReferencedObjectPreSave(UObject* Object, FObjectPreSaveContext SaveContext)
{
	// early out if the parent node is invalid
	UTG_Node* ParentNode = GetParentNode();
	if (ParentNode == nullptr || !ParentNode->GetId().IsValid())
		return;
	
	// every editor should check if your texture graph is dependent on the object being saved
	UMaterialInterface* MaterialBeingSaved = Cast<UMaterialInterface>(Object);
	UMaterialInterface* ReferencedMaterial = InputMaterial.GetMaterial();

	// if object being saved is our linked TextureGraph, we re-create
	if (MaterialBeingSaved && ReferencedMaterial && (MaterialBeingSaved == ReferencedMaterial))
	{
		SetMaterialInternal(MaterialBeingSaved);
	}
}
#endif // WITH_EDITOR


UTG_Expression_Material::UTG_Expression_Material()
{
#if WITH_EDITOR
	// listener for UObject saves, so we can synchronise when linked TextureGraphs get updated 
	PreSaveHandle = FCoreUObjectDelegates::OnObjectPreSave.AddUObject(this, &UTG_Expression_Material::OnReferencedObjectPreSave);
#endif
}
UTG_Expression_Material::~UTG_Expression_Material()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPreSave.Remove(PreSaveHandle);
#endif
}

void UTG_Expression_Material::PostLoad()
{
	Super::PostLoad();

	// Only do this during editor time because that is when we can fix it up.
	// This means that properties will be fixed up during a cook!
#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

		if (Material_DEPRECATED)
		{
			InputMaterial.SetMaterial(Material_DEPRECATED);
			Material_DEPRECATED = nullptr;
		}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
}

void UTG_Expression_Material::Initialize()
{
	// If the referenced material is valid, then we need to recreate a duplicate
 	if (InputMaterial.IsValid() && InputMaterial.GetMaterial() != nullptr)
	{
		MaterialCopy = DuplicateObject(InputMaterial.GetMaterial(), this);
		MaterialCopy->ClearFlags(RF_Standalone | RF_Public);
		MaterialCopy->SetFlags(RF_Transient);
	}

	Super::Initialize();
	SetRenderedAttribute(RenderedAttribute); // reassign the RenderedAttribute to make sure it is the correct one
}

void UTG_Expression_Material::SetMaterialInternal(UMaterialInterface* InMaterial)
{
	if (!InMaterial)
	{
		MaterialCopy = nullptr;
	}
	else if (InMaterial->IsA<UMaterial>() || InMaterial->IsA<UMaterialInstance>())
	{
		MaterialCopy = DuplicateObject(InMaterial, this);
		MaterialCopy->ClearFlags(RF_Standalone | RF_Public);
		MaterialCopy->SetFlags(RF_Transient);
	}

	Super::SetMaterialInternal(MaterialCopy);

	SetRenderedAttribute(RenderedAttribute);
}

void UTG_Expression_Material::SetInputMaterial(const FTG_Material& InParamMaterial)
{
	// This is the public setter of the material, 
	// This is NOT called if the Material is modified from the detail panel!!!
	// We catch that case in PostEditChangeProperty, which will call UpdateMaterialInternalFromSettingOrParam 

	// If it is the same ParamMaterial then avoid anymore work, we shoudl be good to go
	if (InputMaterial == InParamMaterial)
	{
		// Just check that the local MaterialCopy is valid, if not reassign below
		if ((!InputMaterial.IsValid()) || (InputMaterial.IsValid() && MaterialCopy && MaterialPermutations.IsValid()))
			return;
	}

	InputMaterial = InParamMaterial;
	SetMaterialInternal(InputMaterial.GetMaterial());
}

void UTG_Expression_Material::SetRenderedAttribute(FName InRenderedAttribute)
{
	if (GetAvailableMaterialAttributeNames().Num())
	{
		int32 RenderAttributeIndex = GetAvailableMaterialAttributeNames().Find(InRenderedAttribute);
		if (RenderAttributeIndex == INDEX_NONE)
		{
			RenderedAttribute = GetAvailableMaterialAttributeNames()[0];
		}
		else
		{
			RenderedAttribute = InRenderedAttribute;
		}
	}
	else
	{
		RenderedAttribute = TEXT("None");
	}
	
}

bool UTG_Expression_Material::CanHandleAsset(UObject* Asset)
{
	const UMaterialInterface* Mat = Cast<UMaterialInterface>(Asset);
	
	return Mat !=nullptr;
}

void UTG_Expression_Material::SetAsset(UObject* Asset)
{
	if(UMaterialInterface* MaterialAsset = Cast<UMaterialInterface>(Asset); MaterialAsset != nullptr)
	{
		InputMaterial.SetMaterial(MaterialAsset);
#if WITH_EDITOR
		// We need to find its property and trigger property change event manually.
		const auto SourcePin = GetParentNode()->GetInputPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Material, InputMaterial));
		check(SourcePin)	
		if(SourcePin)
		{
			FProperty* Property = SourcePin->GetExpressionProperty();
			PropertyChangeTriggered(Property, EPropertyChangeType::ValueSet);
		}
#endif
	}
}

void UTG_Expression_Material::SetTitleName(FName NewName)
{
	if (UTG_Node* ParentNode = GetParentNode())
	{
		if (UTG_Pin* InputMaterialPin = ParentNode->GetInputPin("InputMaterial"))
		{
			InputMaterialPin->SetAliasName(NewName);
			TitleName = InputMaterialPin->GetAliasName();
		}
	}
}

FName UTG_Expression_Material::GetTitleName() const
{
	return TitleName;
}

TArray<FName> UTG_Expression_Material::GetRenderAttributeOptions() const
{
	return GetAvailableMaterialAttributeNames();
}

EDrawMaterialAttributeTarget UTG_Expression_Material::GetRenderedAttributeId()
{
	if (GetAvailableMaterialAttributeNames().Num())
	{
		int32 RenderAttributeIndex = GetAvailableMaterialAttributeNames().Find(RenderedAttribute);
		if (RenderAttributeIndex == INDEX_NONE)
		{
			return GetAvailableMaterialAttributeIds()[0];
		}
		else
		{
			return GetAvailableMaterialAttributeIds()[RenderAttributeIndex];
		}
	}
	else
	{
		return EDrawMaterialAttributeTarget::Emissive;
	}
}

