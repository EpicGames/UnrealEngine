// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Input/TG_Expression_MaterialFunction.h"
#include "Expressions/TG_Expression_MaterialBase.h"
#include "2D/TargetTextureSet.h"
#include "Transform/Utility/T_SplitToTiles.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"
#include "Job/Job.h"
#include "Helper/GraphicsUtil.h"

#include "Transform/Utility/T_CombineTiledBlob.h"
#include "FxMat/FxMaterial_DrawMaterial.h"

#include "UObject/UObjectGlobals.h"

#include "MaterialExpressionIO.h"

#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"

#include "Materials/MaterialExpressionMultiply.h"
#include "Materials/MaterialExpressionScalarParameter.h"
#include "Materials/MaterialExpressionVectorParameter.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionTextureObjectParameter.h"

#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include <Misc/PackageName.h>

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_MaterialFunction)


#if WITH_EDITOR
void UTG_Expression_MaterialFunction::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// First catch if Material changes
	if (PropertyChangedEvent.GetPropertyName() == FName(TEXT("MaterialFunction")) && !(PropertyChangedEvent.ChangeType == EPropertyChangeType::Unspecified))
	{
		UE_LOGF(LogTextureGraph, VeryVerbose, "Material Function Expression PostEditChangeProperty.");
		SetMaterialFunctionInternal(MaterialFunction);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UTG_Expression_MaterialFunction::PostEditUndo()
{
	// Make sure the signature is in sync after undo in case we undo a materialFunction assignment:
	// So recreate it internally without notifying, normally, the node's pins should match
	DynSignature.Reset();
	GetSignature();

	Super::PostEditUndo();
}
#endif

void UTG_Expression_MaterialFunction::Initialize()
{
	if (MaterialFunction && !ReferenceMaterial)
	{
		ReferenceMaterial = CreateMaterialReference();
	}

	Super::Initialize(); // Allocate the BaseMaterial infrastructure using the ReferenceMaterial.
}

void UTG_Expression_MaterialFunction::SetMaterialFunctionInternal(UMaterialFunctionInterface* InMaterialFunction)
{
	if (!InMaterialFunction)
	{
		MaterialFunction = nullptr;
		ReferenceMaterial = nullptr;
	}
	else if (InMaterialFunction->IsA<UMaterialFunction>())
	{
		// create a material instance
		MaterialFunction = InMaterialFunction;
		ReferenceMaterial = CreateMaterialReference();
	}
	else
	{
		MaterialFunction = InMaterialFunction;
		ReferenceMaterial = nullptr;
	}

	SetMaterialInternal(ReferenceMaterial);
}

void UTG_Expression_MaterialFunction::SetMaterialFunction(UMaterialFunctionInterface* InMaterialFunction)
{
	// This is the public setter of the material function, 
	// This is NOT called if the MaterialFunction is modified from the detail panel!!!
	// We catch that case in PostEditChangeProperty, which will call SetMaterialFunctionInternal 

	// If it is the same Material then avoid anymore work, we shoudl be good to go
	if (InMaterialFunction == MaterialFunction)
	{
		// Just check that the MaterialInstance is valid, if not reassign below
		if ((!MaterialFunction) || (MaterialFunction && MaterialPermutations.IsValid()))
			return;
	}

	SetMaterialFunctionInternal(InMaterialFunction);
}

bool UTG_Expression_MaterialFunction::CanHandleAsset(UObject* Asset)
{
	const UMaterialInterface* Mat = Cast<UMaterialInterface>(Asset);
	
	return Mat !=nullptr;
}

void UTG_Expression_MaterialFunction::SetAsset(UObject* Asset)
{
	UMaterialFunctionInterface* MaterialFunctionAsset = Cast<UMaterialFunctionInterface>(Asset);
	if(MaterialFunctionAsset != nullptr)
	{
		SetMaterialFunction(MaterialFunctionAsset);
#if WITH_EDITOR
		// We need to find its property and trigger property change event manually.
		const auto SourcePin = GetParentNode()->GetInputPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_MaterialFunction, MaterialFunction));

		check(SourcePin)
	
		if(SourcePin)
		{
			auto Property = SourcePin->GetExpressionProperty();
			PropertyChangeTriggered(Property, EPropertyChangeType::ValueSet);
		}
#endif
	}
}



bool UTG_Expression_MaterialFunction::CanRenameTitle() const
{
	// TODO: Enable Material as a param
	// InputMaterial is not a param right now
	// Material can require inputs hooking.
	// Need to figure out how we can handle this case at some later point.
	// For now disabling the input material expression renaming.
	// We will enable it at some point
	return false;
}

template<class MaterialExpressionType>
MaterialExpressionType* CreateMaterialExpression(UMaterialInterface* InMaterial, FName ParamName = FName())
{
// This is only possible to create and edit material in editor mode
#if WITH_EDITOR
	MaterialExpressionType* Expression = NewObject<MaterialExpressionType>(InMaterial);

	if (!ParamName.IsNone())
	{
		Expression->SetParameterName(ParamName);
	}

	UMaterial* AsMaterial = Cast<UMaterial>(InMaterial);
	if (AsMaterial)
	{
		AsMaterial->GetExpressionCollection().AddExpression(Expression);
		Expression->Material = AsMaterial;
	}
	// MaterialFunction->FunctionExpressions.Add(NewExpression);

	Expression->UpdateMaterialExpressionGuid(true, true);

	// use for params
	Expression->UpdateParameterGuid(true, true);
	if (Expression->HasAParameterName())
	{
		Expression->ValidateParameterName(false);
	}
	if (AsMaterial)
	{
		AsMaterial->AddExpressionParameter(Expression, AsMaterial->EditorParameters);
	}

	Expression->MarkPackageDirty();

	return Expression;
#else
	return nullptr;
#endif
}

void SaveRuntimeMaterialAsAsset(UMaterial* InMaterial, const FString& FileName = TEXT("SaveRuntimeMat"), const FString& PackagePath = TEXT("/Game/Temp/"))
{
	FString PackageName =  PackagePath + FileName;

	// Create a new package to hold the Material
	UPackage* Package = CreatePackage(*PackageName);

	// Add the Material to the package
	Package->FullyLoad();
	Package->AddToRoot();

	UObject* ObjectToSave = DuplicateObject(InMaterial, Package);

	// Ensure the Material is marked as saveable
	ObjectToSave->SetFlags(RF_Standalone | RF_Public);

	// Rename the Material to the new name
	//ObjectToSave->Rename(*FileName, Package);

	FSavePackageArgs SaveArgs;

	// This is specified just for example
	{
		SaveArgs.TopLevelFlags = RF_Public | RF_Standalone;
		SaveArgs.SaveFlags = SAVE_NoError;
		SaveArgs.bForceByteSwapping = true;
		SaveArgs.bWarnOfLongFilename = true;

	}

	FString PackageFileName = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());
	// Save the package to disk
	const bool bSucceeded = UPackage::SavePackage(Package, ObjectToSave, *PackageFileName, SaveArgs);

	if (!bSucceeded)
	{
		UE_LOGF(LogTemp, Error, "Package '%ls' wasn't saved!", *PackageFileName);
	}
	else
	{
		UE_LOGF(LogTemp, Warning, "Package '%ls' was successfully saved", *PackageFileName);
	}
}

UMaterialInterface* UTG_Expression_MaterialFunction::CreateMaterialReference()
{
#if WITH_EDITOR

	UMaterial* RefMaterial = NewObject<UMaterial>((UObject*)GetTransientPackage());
	RefMaterial->MaterialDomain = EMaterialDomain::MD_Surface;
	RefMaterial->bAllowFrontLayerTranslucency = false;

	UMaterialEditorOnlyData* EditorOnly = RefMaterial->GetEditorOnlyData();
	FExpressionInput* OutputDestination = &EditorOnly->EmissiveColor;

	// this is used for debug only to introduce a coloring parameter to the result out of the MaterialFUnction
	// disabled for now
	#if (0)
	{	// Introduce a multiply as the last node feeding into the output basecolro in order to debug 
		UMaterialExpressionMultiply* Mul = CreateMaterialExpression<UMaterialExpressionMultiply>(RefMaterial);
		OutputDestination->Connect(0, Mul);
		OutputDestination = &Mul->A;

		// Add a debug color multiplied by the value out from the function
		UMaterialExpressionVectorParameter* ExpParam = CreateMaterialExpression<UMaterialExpressionVectorParameter>(RefMaterial, TEXT("Debug"));
		ExpParam->DefaultValue = FLinearColor(0, 0, 1, 1);
		Mul->B.Connect(0, ExpParam);
	}
	#endif

	// Create the Expression for the MaterialFunctionCall and add it to the material
	UMaterialExpressionMaterialFunctionCall* ExpMFC = CreateMaterialExpression<UMaterialExpressionMaterialFunctionCall>(RefMaterial);
	ExpMFC->SetMaterialFunction(MaterialFunction);
	ExpMFC->UpdateFromFunctionResource();
	RefMaterial->AddExpressionParameter(ExpMFC, RefMaterial->EditorParameters);

	// Now collect the inputs and outputs info from the function
	TArray<struct FFunctionExpressionInput> FunctionInputs;
	TArray<struct FFunctionExpressionOutput> FunctionOutputs;
	MaterialFunction->GetInputsAndOutputs(FunctionInputs, FunctionOutputs);
	UE_LOGF(LogTextureGraph, Log, "Function <%ls>\n", *MaterialFunction->GetName());

	// For all the inputs of the function, create a matching material parameter that can be exposed as an input pin of the texture graph 
	for (int i = 0; i < FunctionInputs.Num(); ++i)
	{
		bool SupportedInput = true;
		TObjectPtr<UMaterialExpressionFunctionInput> ExpInput = FunctionInputs[i].ExpressionInput;
		if (ExpInput)
		{
			auto Name = ExpInput->InputName;
			auto InputName = FName(Name.ToString() + TEXT("_param"));
			auto InputType = ExpInput->GetInputValueType(0);
			FString TypeName;
			switch (InputType)
			{
			case MCT_Float:
				TypeName = TEXT("MCT_Float");
				{
					UMaterialExpressionScalarParameter* ExpParam = CreateMaterialExpression<UMaterialExpressionScalarParameter>(RefMaterial, InputName);
					ExpParam->DefaultValue = ExpInput->PreviewValue.X;
					ExpMFC->GetInput(i)->Connect(0, ExpParam);
				}
				break;
			case MCT_Float2:
				TypeName = TEXT("MCT_Float2");
				{
					if ((Name != TEXT("Coordinates")) && (Name != TEXT("UVs")))
					{
						UMaterialExpressionVectorParameter* ExpParam = CreateMaterialExpression<UMaterialExpressionVectorParameter>(RefMaterial, InputName);
						ExpParam->DefaultValue = ExpInput->PreviewValue;
						ExpMFC->GetInput(i)->Connect(0, ExpParam);
					}
				}
				break;
			case MCT_Float3:
				TypeName = TEXT("MCT_Float3");
				{
					UMaterialExpressionVectorParameter* ExpParam = CreateMaterialExpression<UMaterialExpressionVectorParameter>(RefMaterial, InputName);
					ExpParam->DefaultValue = ExpInput->PreviewValue;
					ExpMFC->GetInput(i)->Connect(0, ExpParam);
				}
				break;
			case MCT_Float4:
				TypeName = TEXT("MCT_Float4");
				{
					UMaterialExpressionVectorParameter* ExpParam = CreateMaterialExpression<UMaterialExpressionVectorParameter>(RefMaterial, InputName);
					ExpParam->DefaultValue = ExpInput->PreviewValue;
					ExpMFC->GetInput(i)->Connect(0, ExpParam);
				}
				break;
			case MCT_Texture2D:
				TypeName = TEXT("MCT_Texture2D");
				{
					UMaterialExpressionTextureObjectParameter* ExpParam = CreateMaterialExpression<UMaterialExpressionTextureObjectParameter>(RefMaterial, InputName);
					ExpMFC->GetInput(i)->Connect(0, ExpParam);
				}
				break;
			case MCT_TextureCube:
				TypeName = TEXT("MCT_TextureCube");
				SupportedInput = false;
				break;
			case MCT_Texture2DArray:
				TypeName = TEXT("MCT_Texture2DArray");
				SupportedInput = false;
				break;
			case MCT_TextureExternal:
				TypeName = TEXT("MCT_TextureExternal");
				SupportedInput = false;
				break;
			case MCT_VolumeTexture:
				TypeName = TEXT("MCT_VolumeTexture");
				SupportedInput = false;
				break;
			case MCT_StaticBool:
				TypeName = TEXT("MCT_StaticBool");
				SupportedInput = false;
				break;
			case MCT_Bool:
				TypeName = TEXT("MCT_Bool");
				SupportedInput = false;
				break;
			case MCT_MaterialAttributes:
				TypeName = TEXT("MCT_MaterialAttributes");
				SupportedInput = false;
				break;
			case MCT_Substrate:
				TypeName = TEXT("MCT_Substrate");
				SupportedInput = false;
				break;
			default:
				TypeName = TEXT("MCT_Unknown");
				SupportedInput = false;
			}

			if (SupportedInput)
			{
				UE_LOGF(LogTextureGraph, Log, "    Input <%ls> type <%ls>", *InputName.ToString(), *TypeName);
			}
			else
			{
				UE_LOGF(LogTextureGraph, Log, "    Input <%ls> type <%ls> #### NOT SUPPORTED ####\n", *InputName.ToString(), *TypeName);
			}
		}
	}

	// For all the outputs of the function, root the one we are interested in to the material.base_color
	for (int i = 0; i < FunctionOutputs.Num(); ++i)
	{
		TObjectPtr<UMaterialExpressionFunctionOutput> ExpOutput = FunctionOutputs[i].ExpressionOutput;
		if (ExpOutput)
		{
			auto Name = ExpOutput->OutputName;
			uint32 Type = ExpOutput->GetOutputValueType(0);

			FString TypeName;
			switch (Type)
			{
			case MCT_Float:
				TypeName = TEXT("MCT_Float");
				OutputDestination->Connect(0, ExpMFC);
				break;
			case MCT_Float2:
				TypeName = TEXT("MCT_Float2");
				OutputDestination->Connect(0, ExpMFC);
				break;
			case MCT_Float3:
				TypeName = TEXT("MCT_Float3");
				OutputDestination->Connect(0, ExpMFC);
				break;
			case MCT_Float4:
				TypeName = TEXT("MCT_Float4");
				OutputDestination->Connect(0, ExpMFC);
				break;
			case MCT_Texture2D:
				TypeName = TEXT("MCT_Texture2D");
				break;
			case MCT_TextureCube:
				TypeName = TEXT("MCT_TextureCube");
				break;
			case MCT_Texture2DArray:
				TypeName = TEXT("MCT_Texture2DArray");
				break;
			case MCT_TextureExternal:
				TypeName = TEXT("MCT_TextureExternal");
				break;
			case MCT_VolumeTexture:
				TypeName = TEXT("MCT_VolumeTexture");
				break;
			case MCT_StaticBool:
				TypeName = TEXT("MCT_StaticBool");
				break;
			case MCT_Bool:
				TypeName = TEXT("MCT_Bool");
				break;
			case MCT_MaterialAttributes:
				TypeName = TEXT("MCT_MaterialAttributes");
				break;
			case MCT_Substrate:
				TypeName = TEXT("MCT_Substrate");
				break;
			default:
				TypeName = TEXT("MCT_Unknown");
			}

			UE_LOGF(LogTextureGraph, Log, "    Output <%ls> type <%ls>\n", *Name.ToString(), *TypeName);
		}
	}

	ExpMFC->UpdateFromFunctionResource();

	// let the material update itself if necessary
	RefMaterial->PreEditChange(NULL);
	RefMaterial->PostEditChange();

	// For debug purpose, save the runtime material as an asset
	//SaveRuntimeMaterialAsAsset(RefMaterial);

	return  RefMaterial;
#else
	return nullptr;
#endif
}
