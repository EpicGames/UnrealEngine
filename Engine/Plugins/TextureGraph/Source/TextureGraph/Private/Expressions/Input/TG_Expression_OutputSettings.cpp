// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Input/TG_Expression_OutputSettings.h"

#include "TG_Graph.h"
#include "Model/StaticImageResource.h"
#include "TG_HelperFunctions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_OutputSettings)

void UTG_Expression_OutputSettings::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);
	
	auto SettingsPin = GetParentNode()->GetPin("Settings");
	auto InputPin = GetParentNode()->GetPin("Input");

	if (InputPin->IsConnected() && SettingsPin)
	{
		if (Settings.BaseName.ToString() == "None" || PreviousInput.BaseName == Settings.BaseName)
		{
			Settings.BaseName = Input.BaseName;
		}
		if (Settings.FolderPath.ToString() == "None" || PreviousInput.FolderPath == Settings.FolderPath)
		{
			Settings.FolderPath = Input.FolderPath;
		}
		if (Settings.Height == EResolution::Auto || PreviousInput.Height == Settings.Height)
		{
			Settings.Height = Input.Height;
		}
		if (Settings.Width == EResolution::Auto || PreviousInput.Width == Settings.Width)
		{
			Settings.Width = Input.Width;
		}
		if (Settings.TextureFormat < Input.TextureFormat || PreviousInput.TextureFormat == Settings.TextureFormat)
		{
			Settings.TextureFormat = Input.TextureFormat;
		}
		if (Settings.LODGroup < Input.LODGroup || PreviousInput.LODGroup == Settings.LODGroup)
		{
			Settings.LODGroup = Input.LODGroup;
		}
		if (Settings.Compression < Input.Compression || PreviousInput.Compression == Settings.Compression)
		{
			Settings.Compression = Input.Compression;
		}
		if (Settings.bSRGB < Input.bSRGB || PreviousInput.bSRGB == Settings.bSRGB)
		{
			Settings.bSRGB = Input.bSRGB;
		}
		if (Settings.TexturePresetType < Input.TexturePresetType || PreviousInput.TexturePresetType == Settings.TexturePresetType)
		{
			Settings.TexturePresetType = Input.TexturePresetType;
		}
		
		SettingsPin->EditSelfVar()->EditAs<FTG_OutputSettings>() = Settings;

		Output = Settings;
	}
	else Output = Settings;

	PreviousInput = Input;
}

bool UTG_Expression_OutputSettings::Validate(MixUpdateCyclePtr Cycle)
{
	FString Errors;

	const UTG_Pin* OutputSettingsPin = GetParentNode()->GetPin("Settings");

	if(!OutputSettingsPin->IsConnected() && !Settings.Validate(Errors))
	{
		UMixInterface* ParentMix = Cast<UMixInterface>(GetOutermostObject());
		auto ErrorType = static_cast<int32>(ETextureGraphErrorType::NODE_WARNING);
		TextureGraphEngine::GetErrorReporter(ParentMix)->ReportWarning(ErrorType, Errors, GetParentNode());
	}

	return Super::Validate(Cycle);
}
void UTG_Expression_OutputSettings::SetTitleName(FName NewName)
{
	GetParentNode()->GetPin("Settings")->SetAliasName(NewName);
}

FName UTG_Expression_OutputSettings::GetTitleName() const
{
	return GetParentNode()->GetPin("Settings")->GetAliasName();
}
