// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Input/TG_Expression_String.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_String)


void UTG_Expression_String::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	// The Value is updated either as an input or as a setting and then becomes the output for this expression
	// The pin out is named "ValueOut"
	ValueOut = String;
}

void UTG_Expression_String::SetTitleName(FName NewName)
{
	GetParentNode()->GetInputPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_String, String))->SetAliasName(NewName);
}

FName UTG_Expression_String::GetTitleName() const
{
	return GetParentNode()->GetInputPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_String, String))->GetAliasName();
}
