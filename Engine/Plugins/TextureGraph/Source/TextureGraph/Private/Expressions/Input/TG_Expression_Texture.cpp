// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Input/TG_Expression_Texture.h"

#include "TG_Graph.h"
#include "2D/TextureHelper.h"
#include "Model/StaticImageResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Texture)

void UTG_Expression_Texture::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);
	
	if (Texture && Texture.RasterBlob)
	{
		Output = Texture;
	}
	else if (Source)
	{
		const FString Path = Source->GetPathName();

		if (!Output || OutputPath != Path)
		{
			UStaticImageResource* StaticImageResource = UStaticImageResource::CreateNew<UStaticImageResource>();
			StaticImageResource->SetAssetUUID(Path);

			//Until we have srgb value exposed in the UI we need to set the Srgb of the Output Descriptor here from the source
			//This gets updated for the late bond case but since we do not have the UI to specify the override in other nodes 
			// the override value will always be set to false while combining the buffers
			auto DesiredDesc = Output.GetBufferDescriptor();
			DesiredDesc.bIsSRGB = Source->SRGB;
			Output = StaticImageResource->GetBlob(InContext->Cycle, &DesiredDesc, 0);
			OutputPath = Path;
		}
	}
	else
	{
		Output = FTG_Texture::GetBlack();
	}
}

bool UTG_Expression_Texture::Validate(MixUpdateCyclePtr Cycle)
{
	UMixInterface* ParentMix = Cast<UMixInterface>(GetOutermostObject());
	
	//Check here if Source is VT
	if (Source && !CanHandleAsset(Source))
	{
		auto ErrorType = static_cast<int32>(ETextureGraphErrorType::UNSUPPORTED_TYPE);

		UClass* Type = Source->GetClass();
		FString TypeName = Source->VirtualTextureStreaming ? "Virtual Texture" : Type->GetName();

		TextureGraphEngine::GetErrorReporter(ParentMix)->ReportError(ErrorType, FString::Printf(TEXT("%s not supported at the moment"), *TypeName), GetParentNode());
		return false;
	}
	
	return true;
}


void UTG_Expression_Texture::SetSource(UTexture* InSource)
{
	bool bHasChanged = Source != InSource;
	Source = InSource;

	if (bHasChanged)
	{
		SetSourceInternal();
	}
}

void UTG_Expression_Texture::SetSourceInternal()
{
	FString PrevTexturePath = Texture.TexturePath;
	if (Source)
	{
		Texture.TexturePath = Source->GetPathName();
	}
	else
	{
		Texture.TexturePath = "";
	}
	// We need to find its property and trigger property change event manually.
	const auto SourcePin = GetParentNode()->GetInputPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Texture, Source));
	check(SourcePin)
	if(SourcePin)
	{
		FProperty* Property = SourcePin->GetExpressionProperty();
		NotifyExpressionChanged(FPropertyChangedEvent(Property));
	}
	
	// if TexturePath changed
	if (PrevTexturePath != Texture.TexturePath)
	{
		// We need to find its property and trigger property change event manually.
		const auto TexturePin = GetParentNode()->GetInputPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Texture, Texture));
		check(TexturePin)
		if(TexturePin)
		{
			FProperty* Property = TexturePin->GetExpressionProperty();
			NotifyExpressionChanged(FPropertyChangedEvent(Property));
		}
	}
}

void UTG_Expression_Texture::SetTexture(const FTG_Texture& InTexture)
{
	bool bHasChanged = Texture != InTexture;
	Texture = InTexture;

	if (bHasChanged)
	{
		SetTextureInternal();
	}
}

void UTG_Expression_Texture::SetTextureInternal()
{
	bool bTextureHasPath = !Texture.TexturePath.IsEmpty();
	if (bTextureHasPath)
	{
		FSoftObjectPath ObjectPath(Texture.TexturePath);
		SetSource(Cast<UTexture>(ObjectPath.TryLoad()));
	}
}
#if WITH_EDITOR
void UTG_Expression_Texture::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// if Graph changes catch it first
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTG_Expression_Texture, Source))
	{
		SetSourceInternal();
	}
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UTG_Expression_Texture, Texture))
	{
		SetTextureInternal();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UTG_Expression_Texture::SetTitleName(FName NewName)
{
	GetParentNode()->GetPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Texture, Texture))->SetAliasName(NewName);
}

FName UTG_Expression_Texture::GetTitleName() const
{
	return GetParentNode()->GetPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Texture, Texture))->GetAliasName();
}

void UTG_Expression_Texture::SetAsset(UObject* Asset)
{
	if(CanHandleAsset(Asset))
	{
		Modify();

		Source = Cast<UTexture>(Asset);

#if WITH_EDITOR
		// We need to find its property and trigger property change event manually.
		const auto SourcePin = GetParentNode()->GetInputPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Texture, Source));

		check(SourcePin);
	
		if(SourcePin)
		{
			auto Property = SourcePin->GetExpressionProperty();
			PropertyChangeTriggered(Property, EPropertyChangeType::ValueSet);
		}
#endif
	}
}

bool UTG_Expression_Texture::CanHandleAsset(UObject* Asset)
{
	return TextureHelper::CanSupportTexture(Cast<UTexture>(Asset));
}
