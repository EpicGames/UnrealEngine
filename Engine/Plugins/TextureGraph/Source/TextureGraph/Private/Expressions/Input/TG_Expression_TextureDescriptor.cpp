// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Input/TG_Expression_TextureDescriptor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_TextureDescriptor)

void UTG_Expression_TextureDescriptor::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	FTextureGraphErrorReporter* ErrorReporter = TextureGraphEngine::GetErrorReporter(InContext->Cycle->GetMix());
	static const int32 MinResolution = 8;
	static const int32 MaxResolution = (int32)EResolution::Resolution8192;
	static const int32 Auto = (int32)EResolution::Auto;

	if (Width < MinResolution || Width > MaxResolution)
	{
		if (ErrorReporter)
		{
			ErrorReporter->ReportWarning((int32)ETextureGraphErrorType::NODE_WARNING, FString::Printf(TEXT("Invalid width specified: %d (Range: %d - %d). Using Auto instead."), 
				Width, MinResolution, MaxResolution));
		}
		Width = Auto;
	}

	if (Height < MinResolution || Height > MaxResolution)
	{
		if (ErrorReporter)
		{
			ErrorReporter->ReportWarning((int32)ETextureGraphErrorType::NODE_WARNING, FString::Printf(TEXT("Invalid height specified: %d (Range: %d - %d). Using Auto instead."), 
				Height, MinResolution, MaxResolution));
		}
		Height = Auto;
	}

	// The Value is updated either as an input or as a setting and then becomes the output for this expression
	// The pin out is named "ValueOut"
	Output.Width = (EResolution)Width;
	Output.Height = (EResolution)Height;
	Output.bIsSRGB = bIsSRGB;
	Output.TextureFormat = Format;
}
