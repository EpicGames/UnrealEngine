// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Input/TG_Expression_TexturePath.h"

#include "TG_Graph.h"
#include "2D/TextureHelper.h"
#include "Model/StaticImageResource.h"
#include "HAL/FileManagerGeneric.h"

// Special case for TexturePath Constant signature, we want to keep the Path Input connectable in that case
// so do this in the override version of BuildInputConstantSignature()

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_TexturePath)
FTG_SignaturePtr  UTG_Expression_TexturePath::BuildInputConstantSignature() const
{
	FTG_Signature::FInit SignatureInit = GetSignatureInitArgsFromClass();
	for (auto& Arg : SignatureInit.Arguments)
	{
		if (Arg.IsInput() && Arg.IsParam())
		{
			Arg.ArgumentType = Arg.ArgumentType.Unparamed();
		}
	}
	return MakeShared<FTG_Signature>(SignatureInit);
}

bool UTG_Expression_TexturePath::ValidateInputPath(FString& ValidatedPath) const
{
	// empty but that's ok
	if (Path.IsEmpty())
	{
		return true;
	}

	// Check that the local path exists
	FString LocalPath = Path.TrimQuotes();
	FPackagePath PackagePath;
	FString PathExt = FPaths::GetExtension(Path);

	// Try to find a file in a mounted package
	if (FPackagePath::TryFromMountedName(LocalPath, PackagePath))
	{
		LocalPath = PackagePath.GetLocalFullPath();
		FString LocalPathExt = FPaths::GetExtension(LocalPath);

		if (LocalPathExt != PathExt)
		{
			LocalPath = FPaths::ChangeExtension(LocalPath, PathExt);
		}
		ValidatedPath = FPaths::ConvertRelativePathToFull(LocalPath);
		return true;
	}
	else if(FPaths::FileExists(LocalPath) || FPaths::DirectoryExists(LocalPath) || FPaths::DirectoryExists(Path))
	{
		ValidatedPath = FPaths::ConvertRelativePathToFull(LocalPath);
		return true;
	}

	return false;
}

TiledBlobPtr UTG_Expression_TexturePath::LoadStaticImage(FTG_EvaluationContext* InContext, const FString& LocalPath, BufferDescriptor* DesiredDesc)
{
	UStaticImageResource* StaticImageResource = UStaticImageResource::CreateNew<UStaticImageResource>();
	StaticImageResource->SetAssetUUID(LocalPath);
	StaticImageResource->SetIsFileSystem(true);

	return StaticImageResource->GetBlob(InContext->Cycle, DesiredDesc, 0);
}

void UTG_Expression_TexturePath::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	FString LocalPath;
	bool bValidation = ValidateInputPath(LocalPath);

	// Determine if we actually need to reload
	// Key insight: Don't touch Output unless we're going to replace it!
	bool bOutputIsValid = (Output.Num() > 0) && 
						  Output.Get(0).GetTexture().RasterBlob && 
						  Output.Get(0).GetTexture().RasterBlob->IsValid();
    
	bool bPathChanged = (LocalPath != OutputPath);
	bool bNeedsReload = bPathChanged || !bOutputIsValid;

	if (bNeedsReload)
	{
		if (!Path.IsEmpty() && bValidation)
		{
			FPaths::MakePlatformFilename(LocalPath);
			if (FPaths::FileExists(LocalPath))
			{
				Output.SetNum(1);
				FTG_Texture Item = LoadStaticImage(InContext, LocalPath, nullptr);
				Output.Set(0, Item);
				OutputPath = LocalPath;
			}
			else if (FPaths::DirectoryExists(LocalPath))
			{
				FFileManagerGeneric FileManager;
				TArray<FString> Files;
				FileManager.FindFiles(Files, *LocalPath);

				if (!Files.IsEmpty())
				{
					Output.SetNum(Files.Num());

					/// Now load a texture against all the files
					for (int32 Index = 0; Index < Files.Num(); Index++)
					{
						const FString& File = Files[Index];
						FString Filename = FPaths::Combine(LocalPath, File);
						FTG_Texture Item = LoadStaticImage(InContext, Filename, nullptr);
						Output.Set(Index, Item);
					}
				}
				else
				{
					// Directory exists but empty - set black
					Output.SetNum(1);
					Output.Set(0, FTG_Texture::GetBlack());
				}
			}
			else
			{
				// Path validation passed but file/dir doesn't exist (edge case)
				Output.SetNum(1);
				Output.Set(0, FTG_Texture::GetBlack());
			}
		}
		else
		{
			// Invalid or empty path - set black as fallback
			Output.SetNum(1);
			Output.Set(0, FTG_Texture::GetBlack());
		}
		
		// Update cached path
		OutputPath = LocalPath;
	}
	
	//For the connected pin we will report error here in eveluate because it does not have the updated value during validation.
	UTG_Pin* PathPin = GetParentNode()->GetPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_TexturePath, Path));

	if (PathPin->IsConnected() && !bValidation)
	{
		ReportError(InContext->Cycle);
	}
}

bool UTG_Expression_TexturePath::Validate(MixUpdateCyclePtr Cycle)
{
	FString LocalPath;
	UTG_Pin* PathPin = GetParentNode()->GetPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_TexturePath, Path));

	if (!PathPin->IsConnected() && !ValidateInputPath(LocalPath))
	{
		ReportError(Cycle);
		return true;
	}

	return true;
}

void UTG_Expression_TexturePath::ReportError(MixUpdateCyclePtr Cycle)
{
	auto ErrorType = static_cast<int32>(ETextureGraphErrorType::NODE_WARNING);
	TextureGraphEngine::GetErrorReporter(Cycle->GetMix())->ReportWarning(ErrorType, FString::Printf(TEXT("Input Path <%s> is not a valid local path"), *Path), GetParentNode());
}

void UTG_Expression_TexturePath::SetTitleName(FName NewName)
{
	GetParentNode()->GetPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_TexturePath, Path))->SetAliasName(NewName);
}

FName UTG_Expression_TexturePath::GetTitleName() const
{
	return GetParentNode()->GetPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_TexturePath, Path))->GetAliasName();
}

