// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Maths/TG_Expression_Blend.h"
#include "Transform/Expressions/T_Blend.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Blend)

void UTG_Expression_Blend::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if (!Mask)
	{
		/// Temporary, since our Grayscale texture saving (for test framework) isn't working correctly
		if (TextureGraphEngine::IsTestMode())
		{
			Mask = FTG_Texture::GetWhite();
		}
		else
		{
			Mask = FTG_Texture::GetWhiteMask();
		}
	}

	if (!Background || !Foreground)
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	T_Blend::FBlendSettings BlendSettings;
	BlendSettings.ForegroundTexture = Foreground;
	BlendSettings.BackgroundTexture = Background;
	BlendSettings.Mask = Mask;
	BlendSettings.Opacity = Opacity;
	BlendSettings.bIgnoreAlpha = bIgnoreAlpha;
	BlendSettings.bClamp = bClamp;
	
	Output = T_Blend::Create(InContext->Cycle, Output.GetBufferDescriptor(), InContext->TargetId, BlendMode, &BlendSettings);
}
