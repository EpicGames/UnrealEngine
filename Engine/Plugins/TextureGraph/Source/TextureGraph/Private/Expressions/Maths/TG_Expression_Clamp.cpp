// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Maths/TG_Expression_Clamp.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Expressions/T_Clamp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Clamp)

float UTG_Expression_Clamp::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 3);
	return std::clamp<float>(ValuePtr[0], ValuePtr[1], ValuePtr[2]);
}

FTG_Texture	UTG_Expression_Clamp::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	if (!Input.IsTexture() || !Input.GetTexture())
		return FTG_Texture::GetBlack();

	return T_Clamp::CreateClamp(InContext->Cycle, Output.EditTexture().GetBufferDescriptor(), Input.GetTexture(InContext), 
		MinValue.GetTexture(InContext, FTG_Texture::GetBlack()), MaxValue.GetTexture(InContext, FTG_Texture::GetWhite()), InContext->TargetId);
}
//////////////////////////////////////////////////////////////////////////
float SmoothStep(float Min, float Max, float X) 
{
	// Scale, bias and saturate X to 0..1 range
	X = std::clamp((X - Min) / (Max - Min), 0.0f, 1.0f);
	// Evaluate polynomial
	return X * X * (3 - 2 * X);
}

float UTG_Expression_SmoothStep::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 3);
	return SmoothStep(ValuePtr[1], ValuePtr[2], ValuePtr[0]);
}

FTG_Texture	UTG_Expression_SmoothStep::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	if (!Input.IsTexture() || !Input.GetTexture())
		return FTG_Texture::GetBlack();

	return T_Clamp::CreateSmoothStep(InContext->Cycle, Output.EditTexture().GetBufferDescriptor(), Input.GetTexture(InContext), 
		Min.GetTexture(InContext, FTG_Texture::GetBlack()), Max.GetTexture(InContext, FTG_Texture::GetWhite()), InContext->TargetId);
}

//////////////////////////////////////////////////////////////////////////
float UTG_Expression_Min::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 2);
	return std::min(ValuePtr[0], ValuePtr[1]);
}

FTG_Texture	UTG_Expression_Min::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	return T_Clamp::CreateMin(InContext->Cycle, Output.EditTexture().GetBufferDescriptor(), Input1.GetTexture(InContext), Input2.GetTexture(InContext), InContext->TargetId);
}

//////////////////////////////////////////////////////////////////////////
float UTG_Expression_Max::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 2);
	return std::max(ValuePtr[0], ValuePtr[1]);
}

FTG_Texture	UTG_Expression_Max::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	return T_Clamp::CreateMax(InContext->Cycle, Output.EditTexture().GetBufferDescriptor(), Input1.GetTexture(InContext), Input2.GetTexture(InContext), InContext->TargetId);
}

