// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Maths/TG_Expression_IfThenElse.h"
#include "FxMat/MaterialManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_IfThenElse)

typedef bool (*CmpFunc)(float, float);

static bool CmpGT(float LHS, float RHS) { return LHS > RHS; }
static bool CmpGTE(float LHS, float RHS) { return LHS >= RHS; }
static bool CmpLT(float LHS, float RHS) { return LHS < RHS; }
static bool CmpLTE(float LHS, float RHS) { return LHS <= RHS; }
static bool CmpEQ(float LHS, float RHS) { return std::abs(LHS - RHS) <= std::numeric_limits<float>::epsilon(); }
static bool CmpNEQ(float LHS, float RHS) { return std::abs(LHS - RHS) > std::numeric_limits<float>::epsilon(); }

static constexpr CmpFunc CmpFunctions[(int32)EIfThenElseOperator::NEQ + 1] = { CmpGT, CmpGTE, CmpLT, CmpLTE, CmpEQ, CmpNEQ };

float UTG_Expression_IfThenElse::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count)
{
	check(Value && Count == 4);
	check(Operator >= EIfThenElseOperator:: GT && Operator <= EIfThenElseOperator::NEQ);
	return CmpFunctions[(int32)Operator](Value[0], Value[1]) ? Value[2] : Value[3];
}

FVector4f UTG_Expression_IfThenElse::EvaluateVector_WithValue(FTG_EvaluationContext* InContext, const FVector4f* const Value, size_t Count)
{
	check(Value && Count == 4);
	check(Operator >= EIfThenElseOperator:: GT && Operator <= EIfThenElseOperator::NEQ);

	auto Func = CmpFunctions[(int32)Operator];

	if (ComparisonType == EIfThenElseType::IndividualComponent)
	{
		/// The base implementation does a component-wise EvaluatScalar anyway ...
		return Super::EvaluateVector_WithValue(InContext, Value, Count);
	}
	else if (ComparisonType == EIfThenElseType::AllComponents)
	{
		const float* LHSPtr = reinterpret_cast<const float*>(Value + 0);
		const float* RHSPtr = reinterpret_cast<const float*>(Value + 1);
		bool DidPass = true;

		for (int32 I = 0; I < 4 && DidPass; I++)
			DidPass &= Func(LHSPtr[I], RHSPtr[I]);

		return DidPass ? Value[2] : Value[3];
	}

	float LHSGray = 0.299 * Value[0].X + 0.587 * Value[0].Y + 0.114 * Value[0].Z;
	float RHSGray = 0.299 * Value[1].X + 0.587 * Value[1].Y + 0.114 * Value[1].Z;

	return Func(LHSGray, RHSGray) ? Value[2] : Value[3];
}

FTG_Texture UTG_Expression_IfThenElse::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	return T_Maths_TwoInputs::CreateIfThenElse(InContext->Cycle, Output.EditTexture().GetBufferDescriptor(), InContext->TargetId, 
		LHS.GetTexture(InContext), RHS.GetTexture(InContext), Then.GetTexture(InContext), Else.GetTexture(InContext), Operator, ComparisonType);
}
