// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Maths/TG_Expression_Invert.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Expressions/T_Invert.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Invert)

void UTG_Expression_Invert::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if (!Input)
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_Invert>(TEXT("T_Invert"));
	check(RenderMaterial);

	Output = T_Invert::Create(InContext->Cycle, Output.GetBufferDescriptor(), Input, MaxValue, InContext->TargetId, IncludeAlpha, Clamp);
}
