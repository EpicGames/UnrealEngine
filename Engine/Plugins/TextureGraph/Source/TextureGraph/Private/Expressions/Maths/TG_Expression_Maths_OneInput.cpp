// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Maths/TG_Expression_Maths_OneInput.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Maths_OneInput)

typedef float (*TrigFunc)(float);

static constexpr TrigFunc TrigFunctions[(int32)ETrigFunction::ATan + 1] = { sin, cos, tan, asin, acos, atan };

//////////////////////////////////////////////////////////////////////////
float UTG_Expression_Trigonometry::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 1);
	check(Function >= ETrigFunction::Sin && Function <= ETrigFunction::ATan);

	auto CurrentFunction = TrigFunctions[(int32)Function];
	return CurrentFunction(ValuePtr[0]);
}

FTG_Texture	UTG_Expression_Trigonometry::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	return T_Maths_OneInput::CreateTrigonometry(InContext->Cycle, Output.EditTexture().GetBufferDescriptor(), InContext->TargetId,
		Angle.GetTexture(InContext).RasterBlob, Function);
}

//////////////////////////////////////////////////////////////////////////

#define IMPL_SINGLE_ARG_EXPR(Name, FloatFunc) \
float UTG_Expression_##Name::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count) \
{ \
	check(ValuePtr && Count == 1); \
	return FloatFunc(ValuePtr[0]); \
} \
\
FTG_Texture	UTG_Expression_##Name::EvaluateTexture(FTG_EvaluationContext* InContext) \
{ \
	return T_Maths_OneInput::Create##Name(InContext->Cycle, Output.EditTexture().GetBufferDescriptor(), InContext->TargetId, Input.GetTexture(InContext)); \
}

static float TG_Square(float Value) { return Value * Value; }
static float TG_Cube(float Value) { return Value * Value * Value; }

IMPL_SINGLE_ARG_EXPR(Abs, std::abs);
IMPL_SINGLE_ARG_EXPR(Sqrt, std::sqrt);
IMPL_SINGLE_ARG_EXPR(Square, TG_Square);
IMPL_SINGLE_ARG_EXPR(Cube, TG_Cube);
IMPL_SINGLE_ARG_EXPR(Cbrt, std::cbrt);
IMPL_SINGLE_ARG_EXPR(Exp, std::exp);
IMPL_SINGLE_ARG_EXPR(Log2, std::log2);
IMPL_SINGLE_ARG_EXPR(Log10, std::log10);
IMPL_SINGLE_ARG_EXPR(Log, std::log);
IMPL_SINGLE_ARG_EXPR(Floor, std::floor);
IMPL_SINGLE_ARG_EXPR(Ceil, std::ceil);
IMPL_SINGLE_ARG_EXPR(Round, std::round);

#undef IMPL_SINGLE_ARG_EXPR
