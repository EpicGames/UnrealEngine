// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Maths/TG_Expression_Maths_ThreeInputs.h"
#include "Transform/Expressions/T_Maths_ThreeInputs.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"

//////////////////////////////////////////////////////////////////////////
/// MAD
//////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Maths_ThreeInputs)
float UTG_Expression_MAD::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 3);
	return (ValuePtr[0] * ValuePtr[1]) + ValuePtr[2];
}

FTG_Texture	UTG_Expression_MAD::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	return T_Maths_ThreeInputs::CreateMad(InContext->Cycle, Output.EditTexture().GetBufferDescriptor(), InContext->TargetId,
		Input1.GetTexture(InContext), Input2.GetTexture(InContext), Input3.GetTexture(InContext));
}

//////////////////////////////////////////////////////////////////////////
/// Lerp
//////////////////////////////////////////////////////////////////////////
float UTG_Expression_Lerp::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) 
{
	check(Value && Count == 3);
	return FMath::Lerp(Value[0], Value[1], Value[2]);
}

FTG_Texture	UTG_Expression_Lerp::EvaluateTexture(FTG_EvaluationContext* InContext) 
{
	return T_Maths_ThreeInputs::CreateLerp(InContext->Cycle, Output.EditTexture().GetBufferDescriptor(), InContext->TargetId,
		Input1.GetTexture(InContext), Input2.GetTexture(InContext), LerpValue.GetTexture(InContext));
}
