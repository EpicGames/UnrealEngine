// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Maths/TG_Expression_Maths_TwoInputs.h"
#include "Transform/Expressions/T_Maths_TwoInputs.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "TextureGraphEngine/Helper/MathUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Maths_TwoInputs)

typedef std::function<TiledBlobPtr(MixUpdateCyclePtr /* Cycle */, BufferDescriptor DesiredOutputDesc, int32 /*TargetId*/, TiledBlobPtr /*Operand1*/, TiledBlobPtr /*Operand2*/)> MathOpFunc;

FORCEINLINE TiledBlobPtr GenericMathOp(FTG_EvaluationContext* InContext, MathOpFunc Func, BufferDescriptor DesiredOutputDesc, const FTG_Texture& Operand1, const FTG_Texture& Operand2)
{
	return Func(InContext->Cycle, DesiredOutputDesc, InContext->TargetId, Operand1.RasterBlob, Operand2.RasterBlob);
}

/// Multiply
float UTG_Expression_Multiply::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 2);
	return ValuePtr[0] * ValuePtr[1];
}

FTG_Texture	UTG_Expression_Multiply::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	return GenericMathOp(InContext, T_Maths_TwoInputs::CreateMultiply, Output.EditTexture().GetBufferDescriptor(), Input1.GetTexture(InContext), Input2.GetTexture(InContext));
}

/// Divide
float UTG_Expression_Divide::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 2);
	if (std::abs(ValuePtr[1]) < std::numeric_limits<float>::epsilon())
		return 0;
	return ValuePtr[0] / ValuePtr[1];
}

FTG_Texture	UTG_Expression_Divide::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	return GenericMathOp(InContext, T_Maths_TwoInputs::CreateDivide, Output.EditTexture().GetBufferDescriptor(), Input1.GetTexture(InContext), Input2.GetTexture(InContext));
}

/// Add
float UTG_Expression_Add::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 2);
	return ValuePtr[0] + ValuePtr[1];
}

FTG_Texture	UTG_Expression_Add::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	return GenericMathOp(InContext, T_Maths_TwoInputs::CreateAdd, Output.EditTexture().GetBufferDescriptor(), Input1.GetTexture(InContext), Input2.GetTexture(InContext));
}

/// Subtract
float UTG_Expression_Subtract::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 2);
	return ValuePtr[0] - ValuePtr[1];
}

FTG_Texture	UTG_Expression_Subtract::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	return GenericMathOp(InContext, T_Maths_TwoInputs::CreateSubtract, Output.EditTexture().GetBufferDescriptor(), Input1.GetTexture(InContext), Input2.GetTexture(InContext));
}

//////////////////////////////////////////////////////////////////////////
/// Dot
//////////////////////////////////////////////////////////////////////////
float UTG_Expression_Dot::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count)
{
	return  ValuePtr[0] * ValuePtr[1];
}

FVector4f UTG_Expression_Dot::EvaluateVector_WithValue(FTG_EvaluationContext* InContext, const FVector4f* const Value, size_t Count)
{
	check(Value && Count == 2);
	float Result = FVector3f::DotProduct(Value[0], Value[1]);
	return FVector4f{ Result, Result, Result, 1 };
}

FTG_Texture	UTG_Expression_Dot::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	auto Desc = Output.EditTexture().GetBufferDescriptor();
	return T_Maths_TwoInputs::CreateDot(InContext->Cycle, Desc, InContext->TargetId, Input1.GetTexture(InContext), Input2.GetTexture(InContext));
}

FTG_Variant::EType	UTG_Expression_Dot::EvalExpressionCommonOutputVariantType() const
{
	// Run the default behavior for comon type and force to scalar if it is NOT a texture type
	auto DefaultCommonType = GetCommonInputVariantType();
	switch (DefaultCommonType)
	{
	case FTG_Variant::EType::Scalar:
	case FTG_Variant::EType::Color:
	case FTG_Variant::EType::Vector:
		return FTG_Variant::EType::Scalar;
	case FTG_Variant::EType::Texture:
		return FTG_Variant::EType::Texture;
	case FTG_Variant::EType::Invalid:
	default:
		return FTG_Variant::EType::Invalid; // THis shouldn't happen
	}
}


void UTG_Expression_Dot::Evaluate(FTG_EvaluationContext* InContext)
{
	UTG_Expression::Evaluate(InContext);

	// For dot product, common output variant is scalar for simple types or texture
	switch (GetCommonInputVariantType())
	{
	case FTG_Variant::EType::Scalar:
		Output.EditScalar() = EvaluateScalar(InContext);
		break;
	case FTG_Variant::EType::Color:
	{
		Output.EditScalar() = EvaluateColor(InContext).R;
		break;
	}
	case FTG_Variant::EType::Vector:
	{
		Output.EditScalar() = EvaluateVector(InContext).X;
		break;
	}
	case FTG_Variant::EType::Texture:
	{
		Output.EditTexture() = EvaluateTexture(InContext).RasterBlob;
		break;
	}
	default:
	{
		Output.EditScalar() = 0.0f;
		break;
	}
	}
}

//////////////////////////////////////////////////////////////////////////
/// Cross 
//////////////////////////////////////////////////////////////////////////
float UTG_Expression_Cross::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count)
{
	return 0;
}

FVector4f UTG_Expression_Cross::EvaluateVector_WithValue(FTG_EvaluationContext* InContext, const FVector4f* const Value, size_t Count)
{
	check(Value && Count == 2);
	return FVector3f::CrossProduct(Value[0], Value[1]);
}

FTG_Texture	UTG_Expression_Cross::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	return T_Maths_TwoInputs::CreateCross(InContext->Cycle, Output.EditTexture().GetBufferDescriptor(), InContext->TargetId, Input1.GetTexture(InContext), Input2.GetTexture(InContext));
}

//////////////////////////////////////////////////////////////////////////
/// Step
//////////////////////////////////////////////////////////////////////////
float UTG_Expression_Step::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 2);
	float Y = ValuePtr[0];
	float X = ValuePtr[1];
	return MathUtils::Step(Y, X);
}

FTG_Texture	UTG_Expression_Step::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	return T_Maths_TwoInputs::CreateStep(InContext->Cycle, Output.EditTexture().GetBufferDescriptor(), InContext->TargetId, Input1.GetTexture(InContext), Input2.GetTexture(InContext));
}

//////////////////////////////////////////////////////////////////////////
/// Power
//////////////////////////////////////////////////////////////////////////
float UTG_Expression_Pow::EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count)
{
	check(ValuePtr && Count == 2);
	return std::powf(ValuePtr[0], ValuePtr[1]);
}

FTG_Texture	UTG_Expression_Pow::EvaluateTexture(FTG_EvaluationContext* InContext)
{
	return T_Maths_TwoInputs::CreatePow(InContext->Cycle, Output.EditTexture().GetBufferDescriptor(), InContext->TargetId, 
		Base.GetTexture(InContext), Exponent.GetTexture(InContext));
}
