// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Output/TG_Expression_Output.h"
#include "FxMat/FxMaterial.h"
#include "TG_HelperFunctions.h"
#include "FxMat/MaterialManager.h"
#include "Job/Job.h"
#include "Job/JobArgs.h"
#include "TG_Graph.h"
#include "Transform/Expressions/T_FlatColorTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Output)

void UTG_Expression_Output::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	//There are three cases:

	// - No Source
	// When we don't have a valid source, we need to set the output to black texture and update the target texture set map. This will update the viewport.
	//
	// - Valid source with the same output descriptor
	// We just need to set the output texture and update the texture set map
	//
	// - Valid source with different output descriptor
	// We need to create a job. Resultant tiled blob should be set in the texture set map.
	//
	// We need to make sure Output is updated in every case.

	if (Source)
	{

		if (Source.IsTexture())
		{
			UpdateBufferDescriptorValues();

			BufferDescriptor OutputDesc = Output.EditTexture().GetBufferDescriptor();
			const BufferDescriptor& InputDesc = Source.GetTexture()->GetDescriptor();

			/// If the descriptors are not the same
			if (OutputDesc != InputDesc)
			{
				RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterialOfType_FX<Fx_FullScreenCopy>(TEXT("OutputCopy"));

				check(RenderMaterial);

				JobUPtr RenderJob = std::make_unique<Job>(InContext->Cycle->GetMix(), InContext->TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));

				int32 Format = (int32)OutputDesc.Format;

				RenderJob
					->AddArg(ARG_BLOB(Source.GetTexture().RasterBlob, "SourceTexture"))
					->AddArg(WithUnbounded(ARG_INT(OutputDesc.ItemsPerPoint, "ItemsPerPoint")))
					->AddArg(WithUnbounded(ARG_INT(Format, "Format")))
					->AddArg(WithUnbounded(ARG_BOOL(OutputDesc.bIsSRGB, "sRGB")));


				const FString Name = TEXT("Output");
				OutputDesc.Name = Name;

				Output.EditTexture() = RenderJob->InitResult(Name, &OutputDesc);
				InContext->Cycle->AddJob(InContext->TargetId, std::move(RenderJob));
			}
			else
			{
				Output = Source;
			}
		}
		else
		{
			Output = Source;
		}
	}
	else
	{
		// Resetting it to scalar.
		Output.EditScalar() = 0.0f;
	}

	// We only update the output last render for the root graph. We don't need to call it for subgraphs.
	// Last render eventually gets set for target texture set and then displayed on the viewport.
	if (InContext->GraphDepth == 0)
	{
		if(Output.IsTexture() && Output.GetTexture())
		{
			InContext->Cycle->GetTarget(0)->GetLastRender().SetTexture(GetTitleName(), Output.GetTexture().RasterBlob);
		}
		else if (Output.IsColor())
		{
			BufferDescriptor DesiredDesc = T_FlatColorTexture::GetFlatColorDesc("OutputFlat");			
			auto OutputFlatTexture = Source.GetTexture(InContext, FTG_Texture::GetBlack(), &DesiredDesc);
			InContext->Cycle->GetTarget(0)->GetLastRender().SetTexture(GetTitleName(), OutputFlatTexture.RasterBlob);
		}
		else
		{
			InContext->Cycle->GetTarget(0)->GetLastRender().SetTexture(GetTitleName(), FTG_Texture::GetBlack());
		}
	}
}

bool UTG_Expression_Output::Validate(MixUpdateCyclePtr	Cycle)
{
	FString Errors;
	
	const UTG_Pin* OutputSettingsPin = GetParentNode()->GetPin("OutputSettings");

	if(!OutputSettingsPin->IsConnected() && !OutputSettings.Validate(Errors))
	{
		UMixInterface* ParentMix = Cast<UMixInterface>(GetOutermostObject());
		auto ErrorType = static_cast<int32>(ETextureGraphErrorType::NODE_WARNING);
		TextureGraphEngine::GetErrorReporter(ParentMix)->ReportWarning(ErrorType, Errors, GetParentNode());
	}

	return Super::Validate(Cycle);
}

void UTG_Expression_Output::SetTitleName(FName NewName)
{
	GetParentNode()->GetOutputPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Output, Output))->SetAliasName(NewName);
	OutputSettings.OutputName = GetParentNode()->GetOutputPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Output, Output))->GetAliasName();
}

FName UTG_Expression_Output::GetTitleName() const
{
	return GetParentNode()->GetOutputPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Output, Output))->GetAliasName();
}

void UTG_Expression_Output::UpdateBufferDescriptorValues()
{
	FTG_Texture& OutputTexture = Output.EditTexture();

	OutputTexture.Descriptor.Width = OutputSettings.Width;
	OutputTexture.Descriptor.Height = OutputSettings.Height;
	OutputTexture.Descriptor.TextureFormat = OutputSettings.TextureFormat;
	OutputTexture.Descriptor.bIsSRGB = OutputSettings.bSRGB;
}

void UTG_Expression_Output::InitializeOutputSettings()
{
	SetTitleName(GetDefaultName());
	OutputSettings.Initialize(GetParentNode()->GetGraph()->GetPathName(),GetTitleName());

	UTG_Pin* Settings = GetParentNode()->GetPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Output, OutputSettings));
	Settings->FromString(OutputSettings.ToString());
}

void UTG_Expression_Output::SetShouldExport(bool InShouldExport)
{
	OutputSettings.bShouldExport = InShouldExport;

	UTG_Pin* Settings = GetParentNode()->GetPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Output, OutputSettings));
	Settings->FromString(OutputSettings.ToString());
}
