// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Procedural/TG_Expression_Gradient.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Gradient)

void UTG_Expression_Gradient::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	BufferDescriptor Desc = T_Gradient::InitOutputDesc( Output.GetBufferDescriptor());

	int32 CurrentRotation = 0;

	if (Type == EGradientType::GT_Linear_1)
		CurrentRotation = (int32)Rotation;
	else if (Type == EGradientType::GT_Linear_2)
		CurrentRotation = (int32)RotationLimited;

	T_Gradient::Params params =
	{
		.Type = Type,
		.Interpolation = Interpolation,
		.Rotation = CurrentRotation,
		.Center = Center,
		.Radius = Radius,
		.Point1 = Point1,
		.Point2 = Point2
	};

	Output = T_Gradient::Create(InContext->Cycle, Desc, params, InContext->TargetId);
}

#if WITH_EDITOR
bool UTG_Expression_Gradient::CanEditChange(const FProperty* InProperty) const
{
	bool bEditCondition = Super::CanEditChange(InProperty);

	const FName PropertyName = InProperty->GetFName();

	// Specific logic associated with Property
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Gradient, Rotation))
	{
		bEditCondition = Type == EGradientType::GT_Linear_1;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Gradient, RotationLimited))
	{
		bEditCondition = Type == EGradientType::GT_Linear_2;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Gradient, Interpolation))
	{
		bEditCondition = (int32)Type <= (int32)EGradientType::GT_Linear_2;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Gradient, Center) || 
		PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Gradient, Radius))
	{
		bEditCondition = Type == EGradientType::GT_Radial;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Gradient, Point1) || 
		PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Gradient, Point2))
	{
		bEditCondition = (Type == EGradientType::GT_Axial_1 || Type == EGradientType::GT_Axial_2);
	}

	// Default behaviour
	return bEditCondition;
}
#endif
