// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Procedural/TG_Expression_Noise.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Mask/T_NoiseMask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Noise)

void UTG_Expression_Noise::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	FSH_NoiseMask::FPermutationDomain PermutationVector;
	PermutationVector.Set<FVar_NoiseType>(NoiseType);

	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_NoiseMask>(TEXT("T_NoiseMask"), PermutationVector);
	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(InContext->Cycle->GetMix(), InContext->TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	FTileInfo TileInfo;

	RenderJob
		->AddArg(ARG_FLOAT(Seed, "Seed"))
		->AddArg(ARG_FLOAT(Amplitude, "Amplitude"))
		->AddArg(ARG_FLOAT(Frequency, "Frequency"))
		->AddArg(ARG_INT(Octaves, "Octaves"))
		->AddArg(ARG_FLOAT(Lacunarity, "Lacunarity"))
		->AddArg(ARG_FLOAT(Persistence, "Persistance"))
		->AddArg(ARG_FLOAT(0, "Invert"))
		->AddArg(ARG_INT(static_cast<int32>(NoiseType), "NoiseType")) /// Adding the noise type arg to hash the RenderJob result
		->AddArg(std::make_shared<JobArg_TileInfo>(TileInfo, "TileInfo")) /// Enable the tileinfo parameters
		->AddArg(std::make_shared<JobArg_ForceTiling>()) /// Force hashing individual tiles differently
		;

	const FString Name = TEXT("Noise"); /// FString::Printf(TEXT("[%s].[%d].[%llu] Noise"), GetDefaultName(), InContext->TargetId, InContext->Cycle->Batch()->BatchId());

	BufferDescriptor Desc = Output.GetBufferDescriptor();
	
	// Default format to RGBA8
	if (Desc.Format == BufferFormat::Auto)
		Desc.Format = BufferFormat::Byte;
	if (Desc.ItemsPerPoint == 0)
		Desc.ItemsPerPoint = 4;

	Desc.DefaultValue = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);

	if (Desc.Width <= 0)
	{
		Desc.Width = DefaultSize;
	}
	
	if(Desc.Height <= 0)
	{
		Desc.Height = DefaultSize;
	}

	Output = RenderJob->InitResult(Name, &Desc);

	InContext->Cycle->AddJob(InContext->TargetId, std::move(RenderJob));
}
