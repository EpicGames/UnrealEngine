// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Procedural/TG_Expression_Pattern.h"
#include "FxMat/MaterialManager.h"
#include "Transform/Mask/T_PatternMask.h"
#include "Helper/GraphicsUtil.h"
#include "Job/JobBatch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Pattern)

void UTG_Expression_Pattern::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	FSH_PatternMask::FPermutationDomain PermutationVector;
	PermutationVector.Set<FVar_PatternType>(PatternType);

	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_PatternMask>(TEXT("T_PatternMask"), PermutationVector);
	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(InContext->Cycle->GetMix(), InContext->TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	FTileInfo TileInfo;

	RenderJob
		->AddArg(ARG_VECTOR2(Placement.Repeat(), "Repeat"))
		->AddArg(ARG_VECTOR2(Placement.Spacing(), "Spacing"))
		->AddArg(ARG_VECTOR2(Placement.OffsetValue(), "Offset"))
		->AddArg(ARG_FLOAT(false, "Invert"))
		->AddArg(ARG_INT(PatternType, "PatternType")) /// Adding the pattern type arg to hash the job result

		->AddArg(ARG_FLOAT(Bevel.Bevel, "Bevel"))
		->AddArg(ARG_FLOAT(Bevel.BevelCurve, "BevelCurve"))
		
		//->AddArg(ARG_FLOAT(0.0f, "JS_Start"))
		->AddArg(ARG_FLOAT(Jitter.SizeAmount, "JS_Amount"))
		->AddArg(ARG_FLOAT(Jitter.SizeThreshold, "JS_Threshold"))
		->AddArg(ARG_INT(Jitter.SizeSeed, "JS_Seed"))

		->AddArg(ARG_FLOAT(Jitter.BrightnessAmount, "JB_Amount"))
		->AddArg(ARG_FLOAT(Jitter.BrightnessThreshold, "JB_Threshold"))
		->AddArg(ARG_INT(Jitter.BrightnessSeed, "JB_Seed"))
		
		->AddArg(ARG_VECTOR2(Jitter.Tilt(), "JT_Amount"))	//(pmask->TiltJitterAmount() / 720.0f) + 0.5f slider value
		->AddArg(ARG_FLOAT(1 - Jitter.TiltThreshold, "JT_Threshold"))
		->AddArg(ARG_INT(Jitter.TiltSeed, "JT_Seed"))
		
		->AddArg(ARG_VECTOR2(Jitter.Angle(), "JA_Amount"))
		->AddArg(ARG_INT(Jitter.AngleSeed, "JA_Seed"))
		
		->AddArg(ARG_FLOAT(Cutout.CutoffThreshold, "CO_Threshold"))
		->AddArg(ARG_INT(Cutout.CutoffSeed, "CO_Seed"))

		->AddArg(ARG_VECTOR2(GradientDirection.Value(), "GradientDir"))
		
		->AddArg(std::make_shared<JobArg_TileInfo>(TileInfo, "TileInfo")) /// Enable the tileinfo parameters
		->AddArg(std::make_shared<JobArg_ForceTiling>()) /// Force hashing individual tiles differently
		;

	const FString Name = FString::Printf(TEXT("[%s].[%d].[%llu] Pattern"), *GetDefaultName().ToString(), InContext->TargetId, InContext->Cycle->GetBatch()->GetBatchId());

	BufferDescriptor Desc = Output.GetBufferDescriptor();
	// Default format to RGBA8
	if (Desc.Format == BufferFormat::Auto)
		Desc.Format = BufferFormat::Byte;
	if (Desc.ItemsPerPoint == 0)
		Desc.ItemsPerPoint = 4;

	Desc.DefaultValue = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);

	if (Desc.Width <= 0)
	{
		Desc.Width = DefaultSize;
	}
	
	if(Desc.Height <= 0)
	{
		Desc.Height = DefaultSize;
	}

	Output = RenderJob->InitResult(Name, &Desc);

	InContext->Cycle->AddJob(InContext->TargetId, std::move(RenderJob));
}
