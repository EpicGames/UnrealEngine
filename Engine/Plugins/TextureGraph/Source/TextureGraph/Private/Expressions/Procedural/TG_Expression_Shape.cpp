// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Procedural/TG_Expression_Shape.h"
#include "FxMat/MaterialManager.h"
#include "Transform/Mask/T_ShapeMask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Shape)

void UTG_Expression_Shape::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	FTG_TextureDescriptor DesiredDescriptor = Output.Descriptor;

	/// If it's auto then make it grey scale as we don't need more than a single channel output
	if (DesiredDescriptor.TextureFormat == ETG_TextureFormat::Auto)
	{
		DesiredDescriptor.TextureFormat = ETG_TextureFormat::BGRA8;
	}

	T_ShapeMask::FParams Params{
		.Rotation = Orientation * (PI / 180.0f),
		.Size = { Width, Height },
		.Rounding = Rounding,
		.BevelWidth = BevelWidth,
		.BevelCurve = BevelCurve,
		.BlendSDF = ShowSDF
	};
	
	int ShapeTypeInt = (int) ShapeType;
	EShapeMaskType ShaderShapeType = (EShapeMaskType)ShapeTypeInt;
	if (ShapeType == EShapeType::Polygon)
		ShaderShapeType = (EShapeMaskType)PolygonNumSides;

	Output = T_ShapeMask::Create(InContext->Cycle, DesiredDescriptor, ShaderShapeType, Params, InContext->TargetId);
}
#if WITH_EDITOR
bool UTG_Expression_Shape::CanEditChange(const FProperty* InProperty) const
{
	bool bEditCondition = Super::CanEditChange(InProperty);

	// if already set to false Or InProperty not directly owned by us, early out
	// the CanEditChange function also checks down the PropertyChain,
	// checking direct ownership fixes an issue here where deeper in the hierarchy,
	// FTG_TextureDescriptor::Height was also being affected because of the logic for TG_Expression_Shape::Height below 
	if (!bEditCondition && (this->GetClass() != InProperty->GetOwnerClass()))
	{
		return bEditCondition;
	}

	const FName PropertyName = InProperty->GetFName();
	
	// Specific logic associated with Property
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Shape, PolygonNumSides))
	{
		bEditCondition = ShapeType == EShapeType::Polygon;
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Shape, Height))
	{
		bEditCondition = ShapeType == EShapeType::Rectangle || (ShapeType==EShapeType::Ellipse);
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Shape, Orientation))
	{
		bEditCondition = ShapeType != EShapeType::Circle;
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Shape, Rounding))
	{
		bEditCondition = ShapeType != EShapeType::Circle;
	}

	// Default behaviour
	return bEditCondition;
}
#endif
