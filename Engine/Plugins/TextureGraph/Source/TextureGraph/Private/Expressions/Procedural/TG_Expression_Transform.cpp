// Copyright Epic Games, Inc. All Rights Reserved.
#include "Expressions/Procedural/TG_Expression_Transform.h"
#include "FxMat/MaterialManager.h"
#include "Transform/Expressions/T_Transform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Transform)

void UTG_Expression_Transform::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);


	auto DesiredDescriptor = Output.Descriptor;
	FVector2f Spacing = {0, 0};

	FVector2f OutCoverage = Coverage;
	FVector2f OutOffset = Offset;

	// hide debug grid when exporting
	float ShowDebugGridValue = ShowDebugGrid;
	if (InContext->Cycle->GetDetails().bExporting)
	{
		ShowDebugGridValue = 0.0f;
	}
	
	T_Transform::TransformParameter XformParam{
		.Coverage = OutCoverage,
		.Translation = OutOffset,
		.Pivot = Pivot,
		.RotationXY = Rotation * (PI / 180.0f),
		.Scale = Repeat
	};

	T_Transform::CellParameter CellParam{
		.Zoom = Zoom * 0.01f,
		.StretchToFit = StretchToFit,
		.Spacing = Spacing,
		.Stagger = {(StaggerOffset * StaggerHorizontally), (StaggerOffset * (1 - StaggerHorizontally))},
		.Stride = Stride,
	};

	T_Transform::ColorParameter ColorParam{
		.FillColor = FillColor,
		.WrapFilterMode = WrapMode,
		.MirrorX = MirrorX,
		.MirrorY = MirrorY,
		.ShowDebugGrid = ShowDebugGridValue
	};
	
	Output = T_Transform::Create(InContext->Cycle, DesiredDescriptor, Input, XformParam, CellParam, ColorParam, InContext->TargetId);
}

