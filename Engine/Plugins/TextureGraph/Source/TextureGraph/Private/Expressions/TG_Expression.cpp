// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/TG_Expression.h"
#include "TG_Graph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression)

const FName TG_Category::Default = "Default";
const FName TG_Category::Output = "Output";
const FName TG_Category::Input = "Input";
const FName TG_Category::Adjustment = "Adjustment";
const FName TG_Category::Channel = "Channels";
const FName TG_Category::DevOnly = "Test/Dev";
const FName TG_Category::Procedural = "Procedural";
const FName TG_Category::Maths = "Math";
const FName TG_Category::Utilities = "Utilities";
const FName TG_Category::Filter = "Filter";
const FName TG_Category::Arrays = "Arrays";
const FName TG_Category::Custom = "Custom";

void UTG_Expression::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving())
		InstanceExpressionClassVersion = GetExpressionClassVersion();
	Super::Serialize(Ar);

	// in the case of Loading, let s check the expression version retreived and log if we detect an older version
	if (Ar.IsLoading() && (InstanceExpressionClassVersion < this->GetExpressionClassVersion()))
	{
		UE_LOGF(LogTextureGraph, VeryVerbose, "Detected an expression in a previous version");
	}
}


UTG_Node* UTG_Expression::GetParentNode() const
{
	return Cast<UTG_Node>(GetOuter());
}

void UTG_Expression::NotifyExpressionChanged(const FPropertyChangedEvent& PropertyChangedEvent)const
{
	UTG_Node* ParentNode = GetParentNode();
	if (ParentNode)
		ParentNode->OnExpressionChangedWithoutVar(PropertyChangedEvent);
}
void UTG_Expression::NotifySignatureChanged()const
{
	UTG_Node* ParentNode = GetParentNode();
	if (ParentNode)
		ParentNode->OnSignatureChanged();
}

FTG_Name UTG_Expression::GetDefaultName() const 
{
	const auto Signature = GetSignature();
	return Signature ? Signature->GetName() : FTG_Name();
}

FTG_Signature::FInit UTG_Expression::GetSignatureInitArgsFromClass() const
{
	auto ExpressionName = GetClass()->GetName();
	ExpressionName.Split(TEXT("TG_Expression_"), nullptr, &ExpressionName);

	FTG_Signature::FInit SignatureInit{ FName(*ExpressionName) };

	for (TFieldIterator<FProperty> Prop(GetClass()); Prop; ++Prop) {
		FName PropName = FName(Prop->GetNameCPP());
		FName PropTypeName = FName(Prop->GetCPPType());
		bool bIsNotConnectable = false; // by default all arguments are connectable

#if WITH_EDITORONLY_DATA
		auto PropertyTGType = Prop->GetMetaData(TEXT("TGType"));
		bIsNotConnectable = Prop->HasMetaData(TEXT("TGPinNotConnectable"));
		bool bIsEnum = false;
#else
		FString PropertyTGType;
		bool bIsEnum = false;
#endif

		// Add NotConnectable for certain types:
		{
			FByteProperty* ByteProperty = CastField<FByteProperty>(*Prop);
			bIsNotConnectable |= (ByteProperty != nullptr);
			FEnumProperty* EnumProperty = CastField<FEnumProperty>(*Prop);
			bIsEnum = (EnumProperty != nullptr);
			FNameProperty* NameProperty = CastField<FNameProperty>(*Prop);
			bIsNotConnectable |= (NameProperty != nullptr);
		}
		uint8 Flags = bIsNotConnectable ? static_cast<uint8>(ETG_Access::NotConnectableFlag) : 0;
		Flags |= bIsEnum ? AccessToUint8(EnumFlag) : 0;

		if (PropertyTGType.Compare(TEXT("TG_InputParam")) == 0)
		{
			// Add the new argument to the signature
			SignatureInit.Arguments.Add(FTG_Argument{ PropName, PropTypeName, { ETG_Access::InParam, Flags} });
		}																								   
		else if (PropertyTGType.Compare(TEXT("TG_OutputParam")) == 0)									   
		{																									   
			// Add the new argument to the signature													   
			SignatureInit.Arguments.Add(FTG_Argument{ PropName, PropTypeName, { ETG_Access::OutParam, Flags } });
		}																								   
		else if (PropertyTGType.Compare(TEXT("TG_Input")) == 0)										   
		{													   
			// Add the new argument to the signature													   
			SignatureInit.Arguments.Add(FTG_Argument{ PropName, PropTypeName, { ETG_Access::In, Flags } });
		}																								   
		else if (PropertyTGType.Compare(TEXT("TG_Output")) == 0)										   
		{																	   
			// Add the new argument to the signature													   
			SignatureInit.Arguments.Add(FTG_Argument{ PropName, PropTypeName, { ETG_Access::Out, Flags } });
		}
		else if (PropertyTGType.Compare(TEXT("TG_Setting")) == 0)
		{
			// Add the new argument to the signature as a Setting													   
			SignatureInit.Arguments.Add(FTG_Argument{ PropName, PropTypeName, { ETG_Access::InSetting, Flags } });
		}
		else if (PropertyTGType.Compare(TEXT("TG_Private")) == 0)
		{
			// Add the new argument to the signature as a Private													   
			SignatureInit.Arguments.Add(FTG_Argument{ PropName, PropTypeName, { ETG_Access::Private, Flags } });
		}
		else
		{
			// FProperties not exposed at all as pins
		}
	}
	return SignatureInit;
}

FTG_SignaturePtr UTG_Expression::BuildSignatureFromClass() const
{
	FTG_Signature::FInit SignatureInit = GetSignatureInitArgsFromClass();

	return MakeShared<FTG_Signature>(SignatureInit);
}

#if WITH_EDITOR
void UTG_Expression::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	UE_LOGF(LogTextureGraph, VeryVerbose, "Expression PostEditChangeProperty. ChangeType: %d", (int32)PropertyChangedEvent.ChangeType);

	NotifyExpressionChanged(PropertyChangedEvent);
}

bool UTG_Expression::CanEditChange(const FProperty* InProperty) const
{
	bool bIsEditable = Super::CanEditChange(InProperty);
	if (bIsEditable && InProperty != nullptr)
	{
		// Automatically set property as non-editable if its pin is connected
		UTG_Node* ParentNode = GetParentNode();
		UTG_Pin* Pin = ParentNode->GetPin(InProperty->GetFName());
		if (Pin != nullptr)
		{
			// Make property not editable if pin is connected and is of type input or settings
			bIsEditable = !(Pin->IsConnected() && (Pin->IsInput() || Pin->IsSetting()));	
		}

		const FName PropertyName = InProperty->GetFName();

		if (PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression, BaseOutputSettings))
		{
			bIsEditable = ShouldShowSettings();
		}
	}

	return bIsEditable;
}

void UTG_Expression::PostEditUndo()
{
	GetParentNode()->OnPostUndo();
	
	UObject::PostEditUndo();
}

void UTG_Expression::PropertyChangeTriggered(FProperty* Property, EPropertyChangeType::Type ChangeType)
{
	FPropertyChangedEvent Event(Property, ChangeType);
	PostEditChangeProperty(Event);
}

#endif

bool UTG_Expression::CanRenameTitle() const
{
	const FName Category = GetCategory();
	
	return Category == TG_Category::Input || Category ==  TG_Category::Output;
}

void UTG_Expression::ValidateGenerateConformer(UTG_Pin* InPin)
{
	FProperty* Property = InPin->GetExpressionProperty();

	if (Property)
	{
#if WITH_EDITOR
		// apply conformance (clamping)
		if (Property->HasMetaData("ClampMin") ||
			Property->HasMetaData("ClampMax"))
		{
			FFieldClass* PropertyClass = Property->GetClass();

			if (PropertyClass == FFloatProperty::StaticClass())
			{
				float ClampMin = FCString::Atof(*Property->GetMetaData(TEXT("ClampMin")));
				float ClampMax = FCString::Atof(*Property->GetMetaData(TEXT("ClampMax")));

				InPin->ConformerFunctor = ([this, ClampMin, ClampMax] (FTG_Evaluation::VarConformerInfo& Info)
				{    
					float InitValue = Info.InVar->GetAs<float>();
                    float Val = InitValue;
                    Val = (FMath::Max<float>(ClampMin, Val));
                    Val = (FMath::Min<float>(ClampMax, Val));
                
                    // keep the OutVar always updated to the value so Pin Default Values can pick that up and show in Node
                    Info.OutVar->EditAs<float>() = Val;
                
                    return !FMath::IsNearlyEqual(Val, InitValue);
				});
			}
			else if (PropertyClass == FDoubleProperty::StaticClass())
			{
				double ClampMin = FCString::Atod(*Property->GetMetaData(TEXT("ClampMin")));
				double ClampMax = FCString::Atod(*Property->GetMetaData(TEXT("ClampMax")));

				InPin->ConformerFunctor = ([this, ClampMin, ClampMax] (FTG_Evaluation::VarConformerInfo& Info)
				{    
					double InitValue = Info.InVar->GetAs<double>();
					double Val = InitValue;
					Val = (FMath::Max<double>(ClampMin, Val));
					Val = (FMath::Min<double>(ClampMax, Val));
                
					// keep the OutVar always updated to the value so Pin Default Values can pick that up and show in Node
					Info.OutVar->EditAs<double>() = Val;
                
					return !FMath::IsNearlyEqual(Val, InitValue);
				});
			}
			else if (PropertyClass == FIntProperty::StaticClass())
			{
				int32 ClampMin = FCString::Atoi(*Property->GetMetaData(TEXT("ClampMin")));
				int32 ClampMax = FCString::Atoi(*Property->GetMetaData(TEXT("ClampMax")));

				InPin->ConformerFunctor = ([this, ClampMin, ClampMax] (FTG_Evaluation::VarConformerInfo& Info)
				{    
					int32 InitValue = Info.InVar->GetAs<int32>();
					int32 Val = InitValue;
					Val = (FMath::Max<int32>(ClampMin, Val));
					Val = (FMath::Min<int32>(ClampMax, Val));
                
					// keep the OutVar always updated to the value so Pin Default Values can pick that up and show in Node
					Info.OutVar->EditAs<int32>() = Val;
                
					return Val != InitValue;
				});
			}
			else if (PropertyClass == FUInt32Property::StaticClass())
			{
				int32 ClampMin = FCString::Atoi(*Property->GetMetaData(TEXT("ClampMin")));
				int32 ClampMax = FCString::Atoi(*Property->GetMetaData(TEXT("ClampMax")));

				InPin->ConformerFunctor = ([this, ClampMin, ClampMax] (FTG_Evaluation::VarConformerInfo& Info)
				{    
					int32 IntValue = Info.InVar->GetAs<int32>();
					int32 Val = IntValue;
					Val = (FMath::Max<int32>(ClampMin, Val));
					Val = (FMath::Min<int32>(ClampMax, Val));
					uint32 Value = (uint32)FMath::Max(Val, 0);
					// keep the OutVar always updated to the value so Pin Default Values can pick that up and show in Node
					Info.OutVar->EditAs<int32>() = Value;
                
					return Value != (uint32)IntValue;
				});
			}
		}
#endif
	}
}

void UTG_Expression::SetupAndEvaluate(FTG_EvaluationContext* InContext)
{
	// Copy input var values TO this expression's matching properties
	for(auto& InputVar : InContext->Inputs.VarArguments)
	{
		InputVar.Value.Var.CopyTo(this, InputVar.Value.Argument);
	}

	// Copy Output var values TO this expression's matching properties (Mainly to copy existing thumbs for Output FTG_Textures)
	for (auto& OutputVar : InContext->Outputs.VarArguments)
	{
		if (OutputVar.Value.Argument.IsVariant())
		{
			FTG_Variant& Variant = OutputVar.Value.Var->GetAs<FTG_Variant>();
			if (Variant.IsTexture())
			{
				FTG_Texture& Texture = Variant.GetTexture();
				Texture.Descriptor = BaseOutputSettings;
			}
		}
		else if (OutputVar.Value.Argument.IsTexture())
		{
			FTG_Texture& Texture = OutputVar.Value.Var->GetAs<FTG_Texture>();
			Texture.Descriptor = BaseOutputSettings;
		}
		OutputVar.Value.Var->CopyTo(this, OutputVar.Value.Argument);
	}

	// For debug purpose, log the expression evaluation
	if (InContext->bDoLog)
	{
		LogEvaluation(InContext);
	}

	// This expression is setup, now into the concrete evaluate
	Evaluate(InContext);

	// AFTER evaluation, copy output var values FROM this expression's matching properties
	for (auto& OutputVar : InContext->Outputs.VarArguments)
	{
		OutputVar.Value.Var->CopyFrom(this, OutputVar.Value.Argument);
	}
}

void UTG_Expression::CopyVarToExpressionArgument(const FTG_Argument& Arg, FTG_Var* InVar)
{
	if (InVar->CopyGeneric(this, Arg, true))
		return;

	CopyVarGeneric(Arg, InVar, true);
}

void UTG_Expression::CopyVarFromExpressionArgument(const FTG_Argument& Arg, FTG_Var* InVar)
{
	if (InVar->CopyGeneric(this, Arg, false))
		return;

	CopyVarGeneric(Arg, InVar, false);
}

void UTG_Expression::LogEvaluation(FTG_EvaluationContext* InContext)
{
	TArray<FTG_Id> InVarIds;
	for (auto& VarMapEntry : InContext->Inputs.VarArguments)
		InVarIds.Emplace(VarMapEntry.Value.Var.GetId());
	TArray<FTG_Id> OutVarIds;
	for (auto& VarMapEntry : InContext->Outputs.VarArguments)
		OutVarIds.Emplace(VarMapEntry.Value.Var->GetId());

	UE_LOGF(LogTextureGraph, VeryVerbose, "%-*ls %ls", UTG_Graph::LogHeaderWidth, *GetDefaultName().ToString(), *UTG_Graph::LogCall(InVarIds, OutVarIds));
}


FTG_Variant::EType UTG_Expression::EvalExpressionCommonInputVariantType() const
{
	FTG_Variant::EType Type = FTG_Variant::EType::Invalid;
	const UTG_Node* ParentNode = GetParentNode();
	for (auto Pin : ParentNode->Pins)
	{
		if (Pin->IsInput() && Pin->IsArgVariant())
		{
			if (Pin->IsConnected())
			{
				// now we know the thing connected to the pin variant exist, what is its type?
				UTG_Pin* OtherPin = ParentNode->GetGraph()->GetPin(Pin->GetVarId());

				if (OtherPin)
				{
					FTG_Variant::EType OtherType = FTG_Variant::GetTypeFromName(OtherPin->GetArgumentCPPTypeName());
					Type = FTG_Variant::WhichCommonType(Type, OtherType);
				}
				else
					return FTG_Variant::EType::Invalid;
			}
			// removed the else part here since the selfvar/default will be reset to the common type on the next evaluation
			/*else
				{
				Type = FTG_Variant::WhichCommonType(Type, Pin->GetSelfVar()->GetAs<FTG_Variant>().GetType());
				}*/
		}
	}

	return Type;
}


FTG_Variant::EType UTG_Expression::EvalExpressionCommonOutputVariantType() const
{
	return GetCommonInputVariantType();
}
