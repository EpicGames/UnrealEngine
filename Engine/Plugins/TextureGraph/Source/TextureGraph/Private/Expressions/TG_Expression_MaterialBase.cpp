// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/TG_Expression_MaterialBase.h"

#include "Materials/Material.h"
#include "FxMat/RenderMaterial_BP.h"
#include "Job/Job.h"
#include "Transform/Utility/T_CombineTiledBlob.h"
#include "Transform/Utility/T_SplitToTiles.h"
#include "TG_Texture.h"
#include "TG_Var.h"
#include "FxMat/MaterialManager.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialInstanceConstant.h"

#if WITH_EDITOR
#include "TextureCompiler.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_MaterialBase)


EDrawMaterialAttributeTarget UTG_Expression_MaterialBase::ConvertEMaterialPropertyToEDrawMaterialAttributeTarget(EMaterialProperty InMaterialProperty)
{
	switch (InMaterialProperty)
	{
		case MP_EmissiveColor:
			return EDrawMaterialAttributeTarget::Emissive;
		case MP_Opacity:
			return EDrawMaterialAttributeTarget::Opacity;
		case MP_OpacityMask:
			return EDrawMaterialAttributeTarget::OpacityMask;
		case MP_BaseColor:
			return EDrawMaterialAttributeTarget::BaseColor;
		case MP_Metallic:
			return EDrawMaterialAttributeTarget::Metallic;
		case MP_Specular:
			return EDrawMaterialAttributeTarget::Specular;
		case MP_Roughness:
			return EDrawMaterialAttributeTarget::Roughness;
		case MP_Anisotropy:
			return EDrawMaterialAttributeTarget::Anisotropy;
		case MP_Normal:
			return EDrawMaterialAttributeTarget::Normal;
		case MP_Tangent:
			return EDrawMaterialAttributeTarget::Tangent;

		case MP_DiffuseColor:
		case MP_SpecularColor:
		case MP_WorldPositionOffset:
		case MP_WorldDisplacement_DEPRECATED:
		case MP_TessellationMultiplier_DEPRECATED:
		case MP_SubsurfaceColor:
		case MP_AmbientOcclusion:
		case MP_Refraction:
		case MP_CustomData0:
		case MP_CustomData1:
		case MP_CustomizedUVs0:
		case MP_CustomizedUVs1:
		case MP_CustomizedUVs2:
		case MP_CustomizedUVs3:
		case MP_CustomizedUVs4:
		case MP_CustomizedUVs5:
		case MP_CustomizedUVs6:
		case MP_CustomizedUVs7:
		case MP_PixelDepthOffset:
		case MP_ShadingModel:
		case MP_FrontMaterial:
		case MP_SurfaceThickness:
		case MP_Displacement:
		case MP_MaterialAttributes:
		case MP_CustomOutput:
		default:
			return EDrawMaterialAttributeTarget::Count; // Not supported yet in the shader
	 }
}


bool UTG_Expression_MaterialBase::Validate(MixUpdateCyclePtr	Cycle)
{
	return true;
}

void UTG_Expression_MaterialBase::SetMaterialInternal(UMaterialInterface* InMaterial)
{

	if (!InMaterial)
	{
		MaterialPermutations = FMaterialInstanceStaticSwitchPermutationMap();
	}
	else 
	{
		MaterialPermutations = FMaterialInstanceStaticSwitchPermutationMap::Create(GetMaterial());
	}

	// Detect the set of available material properties for rendering
	GenerateMaterialAttributeOptions();
	
	// Signature is reset, notify the owning node / graph to update itself
	NotifySignatureChanged();
}

void UTG_Expression_MaterialBase::GenerateMaterialAttributeOptions()
{
	// Detect the set of available material properties for rendering
	AvailableMaterialAttributeIds.Empty();
	AvailableMaterialAttributeNames.Empty();

	static const FName AttribNames[static_cast<uint8>(EDrawMaterialAttributeTarget::Count)] = {
		TEXT("Base Color"),
		TEXT("Metallic"),
		TEXT("Specular"),
		TEXT("Roughness"),
		TEXT("Anisotropy"),
		TEXT("Emissive Color"),
		TEXT("Opacity"),
		TEXT("Opacity Mask"),
		TEXT("Normal"),
		TEXT("Tangent")
	};
	
	if (GetMaterial())
	{
		UMaterial* RefMaterial = GetMaterial()->GetMaterial();
		for (int i = MP_EmissiveColor; i < MP_MAX; ++i)
		{
			EMaterialProperty MP = EMaterialProperty(i);
			if (RefMaterial->IsPropertyConnected(MP))
			{
				if (MP == MP_DiffuseColor && !RefMaterial->IsPropertyConnected(MP_BaseColor))
				{
					MP = MP_BaseColor;
				}
				EDrawMaterialAttributeTarget Attribute = ConvertEMaterialPropertyToEDrawMaterialAttributeTarget(MP);
				if (Attribute != EDrawMaterialAttributeTarget::Count)
				{
					AvailableMaterialAttributeIds.Add(Attribute);
					FName AttributeName = AttribNames[static_cast<uint8>(Attribute)];
					AvailableMaterialAttributeNames.Add(AttributeName);
				}
			}
		}
	}
}

void UTG_Expression_MaterialBase::Initialize()
{
	if (GetMaterial() && !MaterialPermutations.IsValid())
	{
		MaterialPermutations = FMaterialInstanceStaticSwitchPermutationMap::Create(GetMaterial());
	}
	GenerateMaterialAttributeOptions(); // also populate the attributes availables
}

UMaterialInstanceConstant* UTG_Expression_MaterialBase::GetEvaluationMaterialInstanceConstant(FTG_EvaluationContext* InContext)
{
	TArray<FStaticSwitchParameter> EvaluationCombination;

	int32 ArgIndex = 0;
	for (auto& ArgToMatParam : ArgToMatParams)
	{
		if (ArgToMatParam.MatType == EMaterialParameterType::StaticSwitch)
		{
			FStaticSwitchParameter StaticSwitchParameter(FMaterialParameterInfo(ArgToMatParam.MatParamName), ArgToMatParam.Value.AsStaticSwitch(), false, ArgToMatParam.MatParamGuid);

			FTG_Var* Var = InContext->Inputs.GetVar(ArgToMatParam.ArgName);
			if (Var && !Var->IsEmpty())
			{
				auto ParamValue = Var->EditAs<bool>();
				StaticSwitchParameter.Value = ParamValue;
				StaticSwitchParameter.bOverride = true;
			}
			EvaluationCombination.Add(StaticSwitchParameter);
		}
	}

	return MaterialPermutations.GetMaterialInstance(EvaluationCombination);
}

void UTG_Expression_MaterialBase::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	
	/// Set it to false always
	TiledMode = true;

	Output = TextureHelper::GetBlack();
	if (GetMaterial())
	{
		UTG_Node* Node = GetParentNode();


		FString AssetName = GetMaterial()->GetName();
		UMaterialInstanceConstant* MIC = GetEvaluationMaterialInstanceConstant(InContext);

		for (int32 i = 0; i < AvailableMaterialAttributeIds.Num(); ++i)
		{
			bool IsDefaultOutput = GetRenderedAttributeId() == AvailableMaterialAttributeIds[i];
			if (IsDefaultOutput || Node->GetPin(MatAttributesOutputArgNames[i])->IsConnected())
			{
				const auto RenderMaterial = std::make_shared<RenderMaterial_BP>(AssetName, MIC, GetNumVirtualTextureWarmupFrames());
				TiledBlobPtr Result = TextureHelper::GetBlack();
				Result = CreateRenderMaterialJob(InContext, RenderMaterial, Output.GetBufferDescriptor(), AvailableMaterialAttributeIds[i]);
				Outputs[i] = Result;

				if (IsDefaultOutput)
				{
					Output = Result;
				}

				// Reassign the generated result to the corresponding Output arg
				FTG_Var* OutVar = InContext->Outputs.GetVar(MatAttributesOutputArgNames[i]);
				if (OutVar)
				{
					OutVar->EditAs<FTG_Texture>() = Result;
				}
			}
			else
			{
				// For non generated attributes, assign black
				FTG_Var* OutVar = InContext->Outputs.GetVar(MatAttributesOutputArgNames[i]);
				if (OutVar)
				{
					OutVar->EditAs<FTG_Texture>() = TextureHelper::GetBlack();
				}
			}
		}
	}

}

TiledBlobPtr UTG_Expression_MaterialBase::CreateRenderMaterialJob(FTG_EvaluationContext* InContext, const FString& InName, const FString& InMaterialPath, const BufferDescriptor& InDescriptor, EDrawMaterialAttributeTarget InDrawMaterialAttributeTarget)
{
	const RenderMaterial_BPPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_BP(InName, InMaterialPath, GetNumVirtualTextureWarmupFrames());
	return CreateRenderMaterialJob(InContext, RenderMaterial, InDescriptor, InDrawMaterialAttributeTarget);
}

TiledBlobPtr UTG_Expression_MaterialBase::CreateRenderMaterialJob(FTG_EvaluationContext* InContext, const RenderMaterial_BPPtr& InRenderMaterial, const BufferDescriptor& InDescriptor, EDrawMaterialAttributeTarget InDrawMaterialAttributeTarget)
{
	/// Set it to false always
	TiledMode = true;

	check(InRenderMaterial);
	InRenderMaterial->Instance()->EnsureIsComplete();
	InRenderMaterial->Instance()->SetForceMipLevelsToBeResident(true, true, -1);

	TArray<UTexture*> ReferencedTextures;
	
	InRenderMaterial->Instance()->GetUsedTextures(ReferencedTextures, GetCurrentMaterialQualityLevelChecked());

	for (int32 i = 0; i < ReferencedTextures.Num(); ++i)
	{
		UTexture* ReferencedTexture = Cast<UTexture>(ReferencedTextures[i]);

		if(ReferencedTexture)
		{
			ReferencedTexture->SetForceMipLevelsToBeResident(30);
			ReferencedTexture->WaitForStreaming();

#if WITH_EDITOR
			const bool IsCompiling = FTextureCompilingManager::Get().IsCompilingTexture(ReferencedTexture); 
			checkSlow(!IsCompiling);
#endif
		}
	}
	
	JobUPtr MaterialJob = std::make_unique<Job>(InContext->Cycle->GetMix(), InContext->TargetId, std::static_pointer_cast<BlobTransform>(InRenderMaterial), GetParentNode());

	FLinearColor PSControl;
	PSControl.R = (int32)InDrawMaterialAttributeTarget;
	PSControl.G = (int32)EDrawMaterialAttributeTarget::Count;
	PSControl.B = 0.0; // Debug: Control the blend with UV colors 
	PSControl.A = 0.0; // Debug: Control the blend with TileUV colors

	FTileInfo tileInfo;

	MaterialJob->AddArg(ARG_STRING(InRenderMaterial->GetMaterial()->GetPathName(), "Material"));
	MaterialJob->AddArg(ARG_VECTOR(PSControl, "PSControl"));
	MaterialJob->AddArg(std::make_shared<JobArg_TileInfo>(tileInfo, "TileInfo")); /// Enable the tileinfo parameters
	MaterialJob->AddArg(std::make_shared<JobArg_ForceTiling>()); /// Force hashing individual tiles differently
	MaterialJob->AddArg(ARG_INT(GetNumVirtualTextureWarmupFrames(), "NumVirtualTextureWarmupFrames")); // Inject the num warmup frames as part of the caching key

	BufferDescriptor Desc = InDescriptor;

	if (Desc.IsAuto())
		Desc.Format = BufferFormat::Byte;

	if (Desc.ItemsPerPoint <= 0)
		Desc.ItemsPerPoint = 4;

	Desc.DefaultValue = FLinearColor::Black;

	LinkMaterialParameters(InContext, MaterialJob, GetMaterial(), Desc);

	const TiledBlob_PromisePtr MaterialResult = std::static_pointer_cast<TiledBlob_Promise>(MaterialJob->InitResult(InRenderMaterial->GetName(), &Desc));
	MaterialJob->AddArg(WithUnbounded(ARG_BOOL(TiledMode, "TiledMode")));
	MaterialJob->SetTiled(TiledMode);
	

	InContext->Cycle->AddJob(InContext->TargetId, std::move(MaterialJob));

	if (TiledMode)
	{
		return MaterialResult;
	}
	else
	{
		MaterialResult->MakeSingleBlob();
		return T_SplitToTiles::Create(InContext->Cycle, InContext->TargetId, MaterialResult);
	}

}

void UTG_Expression_MaterialBase::LinkMaterialParameters(FTG_EvaluationContext* InContext, JobUPtr& InMaterialJob, const UMaterialInterface* InMaterial, BufferDescriptor InDescriptor)
{
	if (InMaterialJob)
	{	
		int32 ArgIndex = 0;
		for (auto& ArgToMatParam : ArgToMatParams)
		{
			FTG_Var* Var = InContext->Inputs.GetVar(ArgToMatParam.ArgName);
			if (Var && !Var->IsEmpty())
			{
				switch (ArgToMatParam.MatType)
				{
				case EMaterialParameterType::Scalar:
				{
					const FTG_Argument* VarArgument = InContext->Inputs.GetVarArgument(ArgToMatParam.ArgName);
					FName CPPType = VarArgument->GetCPPTypeName();
					float ParamValue;
					if (CPPType == TEXT("int32"))
					{
						ParamValue = Var->GetAs<int32>();
					}
					else if (CPPType == TEXT("uint32"))
					{
						ParamValue = Var->GetAs<uint32>();
					}
					else
					{
						ParamValue = Var->GetAs<float>();
					}

					InMaterialJob->AddArg(ARG_FLOAT(ParamValue, TCHAR_TO_UTF8(*ArgToMatParam.MatParamName.ToString())));
				}
				break;
				case EMaterialParameterType::Vector:
				{
					auto ParamValue = Var->EditAs<FLinearColor>();
					InMaterialJob->AddArg(ARG_VECTOR(ParamValue, TCHAR_TO_UTF8(*ArgToMatParam.MatParamName.ToString())));
				}
				break;
				case EMaterialParameterType::DoubleVector:
				{
					/*
					auto ParamValue = Var->EditAs<FVector4d>();
					Job->AddArg(ARG_VECTOR4(ParamValue, TCHAR_TO_UTF8(*ArgToMatParam.MatParamName.ToString())));
					*/
				}
				break;
				case EMaterialParameterType::Texture:
				{
					FTG_Texture& ParamValue = Var->EditAs<FTG_Texture>();

					// there could be a case where the var has an empty blob, we don't need to do anything
					// we fallback to the default internal material parameter value for this input pin
					if (ParamValue.RasterBlob)
					{
						auto CombinedBlob = T_CombineTiledBlob::Create(InContext->Cycle, ParamValue.GetBufferDescriptor(), 0, ParamValue.RasterBlob);

						auto ArgBlob = ARG_BLOB(CombinedBlob, TCHAR_TO_UTF8(*ArgToMatParam.MatParamName.ToString()));

						ArgBlob->SetHandleTiles(TiledMode);

						InMaterialJob->AddArg(ArgBlob);
					}
				}
				break;
				case EMaterialParameterType::StaticSwitch:
					// Static switch are not passed to the BlobTransofrm as arg, they are part of the evaluation method 
					// to decide WHICH material permutation to use
				break;
				default:
					break;
				}
			}

			ArgIndex++;
		}
	}
}

FName UTG_Expression_MaterialBase::CPPTypeNameFromMaterialParamType(EMaterialParameterType InMatType)
{
	switch (InMatType)
	{
	case EMaterialParameterType::Scalar:
		return TEXT("float");
	case EMaterialParameterType::Vector:
		return TEXT("FLinearColor");
	case EMaterialParameterType::DoubleVector:
		return TEXT("FVector4");
	case EMaterialParameterType::Texture:
		return TEXT("FTG_Texture");
	case EMaterialParameterType::StaticSwitch:
		return TEXT("bool");
	default:
		return FName();
	}
}

void UTG_Expression_MaterialBase::AddSignatureParam(const TArray<FMaterialParameterInfo>& OutParameterInfo, const TArray<FGuid>& OutGuids,
		EMaterialParameterType MatParamType, FTG_Signature::FInit& SignatureInit) const
{
	for (int i = 0; i < OutParameterInfo.Num(); ++i)
	{
		const FMaterialParameterInfo& MatParam = OutParameterInfo[i];
		const FGuid& Guid = OutGuids[i];

		TMap<FName, FString> MetaDataMap;
#if WITH_EDITOR
		if(MatParamType == EMaterialParameterType::Scalar)
		{
			// In case of scalar parameter, material has input range. We use that range to handle the input in the node.
			float MinValue, MaxValue;
			if(GetMaterial()->GetScalarParameterSliderMinMax(MatParam.Name, MinValue, MaxValue))
			{
				MetaDataMap.Add("MinValue", FString::SanitizeFloat(MinValue));
				MetaDataMap.Add("MaxValue", FString::SanitizeFloat(MaxValue));
			}
		}
#endif	

		// Check that no other arg have the  same name in the signature already
		// If it's the case, postfix that arg name from the point of view of the signature
		TArray<FName> ArgNames = TG_MakeArrayOfArgumentNames(SignatureInit.Arguments);
		FName ArgName = TG_MakeNameUniqueInCollection(MatParam.Name, ArgNames);
	
		// New Arg of the signature
		FTG_Argument Arg{ ArgName, CPPTypeNameFromMaterialParamType(MatParamType), {ETG_Access::In}, MetaDataMap};
		
		Arg.SetPersistentSelfVar(); // Set the material parameter persistent selfvar in order to save the state
		SignatureInit.Arguments.Add(Arg);

		// Grab the actual default value from the material
		FMaterialParameterMetadata MatParamMetadata;
		GetMaterial()->GetParameterValue(MatParamType, MatParam, MatParamMetadata);

		// And new entry in ArgtoMatParams
		ArgToMatParams.Add({ ArgName, MatParam.Name, Guid, MatParamType, MatParamMetadata.Value});
	}
}

FTG_SignaturePtr UTG_Expression_MaterialBase::BuildSignatureDynamically() const
{
	FTG_Signature::FInit SignatureInit = GetSignatureInitArgsFromClass();

	// the Arg to Material Param array is populated along with the signature
	// start fresh here
	ArgToMatParams.Empty();

	// append
	if (GetMaterial())
	{
		TArray<FMaterialParameterInfo> OutParameterInfo;
		TArray<FGuid> OutParameterIds;

		GetMaterial()->GetAllScalarParameterInfo(OutParameterInfo, OutParameterIds);
		AddSignatureParam(OutParameterInfo, OutParameterIds, EMaterialParameterType::Scalar, SignatureInit);

		GetMaterial()->GetAllVectorParameterInfo(OutParameterInfo, OutParameterIds);
		AddSignatureParam(OutParameterInfo, OutParameterIds, EMaterialParameterType::Vector, SignatureInit);

		// TODO: We need to add Double Vector4 support.
		//GetMaterial()->GetAllDoubleVectorParameterInfo(OutParameterInfo, OutParameterIds);
		//AddSignatureParam(OutParameterInfo, OutParameterIds, EMaterialParameterType::DoubleVector, SignatureInit);

		// The texture param are declared as FTG_Texture so they can be connected from the standard nodes
		GetMaterial()->GetAllTextureParameterInfo(OutParameterInfo, OutParameterIds);
		AddSignatureParam(OutParameterInfo, OutParameterIds, EMaterialParameterType::Texture, SignatureInit);
		
#if WITH_EDITORONLY_DATA
		GetMaterial()->GetAllStaticSwitchParameterInfo(OutParameterInfo, OutParameterIds);
		AddSignatureParam(OutParameterInfo, OutParameterIds, EMaterialParameterType::StaticSwitch, SignatureInit);
#endif		

		// Add the output argument(s) declaration
		// Make the output argument names unique in the Node signature.
		// These are not Param, so we will alias the outputs exactly like the attribute names so it looks
		// like the expected attributes of the material (base color, normal, ....)
		TArray<FName> ArgNames = TG_MakeArrayOfArgumentNames(SignatureInit.Arguments);
		MatAttributesOutputArgNames.Empty();

		Outputs.Empty(AvailableMaterialAttributeNames.Num());
		for (int32 Index = 0; Index < AvailableMaterialAttributeNames.Num(); ++Index)
		{
			FName ArgName = TG_MakeNameUniqueInCollection(AvailableMaterialAttributeNames[Index], ArgNames);
			ArgNames.Add(ArgName);
			MatAttributesOutputArgNames.Add(ArgName);

			TMap<FName, FString> MetaDataMap;
			MetaDataMap.Add(TEXT("Alias"), AvailableMaterialAttributeNames[Index].ToString());

			// New Arg of the signature
			FTG_Argument Arg{ ArgName, TEXT("FTG_Texture"), {ETG_Access::Out}, MetaDataMap };

			SignatureInit.Arguments.Add(Arg);

			Outputs.Add(FTG_Texture());
		}
	}
	
	return MakeShared<FTG_Signature>(SignatureInit);
}


void UTG_Expression_MaterialBase::CopyVarGeneric(const FTG_Argument& Arg, FTG_Var* InVar, bool CopyVarToArg)
{
	FArgToMaterialParamInfo* ArgToMatParam = ArgToMatParams.FindByKey(Arg.Name);
	if (ArgToMatParam)
	{
		switch (ArgToMatParam->MatType)
		{
		case EMaterialParameterType::Scalar:
		{
			if (CopyVarToArg)
			{
				ArgToMatParam->Value = FMaterialParameterValue(InVar->GetAs<float>());
			}
			else
			{
				InVar->EditAs<float>() = ArgToMatParam->Value.AsScalar();
			}
		}
		break;
		case EMaterialParameterType::Vector:
		{
			if (CopyVarToArg)
			{
				ArgToMatParam->Value = FMaterialParameterValue(InVar->GetAs<FLinearColor>());
			}
			else
			{
				InVar->EditAs<FLinearColor>() = ArgToMatParam->Value.AsLinearColor();
			}
		}
		break;
		case EMaterialParameterType::DoubleVector:
		{
			// TODO: Support that case later ?
		}
		break;
		case EMaterialParameterType::Texture:
		{
			// TODO: Support that case later ?
		}
		break;
		case EMaterialParameterType::StaticSwitch:
		{
			if (CopyVarToArg)
			{
				ArgToMatParam->Value = FMaterialParameterValue(InVar->GetAs<bool>());
			}
			else
			{
				InVar->EditAs<bool>() = ArgToMatParam->Value.AsStaticSwitch();
			}
		}
		break;
		default:
			break;
		}
	}
}
