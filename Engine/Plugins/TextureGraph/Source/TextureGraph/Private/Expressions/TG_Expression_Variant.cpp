// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/TG_Expression_Variant.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Variant)

float UTG_Expression_Variant::EvaluateScalar(FTG_EvaluationContext* InContext) 
{ 
	std::vector<FTG_Variant> Args = GetEvaluateArgs();

	if (!Args.empty())
	{
		std::vector<float> ArgsFloat(Args.size());

		for (size_t ArgIndex = 0; ArgIndex < Args.size(); ArgIndex++)
		{
			FTG_Variant& Arg = Args[ArgIndex];
			ArgsFloat[ArgIndex] = Arg.GetScalar();
		}

		return EvaluateScalar_WithValue(InContext, (float*)&ArgsFloat[0], ArgsFloat.size());
	}

	return EvaluateScalar_WithValue(InContext, nullptr, 0);
}

FVector4f UTG_Expression_Variant::EvaluateVector_WithValue(FTG_EvaluationContext* InContext, const FVector4f* const InputVector, size_t Count)
{
	check(InputVector);

	// Common case
	if (Count == 1)
	{
		return FVector4f{
			EvaluateScalar_WithValue(InContext, reinterpret_cast<const float* const>(&InputVector->X), 1),
			EvaluateScalar_WithValue(InContext, reinterpret_cast<const float* const>(&InputVector->Y), 1),
			EvaluateScalar_WithValue(InContext, reinterpret_cast<const float* const>(&InputVector->Z), 1),
			EvaluateScalar_WithValue(InContext, reinterpret_cast<const float* const>(&InputVector->W), 1)
		};
	}

	// Otherwise we construct an array ... 
	// convert to raw pointer so that it's easier to iterate over than a vector array
	const float* const InputVectorPtr = reinterpret_cast<const float* const>(InputVector);
	FVector4f OutputVector;
	float* OutputVectorPtr = reinterpret_cast<float*>(&OutputVector);
	static const size_t MaxComponents = 4;

	// iterate for X, Y, Z and W
	for (size_t ComponentIndex = 0; ComponentIndex < MaxComponents; ComponentIndex++)
	{
		std::vector<float> ScalarInput(Count);
		for (size_t InputIndex = 0; InputIndex < Count; InputIndex++)
		{
			ScalarInput[InputIndex] = (float)(*(InputVectorPtr + InputIndex * MaxComponents + ComponentIndex));
		}

		float ComponentValue = EvaluateScalar_WithValue(InContext, (float*)(&ScalarInput[0]), Count);
		OutputVectorPtr[ComponentIndex] = ComponentValue;
	}

	return OutputVector;
}

FVector4f UTG_Expression_Variant::EvaluateVector(FTG_EvaluationContext* InContext)
{
	std::vector<FTG_Variant> Args = GetEvaluateArgs();

	if (!Args.empty())
	{
		std::vector<FVector4f> ArgsVector(Args.size());

		for (size_t ArgIndex = 0; ArgIndex < Args.size(); ArgIndex++)
		{
			FTG_Variant& Arg = Args[ArgIndex];
			ArgsVector[ArgIndex] = Arg.GetVector();
		}

		return EvaluateVector_WithValue(InContext, (FVector4f*)&ArgsVector[0], ArgsVector.size());
	}

	return EvaluateVector_WithValue(InContext, nullptr, 0);
}

FLinearColor UTG_Expression_Variant::EvaluateColor(FTG_EvaluationContext* InContext)
{
	// Just evaluate a vector and then convert to FLinearColor
	FVector4f Vec = EvaluateVector(InContext);
	return FLinearColor((float)Vec.X, (float)Vec.Y, (float)Vec.Z, (float)Vec.W);
}

void UTG_Expression_Variant::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	FTG_Variant::EType VariantType = GetCommonInputVariantType();

	switch (VariantType)
	{
	case FTG_Variant::EType::Scalar:
		Output.EditScalar() = EvaluateScalar(InContext);
		break;
	case FTG_Variant::EType::Color:
	{
		Output.EditColor() = EvaluateColor(InContext);
		break;
	}
	case FTG_Variant::EType::Vector:
	{
		Output.EditVector() = EvaluateVector(InContext);
		break;
	}
	case FTG_Variant::EType::Texture:
	{
		Output.EditTexture().Descriptor = BaseOutputSettings;
		Output.EditTexture() = EvaluateTexture(InContext).RasterBlob;
		break;
	}
	default:
	{
		Output.EditScalar() = 0.0f;
		break;
	}
	}
}
