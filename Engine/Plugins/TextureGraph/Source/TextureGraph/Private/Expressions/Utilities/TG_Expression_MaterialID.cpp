// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Utilities/TG_Expression_MaterialID.h"

#include "TG_Texture.h"
#include "Transform/Expressions/T_ExtractMaterialIds.h"
#include "Transform/Expressions/T_MaterialIDMask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_MaterialID)

void UTG_Expression_MaterialID::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	if(MaterialIDMap)
	{
		T_ExtractMaterialIds::Create(InContext->Cycle, MaterialIDMap, MaterialIDInfoCollection, MaterialIDMaskInfos, ActiveColorsCount, InContext->TargetId);

		if(ActiveColorsCount > 0)
		{
			// Generate the mask based on active colors.
			Output = T_MaterialIDMask::Create(InContext->Cycle, MaterialIDMap, ActiveColors, ActiveColorsCount, Output.GetBufferDescriptor(), InContext->TargetId);
		}
		else
		{
			Output = TextureHelper::GetBlack();
		}
	}
	else
	{
		Output = TextureHelper::GetBlack();
		MaterialIDInfoCollection.Infos.Empty();
		MaterialIDMaskInfos.Empty();
		ActiveColorsCount = 0;
	}
}

void UTG_Expression_MaterialID::Initialize()
{
	UpdateActiveColors();
}

void UTG_Expression_MaterialID::UpdateActiveColors()
{
	Modify();
	
	ActiveColors.Empty();

	ActiveColors.SetNum(128);

	for(FLinearColor& Color : ActiveColors)
	{
		Color = FLinearColor::Black;
	}

	ActiveColorsCount = 0;
	
	for(const FMaterialIDMaskInfo MaskInfo : MaterialIDMaskInfos)
	{
		if(MaskInfo.bIsEnabled)
		{
			FLinearColor FoundColor = FLinearColor::Black;
	
			if(MaterialIDInfoCollection.GetColor(MaskInfo.MaterialIdReferenceId, FoundColor))
			{
				ActiveColors[ActiveColorsCount] = FoundColor;
				ActiveColorsCount++;	
			}
		}
	}
}

void UTG_Expression_MaterialID::SetMaterialIdMask(int32 Id, bool bEnable)
{
	if(Id >= 0 && Id < MaterialIDMaskInfos.Num())
	{
		MaterialIDMaskInfos[Id].bIsEnabled = bEnable;
	}
	
#if WITH_EDITOR
	// We need to find its property and trigger property change event manually.
	const auto SourcePin = GetParentNode()->GetPin("MaterialIDMaskInfos");

	check(SourcePin);
	
	if(SourcePin)
	{
		auto Property = SourcePin->GetExpressionProperty();
		PropertyChangeTriggered(Property, EPropertyChangeType::ValueSet);
	}
#endif
}

#if WITH_EDITOR
void UTG_Expression_MaterialID::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName MemberPropertyName = (PropertyChangedEvent.MemberProperty != NULL) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UTG_Expression_MaterialID, MaterialIDMaskInfos))
	{
		UpdateActiveColors();
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif


