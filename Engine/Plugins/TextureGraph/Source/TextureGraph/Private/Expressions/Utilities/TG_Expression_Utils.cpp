// Copyright Epic Games, Inc. All Rights Reserved.

#include "Expressions/Utilities/TG_Expression_Utils.h"
#include "Expressions/Utilities/TG_Expression_MaterialID.h"

#include "TG_Texture.h"
#include "Transform/Expressions/T_ExtractMaterialIds.h"
#include "Transform/Expressions/T_MaterialIDMask.h"
#include "Transform/Utility/T_CombineTiledBlob.h"
#include "Transform/Utility/T_SplitToTiles.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Expression_Utils)

#if 0
void UTG_Expression_Utils_GetWidth::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	Width = 0;

	if (Input)
	{
		if (!Input->IsLateBound())
		{
			Width = Input->GetWidth();
		}
		else
		{
			Input->OnFinalise().then([=, this](const Blob* Result)
			{
				Width = Result->GetWidth();
			});
		}
	}
}

//////////////////////////////////////////////////////////
void UTG_Expression_Utils_GetHeight::Evaluate(FTG_EvaluationContext* InContext)
{
	Super::Evaluate(InContext);

	Height = 0;

	if (Input)
	{
		if (!Input->IsLateBound())
		{
			Height = Input->GetHeight();
		}
		else
		{
			Input->OnFinalise().then([=, this](const Blob* Result)
			{
				Height = Result->GetHeight();
			});
		}
	}
}
#endif 

void UTG_Expression_Utils_MakeVector4::Evaluate(FTG_EvaluationContext* InContext)
{
	Output = FVector4f(X, Y, Z, W);
}

//////////////////////////////////////////////////////////
void UTG_Expression_Utils_Resize::Evaluate(FTG_EvaluationContext* InContext)
{
	if (!Input)
	{
		Output = FTG_Texture::GetBlack();
		return;
	}

	CombineSettings Settings
	{
		.bFixed = false,
		.bMaintainAspectRatio = bMaintainAspectRatio,
		.BackgroundColor = BackgroundColor
	};

	TiledBlobPtr CombinedResult = T_CombineTiledBlob::Create(InContext->Cycle, Output.Descriptor, InContext->TargetId, Input, nullptr, &Settings);
	Output = T_SplitToTiles::Create(InContext->Cycle, InContext->TargetId, CombinedResult);
}
