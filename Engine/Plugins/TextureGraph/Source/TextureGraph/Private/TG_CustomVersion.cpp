// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_CustomVersion.h"
#include "Serialization/CustomVersion.h"

//
// auto theOne = FGuid::NewGuid();
// UE_LOGF(LogTextureGraph, Log, "%ls", *theOne.ToString(EGuidFormats::UniqueObjectGuid));

const FGuid FTG_CustomVersion::GUID(0x84F72F2E, 0x4181C255, 0x26721CB5, 0x37DFB222);

// Register the custom version with core
FCustomVersionRegistration GRegisterTG_CustomVersion(FTG_CustomVersion::GUID, FTG_CustomVersion::LatestVersion, TEXT("TG_BaseVer"));
