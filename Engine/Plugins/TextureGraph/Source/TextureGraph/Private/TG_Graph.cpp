// Copyright Epic Games, Inc. All Rights Reserved.

// ReSharper disable All
#include "TG_Graph.h"
#include "Expressions/TG_Expression.h"
#include "Expressions/Output/TG_Expression_Output.h"
#include "TG_OutputSettings.h"
#include "TG_CustomVersion.h"

#include "Algo/Reverse.h"
#include "Expressions/Input/TG_Expression_OutputSettings.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Graph)

#if WITH_EDITOR
void UTG_Graph::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UE_LOGF(LogTextureGraph, VeryVerbose, "UTG_Graph::PostEditChangeProperty.");
}

bool UTG_Graph::Modify(bool bAlwaysMarkDirty)
{
	// Runtime state is about to get dirty
	// And remember it as such for undo redo
	bIsGraphTraversalDirty = true;
	UE_LOGF(LogTextureGraph, VeryVerbose, "UTG_Graph::Modify: Graph Modified.");

	return Super::Modify(bAlwaysMarkDirty);
}


void UTG_Graph::PostEditUndo()
{
	UE_LOGF(LogTextureGraph, VeryVerbose, "UTG_Graph::PostEditUndo.");
	UObject::PostEditUndo();
}

#endif
void UTG_Graph::Reset()
{
	Nodes.Empty();
	Params.Empty();

	// Transient state is dirty
	bIsGraphTraversalDirty = true;
}

void UTG_Graph::Construct(FString InName)
{
	Name = InName;
	Reset(); // really make sure we start fresh with an empty graph...
	// TODO: Should we do more here?
}

void UTG_Graph::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FTG_CustomVersion::GUID);

	int32 Version = Ar.CustomVer(FTG_CustomVersion::GUID);

	UE_LOGF(LogTextureGraph, VeryVerbose, "  %ls Graph: %ls",
		(Ar.IsSaving() ? TEXT("Saved") : TEXT("Loaded")),
		*Name);
}

void UTG_Graph::PostLoad()
{
	Super::PostLoad();

	UE_LOGF(LogTextureGraph, VeryVerbose, "  PostLoad Graph: %ls", *Name);

	// We have to reset the Transactional flag for the UTG_Graph and we do not know why...
	// For the other UObject in the data structure (Script, Node, Expression) it is set
	// once in the new and it sticks the serialization.
	SetFlags(RF_Transactional);

	TArray<uint32> NodeRequiringRemap;
	// Restore all node runtime fields and associated pins
	for (uint32 i = 0; i < (uint32)Nodes.Num(); ++i)
	{
		auto Node = Nodes[i];
		if (Node)
		{
			Node->Initialize(FTG_Id(i));

			// Pins are loaded, we can check that the signature matching the pin's arguments
			bool PinsMatchSignature = Node->CheckPinSignatureAgainstExpression();
			if (!PinsMatchSignature)
			{
				NodeRequiringRemap.Add(i);
				UE_LOGF(LogTextureGraph, VeryVerbose, "Node %ls serialized signature is different from expression", *Node->GetId().ToString());
				Node->WarningStack.Add(FName(FString::Printf(TEXT("Node %s serialized signature is different from expression's signature, pins are regenerated"), *Node->GetId().ToString())));
			}

			int16 j = 0;
			for (auto Pin : Node->Pins)
			{
				FTG_Id PinId(i, j);

				Pin->InitSelfVar();

				// set up conformant functor to the pin here
				Node->ValidateGenerateConformer(Pin);

				// If a Param, let's add it to the Param list
				if (Pin->IsParam())
					Params.Emplace(Pin->GetAliasName(), Pin->GetId());

				++j;
			}
		}
	}

	// Take care of eventual nodes requiring remapping because their signature has changed since serialization
	if (!NodeRequiringRemap.IsEmpty())
	{
		Modify();
		for (auto NodeIdx : NodeRequiringRemap)
		{
			auto Node = Nodes[NodeIdx];
			// erase the signature of the node, this way the regenerate call will work from the actual PIn deduxced signature
			Node->Signature.Reset();
			RegenerateNode(Node);
		}
	}
}

void UTG_Graph::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);
	UE_LOGF(LogTextureGraph, VeryVerbose, "  PreSave Graph: %ls", *Name);
}

// Inner setup node calling allocation of pins and edges and var in cascade
void UTG_Graph::SetupNode(UTG_Node* Node)
{
	// Call in the UObject Modify to snapshot the current state for do/undo feature
	// Done BEFORE doing any change to the graph
	Modify();

	// Initialize the node, grab its signature
	Node->Initialize(FTG_Id(Nodes.Num()));

	// Allocate pins from signature
	AllocateNodePins(Node);

	// Last but not least, add the node to the graph
	Nodes.Emplace(Node);

	OnNodeAdded(Node);
}


void UTG_Graph::RegenerateNode(UTG_Node* InNode)
{
	// Call in the UObject Modify to snapshot the current state for do/undo feature
	// Done BEFORE doing any change to the node
	InNode->Modify();
	Modify();
	
	FTG_SignaturePtr OldSignature = InNode->Signature;
	// If the old signature is null, rebuild one from the current PIn arguments
	if (!OldSignature)
		OldSignature = MakeShared<FTG_Signature>(FTG_Signature::FInit{ "", InNode->GetPinArguments()});

	FTG_SignaturePtr NewSignature = InNode->GetExpression()->GetSignature();
	// Generate the table mapping from NewArgument Idx to existing PinId
	// the element is Invalid if there is no matching existing Pin
	FTG_Indices NewIdxToOld = FTG_Signature::GenerateMappingArgIdxTable((*OldSignature), (*NewSignature));

	//Save the edited alias name so we can reaply them
	//Alias names can be edited when used as a title for node
	TArray<FName> EditedAliasNames;
	for (int32 i = 0; i < InNode->Pins.Num(); ++i)
	{
		FName EditedAlias = InNode->Pins[i]->HasAliasName() ? InNode->Pins[i]->GetAliasName() : NAME_None;
		EditedAliasNames.Add(EditedAlias);
	}

	// Go through the allocated pins and reuse the matching arguments
	// Store the future "Pins" array of the node in the "NewPinArray"
	FTG_Index InNodeIdx = InNode->GetId().NodeIdx();
	TArray<TObjectPtr<UTG_Pin>> NewPinArray;
	
	for (int32 i = 0; i < NewIdxToOld.Num(); ++i)
	{
		FTG_Index OldIdx = NewIdxToOld[i];
		// If the mapped old Idx for this Idx looks legit
		if ((OldIdx != FTG_Id::INVALID_INDEX) && (OldIdx < InNode->Pins.Num()))
		{
			// Old pin maybe exists, it can be reused!
			TObjectPtr<UTG_Pin> ReusedPin = InNode->Pins[OldIdx];

			// remove the pin from Params if it was a Param, will be added back below if it s a param again in the new signature
			if (ReusedPin->IsParam())
				Params.Remove(ReusedPin->GetAliasName());

			// Make sure the reused pin self var is correctly reset to whatever default value is coming from the Expression instance
			ReusedPin->ResetSelfVarDefault();

			// the pin need to exists and be remaped to a different id
			if (ReusedPin && (i != OldIdx))
			{
				// Move the ReusedPin at the right id
				FTG_Id NewId(InNode->GetId().NodeIdx(), i);
				ReusedPin->RemapId(NewId);
				ReusedPin->InitSelfVar();
				for (FTG_Id Edge : ReusedPin->GetEdges())
				{
					GetPin(Edge)->RemapEdgeId(FTG_Id(InNodeIdx, OldIdx), NewId);
				}
			}
			
			
			InNode->Pins[OldIdx] = nullptr;
			NewPinArray.Add(ReusedPin);
		}
		else
		{
			NewPinArray.Add(nullptr);
		}
	}

	// Kill the existing pins which are not reused
	for (int32 i = 0; i < InNode->Pins.Num(); ++i)
	{
		// The pin is still valid so kill it
		UTG_Pin* Pin = InNode->Pins[i].Get();
		
		if (Pin && IsValidPin(Pin->GetId()))
		{
			KillPin(Pin->GetId()); // This also remove the Pin from the Params
		}
	}

	// Clean the node members reflecting the old signature
	InNode->Pins.Empty();

	// Re initialize the InNode
	InNode->Initialize(InNode->GetId());

	// Assign the NewPinArray as the InNode->Pins
	InNode->Pins = NewPinArray;
	// Go again over all the pins of the node
	bool bHasVariant = false;
	for (int32 i = 0; i < InNode->Pins.Num(); ++i)
	{
		// If pin is null, let's allocate
		if (!InNode->Pins[i])
		{
			auto& Arg = NewSignature->GetArgument(i);
			AllocatePin(InNode, Arg, i);

			if (i >= 0 && i < EditedAliasNames.Num())
			{
				//if the pin reallocated ever had an edited Name we should keep it
				FName EditedAlias = EditedAliasNames[i];
				if (!EditedAlias.IsNone())
				{
					InNode->Pins[i]->SetAliasName(EditedAlias);
				}
			}
		}
		// If pin is reused, reassign the new signature argument to update potential changes in the ArgFlags
		else
		{
			auto Pin = InNode->Pins[i];
			Pin->Modify();
			Pin->Argument.ArgumentType = NewSignature->GetArgument(i).ArgumentType;
			
			// If a Param, let's add it to the Param list (after we removed it above since this is a reused pin)
			if (Pin->IsParam())
			{
				Params.Emplace(Pin->GetAliasName(), Pin->GetId());
			}
		}
		bHasVariant |= InNode->Pins[i]->IsArgVariant();
	}

	// Last and not least, if there is a variant pin, let's reevaluate the common variant type
	if (bHasVariant)
	{
		InNode->GetExpression()->ResetCommonInputVariantType();
	}

	// Make sure the node is finally renamed correctly, and potentially any pin alias name
	// which is what we use for input param pin/node
	InNode->GetExpression()->SetTitleName(InNode->GetExpression()->GetTitleName());
}

void UTG_Graph::AllocateNodePins(UTG_Node* Node)
{
	// All the pins are created from the signature 
	for (auto& Arg : Node->GetSignature().GetInArguments())
		AllocatePin(Node, Arg);
	for (auto& Arg : Node->GetSignature().GetOutArguments())
		AllocatePin(Node, Arg);
	for (auto& Arg : Node->GetSignature().GetPrivateArguments())
		AllocatePin(Node, Arg);
}

void UTG_Graph::KillNodePins(UTG_Node* InNode)
{
	// All the associated Pins are destroyed
	for (auto pin : InNode->Pins)
		KillPin(pin->GetId());

	InNode->Pins.Empty();
}

// Inner allocate pin
FTG_Id UTG_Graph::AllocatePin(UTG_Node* Node, const FTG_Argument& Arg, int32 InPinIdx)
{
	check(Node);
	int32 PinIdx = (InPinIdx < 0 ? Node->Pins.Num() : InPinIdx);

	FTG_Id PinId(Node->GetId().NodeIdx(), (int16)PinIdx);

	UTG_Pin* NewPin = NewObject<UTG_Pin>(Node, UTG_Pin::StaticClass(), NAME_None, RF_Transactional);
	NewPin->Construct(PinId, Arg);

	if (InPinIdx < 0)
		Node->Pins.Emplace(NewPin);
	else
		Node->Pins[PinIdx] = NewPin;

	// A brand new pin is probably correctly named in the scope of the node
	// but maybe its alias name needs to be eventually edited if it is a param of the graph
	// Alias name can be assigned from signature through metadata and is the same as arg name by default
	NewPin->AliasName = Node->ValidateGeneratePinAliasName(NewPin->AliasName, PinId);
	
	// set up conformant functor to the pin here
	Node->ValidateGenerateConformer(NewPin);

	// Add the pin as a parameter if needed
	if (Arg.IsParam())
	{
		FName ParamName = Node->Pins[PinId.PinIdx()]->GetAliasName();
		Params.Emplace(ParamName, PinId);
	}

	return PinId;
}

// Inner kill pin
void UTG_Graph::KillPin(FTG_Id InPinId)
{
	auto Pin = GetPin(InPinId);
	check(Pin); // Pin should be valid
	if (!Pin)
		return; // but just exit if that ever happens

	Pin->Modify();

	// Kill all the connection edges
	for (auto OtherPinId : Pin->GetEdges())
	{
		auto OtherPin = GetPin(OtherPinId);
		if (OtherPin)
		{
			OtherPin->RemoveEdge(InPinId);
		}
	}

	// Only update for Param Pin's. 
	if(Pin->IsParam())
	{
		// Remove the pin form the param just in case
		Params.Remove(Pin->GetAliasName());
	}
}

void UTG_Graph::RemoveNode(UTG_Node* InNode)
{
	check(InNode); // InNode should be valid
	if (!InNode)
		return; // but just exit if that ever happens

	auto Title = InNode->GetNodeName();
	Modify();
	InNode->Modify();

	// Kill the node pins
	KillNodePins(InNode);

	// The node is destroyed, its Uuid becomes invalid
	// The associated Expression is dereferenced and potentially GCed
	Nodes[InNode->GetId().NodeIdx()] = nullptr;

	OnNodeRemoved(InNode, Title);
}

// Inner setup node calling allocation of pins and edges and var in cascade
void UTG_Graph::AddPostPasteNode(UTG_Node* Node)
{
	SetupNode(Node);
}


UTG_Node* UTG_Graph::CreateExpressionNode(const UClass* ExpressionClass)
{
	UTG_Node* NewNode = NewObject<UTG_Node>(this, UTG_Node::StaticClass(), NAME_None, RF_Transactional);
	UTG_Expression* NewExpression = NewObject<UTG_Expression>(NewNode, ExpressionClass, NAME_None, RF_Transactional);
	NewNode->Construct(NewExpression);
	SetupNode(NewNode);

	return NewNode;
}

UTG_Node* UTG_Graph::CreateExpressionNode(UTG_Expression* NewExpression)
{
	UTG_Node* NewNode = NewObject<UTG_Node>(this, UTG_Node::StaticClass(), NAME_None, RF_Transactional);
	NewExpression->Rename(nullptr, NewNode, RF_Transactional);
	NewNode->Construct(NewExpression);
	SetupNode(NewNode);

	return NewNode;
}

bool UTG_Graph::Connect(UTG_Node& NodeFrom, FTG_Name& PinFromName, UTG_Node& NodeTo, FTG_Name& PinToName)
{
	auto PinFrom = NodeFrom.GetOutputPin(PinFromName);
	if (PinFrom == nullptr)
	{
		return false;
	}

	auto PinTo = NodeTo.GetInputPin(PinToName);
	if (PinTo == nullptr)
	{
		return false;
	}

	// Test the validity of the connection and grab the converterkey eventually
	FName InputVarConverterKey;
	if (ConnectionCausesLoop(PinFrom, PinTo) || !ArePinsCompatible(PinFrom, PinTo, InputVarConverterKey))
	{
		return false;
	}

	// Call in the UObject Modify to snapshot the current state for do/undo feature
	// Done BEFORE doing any change to the graph
	Modify();

	// detect if there is already an edge arriving to destination we should kill
	FTG_Id PrevEdgePinId;
	if (!PinTo->GetEdges().IsEmpty())
	{
		// Catch the current edge other pin to this pin
		PrevEdgePinId = PinTo->GetEdges()[0];

		// Then remove the connection mutually
		PinTo->RemoveEdge(PrevEdgePinId);
		auto OtherPin = GetPin(PrevEdgePinId);
		if (OtherPin)
		{
			OtherPin->RemoveEdge(PinTo->GetId());
		}
	}

	// Add the connection to both pin's edges
	PinTo->AddEdge(PinFrom->GetId());
	PinFrom->AddEdge(PinTo->GetId());

	// And assign the PinTo's InputVarConverterKey
	PinTo->InstallInputVarConverterKey(InputVarConverterKey);

	// Notify the Destination node a connection changed in case it needs to change something for it
	PinTo->GetNodePtr()->OnPinConnectionChanged(PinTo->GetId(), PrevEdgePinId, PinFrom->GetId());

	NotifyGraphChanged();

	return true;
}

void UTG_Graph::CollectOutputSettings(TMap<FTG_Id, FTG_OutputSettings>& OutExportSettingsMap) const
{
	ForEachNodes([&](const UTG_Node* Node, uint32 Index)
	{
		if (UTG_Expression_Output* OutputExpression = dynamic_cast<UTG_Expression_Output*>(Node->GetExpression()))
		{
			UTG_Pin* SettingsPin = Node->GetPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Output, OutputSettings));
			
			// skip connected pins as they will be exposed as Input Parameters
			if (!SettingsPin->IsConnected())
			{
				OutExportSettingsMap.Add(Node->Id, OutputExpression->OutputSettings);
			}
		}
	});
}

void UTG_Graph::SetOutputSettings(const TMap<FTG_Id, FTG_OutputSettings>& OutputSettingsMap)
{
	ForEachNodes([&](const UTG_Node* Node, uint32 Index)
	{
		if (UTG_Expression_Output* OutputExpression = dynamic_cast<UTG_Expression_Output*>(Node->GetExpression()))
		{
			const FTG_OutputSettings* OutputSettingsToSet = OutputSettingsMap.Find(Node->Id);
			if (OutputSettingsToSet)
			{
				OutputExpression->OutputSettings = *OutputSettingsToSet;
				UTG_Pin* SettingsPin = Node->GetPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Output, OutputSettings));
				SettingsPin->FromString(OutputSettingsToSet->ToString());
			}
		}
	});
}


bool UTG_Graph::Validate(MixUpdateCyclePtr Cycle)
{
	// TODO: Add any graph level validation steps here
	bool IsValid = true;
	// Validate all nodes
	Traverse([&](UTG_Node* n, int32_t i, int32_t l)
	{
		IsValid &= n->Validate(Cycle);
	});

	return IsValid;
}

bool UTG_Graph::ConnectionCausesLoop(const UTG_Pin* PinFrom, const UTG_Pin* PinTo)
{
	if (PinFrom && PinTo)
	{
		auto NodeFrom = PinFrom->GetNodePtr();
		auto NodeTo = PinTo->GetNodePtr();
		if (NodeFrom != NodeTo)
		{
			if (NodeFrom->GetGraph() == NodeTo->GetGraph())
			{
				// Gather all the nodes feeding the From Node
				auto SourceNodeIds = NodeFrom->GetGraph()->GatherAllSourceNodes(NodeFrom);

				// if the To node is found then the proposed connection is causing a loop
				if (SourceNodeIds.Find(NodeTo->GetId()) != INDEX_NONE)
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool UTG_Graph::ArePinsCompatible(const UTG_Pin* PinFrom, const UTG_Pin* PinTo, FName& ConverterKey)
{
	if (PinFrom && PinTo)
	{
		UTG_Node* NodeFrom = PinFrom->GetNodePtr();
		UTG_Node* NodeTo = PinTo->GetNodePtr();

		/// We don't allow connecting an array parameter to an output node. Too many complications with subgraphs etc
		if (NodeFrom != NodeTo && !(PinFrom->IsArgArray() && Cast<UTG_Expression_Output>(NodeTo->GetExpression())))
		{
			if (NodeFrom->GetGraph() == NodeTo->GetGraph())
			{
				return FTG_Evaluation::AreArgumentsCompatible(PinFrom->GetArgument(), PinTo->GetArgument(), ConverterKey);
			}
		}
	}
	return false;
}

void UTG_Graph::RemovePinEdges(UTG_Node& InNode, FTG_Name& InPinName)
{
	auto Pin = InNode.GetPin(InPinName);
	if (Pin == nullptr || !Pin->IsConnected())
	{
		// Avoid to work if not needed, unvalid pin, or pin without any edge
		return;
	}

	// Call in the UObject Modify to snapshot the current state for do/undo feature
	// Done BEFORE doing any change to the graph
	Modify();

	if (Pin->IsOutput())
	{
		// Kill all the connection edges other side
		for (auto OtherPinId : Pin->GetEdges())
		{
			auto OtherPin = GetPin(OtherPinId);
			if (OtherPin)
			{
				OtherPin->RemoveEdge(Pin->GetId());
				OtherPin->GetNodePtr()->OnPinConnectionChanged(OtherPinId, Pin->GetId(), FTG_Id());
			}
		}
		Pin->RemoveAllEdges();
	}
	else // IsInput
	{
		auto OtherPin = GetPin(Pin->GetEdges()[0]); // There is one edge to the pin
		Pin->RemoveEdge(OtherPin->GetId());
		OtherPin->RemoveEdge(Pin->GetId());

		Pin->GetNodePtr()->OnPinConnectionChanged(Pin->GetId(), OtherPin->GetId(), FTG_Id());
	}

	NotifyGraphChanged();
}

void UTG_Graph::RemoveEdge(UTG_Node& NodeFrom, FTG_Name& PinFromName, UTG_Node& NodeTo, FTG_Name& PinToName)
{
	auto PinFrom = NodeFrom.GetOutputPin(PinFromName);
	if (PinFrom == nullptr || !PinFrom->IsConnected())
	{
		// Avoid to work if not needed, unvalid pin, or pin without any edge
		return;
	}

	auto PinTo = NodeTo.GetInputPin(PinToName);
	if (PinTo == nullptr || !PinTo->IsConnected())
	{
		// Avoid to work if not needed, unvalid pin, or pin without any edge
		return;
	}

	// Call in the UObject Modify to snapshot the current state for do/undo feature
	// Done BEFORE doing any change to the graph
	Modify();

	PinTo->RemoveEdge(PinFrom->GetId());
	PinFrom->RemoveEdge(PinTo->GetId());

	// Notify the Destination node a connection changed in case it needs to change something for it
	PinTo->GetNodePtr()->OnPinConnectionChanged(PinTo->GetId(), PinFrom->GetId(), FTG_Id());


	NotifyGraphChanged();
}


void UTG_Graph::AppendParamsSignature(FTG_Arguments& InOutArguments, TArray<FTG_Id>& InParams,
                                      TArray<FTG_Id>& OutParams) const
{
	for (auto PinId : Params)
	{
		const UTG_Pin* ParamPin = GetPin(PinId.Value);

		/// The pin has to be marked connectable to show up
		if (!ParamPin->IsNotConnectable())
		{
			FTG_Argument Argument = {
				ParamPin->GetAliasName(), // Use the alias name to export the param as the graph interface
				ParamPin->GetArgumentCPPTypeName(), // Same CPP type name
				ParamPin->GetArgumentType().Unparamed() // Remove the param tag since it is no longer a param
			}; 

			InOutArguments.Emplace(Argument);

			if (ParamPin->IsInput())
				InParams.Emplace(PinId.Value);
			else
				OutParams.Emplace(PinId.Value);
		}
	}
}

void UTG_Graph::NotifyGraphChanged(UTG_Node* InNode /*=nullptr*/, bool bIsTweaking /* = false */)
{
#if WITH_EDITOR
	// notify listeners (EdGraph etc.)
	OnGraphChangedDelegate.Broadcast(this, InNode, bIsTweaking);
#endif
}

void UTG_Graph::OnNodeAdded(UTG_Node* InNode)
{
#if WITH_EDITOR
	if (OnTGNodeAddedDelegate.IsBound())
		OnTGNodeAddedDelegate.Broadcast(InNode);
#endif
	NotifyGraphChanged(InNode);
}

void UTG_Graph::OnNodeRemoved(UTG_Node* InNode, FName Title)
{
#if WITH_EDITOR
	if (OnTGNodeRemovedDelegate.IsBound())
		OnTGNodeRemovedDelegate.Broadcast(InNode, Title);
#endif
	NotifyGraphChanged(InNode);
}

void UTG_Graph::OnNodeRenamed(UTG_Node* InNode, FName OldName)
{
#if WITH_EDITOR
	if (OnTGNodeRenamedDelegate.IsBound())
		OnTGNodeRenamedDelegate.Broadcast(InNode, OldName);
#endif
}

bool UTG_Graph::RenameParam(FName OldName, FName NewName)
{
	if(Params.Contains(OldName) && !Params.Contains(NewName))
	{
		auto id = Params[OldName];

		Params.Remove(OldName);
		Params.Add(NewName, id);

		return true;
	}

	return false;
}

void UTG_Graph::OnNodeChanged(UTG_Node* InNode, bool bIsTweaking)
{
	UE_LOGF(LogTextureGraph, VeryVerbose, "UTG_Graph::Node Changed")
	NotifyGraphChanged(InNode, bIsTweaking);
}

void UTG_Graph::OnNodeSignatureChanged(UTG_Node* InNode)
{
	UE_LOGF(LogTextureGraph, VeryVerbose, "UTG_Graph::Node Recreate");
	// recreate node with expression
	RegenerateNode(InNode);

#if WITH_EDITOR
	// notify listeners (EdGraph etc.)
	OnNodeSignatureChangedDelegate.Broadcast(InNode);
#endif

	NotifyGraphChanged(InNode);
}

void UTG_Graph::OnNodePinChanged(FTG_Id InPinId, UTG_Node* InNode)
{
	UE_LOGF(LogTextureGraph, VeryVerbose, "UTG_Graph::Pin Changed")

	// Commented this code
	// Right now pin changed is being called only from renaming pin.
	// We dont want to trigger graph change just from renaming.
	// NotifyGraphChanged(InNode);
}


void UTG_Graph::NotifyNodePostEvaluate(UTG_Node* InNode, const FTG_EvaluationContext* InContext)
{
#if WITH_EDITOR
	// notify listeners (EdGraph etc.)
	OnNodePostEvaluateDelegate.Broadcast(InNode, InContext);
#endif
}

FName UTG_Graph::GetParamName(const FTG_Id& InParamId) const
{
	const UTG_Pin* Pin = GetPin(InParamId);
	if (Pin)
	{
		if (Pin->IsParam())
		{
			return Pin->GetAliasName();
		}
	}
	return FName();
}

FTG_Ids			UTG_Graph::GetInputParamIds() const
{
	FTG_Ids Ids;
	ForEachParams([&](const UTG_Pin* Pin, uint32 Index) {
		if (Pin->IsInput())
		{
			Ids.Add(Pin->GetId());
		}
	});
	return Ids;
}
FTG_Ids			UTG_Graph::GetOutputParamIds() const
{
	FTG_Ids Ids;
	ForEachParams([&](const UTG_Pin* Pin, uint32 Index) {
		if (Pin->IsOutput())
		{
			Ids.Add(Pin->GetId());
		}
	});
	return Ids;
}

void UTG_Graph::ForEachNodes(std::function<void(const UTG_Node* /*node*/, uint32 /*index*/)> visitor) const
{
	for (uint32 i = 0; i < (uint32)Nodes.Num(); ++i)
	{
		if (Nodes[i])
			visitor(Nodes[i], i);
	}
}

void UTG_Graph::ForEachPins(
	std::function<void(const UTG_Pin* /*pin*/, uint32 /*pin_index*/, uint32 /*node_index*/)> visitor) const
{
	ForEachNodes([&](const UTG_Node* Node, uint32 N)
	{
		for (uint32 i = 0; i < (uint32)Node->Pins.Num(); ++i)
		{
			if (Node->Pins[i]->IsValid())
				visitor(Node->Pins[i], i, N);
		}
	});
}

void UTG_Graph::ForEachVars(
	std::function<void(const FTG_Var* /*var*/, uint32 /*index*/, uint32 /*node_index*/)> visitor) const
{
	ForEachPins([&](const UTG_Pin* Pin, uint32 I, uint32 N)
	{
		if (Pin->IsValidSelfVar())
			visitor(Pin->GetSelfVar(), I, N);
	});
}

void UTG_Graph::ForEachParams(std::function<void(const UTG_Pin* /*pin*/, uint32 /*index*/)> visitor) const
{
	uint32 i = 0;
	for (auto Param : Params)
	{
		const UTG_Pin* ParamPin = GetPin(Param.Value);
		if (ParamPin)
			visitor(ParamPin, i);
		++i;
	}
}

void UTG_Graph::ForEachEdges(std::function<void(const UTG_Pin* /*pinFrom*/, const UTG_Pin* /*pinTo*/)> visitor) const
{
	// Go through all the pin's and their list of edges
	ForEachPins(
		[&](const UTG_Pin* APin, uint32 pinIdx, uint32 nodeIdx)
		{
			bool APinIsOutputAkaSource = APin->IsOutput();
			FTG_Id APinId = APin->GetId();
			for (auto BPinId : APin->GetEdges())
			{
				if (APinId < BPinId)
				{
					const UTG_Pin* BPin = GetPin(BPinId);

					if (BPin)
					{
						if (APinIsOutputAkaSource)
							visitor(APin, BPin);
						else
							visitor(BPin, APin);
					}
				}
			}
		}
	);
}

void UTG_Graph::ForEachOutputSettings( std::function<void(const FTG_OutputSettings& /*settings*/)> visitor)
{
		ForEachNodes([visitor](const UTG_Node* Node, uint32 Index)
		{
			UTG_Expression_Output* TargetExpression = Cast<UTG_Expression_Output>(Node->GetExpression());
			if (TargetExpression)
			{
				visitor(TargetExpression->OutputSettings);
			}
		});
}

int UTG_Graph::GetOutputParamTextures(TArray<FName>& OutNames, FTG_Ids& OutPinIds) const
{
	int NumFounds = 0;
	ForEachParams([&](const UTG_Pin* Pin, uint32 Index) {
		if (Pin->IsOutput())
		{
			if (Pin->IsArgTexture())
			{
				OutNames.Add(Pin->GetAliasName());
				OutPinIds.Add(Pin->GetId());
				NumFounds++;
			}
		}
		});
	return NumFounds;
}

bool UTG_Graph::GetOutputParamValue(const FName& ParamName, FTG_Variant& OutVariant) const
{
	auto ParamPin = FindParamPin(ParamName);
	if (ParamPin && ParamPin->IsOutput())
	{
		return ParamPin->GetValue(OutVariant);
	}

	return false;
}

int UTG_Graph::GetAllOutputParamValues(TArray<FTG_Variant>& OutVariants, TArray<FName>* OutNames) const
{
	int NumFounds = 0;
	ForEachParams([&](const UTG_Pin* Pin, uint32 Index) {
		if (Pin->IsOutput())
		{
			if (Pin->IsArgVariant())
			{
				// For each valid output param variant, grab the result variant if valid
				// and the name if container provided
				FTG_Variant OutVariant;
				if (Pin->GetValue(OutVariant))
				{
					OutVariants.Add(OutVariant);
					if (OutNames)
						OutNames->Add(Pin->GetAliasName());
					NumFounds++;
				}
				else
				{
					UE_LOGF(LogTextureGraph, VeryVerbose, "Output %ls variant failed to access", *(Pin->GetAliasName().ToString()));
				}
			}
		}
		});
	return NumFounds;
}

FTG_VarMap UTG_Graph::GetInputParamsVarMap()
{
	FTG_VarMap Map;
	for (auto Param : Params)
	{
		UTG_Pin* ParamPin = GetPin(Param.Value);
		if (ParamPin)
		{
			if (ParamPin->IsInput())
			{
				const auto VarId = ParamPin->GetVarId();
				FTG_Var* Var = GetVar(VarId);
				Map.VarArguments.Add(ParamPin->GetAliasName(), { *Var, ParamPin->GetArgument() });
			}
		}
	};
	return Map;
}

void UTG_Graph::SetInputParamsFromVarMap(FTG_VarMap InMap)
{
	auto InputIds = GetInputParamIds();
	for (auto InputId : InputIds)
	{
		UTG_Pin* Pin = Cast<UTG_Pin>(GetPin(InputId));
		FTG_Var* VarToCopy = InMap.GetVar(Pin->GetAliasName());
		const FTG_Argument* VarArgument = InMap.GetVarArgument(Pin->GetAliasName());

		if (VarToCopy)
		{
			const auto VarId = Pin->GetVarId();
			FTG_Var* Var = GetVar(VarId);
			Var->CopyFrom(VarToCopy);

			UTG_Expression* OwnerExpression = Pin->GetNodePtr()->GetExpression();
			// also copy to expression
			OwnerExpression->CopyVarToExpressionArgument(*VarArgument, Var);
		}
	};
	
}

FTG_Ids UTG_Graph::GatherSourceNodes(const UTG_Node* InNode) const
{
	FTG_Ids SourceNodes; // the array of source nodes that will be return, empty for now

	if (InNode) // Only work for a valid node
	{
		bool AllInConnected = true;
		bool AllInDisconnected = true;
		auto InPinIds = InNode->GetInputPinIds();

		for (auto InPinId : InPinIds) // Over all the Input pins, find the node feeding them
		{
			if (InPinId.PinIdx() < InNode->Pins.Num())
			{
				const UTG_Pin* Pin = InNode->Pins[InPinId.PinIdx()];
				if (Pin->GetEdges().IsEmpty())
				{
					AllInConnected = false;
				}
				else
				{
					AllInDisconnected = false;
					auto EdgePinId = Pin->GetEdges()[0];
					const UTG_Pin* SourcePin = GetPin(EdgePinId);

					if (SourcePin)
					{
						check(SourcePin);

						// only add the source node if not already there
						if (SourceNodes.Find(SourcePin->GetNodeId()) == INDEX_NONE)
							SourceNodes.Emplace(SourcePin->GetNodeId());
					}
				}
			}
		}
	}
	return SourceNodes;
}

FTG_Ids UTG_Graph::GatherAllSourceNodes(const UTG_Node* Node) const
{
	FTG_Ids AllSourceNodes; // the array of source nodes that will be return, empty for now

	FTG_Ids PassNodes[2];
	int32 PassIdx = 0;
	int32 NextPassIdx = 1;
	PassNodes[PassIdx].Add(Node->GetId());

	while (!PassNodes[PassIdx].IsEmpty())
	{
		for (auto NodeId : PassNodes[PassIdx])
		{
			FTG_Ids ThisNodeSourceNodes = GatherSourceNodes(GetNode(NodeId));
			for (auto InputNodeID : ThisNodeSourceNodes)
			{
				// only add the input  node if not already there
				if (PassNodes[NextPassIdx].Find(InputNodeID) == INDEX_NONE)
					PassNodes[NextPassIdx].Emplace(InputNodeID);


				AllSourceNodes.Emplace(InputNodeID);
			}
		}

		PassNodes[PassIdx].Empty();
		PassIdx = NextPassIdx;
		NextPassIdx = (NextPassIdx + 1) % 2;
	}

	return AllSourceNodes;
}

void UTG_Graph::GatherOuterNodes(const TArray<FTG_Ids>& sourceNodesPerNode, TSet<FTG_Id>& nodeReservoirA,
                                 TSet<FTG_Id>& nodeReservoirB) const
{
	// Fill reservoir B with the nodes feeding inputs to some other nodes
	for (auto a : nodeReservoirA)
	{
		for (auto b : sourceNodesPerNode[a.NodeIdx()])
		{
			nodeReservoirB.Add(b);
		}
	}

	// Now remove the inputing nodes from reservoir a
	for (auto b : nodeReservoirB)
	{
		nodeReservoirA.Remove(b);
	}

	// Reservoir A is left with only outer nodes
}

void UTG_Graph::EvalInOutPins() const
{
	Traversal.InPins.Empty();
	Traversal.OutPins.Empty();

	// just grab the ins and outs pins for the full graph
	ForEachPins([&](const UTG_Pin* Pin, uint32 I, uint32 Node)
	{
		if (Pin->IsValid() && Pin->IsConnected())
		{
			auto a = Pin->GetArgument().GetType().GetAccess();
			if (a == ETG_Access::In)
			{
				Traversal.InPins.Push(Pin->GetId());
			}
			else if (a == ETG_Access::Out)
			{
				Traversal.OutPins.Push(Pin->GetId());
			}
		}
	});
}

void UTG_Graph::EvalTraverseOrder() const
{
	Traversal.TraverseOrder.Empty();
	Traversal.InNodesCount = 0;
	Traversal.OutNodesCount = 0;
	Traversal.NodeWavesCount = 0;

	// Store all the nodes in a set, the starting reservoir
	TSet<FTG_Id> nodeReservoirs[2];
	for (auto& Node : Nodes)
	{
		if (Node)
			nodeReservoirs[0].Add(Node->GetId());
	}

	// For every node, the source nodes needed
	// This array has the same size as 'Nodes" since we will use the same NodeId to index in it
	// so don;'t worry about null nodes here
	TArray<FTG_Ids> sourceNodesArray;
	for (auto& Node : Nodes)
	{
		sourceNodesArray.Emplace(GatherSourceNodes(Node));
	}

	// Find the outer nodes from the reservoir
	// start from the full reservoir, every pass, separate the outers and 
	// record them in the traverse order.
	auto pass = 0;
	auto reservoirIndex = pass % 2;
	while (nodeReservoirs[reservoirIndex].Num())
	{
		GatherOuterNodes(sourceNodesArray, nodeReservoirs[reservoirIndex], nodeReservoirs[(pass + 1) % 2]);
		if (nodeReservoirs[reservoirIndex].IsEmpty())
			break; /// THis should not happen, means a 

		// Add all the outer nodes in the traverse order
		for (auto n : nodeReservoirs[reservoirIndex])
			Traversal.TraverseOrder.Push(n);

		// Store the number of outers in this pass
		Traversal.InNodesCount = nodeReservoirs[reservoirIndex].Num();

		// How many last outer nodes
		if (pass == 0)
			Traversal.OutNodesCount = Traversal.InNodesCount;

		// next pass start fresh with the B reservoir
		nodeReservoirs[reservoirIndex].Empty();
		++pass;
		reservoirIndex = pass % 2;
	}
	Traversal.NodeWavesCount = pass;
	Algo::Reverse(Traversal.TraverseOrder);
}
#if WITH_EDITOR
void UTG_Graph::SetExtraEditorNodes(const TArray<TObjectPtr<const UObject>>& InNodes)
{
	ExtraEditorNodes.Empty();

	for (const UObject* Node : InNodes)
	{
		ExtraEditorNodes.Add(DuplicateObject(Node, this));
	}
}
#endif
const FTG_GraphTraversal& UTG_Graph::GetTraversal() const
{
	UE_LOGF(LogTextureGraph, VeryVerbose, "UTG_Graph::GetTraversal: Dirty = %ls",
	       bIsGraphTraversalDirty ? TEXT("true") : TEXT("false"));
	if (bIsGraphTraversalDirty)
	{
		EvalInOutPins();
		EvalTraverseOrder();
		bIsGraphTraversalDirty = false;
	}
	return Traversal;
}

void UTG_Graph::Traverse(NodeVisitorFunction visitor, int32 graph_depth) const
{
	GetTraversal(); // update the Traversal first if needed

	int32_t i = 0;
	for (auto ni : Traversal.TraverseOrder)
	{
		visitor(Nodes[ni.NodeIdx()], i, graph_depth);
		++i;
	}
}

void UTG_Graph::Evaluate(FTG_EvaluationContext* InContext)
{
	FTG_Evaluation::EvaluateGraph(this, InContext);
}


FString NewLine = TEXT("\r\n");

void UTG_Graph::Log() const
{
	FString LogMessage;
	FString Tab = "   ";

	LogMessage += TEXT("**** Graph State ****") + NewLine;
	LogMessage += LogNodes(Tab);
	LogMessage += LogVars(Tab);
	LogMessage += LogParams(Tab);
	LogMessage += LogTraversal(Tab);
	LogMessage += TEXT("**** *********** ****") + NewLine;

	TArray<FString> Lines;
	LogMessage.ParseIntoArray(Lines, TEXT("\n"));
	for (const FString& Line : Lines)
	{
		UE_LOGF(LogTextureGraph, Display, "%ls", *Line);
	}
}

FString UTG_Graph::LogNodes(FString InTab) const
{
	FString Tab = InTab + "    ";
	FString PinTab = Tab + "    ";
	FString LogMessage;
	LogMessage += Tab + TEXT("**** Nodes ") + NewLine;
	ForEachNodes([&](const UTG_Node* n, int32_t i)
	{
		FString token = n->LogPins(PinTab);
		LogMessage += Tab + FString::Printf(TEXT("%-*s"), LogHeaderWidth, *n->LogHead()) + NewLine;
		LogMessage += token;
	});
	LogMessage += Tab + TEXT("****") + NewLine;

	return LogMessage;
}

FString UTG_Graph::LogParams(FString InTab) const
{
	FString Tab = InTab + "    ";
	FString ParamTab = InTab + "    " + "    ";
	FString LogMessage;
	LogMessage += Tab + TEXT("**** Params ") + NewLine;
	for (auto Param : Params)
	{
		const UTG_Pin* Pin = GetPin(Param.Value);
		if (!Pin || !Pin->IsValid())
		{
			LogMessage += FString::Printf(
				TEXT("Invalid param <%s> referencing pin%s"), *Param.Key.ToString(), *Param.Value.ToString()) + NewLine;
		}
		else
		{
			FString token = Pin->Log(ParamTab);
			LogMessage += FString::Printf(TEXT("%s"), *token) + NewLine;
		}
	}
	LogMessage += Tab + TEXT("****") + NewLine;

	return LogMessage;
}

FString UTG_Graph::LogVars(FString InTab) const
{
	FString Tab = InTab + "    ";

	FString LogMessage;
	LogMessage += Tab + TEXT("**** Vars ") + NewLine;
	ForEachVars([&](const FTG_Var* v, int32 i, int32 n)
	{
		FString token = v->LogValue();
		LogMessage += Tab + FString::Printf(TEXT("%-*s-> %s"), LogHeaderWidth, *v->LogHead(), *token) + NewLine;
	});
	LogMessage += Tab + TEXT("****") + NewLine;

	return LogMessage;
}

FString UTG_Graph::LogTraversal(FString InTab) const
{
	FString Tab = InTab + "    ";

	FString LogMessage;
	LogMessage += Tab + TEXT("**** Traversal ") + NewLine;
	Traverse([&](UTG_Node* n, int32_t i, int32_t l)
	{
		LogMessage += Tab + FString::Printf(TEXT("%-*s%s"), LogHeaderWidth, *n->LogHead(),
		                                    *LogCall(n->GetInputVarIds(), n->GetOutputVarIds())) + NewLine;
	});

	LogMessage += Tab + TEXT("****") + NewLine;

	return LogMessage;
}

FString UTG_Graph::LogCall(const TArray<FTG_Id>& Inputs, const TArray<FTG_Id>& Outputs, int32 InputLogWidth)
{
	FString InputsToken = TEXT("(");
	bool first = true;
	for (auto inId : Inputs)
	{
		InputsToken += FString::Printf(TEXT("%sv%s"), (first ? TEXT("") : TEXT(", ")), *inId.ToString());
		first = false;
	}
	InputsToken += TEXT(")");

	FString OutputsToken = TEXT("[");
	first = true;
	for (auto outId : Outputs)
	{
		OutputsToken += FString::Printf(TEXT("%sv%s"), (first ? TEXT("") : TEXT(", ")), *outId.ToString());
		first = false;
	}
	OutputsToken += TEXT("]");

	return FString::Printf(TEXT("%*s -> %s"), InputLogWidth, *InputsToken, *OutputsToken);
}
