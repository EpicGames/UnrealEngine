// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_GraphEvaluation.h"
#include "TG_Graph.h"
#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "TG_Variant.h"
#include "Transform/Expressions/T_FlatColorTexture.h"
#include "TextureGraphEngine/TextureGraphEngine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_GraphEvaluation)

int FTG_Evaluation::FilterArrayInputs(FTG_EvaluationContext* InContext, const TArray<TObjectPtr<UTG_Pin>>& InPins, 
	TArray<TObjectPtr<UTG_Pin>>& ArrayInputs, TArray<TObjectPtr<UTG_Pin>>& NonArrayPins)
{
	int MaxCount = -1;
	for (const TObjectPtr<UTG_Pin>& InPin : InPins)
	{
		FTG_Argument Arg = InPin->GetArgument();

		const auto VarId = InPin->GetVarId();
		FTG_Var* Var = InContext->Graph->GetVar(VarId);

		if (Var)
		{
			/// If it's input and not an array but the var is an array
			if (Arg.IsInput() && !Arg.IsArray() && Var->IsArray())
			{
				ArrayInputs.Add(InPin);
				MaxCount = FMath::Max<int>(MaxCount, Var->GetAs<FTG_VariantArray>().Num());
			}
			else
				NonArrayPins.Add(InPin);
		}
	}

	return MaxCount;
}

void FTG_Evaluation::TransferVarToPin(UTG_Pin* InPin, FTG_EvaluationContext* Context, int Index)
{
	FTG_Argument Arg = InPin->GetArgument();

	const auto VarId = InPin->GetVarId();
	FTG_Var* Var = Context->Graph->GetVar(VarId);

	if (!Var)
		return;

	check(Var);

	if (Arg.IsInput())
	{
		bool bNeedsConversion = InPin->ConnectionNeedsConversion();
		FName VarConverterKey = InPin->GetInputVarConverterKey();
		FTG_Var* OutVar = InPin->EditConvertedVar(); // Generate the converted version of the connected input into the CoonvertedVar of the Pin
		bool bIsEnum = InPin->GetArgument().ArgumentType.IsEnum();

		// Filter the case if a var is an array
		if (Var->IsArray() && !InPin->EditSelfVar()->IsArray())
		{
			// Introduce a conversion specifically from VarianArray to the target type
			// if there was ALREADY a conversion it is likely compatible anyway
			VarConverterKey = MakeConvertKey("FTG_VariantArray", Arg.CPPTypeName);
			// If no conversion was installed
			if (!bNeedsConversion)
			{
				// TODO: SG : I need to double check that this is needed, it doesn't seem necessary
				OutVar = InPin->EditSelfVar();
				bNeedsConversion = true;
			}
		}

		if (bNeedsConversion)
		{
			FTG_Evaluation::VarConverter* Converter = FTG_Evaluation::DefaultConverters.Find(VarConverterKey);
			// If converter is null then the var is compatible, no need to convert.
			// Else COnverter is valid and we need to run it 
			if ((*Converter))
			{
				FTG_Evaluation::VarConverterInfo Info
				{
					.InVar = Var,
					.OutVar = OutVar,
					.Index = Index,
					.Context = Context
				};

				(*Converter)(Info);

				// Since the source var is converted into the converted var
				// we need to pass the self var as the input for the pin
				Var = Info.OutVar;
			}
		}

		if (InPin->IsArgVariant() && !InPin->IsConnected())
		{
			auto VariantType = Context->CurrentNode->GetExpressionCommonInputVariantType();
			Var->EditAs<FTG_Variant>().ResetTypeAs(VariantType);
		}

		if (InPin->NeedsConformance())
		{
			FTG_Evaluation::VarConformerInfo Info
			{
				.InVar = Var,
				.OutVar = InPin->EditSelfVar(),
				.Index = Index,
				.Context = Context
			};

			bool bResult = (InPin->ConformerFunctor)(Info);
	 		if (bResult)
			{
				// Since the source var is conformed into the SelfVar
	            // we need to pass the self var as the input for the pin
	            Var = Info.OutVar;
			}
		}

		Context->Inputs.VarArguments.Add(Arg.GetName(), { *Var, Arg });
	}
	else if (Arg.IsOutput())
	{
		/// OK, if the output is an array then we just reset the VAR to NULL. 
		/// This will get copied to anyway when the expression runs. 
		if (Var->IsArray())
			Var->Reset();
		Context->Outputs.VarArguments.Add(Arg.GetName(), { Var, Arg });
	}
}

void FloatToInt_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<float>();
	Info.OutVar->EditAs<int>() = Input;
}

void FloatToUInt_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	float Input = Info.InVar->GetAs<float>();
	uint32 Output = 0;
	if (Input > 0)
		Output = (uint32)Input;
	Info.OutVar->EditAs<uint32>() = Output;
}

void FloatToFLinearColor_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<float>();
	Info.OutVar->EditAs<FLinearColor>() = FLinearColor(Input, Input, Input);
}

void FloatToFVector4f_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<float>();
	Info.OutVar->EditAs<FVector4f>() = FLinearColor(Input, Input, Input);
}

void FloatToFVector2f_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<float>();
	Info.OutVar->EditAs<FVector2f>() = FVector2f(Input, Input);
}

void FLinearColorToFVector4f_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<FLinearColor>();
	Info.OutVar->EditAs<FVector4f>() = Input;
}
void FVector4fToFLinearColor_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<FVector4f>();
	Info.OutVar->EditAs<FLinearColor>() = Input;
}

void FLinearColorToFVector2f_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<FLinearColor>();
	Info.OutVar->EditAs<FVector2f>() = FVector2f(Input.R, Input.G);
}
void FVector4fToFVector2f_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<FVector4f>();
	Info.OutVar->EditAs<FVector2f>() = FVector2f(Input);
}


const FString FTG_Evaluation::GVectorToTextureAutoConv_Name = TEXT("_Auto_Conv_Vector_To_Tex_");
const FString FTG_Evaluation::GColorToTextureAutoConv_Name = TEXT("_Auto_Conv_LinearColor_To_Tex_");
const FString FTG_Evaluation::GFloatToTextureAutoConv_Name = TEXT("_Auto_Conv_Float_To_Tex_");

// Produce a BufferDescriptor ideal to store a constant value of the type specified by Variant Type
// The texture generated with the Descriptor will contains enough precision for the consstant to save
BufferDescriptor GetFlatColorDesc(ETG_VariantType InVariantType)
{
	switch (InVariantType)
	{
	case ETG_VariantType::Scalar:
		return T_FlatColorTexture::GetFlatColorDesc(FTG_Evaluation::GFloatToTextureAutoConv_Name, BufferFormat::Half);

	case ETG_VariantType::Color:
		return T_FlatColorTexture::GetFlatColorDesc(FTG_Evaluation::GColorToTextureAutoConv_Name, BufferFormat::Byte);

	case ETG_VariantType::Vector:
		return T_FlatColorTexture::GetFlatColorDesc(FTG_Evaluation::GVectorToTextureAutoConv_Name, BufferFormat::Half);

	default:
		return  BufferDescriptor();
	}
}


void FloatToFTG_Texture_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<float>();
	auto& Output = Info.OutVar->EditAs<FTG_Texture>();
	BufferDescriptor Desc = GetFlatColorDesc(ETG_VariantType::Scalar);
	Output = T_FlatColorTexture::Create(Info.Context->Cycle, Desc, FLinearColor(Input, Input, Input), Info.Context->TargetId);
}

void FLinearColorToFTG_Texture_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<FLinearColor>();
	auto& Output = Info.OutVar->EditAs<FTG_Texture>();
	BufferDescriptor Desc = GetFlatColorDesc(ETG_VariantType::Color);
	Output = T_FlatColorTexture::Create(Info.Context->Cycle, Desc, Input, Info.Context->TargetId);
}

void FVector4fToFTG_Texture_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<FVector4f>();
	auto& Output = Info.OutVar->EditAs<FTG_Texture>();
	BufferDescriptor Desc = GetFlatColorDesc(ETG_VariantType::Vector);
	Output = T_FlatColorTexture::Create(Info.Context->Cycle, Desc, FLinearColor(Input.X, Input.Y, Input.Z, Input.W), Info.Context->TargetId);
}


void FloatToFTG_Variant_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<float>();
	auto& Output = Info.OutVar->EditAs<FTG_Variant>();

	FTG_Variant::EType VariantType = Info.Context->CurrentNode->GetExpressionCommonInputVariantType();

	switch (VariantType)
	{
	case FTG_Variant::EType::Scalar:
		Output.Data.Set<float>(Input);
		break;
	case FTG_Variant::EType::Color:
		Output.Data.Set<FLinearColor>(FLinearColor((float)Input, (float)Input, (float)Input));
		break;
	case FTG_Variant::EType::Vector:
		Output.Data.Set<FVector4f>(FVector4f((float)Input));
		break;
	case FTG_Variant::EType::Texture:
		BufferDescriptor Desc = GetFlatColorDesc(ETG_VariantType::Scalar);
		auto texture = FTG_Texture(
			T_FlatColorTexture::Create(Info.Context->Cycle, Desc,
				FLinearColor((float)Input, (float)Input, (float)Input),
				Info.Context->TargetId));
		Output.Data.Set<FTG_Texture>(texture);
		break;
	}
}

void FLinearColorToFTG_Variant_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<FLinearColor>();
	auto& Output = Info.OutVar->EditAs<FTG_Variant>();

	FTG_Variant::EType VariantType = Info.Context->CurrentNode->GetExpressionCommonInputVariantType();

	switch (VariantType)
	{
	case FTG_Variant::EType::Scalar:
	case FTG_Variant::EType::Color:
		Output.Data.Set<FLinearColor>(Input);
		break;
	case FTG_Variant::EType::Vector:
		Output.Data.Set<FVector4f>(FVector4f(Input.R, Input.G, Input.B, Input.A));
		break;
	case FTG_Variant::EType::Texture:
		BufferDescriptor Desc = GetFlatColorDesc(ETG_VariantType::Color);
		auto texture = FTG_Texture(
			T_FlatColorTexture::Create(Info.Context->Cycle, Desc,
				Input,
				Info.Context->TargetId));
		Output.Data.Set<FTG_Texture>(texture);
		break;
	}
}

void FVector4fToFTG_Variant_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<FVector4f>();
	auto& Output = Info.OutVar->EditAs<FTG_Variant>();

	FTG_Variant::EType VariantType = Info.Context->CurrentNode->GetExpressionCommonInputVariantType();

	switch (VariantType)
	{
	case FTG_Variant::EType::Scalar:
	case FTG_Variant::EType::Color:
	case FTG_Variant::EType::Vector:
		Output.Data.Set<FVector4f>(Input);
		break;
	case FTG_Variant::EType::Texture:
		BufferDescriptor Desc = GetFlatColorDesc(ETG_VariantType::Vector);
		auto texture = FTG_Texture(
			T_FlatColorTexture::Create(Info.Context->Cycle, Desc,
				FLinearColor(Input.X, Input.Y, Input.Z, Input.W),
				Info.Context->TargetId));
		Output.Data.Set<FTG_Texture>(texture);
		break;
	}

	Output.Data.Set<FVector4f>(Input);
}

void FTG_TextureToFTG_Variant_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	FTG_Texture Input = FTG_Texture::GetBlack();

	if (Info.InVar->IsArray())
	{
		FTG_VariantArray VarArray = Info.InVar->GetAs<FTG_VariantArray>();
		check(Info.Index >= 0 && Info.Index < VarArray.Num());
		Input = VarArray.Get(Info.Index).GetTexture();
	}
	else
		Input = Info.InVar->GetAs<FTG_Texture>();
	FTG_Variant& Output = Info.OutVar->EditAs<FTG_Variant>();
	Output.Data.Set<FTG_Texture>(Input);
}

void FTG_VariantToFloat_Converter_Internal(float& Output, const FTG_Variant& Input)
{
	FTG_Variant::EType SourceType = Input.GetType();

	switch (SourceType)
	{
	case FTG_Variant::EType::Texture:
		return;
	case FTG_Variant::EType::Scalar:
		Output = Input.Data.Get<float>();
		break;
	case FTG_Variant::EType::Color:
		Output = Input.Data.Get<FLinearColor>().R;
		break;
	case FTG_Variant::EType::Vector:
		Output = Input.Data.Get<FVector4f>().X;
		break;
	}
}

void FTG_VariantToFloat_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<FTG_Variant>();
	auto& Output = Info.OutVar->EditAs<float>();
	FTG_VariantToFloat_Converter_Internal(Output, Input);
}

void FTG_VariantToFLinearColor_Converter_Internal(FLinearColor& Output, const FTG_Variant& Input)
{
	FTG_Variant::EType SourceType = Input.GetType();

	switch (SourceType)
	{
	case FTG_Variant::EType::Texture:
		return;
	case FTG_Variant::EType::Scalar:
		Output = FLinearColor((float)Input.Data.Get<float>(), (float)Input.Data.Get<float>(), (float)Input.Data.Get<float>());
		break;
	case FTG_Variant::EType::Color:
		Output = Input.Data.Get<FLinearColor>();
		break;
	case FTG_Variant::EType::Vector:
		Output = FLinearColor(Input.Data.Get<FVector4f>().X, Input.Data.Get<FVector4f>().Y, Input.Data.Get<FVector4f>().Z, Input.Data.Get<FVector4f>().W);
		break;
	}
}

void FTG_VariantToFLinearColor_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<FTG_Variant>();
	auto& Output = Info.OutVar->EditAs<FLinearColor>();
	FTG_VariantToFLinearColor_Converter_Internal(Output, Input);
}

void FTG_VariantToFVector4f_Converter_Internal(FVector4f& Output, const FTG_Variant& Input)
{
	FTG_Variant::EType SourceType = Input.GetType();

	switch (SourceType)
	{
	case FTG_Variant::EType::Texture:
		return;
	case FTG_Variant::EType::Scalar:
		Output = FVector4f((float)Input.Data.Get<float>());
		break;
	case FTG_Variant::EType::Color:
		Output = FVector4f(Input.Data.Get<FLinearColor>().R, Input.Data.Get<FLinearColor>().G, Input.Data.Get<FLinearColor>().B, Input.Data.Get<FLinearColor>().A);
		break;
	case FTG_Variant::EType::Vector:
		Output = Input.Data.Get<FVector4f>();
		break;
	}
}

void FTG_VariantToFVector4f_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<FTG_Variant>();
	auto& Output = Info.OutVar->EditAs<FVector4f>();
	FTG_VariantToFVector4f_Converter_Internal(Output, Input);
}

void FTG_VariantToFVector2f_Converter_Internal(FVector2f& Output, const FTG_Variant& Input)
{
	FTG_Variant::EType SourceType = Input.GetType();

	switch (SourceType)
	{
	case FTG_Variant::EType::Texture:
		return;
	case FTG_Variant::EType::Scalar:
		Output = FVector2f((float)Input.Data.Get<float>());
		break;
	case FTG_Variant::EType::Color:
		Output = FVector2f(Input.Data.Get<FLinearColor>().R, Input.Data.Get<FLinearColor>().G);
		break;
	case FTG_Variant::EType::Vector:
		Output = FVector2f(Input.Data.Get<FVector4f>());
		break;
	}
}

void FTG_VariantToFTG_VariantArray_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	FTG_VariantArray& Output = Info.OutVar->EditAs<FTG_VariantArray>();

	if (!Info.InVar->IsArray())
	{
		FTG_Variant& Input = Info.InVar->GetAs<FTG_Variant>();
		int32 Index = 0;
		if (Info.Index >= 0)
		{
			Index = Info.Index;
		}

		if (Output.Num() <= Index)
			Output.SetNum(Index + 1);

		Output.Set(Index, Input);
	}
	else
	{
		Output = Info.InVar->GetAs<FTG_VariantArray>();
	}
}

void FTG_TextureToFTG_VariantArray_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	FTG_VariantArray& Output = Info.OutVar->EditAs<FTG_VariantArray>();

	if (!Info.InVar->IsArray())
	{
		FTG_Texture InTexture = Info.InVar->GetAs<FTG_Texture>();
		int32 Index = 0;
		if (Info.Index >= 0)
		{
			Index = Info.Index;
		}

		if (Output.Num() <= Index)
			Output.SetNum(Index + 1);

		Output.Set(Index, InTexture);
	}
	else
	{
		Output = Info.InVar->GetAs<FTG_VariantArray>();
	}
}

void FTG_VariantToFVector2f_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<FTG_Variant>();
	auto& Output = Info.OutVar->EditAs<FVector2f>();
	FTG_VariantToFVector2f_Converter_Internal(Output, Input);
}

void FTG_VariantToFTG_Texture_Converter_Internal(FTG_Texture& Output, const FTG_Variant& Input, FTG_Evaluation::VarConverterInfo& Info)
{
	FTG_Variant::EType SourceType = Input.GetType();

	FLinearColor Color = FLinearColor::Black;

	switch (SourceType)
	{
	case FTG_Variant::EType::Texture:
		Output = Input.Data.Get<FTG_Texture>();
		return;
	case FTG_Variant::EType::Scalar:
		Color = FLinearColor((float)Input.Data.Get<float>(), (float)Input.Data.Get<float>(), (float)Input.Data.Get<float>());
		break;
	case FTG_Variant::EType::Color:
		Color = Input.Data.Get<FLinearColor>();
		break;
	case FTG_Variant::EType::Vector:
		Color = FLinearColor(Input.Data.Get<FVector4f>().X, Input.Data.Get<FVector4f>().Y, Input.Data.Get<FVector4f>().Z, Input.Data.Get<FVector4f>().W);
		break;
	}
	BufferDescriptor Desc = GetFlatColorDesc(SourceType);
	Output = T_FlatColorTexture::Create(Info.Context->Cycle, Desc, Color, Info.Context->TargetId);
}

void FTG_VariantToFTG_Texture_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<FTG_Variant>();
	auto& Output = Info.OutVar->EditAs<FTG_Texture>();
	FTG_VariantToFTG_Texture_Converter_Internal(Output, Input, Info);
}

void FTG_VariantArrayToFTG_Variant_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	const FTG_VariantArray& InputArray = Info.InVar->GetAs<FTG_VariantArray>();
	check(Info.Index >= 0 && Info.Index < InputArray.Num());
	const FTG_Variant& Input = InputArray.GetArray()[Info.Index];
	FTG_Variant& Output = Info.OutVar->EditAs<FTG_Variant>();
	Output = Input;
}

void FTG_VariantArrayToFTG_Texture_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	const FTG_VariantArray& InputArray = Info.InVar->GetAs<FTG_VariantArray>();
	check(Info.Index >= 0 && Info.Index < InputArray.Num());
	const FTG_Variant& Input = InputArray.GetArray()[Info.Index];

	FTG_Texture& Output = Info.OutVar->EditAs<FTG_Texture>();
	FTG_VariantToFTG_Texture_Converter_Internal(Output, Input, Info);
}

void FloatToEnum_Converter(FTG_Evaluation::VarConverterInfo& Info)
{
	auto Input = Info.InVar->GetAs<float>();
	Info.OutVar->EditAs<int32>() = (int32)Input;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
#define VAR_CONVERTER(nameFrom, nameTo, function)	{ MakeConvertKey(TEXT(#nameFrom), TEXT(#nameTo)), &function }
#define VAR_CONVERTER_DEF(nameFrom, nameTo, function)		VAR_CONVERTER(#nameFrom, #nameTo, #function)
#define VAR_CONVERTER_NULL(nameFrom, nameTo)	{ MakeConvertKey(TEXT(#nameFrom), TEXT(#nameTo)), nullptr }

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
FTG_Evaluation::ConverterMap FTG_Evaluation::DefaultConverters
(
	{
		VAR_CONVERTER(float, int, FloatToInt_Converter),
		VAR_CONVERTER(float, int32, FloatToInt_Converter),
		VAR_CONVERTER(float, uint32, FloatToUInt_Converter),
		VAR_CONVERTER(float, FLinearColor, FloatToFLinearColor_Converter),
		VAR_CONVERTER(float, FVector4f, FloatToFVector4f_Converter),
		VAR_CONVERTER(float, FTG_Texture, FloatToFTG_Texture_Converter),
		VAR_CONVERTER(float, FVector2f, FloatToFVector2f_Converter),
		VAR_CONVERTER(float, Enum, FloatToEnum_Converter),

		VAR_CONVERTER(FLinearColor, FVector4f, FLinearColorToFVector4f_Converter),
		VAR_CONVERTER(FLinearColor, FTG_Texture, FLinearColorToFTG_Texture_Converter),
		VAR_CONVERTER(FLinearColor, FVector2f, FLinearColorToFVector2f_Converter),

		VAR_CONVERTER(FVector4f, FLinearColor, FVector4fToFLinearColor_Converter),
		VAR_CONVERTER(FVector4f, FTG_Texture, FVector4fToFTG_Texture_Converter),
		VAR_CONVERTER(FVector4f, FVector2f, FVector4fToFVector2f_Converter),

		VAR_CONVERTER(float, FTG_Variant, FloatToFTG_Variant_Converter),
		VAR_CONVERTER(FLinearColor, FTG_Variant, FLinearColorToFTG_Variant_Converter),
		VAR_CONVERTER(FVector4f, FTG_Variant, FVector4fToFTG_Variant_Converter),
		VAR_CONVERTER(FTG_Texture, FTG_Variant, FTG_TextureToFTG_Variant_Converter),

		VAR_CONVERTER(FTG_VariantArray, FTG_Variant, FTG_VariantArrayToFTG_Variant_Converter),
		VAR_CONVERTER(FTG_VariantArray, FTG_Texture, FTG_VariantArrayToFTG_Texture_Converter),

		VAR_CONVERTER(FTG_Variant.Scalar, float, FTG_VariantToFloat_Converter),

		VAR_CONVERTER(FTG_Variant.Scalar, FLinearColor, FTG_VariantToFLinearColor_Converter),
		VAR_CONVERTER(FTG_Variant.Color, FLinearColor, FTG_VariantToFLinearColor_Converter),
		VAR_CONVERTER(FTG_Variant.Vector, FLinearColor, FTG_VariantToFLinearColor_Converter),

		VAR_CONVERTER(FTG_Variant.Scalar, FVector4f, FTG_VariantToFVector4f_Converter),
		VAR_CONVERTER(FTG_Variant.Color, FVector4f, FTG_VariantToFVector4f_Converter),
		VAR_CONVERTER(FTG_Variant.Vector, FVector4f, FTG_VariantToFVector4f_Converter),

		VAR_CONVERTER(FTG_Variant.Scalar, FTG_Texture, FTG_VariantToFTG_Texture_Converter),
		VAR_CONVERTER(FTG_Variant.Color, FTG_Texture, FTG_VariantToFTG_Texture_Converter),
		VAR_CONVERTER(FTG_Variant.Vector, FTG_Texture, FTG_VariantToFTG_Texture_Converter),
		VAR_CONVERTER(FTG_Variant.Texture, FTG_Texture, FTG_VariantToFTG_Texture_Converter),
		VAR_CONVERTER(FTG_Variant, FTG_Texture, FTG_VariantToFTG_Texture_Converter),

		VAR_CONVERTER(FTG_Variant.Scalar, FVector2f, FTG_VariantToFVector2f_Converter),
		VAR_CONVERTER(FTG_Variant.Color, FVector2f, FTG_VariantToFVector2f_Converter),
		VAR_CONVERTER(FTG_Variant.Vector, FVector2f, FTG_VariantToFVector2f_Converter),

		VAR_CONVERTER(FTG_Variant.Scalar, FTG_VariantArray, FTG_VariantToFTG_VariantArray_Converter),
		VAR_CONVERTER(FTG_Variant.Color, FTG_VariantArray, FTG_VariantToFTG_VariantArray_Converter),
		VAR_CONVERTER(FTG_Variant.Vector, FTG_VariantArray, FTG_VariantToFTG_VariantArray_Converter),
		VAR_CONVERTER(FTG_Variant.Texture, FTG_VariantArray, FTG_VariantToFTG_VariantArray_Converter),
		VAR_CONVERTER(FTG_Texture, FTG_VariantArray, FTG_TextureToFTG_VariantArray_Converter),

		VAR_CONVERTER_NULL(FTG_Variant.Scalar, FTG_Variant),
		VAR_CONVERTER_NULL(FTG_Variant.Color, FTG_Variant),
		VAR_CONVERTER_NULL(FTG_Variant.Vector, FTG_Variant),
		VAR_CONVERTER_NULL(FTG_Variant.Texture, FTG_Variant),
	}
);

FName FTG_Evaluation::MakeConvertKey(FName From, FName To)
{
	return FName(From.ToString() + TEXT("To") + To.ToString());
}

FName FTG_Evaluation::MakeConvertKey(const FTG_Argument& ArgFrom, const FTG_Argument& ArgTo)
{
	if (!ArgTo.ArgumentType.IsEnum())
		return MakeConvertKey(ArgFrom.GetCPPTypeName(), ArgTo.GetCPPTypeName());

	return FName(ArgFrom.GetCPPTypeName().ToString() + TEXT("ToEnum"));
}

bool FTG_Evaluation::AreArgumentsCompatible(const FTG_Argument& ArgFrom, const FTG_Argument& ArgTo, FName& ConverterKey)
{
	ConverterKey = FName();
	auto AT = ArgFrom.GetCPPTypeName();
	auto BT = ArgTo.GetCPPTypeName();

	if (AT == BT)
		return true;

	FName FromToName = MakeConvertKey(ArgFrom, ArgTo);
	auto Found = DefaultConverters.Find(FromToName);
	if (Found)
	{
		ConverterKey = FromToName;
		return true;
	}

	return false;
}

void FTG_Evaluation::EvaluateGraph(UTG_Graph* InGraph, FTG_EvaluationContext* InContext)
{
	// Entering a new Graph scope, we build a new EvaluationContext
	FTG_EvaluationContext EvalContext;
	EvalContext.Cycle = InContext->Cycle;
	EvalContext.Graph = InGraph;
	EvalContext.GraphDepth = InContext->GraphDepth;
	
	// Gather the external vars which are connected to Params of this graph
	// As inputs
	for (auto V : InContext->Inputs.VarArguments)
	{
		auto ParamId = InGraph->FindParamPinId(V.Key);
		if (ParamId.IsValid())
		{
			auto TheVar = InGraph->GetVar(ParamId);
			if (TheVar)
			{
				V.Value.Var.CopyTo(TheVar);
				//TheVar->ShareData(*V.Value.Var);
				EvalContext.ConnectedInputParamIds.Add(ParamId);
			}
		}
	}
	// As outputs
	for (auto V : InContext->Outputs.VarArguments)
	{
		auto ParamId = InGraph->FindParamPinId(V.Key);
		if (ParamId.IsValid())
		{
			EvalContext.ConnectedOutputParamIds.Add(ParamId);
		}
	}

	// The graph evaluation context is initialized and becomes the expression evaluation context
	auto ExpressionEvalContext = &EvalContext;
	InGraph->Traverse([&, ExpressionEvalContext](UTG_Node* n, int32_t i, int32_t l)
		{
			FTG_Evaluation::EvaluateNode(n, ExpressionEvalContext);
		});

	// After evaluation, transfer the output param data to the upper graph's vars
	for (auto V : InContext->Outputs.VarArguments)
	{
		auto ParamId = InGraph->FindParamPinId(V.Key);
		if (ParamId.IsValid())
		{
			auto TheVar = InGraph->GetVar(ParamId);
			if (TheVar)
			{
				V.Value.Var->ShareData(*TheVar);
			}
		}
	}
}

void FTG_Evaluation::EvaluateNodeArray(UTG_Node* InNode, const TArray<TObjectPtr<UTG_Pin>>& ArrayInputs, 
	const TArray<TObjectPtr<UTG_Pin>>& NonArrayPins, int MaxCount, FTG_EvaluationContext* InContext)
{
	check(!ArrayInputs.IsEmpty());
	check(MaxCount > 0);

	/// Initialize an array for each of the output pins/vars
	TMap<FName, FTG_VariantArray> ArrayOutputs;

	/// Non array pins only need to have transfer once. We use the same loop to construct our
	/// array pins and copy over the value of the pins that were evaluated as arrays in prevous
	/// update cycles. This way we retain the texture descriptors
	for (UTG_Pin* Pin : NonArrayPins)
	{
		FTG_Argument Arg = Pin->GetArgument();
		const auto VarId = Pin->GetVarId();
		FTG_Var* Var = InContext->Graph->GetVar(VarId);

		if (Arg.IsOutput())
		{
			FTG_VariantArray ArrayOutput;
			if (Var->IsArray())
				ArrayOutput = Var->GetAs<FTG_VariantArray>();
			ArrayOutput.SetNum(MaxCount);
			ArrayOutputs.Add(Arg.GetName(), std::move(ArrayOutput));
		}

		FTG_Evaluation::TransferVarToPin(Pin, InContext, -1);
	}

	for (int VarIndex = 0; VarIndex < MaxCount; VarIndex++)
	{
		FTG_Variant::EType CommonVariantType = FTG_Variant::EType::Invalid;

		/// Run through array inputs, one at a time
		for (int ArrayIndex = 0; ArrayIndex < ArrayInputs.Num(); ArrayIndex++)
		{
			UTG_Pin* ArrayPin = ArrayInputs[ArrayIndex];
			FTG_Var* Var = InContext->Graph->GetVar(ArrayPin->GetVarId());
			check(Var && Var->IsArray());
			const FTG_VariantArray& VarArray = Var->GetAs<FTG_VariantArray>();

			/// If an array is smaller than the largest array supplied in the inputs, then we 
			/// simply clamp it. Currently, it's the responsibility of the user to ensure that 
			/// inputs match up in terms of their arrays (with the exception that the array has
			/// one input)
			int VarArrayIndex = FMath::Min(VarIndex, VarArray.Num() - 1);

			if (VarIndex >= VarArray.Num() && VarArray.Num() > 1)
			{
				FString ErrorMsg = FString::Printf(TEXT("Input array mismatch. Array input %s has total number of %d items but the maximum input array length for the node is: %d"),
					*ArrayPin->GetArgumentName().ToString(), VarArray.Num(), MaxCount);
				TextureGraphEngine::GetErrorReporter(InContext->Cycle->GetMix())
					->ReportWarning((int32)ETextureGraphErrorType::INPUT_ARRAY_WARNING, ErrorMsg, InNode);
			}

			const FTG_Variant& Variant = VarArray.GetArray()[VarArrayIndex];
			if (Variant.GetType() > CommonVariantType)
				CommonVariantType = Variant.GetType();

			FTG_Evaluation::TransferVarToPin(ArrayPin, InContext, VarArrayIndex);
		}

		/// Now we evaluate the expression
		UTG_Expression* Expression = InNode->GetExpression();
		Expression->ResetCommonInputVariantType(CommonVariantType);

		/// Trigger the Evaluation
		Expression->SetupAndEvaluate(InContext);

		for (auto& OutputVar : InContext->Outputs.VarArguments)
		{
			FTG_VariantArray* ArrayOutput = ArrayOutputs.Find(OutputVar.Key);
			check(ArrayOutput && ArrayOutput->Num() > VarIndex);

			FString FromType = OutputVar.Value.Argument.GetCPPTypeName().ToString();
			FString ToType = TEXT("FTG_Variant");

			/// If the output type is not a variant then we need to convert it variant
			if (FromType.Find(ToType) != 0)
			{
				FName FromToName = MakeConvertKey(FName(*FromType), FName(*ToType));
				VarConverter* Converter = DefaultConverters.Find(FromToName);
				check(Converter);

				FTG_Var VariantVar;
				FTG_Variant Variant;
				VariantVar.SetAs<FTG_Variant>(Variant);

				FTG_Evaluation::VarConverterInfo Info;
				Info.Context = InContext;
				Info.InVar = OutputVar.Value.Var;
				Info.OutVar = &VariantVar; // Generate the converted version of the connected input into the CoonvertedVar of the Pin
				Info.Index = -1;

				(*Converter)(Info);

				/// We just copy it over in the array
				ArrayOutput->Set(VarIndex, VariantVar.GetAs<FTG_Variant>());
			}
			else if (FromType == TEXT("FTG_VariantArray"))
			{
				ArrayOutput->CopyFrom(OutputVar.Value.Var->GetAs<FTG_VariantArray>());
			}
			else
			{
				ArrayOutput->Set(VarIndex, OutputVar.Value.Var->GetAs<FTG_Variant>());
			}
		}
	}

	/// We secretly replace the output var(s) with the array outputs that we've accumulated in the 
	/// loop above. This will ensure that the array of outputs flows downstream to other connected nodes
	for (auto& OutputVar : InContext->Outputs.VarArguments)
	{
		FTG_VariantArray* ArrayOutput = ArrayOutputs.Find(OutputVar.Key);
		OutputVar.Value.Var->ResetAs<FTG_VariantArray>();
		OutputVar.Value.Var->SetAs<FTG_VariantArray>(*ArrayOutput);
		OutputVar.Value.Var->SetArray();
	}
}

void FTG_Evaluation::EvaluateNode(UTG_Node* InNode, FTG_EvaluationContext* InContext)
{
	const auto Expression = InNode->GetExpression();
	if (Expression)
	{
		InContext->CurrentNode = InNode;

		// Grab the vars from the pins and load them in the context's input and output arrays
		InContext->Inputs.Empty();
		InContext->Outputs.Empty();

		TArray<TObjectPtr<UTG_Pin>> ArrayInputs, NonArrayPins;
		int MaxCount = FilterArrayInputs(InContext, InNode->Pins, ArrayInputs, NonArrayPins); 

		if (ArrayInputs.IsEmpty())
		{
			/// Just do normal evaluation
			for (UTG_Pin* Pin : InNode->Pins)
			{
				FTG_Evaluation::TransferVarToPin(Pin, InContext, -1);
			}

			// Trigger the evaluation of the expression
			Expression->SetupAndEvaluate(InContext);
		}
		else
		{
			EvaluateNodeArray(InNode, ArrayInputs, NonArrayPins, MaxCount, InContext);
		}

		// After the evaluation, notify the postEvaluate for thumbnails
		InNode->GetGraph()->NotifyNodePostEvaluate(InNode, InContext);

		InContext->CurrentNode = nullptr;
	}
}
