// Copyright Epic Games, Inc. All Rights Reserved.
#include "TG_Hash.h"

#include "Internationalization/TextLocalizationResource.h"

FTG_Hash TG_HashName(FTG_Name& Name)
{
	return FTextLocalizationResource::HashString(Name.ToString());
}

FTG_Hash TG_Hash(FTG_Hash H)
{
	return std::hash<FTG_Hash>{}(H);
}

DEFINE_LOG_CATEGORY(LogTextureGraph);


// Util function to validate that a name is unique in a collection and make a new candidate name if not
// It returns a FName equal to the Candidate if unique in the collection or a proposed edited candidate name which is unique
FName TG_MakeNameUniqueInCollection(FName CandidateName, const TArray<FName>& Collection, FName RecursionPreFix, int32 RecursionCount)
{
	FName Tested = CandidateName;
	if (RecursionCount > 0)
	{
		Tested = FName(CandidateName.ToString() + RecursionPreFix.ToString() + FString::FromInt(RecursionCount));
	}
	auto Found = Collection.Find(Tested);
	if (Found == INDEX_NONE)
	{
		return Tested;
	}
	else
	{
		++RecursionCount;
		return 	TG_MakeNameUniqueInCollection(CandidateName, Collection, RecursionPreFix, RecursionCount);
	}
}