// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_HelperFunctions.h"

#include "EngineAnalytics.h"
#include "Job/Scheduler.h"
#include "TG_Texture.h"
#include "TG_Graph.h"
#include "Export/TextureExporter.h"
#include "Expressions/Output/TG_Expression_Output.h"
#include "Expressions/TG_Expression.h"
#include "Model/Mix/MixManager.h"
#include "Transform/Expressions/T_FlatColorTexture.h"
#include "Logging/MessageLog.h"
#include "Misc/FileHelper.h"
#include "TextureGraph.h"
TArray<BlobPtr> FTG_HelperFunctions::GetTexturedOutputs(const UTG_Node* Node, FTG_EvaluationContext* TextureConversionContext /*= nullptr*/)

{
	TArray<BlobPtr> Outputs;

	if (Node)
	{
		auto OutPinIds = Node->GetOutputPinIds();
		for (auto Id : OutPinIds)
		{
			//This is a work around for checking the type of the output
			//Probably we need to have a better solution for checking output type
			auto Pin = Node->GetGraph()->GetPin(Id);
			FTG_Texture Texture;
			Pin->GetValue(Texture);

			if (Texture && Texture.RasterBlob)
			{
				Outputs.Add(Texture.RasterBlob);
			}
			// if there is a conversion context provided we assume we want to force convert the variant to a texture 
			else if (TextureConversionContext != nullptr)
			{
				UTG_Expression_Output* OutputExpression = Cast<UTG_Expression_Output>(Node->GetExpression());
				// convert to a texture if forced
				if (OutputExpression)
				{
					BufferDescriptor DesiredDesc = OutputExpression->Output.EditTexture().GetBufferDescriptor();
					if (DesiredDesc.Width == 0 || DesiredDesc.Height == 0)
					{
						FTG_OutputSettings& OutputSetting = OutputExpression->OutputSettings;
				
						DesiredDesc = T_FlatColorTexture::GetFlatColorDesc("Output");
						DesiredDesc.Width = (uint32)OutputSetting.Width;
						DesiredDesc.Height = (uint32)OutputSetting.Height;
					}

					// OutputExpression->Output.EditTexture() =
					auto ConvertedTexture = OutputExpression->Source.GetTexture(TextureConversionContext, FTG_Texture::GetBlack(), &DesiredDesc);
					Outputs.Add(ConvertedTexture.RasterBlob);
				}
			}
		}
	}

	return Outputs;
}

void FTG_HelperFunctions::EnsureOutputIsTexture(MixUpdateCyclePtr Cycle, UTG_Node* Node)
{
	check(Cycle);

	TMap<UTG_Pin*, FTG_Texture*> Textures;

	if (Node)
	{
		auto OutPinIds = Node->GetOutputPinIds();
		for (auto Id : OutPinIds)
		{
			auto Pin = Node->GetGraph()->GetPin(Id);
			auto TypeName = Pin->GetArgument().GetCPPTypeName().ToString();

			if (TypeName.Contains("FTG_Variant"))
			{
				auto Var = Node->GetGraph()->GetVar(Id);
				if (Var != nullptr)
				{
					FTG_Variant Variant = Var->EditAs<FTG_Variant>();
					FTG_EvaluationContext EvaluationContext;
					EvaluationContext.Cycle = Cycle;

					/// This will ensure 
					Variant.EditTexture() = Variant.GetTexture(&EvaluationContext);
				}
			}
		}
	}
}

JobBatchPtr FTG_HelperFunctions::InitExportBatch(UTextureGraphBase* InTextureGraph, FString ExportPath, FString AssetName, FExportSettings& TargetExportSettings,
	bool OverrideExportPath, bool OverwriteTextures, bool ExportAllOutputs, bool bSave)
{
	FString ErrorMessage = "";
	bool AnyValidExport = false;
	TargetExportSettings.Reset();

	FInvalidationDetails Details;
	Details.All();
	Details.Mix = InTextureGraph;
	Details.bExporting = true;
	auto Batch = JobBatch::Create(Details);
	/// Update the mix so that the rendering Cycle gets populated
	MixUpdateCyclePtr Cycle = Batch->GetCycle();
	InTextureGraph->Update(Cycle);

	InTextureGraph->Graph()->ForEachNodes([=,&TargetExportSettings, &ErrorMessage , &AnyValidExport](const UTG_Node* Node, uint32 Index)
		{
			UTG_Expression_Output* TargetExpression = Cast<UTG_Expression_Output>(Node->GetExpression());
			if (TargetExpression)
			{
				FTG_OutputSettings& OutputSettings = TargetExpression->OutputSettings;

				if (TargetExpression->GetShouldExport() || ExportAllOutputs)
				{
					FTG_EvaluationContext EvaluationContext;
					EvaluationContext.Cycle = Batch->GetCycle();

					auto ExportBlobs = FTG_HelperFunctions::GetTexturedOutputs(Node, &EvaluationContext);

					FString Path = OutputSettings.FolderPath.ToString();
					if (OverrideExportPath)
					{
						Path = ExportPath;
					}

					FString FileName = AssetName.IsEmpty() ? OutputSettings.GetFullOutputName() : AssetName;

					FString PathErrors;
					bool IsPathValid = TextureExporter::IsFilePathValid(FName(*FileName), FName(*Path), PathErrors);
					bool IsPackageValid = TextureExporter::IsPackageNameValid(Path, FileName);
					
					bool HasOutputs = ExportBlobs.Num() > 0;

					if (HasOutputs && IsPathValid && IsPackageValid)
					{
						TiledBlobPtr Output = std::static_pointer_cast<TiledBlob>(ExportBlobs[0]);//Dealing with one output per Node for now
						FExportMapSettings MapSettings = TextureExporter::GetExportSettingsForTarget(TargetExportSettings, std::static_pointer_cast<TiledBlob>(Output), *FileName);
						MapSettings.Name = FName(*FileName);
						MapSettings.Path = Path;
						MapSettings.UseOverridePath = OverrideExportPath;
						MapSettings.OverwriteTextures = OverwriteTextures;
						MapSettings.LODGroup = OutputSettings.LODGroup;
						MapSettings.Compression = OutputSettings.Compression;
						MapSettings.IsSRGB = OutputSettings.bSRGB;
						MapSettings.Width = (int32)OutputSettings.Width;
						MapSettings.Height = (int32)OutputSettings.Height;
						MapSettings.bSave = bSave;
						TargetExportSettings.ExportPreset.push_back(std::pair<FName, FExportMapSettings>{ MapSettings.Name, MapSettings });
						AnyValidExport = true;
					}
					else
					{
						//Log Error to Error System
						if (!HasOutputs)
						{
							ErrorMessage += FString::Format(TEXT("Texture Export Error : No valid output found for OutputSetting {0}"), { OutputSettings.OutputName.ToString() });
							ErrorMessage += "\n";
						}
						if (!IsPathValid)
						{
							ErrorMessage += FString::Format(TEXT("Texture Export Error : {0} OutputSettings Node: {1}"), { PathErrors, OutputSettings.OutputName.ToString() });
							ErrorMessage += "\n";
						}
						if (!IsPackageValid)
						{
							FString Error = "Invalid Package name";
							ErrorMessage += FString::Format(TEXT("Texture Export Error : {0} OutputSettings Node: {1}"), { Error, OutputSettings.OutputName.ToString() });
							ErrorMessage += "\n";
						}
					}
				}
			}
		});

	if (!ErrorMessage.IsEmpty())
	{
		ErrorMessage = ErrorMessage.LeftChop(1);
		auto ErrorType = static_cast<int32>(ETextureGraphErrorType::UNSUPPORTED_TYPE);
		TextureGraphEngine::GetErrorReporter(InTextureGraph)->ReportError(ErrorType, ErrorMessage, nullptr);

		FMessageLog("PIE").Error()
			->AddToken(FTextToken::Create(FText::FromString(ErrorMessage)));

		if (!AnyValidExport)
		{
			return nullptr;
		}
	}

	return Batch;
}

AsyncInt FTG_HelperFunctions::ExportAsync(UTextureGraphBase* InTextureGraph, FString ExportPath, FString AssetName, FExportSettings& TargetExportSettings, bool OverrideExportPath, bool OverwriteTextures /*= true*/, bool ExportAllOutputs /*= false*/,bool bSave /*= true*/)
{
	// Make a copy of the settings so that we don't have any threading issues or issues if TG window is closed
	// while the async operation is in progress
	TSharedRef<FExportSettings> SessionSettings = MakeShared<FExportSettings>(TargetExportSettings); 
	
	JobBatchPtr Batch = InitExportBatch(InTextureGraph, ExportPath, AssetName, *SessionSettings, OverrideExportPath, OverwriteTextures, ExportAllOutputs, bSave);

	if (!Batch)
	{
		// Nothing to export → notify and resolve to 0.
		Util::OnGameThread([Sess = SessionSettings]() { Sess->OnDone.ExecuteIfBound(); });
		return cti::make_ready_continuable<int32>(0);
	}
	
	return RenderAsync(InTextureGraph, Batch)
		.then([InTextureGraph, SessionSettings, ExportPath](auto result) mutable
		{
			return TextureExporter::ExportAsUAsset(InTextureGraph, SessionSettings, ExportPath);
		})
		.then([InTextureGraph, SessionSettings](int32 NumExports) mutable
		{
			if (IsValid(InTextureGraph))
			{
				InTextureGraph->InvalidateAll();
			}
				
			// Add analytics tag
			if (FEngineAnalytics::IsAvailable())
			{
				TArray<FAnalyticsEventAttribute> Attributes;
				Attributes.Add(FAnalyticsEventAttribute(TEXT("NumExports"),  NumExports));
				// Send Analytics event 
				FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.TextureGraph.Export"), Attributes);
			}
			return NumExports;
		});
}

void FTG_HelperFunctions::InitTargets(UTextureGraphBase* InTextureGraph)
{
	TextureGraphEngine::RegisterErrorReporter(InTextureGraph, std::make_shared<FTextureGraphErrorReporter>());
	/// Now run the update Cycle

	int num = 1;
	UMixSettings* Settings = InTextureGraph->GetSettings();
	Settings->FreeTargets();
	Settings->InitTargets(num);

	/// Now we add these to the scene
	for (size_t i = 0; i < num; i++)
	{
		TargetTextureSetPtr target = std::make_unique<TargetTextureSet>(
			(int32)i,
			TEXT(""),
			nullptr,
			InTextureGraph->Width(),
			InTextureGraph->Height()
		);

		target->Init();

		Settings->SetTarget(i, target);
	}
}

JobBatchPtr FTG_HelperFunctions::InitRenderBatch(UTextureGraphBase* InTextureGraph, JobBatchPtr ExistingBatch /* = nullptr */)
{
	/// Now run the update Cycle
	JobBatchPtr NewBatch = nullptr;
	JobBatchPtr Batch = ExistingBatch;

	if (!ExistingBatch)
	{
		FInvalidationDetails Details;
		Details.All();
		Details.Mix = InTextureGraph;

		NewBatch = JobBatch::Create(Details);
		MixUpdateCyclePtr Cycle = NewBatch->GetCycle();

		/// Update the mix so that the rendering Cycle gets populated
		InTextureGraph->Update(Cycle);

		Batch = NewBatch;
	}

	return Batch;
}

AsyncBool FTG_HelperFunctions::RenderAsync(UTextureGraphBase* InTextureGraph, JobBatchPtr ExistingBatch /* = nullptr */)
{
	JobBatchPtr Batch = InitRenderBatch(InTextureGraph, ExistingBatch);

	if (!Batch)
	{
		return cti::make_ready_continuable(true);
	}

	std::atomic_bool* IsMixRendered = new std::atomic_bool(false);
	std::mutex* mutex = new std::mutex();

	/// This needs to update because now we're dependent on queues processing in
	/// a separate thread. Those queues start when Device::Update is called for
	/// the first time. 
	Util::OnBackgroundThread([=]()
	{
		{
			/// Lock the mutex so that the Batch->OnDone callback doesn't get executed past the 
			/// *IsMixRendered = true atomic flag set
			std::unique_lock<std::mutex> lock(*mutex);

			/// Lock the engine
			if (TextureGraphEngine::IsTestMode())
				TextureGraphEngine::Lock();

			/// We must get out of the loop if the engine is being destroyed. This can happen
			/// when an test has offended the time limit for the test. Can result in tests exiting
			/// and calling TextureGraphEngine::Destroy even though this loop might still be running. 
			/// We need a safe passage out of this.
			while (!*IsMixRendered && !TextureGraphEngine::IsDestroying())
			{
				Util::OnGameThread([]()
				{
					if (!TextureGraphEngine::IsDestroying())
						TextureGraphEngine::Update(0);
				});

				std::this_thread::sleep_for(std::chrono::milliseconds(10));
			}

			/// Unlock it
			if (TextureGraphEngine::IsTestMode())
				TextureGraphEngine::Unlock();
		}
	});

	return cti::make_continuable<bool>([=](auto&& promise) mutable 
	{
		Batch->OnDone([=, FWD_PROMISE(promise)](JobBatch*) mutable
		{
			/// We need to do this on the background thread because we can potentially have a deadlock
			/// where this thread blocks on the mutex wait and the TextureGraphEngine::Update above is waiting
			/// for Util::OnGameThread update. This situation cannot be allowed to happen
			Util::OnBackgroundThread([=, FWD_PROMISE(promise)]() mutable
			{
				/// Set the atomic so that the loop above with TextureGraphEngine::Update can exit
				/// and release the mutex that we're going to acquire below
				*IsMixRendered = true;

				{
					std::unique_lock<std::mutex> lock(*mutex);

					delete IsMixRendered;
					IsMixRendered = nullptr;
				}

				delete mutex;
				mutex = nullptr;

				Util::OnGameThread([=, FWD_PROMISE(promise)]() mutable
				{
					promise.set_value(true);
				});
			});
		});

		if (Batch)
		{
			Util::OnGameThread([=]()
				{
					TextureGraphEngine::GetScheduler()->AddBatch(Batch);
				});
		}
	}); 
}
