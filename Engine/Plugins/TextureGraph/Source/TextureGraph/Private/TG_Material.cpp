// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_Material.h"

#include "TG_CustomVersion.h"
#include "TG_Var.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Material)

template <> FString TG_Var_LogValue(FTG_Material& Value)
{
	return FString::Printf(TEXT("FTG_Material %s"), (*Value.AssetPath.ToString()));
}

bool FTG_Material::IsValid() const
{
	return AssetPath.IsValid();
}

UMaterialInterface* FTG_Material::GetMaterial() const
{
	return Cast<UMaterialInterface>(AssetPath.TryLoad());
}

void FTG_Material::SetMaterial(UMaterialInterface* InMaterial)
{
	if (InMaterial)
	{
		AssetPath = FSoftObjectPath(InMaterial);
	}
	else
	{
		AssetPath.Reset();
	}
}

void FTG_Material::ResetTexturePath()
{
	AssetPath.Reset();
}

FArchive& operator<<(FArchive& Ar, FTG_Material& T)
{
	Ar.UsingCustomVersion(FTG_CustomVersion::GUID);

	Ar << T.AssetPath;

	return Ar;
}

bool FTG_Material::Serialize(FArchive& Ar)
{
	Ar << *this;
	return true;
}
