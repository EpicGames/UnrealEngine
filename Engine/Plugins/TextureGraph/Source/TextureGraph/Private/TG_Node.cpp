// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_Node.h"

#include "TG_Graph.h"
#include "Expressions/TG_Expression.h"
#include "TG_CustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Node)

UTG_Graph* UTG_Node::GetGraph() const { return Cast<UTG_Graph>(GetOuter()); }

bool UTG_Node::Validate(MixUpdateCyclePtr Cycle)
{
	// Add node level checks here
	if (!WarningStack.IsEmpty())
	{
		UMixInterface* ParentMix = Cast<UMixInterface>(GetOutermostObject());
		for (auto Warning : WarningStack)
		{
			auto ErrorType = static_cast<int32>(ETextureGraphErrorType::NODE_WARNING);
			FTextureGraphErrorReport Report = TextureGraphEngine::GetErrorReporter(ParentMix)->ReportWarning(ErrorType, Warning.ToString(), this);
		}
	}

	// Validate the expression
	return Expression->Validate(Cycle);
	
}

FTG_Name UTG_Node::GetNodeName() const
{
	return GetExpression()->GetTitleName();
}

#if WITH_EDITOR
void UTG_Node::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UE_LOGF(LogTextureGraph, VeryVerbose, "UTG_Node::PostEditChangeProperty.");
}

void UTG_Node::PostEditUndo()
{
	OnPostUndo();
	
	UObject::PostEditUndo();
}

void UTG_Node::OnPostUndo()
{
	if (IsValidChecked(this))
	{
		// We dont know what got modified.
		// For now, just copying all the values from the expression.
		for (const auto& Pin : Pins)
		{
			/// exclude the case of input textures but only for expressions that can't handle it (MaterialID change made by Zohaib Javed in CL: 31512086)
			if (!(Pin->IsInput() && Pin->IsArgTexture() && Expression && Expression->IgnoreInputTextureOnUndo()))
			{
				auto Arg = Pin->GetArgument();
				Pin->EditSelfVar()->CopyFrom(GetExpression(), Arg);
			}
		}

		Signature = GetExpression()->GetSignature();
	}
}
#endif

TArray<FName> UTG_Node::GetPinAliasNames(uint8 InAccessFilter) const
{
	TArray<FName> Names;

	for (auto Pin : Pins)
	{
		if (Pin && Pin->Argument.ArgumentType.MatchAccessFilter(InAccessFilter))
			Names.Add(Pin->GetAliasName());
	}

	return Names;
}


FName UTG_Node::ValidateGeneratePinAliasName(FName CandidateName, const FTG_Id& PinId) const
{
	// The pin is testing a "new" Alias name and we want to make sure it is unique...
	const UTG_Pin* TestedPin = Pins[PinId.PinIdx()];
	uint8 Filter = 0;
	if (TestedPin->IsOutput())
	{
		Filter =AccessToUint8(OutputBitMask);
	}

	// ... in the scope of the Node
	// Among all the Node's pins AliasNames
	TArray<FName> Names = GetPinAliasNames(Filter);
	// Remove the Pin's Alias name we are testing from the collection
	Names.Remove(CandidateName); 

	// ... in the scope of the Graph if it is a Param
	// Need to add the param names EXCEPT the one for this
	if (TestedPin->IsParam())
	{
		TArray<FName> ParamNames = GetGraph()->GetParamNames();
		for (auto Param : ParamNames)
		{
			if (PinId != GetGraph()->FindParamPinId(Param))
			{
				Names.Add(Param);
			}
		}
	}

	auto CheckedName = TG_MakeNameUniqueInCollection(CandidateName, Names);
	if (CheckedName == CandidateName)
	{
		// The CandidateName is unique, good to go
		// Not modified
		return CandidateName;
	}
	else
	{
		// Modify the pin's alias name to the edited version
		return CheckedName;
	}
}

void UTG_Node::ValidateGenerateConformer(UTG_Pin* InPin)
{
	GetExpression()->ValidateGenerateConformer(InPin);
}

const FTG_Signature& UTG_Node::GetSignature() const
{
	return (Signature ? *Signature : *(Expression->GetSignature()));
}


void UTG_Node::NotifyGraphOfNodeChange(bool bIsTweaking)
{
	// In some rare transient cases, the Outer is NOT the graph, so do not notify
	if (GetGraph())
		GetGraph()->OnNodeChanged(this, bIsTweaking);
}

void UTG_Node::OnExpressionChangedWithoutVar(const FPropertyChangedEvent& PropertyChangedEvent)
{
	UE_LOGF(LogTextureGraph, Verbose, "UTG_Node::PropertyChangedEvent ChangeType: %d", PropertyChangedEvent.ChangeType);

	if (PropertyChangedEvent.Property)
	{
		// MemberProperty name is what we're interested in, the expression's member property has triggered an event
		// and we want to find the matching pin and take the value in from the expression's member
		FTG_Id PinId = GetPinId(PropertyChangedEvent.GetMemberPropertyName());

		if (PinId.IsValid())
		{
			auto ModifiedPin = GetPin(PinId);

			if(ModifiedPin)
			{
				auto Arg = ModifiedPin->GetArgument();
				ModifiedPin->EditSelfVar()->CopyFrom(GetExpression(), Arg);
			}
		}
	}

	NotifyGraphOfNodeChange(PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive);
}

void UTG_Node::OnSignatureChanged()
{
	GetGraph()->OnNodeSignatureChanged(this);
}

void UTG_Node::OnPinRenamed(FTG_Id InPinId, FName OldName)
{
	GetGraph()->OnNodePinChanged(InPinId,this);
	
	UTG_Pin* Pin = GetPin(InPinId);

	// Param pin is used to set the title of the node.
	// Right now, Inputs and outputs expressions can be params.
	// We assume each expression has a single param.
	if(Pin->IsParam())
	{
		GetGraph()->OnNodeRenamed(this, OldName);
	}
}

void UTG_Node::OnPinConnectionChanged(FTG_Id InPinId, FTG_Id OldPinId, FTG_Id NewPinId)
{
	UTG_Pin* ThePin = GetPin(InPinId);
	UTG_Pin* OldPin = GetPin(OldPinId);
	UTG_Pin* NewPin = GetPin(NewPinId);

	if (ThePin->IsArgVariant())
	{
		GetExpression()->NotifyCommonInputVariantTypeChanged(); // notify and update the common type and the signature accordingly
	}

	if (!ThePin->IsConnected())
	{
		if (ThePin->IsArgTexture())
		{
			//Reset the FTGTexture
			ThePin->EditSelfVar()->EditAs<FTG_Texture>() = nullptr;
			GetExpression()->CopyVarToExpressionArgument(ThePin->GetArgument(), ThePin->EditSelfVar());
		}
	}
}

void UTG_Node::OnPinConnectionUndo(FTG_Id InPinId)
{
	UTG_Pin* PinTo = GetPin(InPinId);

	if (PinTo)
	{
		// Reset the Pin to Default in this case
		if (PinTo->IsArgVariant())
		{
			GetExpression()->NotifyCommonInputVariantTypeChanged(FTG_Variant::EType::Scalar); // notify and update the common type and the signature accordingly
		}
		else if (PinTo->IsArgTexture())
		{
			//Reset the FTGTexture
			PinTo->EditSelfVar()->EditAs<FTG_Texture>() = nullptr;
			GetExpression()->CopyVarToExpressionArgument(PinTo->GetArgument(), PinTo->EditSelfVar());
		}
	}
}


void UTG_Node::Construct(UTG_Expression* InExpression)
{
	Expression = InExpression;
}

void UTG_Node::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FTG_CustomVersion::GUID);

	if (!Expression)
	{
		UE_LOGF(LogTextureGraph, Verbose, "    %ls Node: NUll Expression???",
			(Ar.IsSaving() ? TEXT("Saved") : TEXT("Loaded")));
		Expression = NewObject<UTG_Expression_Null>(this, UTG_Expression_Null::StaticClass(), NAME_None, RF_Transactional);
	}

	int32 Version = Ar.CustomVer(FTG_CustomVersion::GUID);

	/// Backwards compatibility with Texture descriptor
	if (Version < FTG_CustomVersion::TGExpressionAddedBaseOutputSettings)
	{
		TArray<FTG_Texture> Textures;
		GetAllOutputValues(Textures, nullptr);

		if (!Textures.IsEmpty())
		{
			Expression->BaseOutputSettings = Textures[0].Descriptor;
		}
		else /// Try to see if there are texture variants 
		{
			TArray<FTG_Variant> Variants;
			GetAllOutputValues(Variants, nullptr);

			/// Just copy the descriptor off the first texture that we encounter
			for (int32 Index = 0; Index < Variants.Num(); Index++)
			{
				if (Variants[Index].IsTexture())
				{
					FTG_Texture Texture = Variants[Index].GetTexture();
					Expression->BaseOutputSettings = Texture.Descriptor;
					break;
				}
			}
		}
	}

	UE_LOGF(LogTextureGraph, VeryVerbose, "    %ls Node: %ls",
		(Ar.IsSaving() ? TEXT("Saved") : TEXT("Loaded")),
		*GetId().ToString());
}

void UTG_Node::Initialize(FTG_Id InId)
{
	check(GetGraph());  // a Graph MUST be the outer owning this node
	Id = InId;
	check(Expression); // an expression MUST have been assigned or recovered from unserialization
	check(Expression->GetOuter() == this); // and the expression outer must be this

	// Initialize the expression in cascade allowing it to re create transient data
	// This is called in the PostLoad of the Graph
	Expression->Initialize();

	Signature = Expression->GetSignature();
}

int32 UTG_Node::ForEachInputPins(std::function<void(const UTG_Pin* /*pin*/, uint32 /*index*/)> visitor) const
{
	int32 Num = GetSignature().GetInArguments().Num();
	for (int32 i = 0; i < Num; ++i)
	{
		visitor(Pins[i], i);
	}
	return Num;
}

int32 UTG_Node::ForEachOutputPins(std::function<void(const UTG_Pin* /*pin*/, uint32 /*index*/)> visitor) const
{
	int32 Offset = GetSignature().GetInArguments().Num();
	int32 Num = GetSignature().GetOutArguments().Num();
	for (int32 i = 0; i < Num; ++i)
	{
		if (i + Offset < Pins.Num())
			visitor(Pins[i + Offset], i);
	}
	return Num;
}

int UTG_Node::GetInputPins(TArray<const UTG_Pin*>& OutPins) const
{
	return ForEachInputPins([&](const UTG_Pin* OutPin, uint32 Index) {
		OutPins.Add(OutPin);
		});

}

int UTG_Node::GetOutputPins(TArray<const UTG_Pin*>& OutPins) const
{
	return ForEachOutputPins([&](const UTG_Pin* OutPin, uint32 Index) {
		OutPins.Add(OutPin);
		});
}

int UTG_Node::GetInputPins(TArray<UTG_Pin*>& OutPins) const
{
	return ForEachInputPins([&](const UTG_Pin* OutPin, uint32 Index) {
		OutPins.Add(const_cast<UTG_Pin*>(OutPin));
		});

}

int UTG_Node::GetOutputPins(TArray<UTG_Pin*>& OutPins) const
{
	return ForEachOutputPins([&](const UTG_Pin* OutPin, uint32 Index) {
		OutPins.Add(const_cast<UTG_Pin*>(OutPin));
		});
}

FTG_Id UTG_Node::GetPinId(FName Name) const
{
	FTG_Index i = 0;
	for (const auto& it : Pins)
	{
		if (it->GetArgumentName().Compare(Name) == 0)
			return FTG_Id(GetId().NodeIdx(), i);
		++i;
	}
	
	return FTG_Id();
}


FTG_Id UTG_Node::GetInputPinIdAt(FTG_Index inIndex) const
{
	if (inIndex >= 0 && inIndex < GetSignature().GetInArguments().Num())		
		return FTG_Id(GetId().NodeIdx(), inIndex);
	return FTG_Id();
}
FTG_Id UTG_Node::GetOutputPinIdAt(FTG_Index outIndex) const
{
	if (outIndex >= 0 && outIndex < GetSignature().GetOutArguments().Num())
		return FTG_Id(GetId().NodeIdx(), outIndex + GetSignature().GetInArguments().Num());
	return FTG_Id();
}
FTG_Id UTG_Node::GetPrivatePinIdAt(FTG_Index privateIndex) const
{
	if (privateIndex >= 0 && privateIndex < GetSignature().GetPrivateArguments().Num())
		return FTG_Id(GetId().NodeIdx(), privateIndex + GetSignature().GetInArguments().Num() + GetSignature().GetOutArguments().Num());
	return FTG_Id();
}

TArray<FTG_Id> MakeIdArray(int32 NodeIdx, int32 Num, int32 IdxOffset)
{
	FTG_Ids Ids;
	Ids.Reserve(Num);
	for (FTG_Index i = 0; i < Num; ++i)
	{
		Ids.Add(FTG_Id(NodeIdx, i + IdxOffset));
	}
	return Ids;
}

TArray<FTG_Id> UTG_Node::GetInputPinIds() const
{
	return MakeIdArray(GetId().NodeIdx(), GetSignature().GetInArguments().Num(), 0);
}
TArray<FTG_Id> UTG_Node::GetOutputPinIds() const
{
	int32 IdxOffset = GetSignature().GetInArguments().Num();
	return MakeIdArray(GetId().NodeIdx(), GetSignature().GetOutArguments().Num(), IdxOffset);
}
TArray<FTG_Id> UTG_Node::GetPrivatePinIds() const
{
	int32 IdxOffset = GetSignature().GetInArguments().Num() + GetSignature().GetOutArguments().Num();
	return MakeIdArray(GetId().NodeIdx(), GetSignature().GetPrivateArguments().Num(), IdxOffset);
}

FTG_Id UTG_Node::GetInputPinID(FTG_Name& Name) const
{
    FTG_Index i = GetSignature().FindInputArgument(Name);
    return (i == FTG_Id::INVALID_INDEX ? FTG_Id::INVALID : GetInputPinIdAt(i));
}
FTG_Id UTG_Node::GetOutputPinID(FTG_Name& Name) const
{
	FTG_Index i = GetSignature().FindOutputArgument(Name);
	return (i == FTG_Id::INVALID_INDEX ? FTG_Id::INVALID : GetOutputPinIdAt(i));
}
FTG_Id UTG_Node::GetPrivatePinID(FTG_Name& Name) const
{
	FTG_Index i = GetSignature().FindPrivateArgument(Name);
	return (i == FTG_Id::INVALID_INDEX ? FTG_Id::INVALID : GetPrivatePinIdAt(i));
}

UTG_Pin* UTG_Node::GetPin(FName Name) const
{
	return GetPin(GetPinId(Name));
}

UTG_Pin* UTG_Node::GetInputPin(FTG_Name& name) const
{
    return GetGraph()->GetPin(GetInputPinID(name));
}
UTG_Pin* UTG_Node::GetOutputPin(FTG_Name& name) const
{
    return GetGraph()->GetPin(GetOutputPinID(name));
}
UTG_Pin* UTG_Node::GetPrivatePin(FTG_Name& name) const
{
	return GetGraph()->GetPin(GetPrivatePinID(name));
}

UTG_Pin* UTG_Node::GetInputPinAt(FTG_Index inIndex) const
{
    return GetGraph()->GetPin(GetInputPinIdAt(inIndex));
}
UTG_Pin* UTG_Node::GetOutputPinAt(FTG_Index outIndex) const
{
    return GetGraph()->GetPin(GetOutputPinIdAt(outIndex));
}
UTG_Pin* UTG_Node::GetPrivatePinAt(FTG_Index privateIndex) const
{
	return GetGraph()->GetPin(GetPrivatePinIdAt(privateIndex));
}

UTG_Pin* UTG_Node::GetPin(FTG_Id pinID) const
{
    return GetGraph()->GetPin(pinID);
}

TArray<FTG_Id> UTG_Node::GetInputVarIds() const
{
	TArray<FTG_Id> VarIds;
	auto InPinIds = GetInputPinIds();
	for (auto PinId : InPinIds)
		VarIds.Emplace(GetGraph()->GetPin(PinId)->GetVarId());
	return VarIds;
}

TArray<FTG_Id> UTG_Node::GetOutputVarIds() const
{
	TArray<FTG_Id> VarIds;
	auto OutPinIds = GetOutputPinIds();
	for (auto PinId : OutPinIds)
		VarIds.Emplace(GetGraph()->GetPin(PinId)->GetVarId());
	return VarIds;
}

FString UTG_Node::LogHead() const
{
	return FString::Printf(TEXT("n%s<%s>"), *Id.ToString(), *GetNodeName().ToString());
}

FString UTG_Node::LogPins(FString Tab) const
{
	FString LogMessage;
	auto InPinIds = GetInputPinIds();
	for (auto i : InPinIds)
	{
		auto p = GetGraph()->GetPin(i);
		LogMessage += p->Log(Tab);
	}
	auto OutPinIds = GetOutputPinIds();
	for (auto o : OutPinIds)
	{
		auto p = GetGraph()->GetPin(o);
		LogMessage += p->Log(Tab);
	}
	auto PrivateFieldPinIds = GetPrivatePinIds();
	for (auto f : PrivateFieldPinIds)
	{
		auto p = GetGraph()->GetPin(f);
		LogMessage += p->Log(Tab);
	}
	return LogMessage;
}

FTG_Arguments UTG_Node::GetPinArguments() const
{
	FTG_Arguments Arguments;

	for (auto Pin : Pins)
	{
		Arguments.Add(Pin->Argument);
	}

	return Arguments;
}

bool UTG_Node::CheckPinSignatureAgainstExpression() const
{
	FTG_Arguments ExpArguments = GetSignature().GetArguments();
	FTG_Arguments PinArguments = GetPinArguments();
	
	return ExpArguments == PinArguments;
}

FTG_Variant::EType UTG_Node::GetExpressionCommonInputVariantType() const
{
	return GetExpression()->GetCommonInputVariantType();
}

int UTG_Node::GetAllOutputValues(TArray<FTG_Variant>& OutVariants, TArray<FName>* OutNames) const
{
	int NumFounds = 0;
	for (auto Pin : Pins)
	{
		if (Pin->IsOutput())
		{
			// For each valid output pin, grab the result in a variant if valid
			// and the name if container provided
			if (!Pin->IsArgArray())
			{
				FTG_Variant OutVariant;
				if (Pin->GetValue(OutVariant))
				{
					OutVariants.Add(OutVariant);
					if (OutNames)
						OutNames->Add(Pin->GetAliasName());
					NumFounds++;
				}
				else
				{
					UE_LOGF(LogTextureGraph, Warning, "Output %ls failed to collect as variant", *(Pin->GetAliasName().ToString()));
				}
			}
			else
			{
				FTG_VariantArray VarArray;
				Pin->GetValue(VarArray);

				for (int Index = 0; Index < VarArray.Num(); Index++)
				{
					OutVariants.Add(VarArray.GetArray()[Index]);
					if (OutNames)
						OutNames->Add(Pin->GetAliasName());
					NumFounds++;
				}
			}
		}
	}

	return NumFounds;
}


int UTG_Node::GetAllOutputValues(TArray<FTG_Texture>& OutTextures, TArray<FName>* OutNames) const
{
	int NumFounds = 0;
	for (auto Pin : Pins)
	{
		if (Pin->IsOutput() && Pin->IsArgTexture())
		{
			// For each Texture compatible output pin, grab the result if valid
			// and the name if container provided
			FTG_Texture OutTexture;
			if (Pin->GetValue(OutTexture))
			{
				OutTextures.Add(OutTexture);
				if (OutNames)
					OutNames->Add(Pin->GetAliasName());
				NumFounds++;
			}
			else
			{
				UE_LOGF(LogTextureGraph, Log, "Output %ls failed to collect as Texture", *(Pin->GetAliasName().ToString()));
			}
		}
	}

	return NumFounds;
}
