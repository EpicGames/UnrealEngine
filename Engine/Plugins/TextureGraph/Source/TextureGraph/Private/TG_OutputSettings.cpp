// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_OutputSettings.h"
#include "Expressions/Output/TG_Expression_Output.h"
#include "TextureGraph.h"
#include "TG_Node.h"
#include "TG_HelperFunctions.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_OutputSettings)

void FTG_OutputSettings::Set(int InWidth, int InHeight, FName Name /*= "None"*/, FName Path /*= "None"*/, ETG_TextureFormat Format /*= ETG_TextureFormat::BGRA8*/, ETG_TexturePresetType InTextureType /*= ETG_TexturePresetType::None*/,
	TextureCompressionSettings InCompression /*= TextureCompressionSettings::TC_Default*/, TextureGroup InLodGroup /*= TextureGroup::TEXTUREGROUP_World*/, bool InbSRGB /*= false*/)
{
	BaseName = Name;
	FolderPath = Path;
	Width = (EResolution)InWidth;
	Height = (EResolution)InHeight;
	TextureFormat = Format;
	TexturePresetType = InTextureType;

	if (TexturePresetType != ETG_TexturePresetType::None)
	{
		OnSetTexturePresetType(TexturePresetType);
	}
	else
	{
		Compression = InCompression;
		LODGroup = InLodGroup;
		bSRGB = InbSRGB;
	}
}

void FTG_OutputSettings::OnSetTexturePresetType(ETG_TexturePresetType Type)
{
	switch (Type)
	{
	case ETG_TexturePresetType::None:
		LODGroup = TextureGroup::TEXTUREGROUP_World;
		Compression = TextureCompressionSettings::TC_Default;
		bSRGB = true;
		break;
	case ETG_TexturePresetType::Diffuse:
	case ETG_TexturePresetType::Emissive:
		LODGroup = TextureGroup::TEXTUREGROUP_Character;
		Compression = TextureCompressionSettings::TC_Default;
		bSRGB = true;
		break;
	case ETG_TexturePresetType::FX:
		LODGroup = TextureGroup::TEXTUREGROUP_Character;
		Compression = TextureCompressionSettings::TC_Masks;
		bSRGB = false;
		break;
	case ETG_TexturePresetType::Normal:
		LODGroup = TextureGroup::TEXTUREGROUP_CharacterNormalMap;
		Compression = TextureCompressionSettings::TC_Normalmap;
		bSRGB = false;
		break;
	case ETG_TexturePresetType::MaskComp:
	case ETG_TexturePresetType::Specular:
		LODGroup = TextureGroup::TEXTUREGROUP_CharacterSpecular;
		Compression = TextureCompressionSettings::TC_Masks;
		bSRGB = false;
		break;
	case ETG_TexturePresetType::Tangent:
		LODGroup = TextureGroup::TEXTUREGROUP_CharacterSpecular;
		Compression = TextureCompressionSettings::TC_Default;
		bSRGB = false;
		break;
	default:
		LODGroup = TextureGroup::TEXTUREGROUP_World;
		Compression = TextureCompressionSettings::TC_Default;
		bSRGB = true;
		break;
	}
}

bool FTG_OutputSettings::Validate(FString& OutErrors) const
{
	return TextureExporter::IsFilePathValid(BaseName, FolderPath, OutErrors);
}

void FTG_OutputSettings::Initialize(FString PathName,FName InName /*= "Output"*/)
{
	OutputName = InName;
	BaseName = InName;
	FString AssetPath = PathName;
	FString DefaultDirectory = FPaths::GetPath(AssetPath);

	FolderPath = FName(DefaultDirectory);
	TextureFormat = ETG_TextureFormat::BGRA8;
}

FArchive& operator<<(FArchive& Ar, FTG_OutputSettings& T)
{
	Ar << T.BaseName;
	Ar << T.OutputName;
	Ar << T.FolderPath;
	Ar << T.Width;
	Ar << T.Height;
	Ar << T.TextureFormat;
	Ar << T.TexturePresetType;
	Ar << T.LODGroup;
	Ar << T.Compression;
	Ar << T.bSRGB;
	return Ar;
}
