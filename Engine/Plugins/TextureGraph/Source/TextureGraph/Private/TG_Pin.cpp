// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_Pin.h"

#include "Expressions/TG_Expression.h"
#include "TG_Graph.h"
#include "TG_CustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Pin)


UTG_Node* UTG_Pin::GetNodePtr() const { return Cast<UTG_Node>(GetOuter()); }

void UTG_Pin::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FTG_CustomVersion::GUID);

	// Serialize the Var with the knowledge of the 
	SelfVar.Serialize(Ar, GetId(), GetArgument());

	UE_LOGF(LogTextureGraph, VeryVerbose, "      %ls Pin: %ls : %ls",
		(Ar.IsSaving() ? TEXT("Saved") : TEXT("Loaded")),
		*GetId().ToString(),
		*GetArgumentName().ToString());
}

void UTG_Pin::Construct(FTG_Id InId, const FTG_Argument& InArgument)
{
	Id = InId;
	Argument = InArgument;

	const FString* DefaultAliasName = InArgument.MetaDataMap.Find(TEXT("Alias"));
	if (DefaultAliasName)
	{
		AliasName = FName(*DefaultAliasName);
	}
	else
	{
		AliasName = Argument.GetName();
	}

	InitSelfVar();
}

void UTG_Pin::InitSelfVar()
{
	SelfVar.PinId = Id; // Assign the Pinid as the Id of the Var
	
	// The concrete value of the selfvar is only initialized for inputs, not output
	if (IsInput())
	{
		// Setting and input pin's var are allocated and copied over from the data in the matching expression argument
		FTG_Argument Arg = GetArgument();
		if (Arg.IsPersistentSelfVar())
		{
			if (SelfVar.IsEmpty()) // At creation of the Pin, SelfVar is empty, go get the initial value from the Expression
			{
				SelfVar.CopyFrom(GetNodePtr()->GetExpression(), Arg);
			}
			else // Else, Pin and selfvar already existed, IT contains the current value
			{
				SelfVar.CopyTo(GetNodePtr()->GetExpression(), Arg);
			}
		}
		else
		{
			SelfVar.CopyFrom(GetNodePtr()->GetExpression(), Arg);
		}
	}

	// Output pin IF and ONLY IF it s a FTG_TEXTURE we need to recover the saved texture descriptor value
	if (IsOutput())
	{
		FTG_Argument Arg = GetArgument();
		if (Arg.IsTexture())
		{
			SelfVar.CopyFrom(GetNodePtr()->GetExpression(), Arg);
		}
	}

	if (ConnectionNeedsConversion())
	{
		ConvertedVar.PinId = Id; // assign the same id as the pin
	}
}

void UTG_Pin::ResetSelfVarDefault()
{
	SelfVar.PinId = Id; // Assign the Pinid as the Id of the Var
	// The concrete value of the selfvar is only initialized for inputs, not output
	if (IsInput())
	{
		FTG_Argument Arg = GetArgument();
		SelfVar.CopyFrom(GetNodePtr()->GetExpression(), Arg);
	}
	if (IsOutput())
	{
		FTG_Argument Arg = GetArgument();
		if (Arg.IsTexture())
		{
			SelfVar.CopyFrom(GetNodePtr()->GetExpression(), Arg);
		}
	}
}

void UTG_Pin::RemapId(FTG_Id InId)
{
	Modify();
	Id = InId;
	if (!IsPrivate()) // SelfVar is only valid for non private fields
	{
		SelfVar.PinId = InId;
	}
}

void UTG_Pin::RemapEdgeId(FTG_Id OldId, FTG_Id NewId)
{
	Modify();
	for (int i = 0; i < Edges.Num(); ++i)
	{
		if (Edges[i] == OldId)
		{
			Edges[i] = NewId;
			break;
		}
	}
}

void UTG_Pin::RemoveAllEdges()
{
	Modify();
	Edges.Empty();
	RemoveInputVarConverterKey();
}
void UTG_Pin::RemoveEdge(FTG_Id EdgeId)
{
	Modify();
	for (int i = 0; i < Edges.Num(); ++i)
	{
		if (Edges[i] == EdgeId)
		{
			Edges.RemoveAt(i);
			break;
		}
	}
	RemoveInputVarConverterKey();
}
void UTG_Pin::AddEdge(FTG_Id EdgeId)
{
	Modify();
	Edges.Emplace(EdgeId);
}


void UTG_Pin::InstallInputVarConverterKey(FName ConverterKey)
{
	assert(InputVarConverterKey.IsNone());
	if (!ConverterKey.IsNone())
	{
		InputVarConverterKey = ConverterKey;
		ConvertedVar.PinId = GetId();
	}
}
void UTG_Pin::RemoveInputVarConverterKey()
{
	if (!InputVarConverterKey.IsNone())
	{
		InputVarConverterKey = FName();
		ConvertedVar.PinId = FTG_Id::INVALID;
		ConvertedVar.Concept = nullptr;
	}
}

FTG_Hash UTG_Pin::Hash(const UTG_Pin& Pin)
{
    auto v = TG_HashName(Pin.GetArgumentName());
    v += Pin.Id.IndexRaw();
    v += Pin.GetNodeId().IndexRaw();
    return TG_Hash(v);
}

FTG_Id UTG_Pin::GetNodeId() const
{
	return (GetNodePtr() ? GetNodePtr()->GetId() : FTG_Id::INVALID);
}

void UTG_Pin::SetAliasName(FName InAliasName)
{
	// Avoid any thinking if the new alias name is the same as what we have already
	if (InAliasName == AliasName)
		return;

	FName CheckedName = GetNodePtr()->ValidateGeneratePinAliasName(InAliasName, GetId());

	Modify(); // Mark the state before the change

	FName OldAliasName = AliasName;

	// Update the Alias with the CHecked one
	AliasName = CheckedName;

	// Update the params list
	if(IsParam())
	{
		GetNodePtr()->GetGraph()->RenameParam(OldAliasName, AliasName);
	}

	// Notify node for pin change
	GetNodePtr()->OnPinRenamed(Id, OldAliasName);
}
void UTG_Pin::NotifyPinSelfVarChanged(bool bIsTweaking /*= false*/)
{
	// The value of the PinSelfVar has changed
	// If the pin is not connected and argument is an input 
	// Or if an output
	// then we also transfer the new value to the Expression's own matching value.
	if ((IsInput() && !IsConnected()) || (IsOutput()))
	{
		if (!IsArgTextureArray())
			EditSelfVar()->CopyTo(GetNodePtr()->GetExpression(), Argument);
	}

#if WITH_EDITOR
	UTG_Node* TGNode = this->GetNodePtr();
	check (TGNode != nullptr);
	GetNodePtr()->NotifyGraphOfNodeChange(bIsTweaking);
	
#endif
}

#if WITH_EDITOR
void UTG_Pin::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if ((PropertyName == GET_MEMBER_NAME_CHECKED(UTG_Pin, AliasName)))
	{
		// Set the property value (that just changed) through the proper setter which will validate the datta and propagate the event if needed
		SetAliasName(AliasName);
	}
}

bool UTG_Pin::Modify(bool bAlwaysMarkDirty)
{
	//Calling the Pin expression Modify here to make sure we have the snapshot of data
	GetNodePtr()->GetExpression()->Modify();
	return Super::Modify(bAlwaysMarkDirty);
}
#endif

FProperty* UTG_Pin::GetExpressionProperty() const
{
	FProperty* Property = GetNodePtr()->GetExpression()->GetClass()->FindPropertyByName(GetArgumentName());
	return Property;
}


FString UTG_Pin::GetEvaluatedVarValue() const
{
	// First acces the true Var used from this Pin
	const FTG_Var* CurrentVar = GetSelfVar();
	if (IsInput() && IsConnected())
	{
		if (ConnectionNeedsConversion())
		{
			CurrentVar = &ConvertedVar;
		}
		else
		{
			CurrentVar = GetNodePtr()->GetGraph()->GetVar(Edges[0]);
		}
	}

	if (!CurrentVar)
		return FString();

	FString CurrentValue = CurrentVar->LogValue();

	// Enum requires special-case handling
	// Needs help from the FProperty as the Var does not know the Enum type
	FProperty* Property = GetExpressionProperty();
	FByteProperty* ByteProperty = CastField<FByteProperty>(Property);
	if (ByteProperty)
	{
		UEnum* Enum = ByteProperty->GetIntPropertyEnum();
		if (Enum)
		{
			CurrentValue = Enum->GetNameStringByIndex(FCString::Atoi(*CurrentValue));
		}
	}
	FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property);
	if (EnumProperty)
	{
		UEnum* Enum = EnumProperty->GetEnum();
		if (Enum)
		{
			CurrentValue = Enum->GetNameStringByIndex(FCString::Atoi(*CurrentValue));
		}
	}

	return CurrentValue;
}

void UTG_Pin::FromString(const FString& InValueStr, bool bIsTweaking /*= false*/)
{
	Modify();

	// Pin->DefaultValue
	FString DefaultValue = InValueStr;

	// Enum requires special-case handling
	// Needs help from the FProperty as the Var does not know the Enum type
	FProperty* Property = GetExpressionProperty();
	if (Property)
	{
		FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property);
		FByteProperty* ByteProperty = CastField<FByteProperty>(Property);
		if (ByteProperty && ByteProperty->Enum)
		{
			DefaultValue = FString::FromInt(ByteProperty->Enum->GetValueByNameString(DefaultValue));
		}
		else if (EnumProperty && EnumProperty->GetEnum())
		{
			DefaultValue = FString::FromInt(EnumProperty->GetEnum()->GetValueByNameString(DefaultValue));
		}
	}

	EditSelfVar()->SetValueFromStr(DefaultValue);

	NotifyPinSelfVarChanged(bIsTweaking);
}

FString	UTG_Pin::LogHead() const
{
	return FString::Printf(TEXT("%s.%s"),
			*Id.ToString(),
			*(HasAliasName() ? GetAliasName().ToString() + TEXT("->") + GetArgumentName().ToString() : GetArgumentName().ToString())
			);
}

FString	UTG_Pin::LogTooltip() const
{
	FString ExprTooltip = TEXT("");

#if WITH_EDITOR
	const FProperty* Prop = GetExpressionProperty();

	if (Prop)
		ExprTooltip = *Prop->GetToolTipText().ToString();
#endif 
	
	return FString::Printf(TEXT("%s : %s - %s"),
		*GetAliasName().ToString(),
		*GetArgumentCPPTypeName().ToString(),
		*ExprTooltip);
}


FString	UTG_Pin::Log(FString Tab) const
{
	FString LogMessage;

	FProperty* Property = GetExpressionProperty();
	FString PropertyName = TEXT("");
	if (Property)
		PropertyName = Property->GetClass()->GetName();

	LogMessage += Tab + FString::Printf(TEXT("%-*s%*s : %-*s %-*s"),
		UTG_Graph::LogHeaderWidth, *LogHead(),
		UTG_Graph::LogHeaderWidth / 2, *GetArgumentType().ToString(),
		UTG_Graph::LogHeaderWidth, *GetArgumentCPPTypeName().ToString(),
		UTG_Graph::LogHeaderWidth, *PropertyName);
	
	FString ConnectionMessage;
	if (IsConnected())
	{
		ConnectionMessage += (IsInput() || IsSetting() ? TEXT(" <- [") : TEXT(" -> ["));

		bool first = true;
		for (auto sourcePinId : GetEdges())
		{
			auto OtherPin = GetNodePtr()->GetGraph()->GetPin(sourcePinId);
			check(OtherPin);
			ConnectionMessage += FString::Printf(TEXT("%s%s"), (first ? TEXT("") : TEXT(", ")), *OtherPin->LogHead());
			first = false;
		}
		ConnectionMessage += TEXT("]");

		if (ConnectionNeedsConversion())
		{
			ConnectionMessage += FString::Printf(TEXT(" >> %s >> v%s"), *GetInputVarConverterKey().ToString(), *GetSelfVarId().ToString());
		}
	}
	LogMessage += FString::Printf(TEXT("%-*s"), UTG_Graph::LogHeaderWidth, *ConnectionMessage);

	LogMessage += TEXT("\r\n");

	return LogMessage;
}


bool UTG_Pin::GetValue(float& OutValue) const
{
	if (IsArgScalar())
	{
		if (IsArgVariant())
		{
			// By calling Edit here we also make sure the variant is reset to the expected type in case this wasn't the case yet
			// this is the case for an output pin, right after loading or creating the pin, the inner variant hasn't been reseted yet
			OutValue = GetSelfVar()->GetAs<FTG_Variant>().EditScalar();
		}
		else
		{
			FName CPPType = GetArgumentCPPTypeName();
			if (CPPType == TEXT("int32"))
			{
				OutValue = GetSelfVar()->GetAs<int32>();
			}
			else if (CPPType == TEXT("uint32"))
			{
				OutValue = GetSelfVar()->GetAs<uint32>();
			}
			else
			{
				OutValue = GetSelfVar()->GetAs<float>();
			}
		}
		return true;
	}
	return false;
}

bool UTG_Pin::GetValue(bool& OutValue) const
{
	if (IsArgBool())
	{
		OutValue = GetSelfVar()->GetAs<bool>();
		return true;
	}
	return false;
}

bool UTG_Pin::GetValue(FString& OutValue) const
{
	if (IsArgString())
	{
		OutValue = GetSelfVar()->GetAs<FString>();
		return true;
	}
	return false;
}

bool UTG_Pin::GetValue(FLinearColor& OutValue) const
{
	if (IsArgColor())
	{
		if (IsArgVariant())
		{
			// By calling Edit here we also make sure the variant is reset to the expected type in case this wasn't the case yet
			// this is the case for an output pin, right after loading or creating the pin, the inner variant hasn't been reseted yet
			OutValue = GetSelfVar()->GetAs<FTG_Variant>().EditColor();
		}
		else
		{
			OutValue = GetSelfVar()->GetAs<FLinearColor>();
		}
		return true;
	}
	else if (IsArgVector())
	{
		if (IsArgVariant())
		{
			// By calling Edit here we also make sure the variant is reset to the expected type in case this wasn't the case yet
			// this is the case for an output pin, right after loading or creating the pin, the inner variant hasn't been reseted yet
			FVector4f VecValue = GetSelfVar()->GetAs<FTG_Variant>().EditVector();
			OutValue = *(reinterpret_cast<FLinearColor*>(&VecValue));
		}
		else
		{
			FVector4f VecValue = GetSelfVar()->GetAs<FVector4f>();
			OutValue = *(reinterpret_cast<FLinearColor*>(&VecValue));
		}
		return true;
	}
	return false;
}


bool UTG_Pin::GetValue(FVector4f& OutValue) const
{
	if (IsArgColor())
	{
		if (IsArgVariant())
		{
			// By calling Edit here we also make sure the variant is reset to the expected type in case this wasn't the case yet
			// this is the case for an output pin, right after loading or creating the pin, the inner variant hasn't been reseted yet
			FLinearColor ColValue = GetSelfVar()->GetAs<FTG_Variant>().EditColor();
			OutValue = *(reinterpret_cast<FVector4f*>(&ColValue));
		}
		else
		{
			FLinearColor ColValue = GetSelfVar()->GetAs<FLinearColor>();
			OutValue = *(reinterpret_cast<FVector4f*>(&ColValue));
		}
		return true;
	}
	else if (IsArgVector())
	{
		if (IsArgVariant())
		{
			// By calling Edit here we also make sure the variant is reset to the expected type in case this wasn't the case yet
			// this is the case for an output pin, right after loading or creating the pin, the inner variant hasn't been reseted yet
			OutValue = GetSelfVar()->GetAs<FTG_Variant>().EditVector();
		}
		else
		{
			OutValue = GetSelfVar()->GetAs<FVector4f>();
		}
		return true;
	}
	return false;
}

bool UTG_Pin::GetValue(FTG_Texture& OutValue) const
{
	if (IsArgArray())
	{
		FTG_VariantArray& VarArray = GetSelfVar()->GetAs<FTG_VariantArray>();
		OutValue = VarArray.GetViewTexture();
		return true;
	}
	else if (IsArgTexture())
	{
		if (IsArgVariant())
		{
			// By calling Edit here we also make sure the variant is reset to the expected type in case this wasn't the case yet
			// this is the case for an output pin, right after loading or creating the pin, the inner variant hasn't been reseted yet
			OutValue = GetSelfVar()->GetAs<FTG_Variant>().EditTexture();
		}
		else
		{
			OutValue = GetSelfVar()->GetAs<FTG_Texture>();
		}
		return true;
	}
	return false;
}

bool UTG_Pin::GetValue(FTG_VariantArray& OutValue) const
{
	if (IsArgArray())
	{
		OutValue = GetSelfVar()->GetAs<FTG_VariantArray>();
		return true;
	}
	return false;
}


// Variant value getter only if a Variant argument OR if the argument is one of the supported type by Variant (Scalar/Color/Vector/Texture)
// After getting the value, the next step is to check the type contained in the variant.
bool UTG_Pin::GetValue(FTG_Variant& OutValue) const
{
	if (IsArgVariant())
	{
		check(!IsArgArray());
		OutValue = GetSelfVar()->GetAs<FTG_Variant>();
		return true;
	}

	if (GetValue(OutValue.EditScalar()))
		return true;

	if (IsArgColor())
	{
		return GetValue(OutValue.EditColor());
	}
	else if (IsArgVector())
	{
		return GetValue(OutValue.EditVector());
	}

	if (GetValue(OutValue.EditTexture()))
		return true;

	return false;
}

bool UTG_Pin::SetValue(const float* Value, size_t Count)
{
	if (Count == 1)
		return SetValue(*Value);
	else if (Count == 4)
	{
		if (IsArgColor() || IsArgVector())
			return SetValue(*reinterpret_cast<const FLinearColor*>(Value));
	}

	check(false);
	return false;
}

bool UTG_Pin::SetValue(const FTG_TextureDescriptor& Value)
{
	Modify(true);
	EditSelfVar()->EditAs<FTG_TextureDescriptor>() = Value;
	NotifyPinSelfVarChanged();
	return true;
}

bool UTG_Pin::SetValue(float Value)
{
	if (IsArgScalar())
	{
		Modify();

		if (IsArgVariant())
		{
			EditSelfVar()->EditAs<FTG_Variant>().EditScalar() = Value;
		}
		else
		{
			FName CPPType = GetArgumentCPPTypeName();
			if (CPPType == TEXT("int32"))
			{
				EditSelfVar()->EditAs<int32>() = (int32)Value;
			}
			else if (CPPType == TEXT("uint32"))
			{
				EditSelfVar()->EditAs<uint32>() = (uint32)Value;
			}
			else
			{
				EditSelfVar()->EditAs<float>() = Value;
			}
		}

		NotifyPinSelfVarChanged();
		return true;
	}
	return false;
}

bool UTG_Pin::SetValue(bool bValue)
{
	if (IsArgBool())
	{
		Modify();
		EditSelfVar()->EditAs<bool>() = bValue;
		NotifyPinSelfVarChanged();

		return true;
	}
	return false;
}

bool UTG_Pin::SetValue(FString Value)
{
	if (IsArgString())
	{
		Modify();
		EditSelfVar()->EditAs<FString>() = Value;
		NotifyPinSelfVarChanged();

		return true;
	}
	return false;
}

bool UTG_Pin::SetValue(const FLinearColor& Value)
{
	if (IsArgColor())
	{
		Modify();

		if (IsArgVariant())
		{
			EditSelfVar()->EditAs<FTG_Variant>().EditColor() = Value;
		}
		else
		{
			EditSelfVar()->EditAs<FLinearColor>() = Value;
		}

		NotifyPinSelfVarChanged();
		return true;
	}
	else if (IsArgVector())
	{
		Modify();

		if (IsArgVariant())
		{
			EditSelfVar()->EditAs<FTG_Variant>().EditVector() = Value;
		}
		else
		{
			EditSelfVar()->EditAs<FVector4f>() = Value;
		}

		NotifyPinSelfVarChanged();
		return true;
	}
	return false;
}


bool UTG_Pin::SetValue(const FVector4f& Value)
{
	if (IsArgColor())
	{
		Modify();

		if (IsArgVariant())
		{
			EditSelfVar()->EditAs<FTG_Variant>().EditColor() = Value;
		}
		else
		{
			EditSelfVar()->EditAs<FLinearColor>() = Value;
		}

		NotifyPinSelfVarChanged();
		return true;
	}
	else if (IsArgVector())
	{
		Modify();

		if (IsArgVariant())
		{
			EditSelfVar()->EditAs<FTG_Variant>().EditVector() = Value;
		}
		else
		{
			EditSelfVar()->EditAs<FVector4f>() = Value;
		}

		NotifyPinSelfVarChanged();
		return true;
	}
	return false;
}

