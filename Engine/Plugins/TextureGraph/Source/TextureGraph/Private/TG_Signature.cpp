// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_Signature.h" 
#include "TG_Hash.h"
#include "TG_Variant.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Signature)

const FTG_Id FTG_Id::INVALID;

const FTG_Argument FTG_Argument::Invalid = {};

bool FTG_Argument::IsScalar() const
{
	return (CPPTypeName == TEXT("float")) || (CPPTypeName == TEXT("int32")) || (CPPTypeName == TEXT("uint32")) || CPPTypeName == FTG_Variant::GetArgNameFromType(ETG_VariantType::Scalar);
}

bool FTG_Argument::IsBool() const
{
	return CPPTypeName == TEXT("bool");
}

bool FTG_Argument::IsString() const
{
	return CPPTypeName == TEXT("FString") || 
		CPPTypeName == TEXT("FText");
}

bool FTG_Argument::IsColor() const
{
	return CPPTypeName == TEXT("FLinearColor") || CPPTypeName == FTG_Variant::GetArgNameFromType(ETG_VariantType::Color);
}

bool FTG_Argument::IsVector() const
{
	return (CPPTypeName == TEXT("FVector4f")) || CPPTypeName == FTG_Variant::GetArgNameFromType(ETG_VariantType::Vector);
}

bool FTG_Argument::IsTexture() const
{
	return CPPTypeName == TEXT("FTG_Texture") || CPPTypeName == FTG_Variant::GetArgNameFromType(ETG_VariantType::Texture);
}

bool FTG_Argument::IsArray() const
{
	return CPPTypeName == TEXT("FTG_VariantArray");
}
bool FTG_Argument::IsVariant() const
{
	if (CPPTypeName == TEXT("FTG_Variant"))
		return true;

	for (auto VariantType : TEnumRange<ETG_VariantType>())
	{
		if (CPPTypeName == FTG_Variant::GetArgNameFromType(VariantType))
			return true;
	}

	return false;
}

bool FTG_Argument::IsObject(UClass* InClass) const
{
	UClass* Result = StaticLoadClass(UObject::StaticClass(), nullptr, *CPPTypeName.ToString(), nullptr, LOAD_None, nullptr);
	if (Result) // The argument is actually a UObject
	{
		return Result->IsChildOf(InClass);
	}
	return false;
}

bool FTG_Argument::GetMetaData(const FName KeyToFind, FString& OutValue)
{
	OutValue = "";
	
	if(MetaDataMap.Contains(KeyToFind))
	{
		OutValue = MetaDataMap[KeyToFind];

		return true;
	}

	return false;
}

FTG_Hash FTG_Argument::Hash(const FTG_Argument& Argument)
{
	FTG_Hash v = TG_HashName(Argument.Name);
	v += static_cast<uint8>(Argument.ArgumentType.Flags);
	return TG_Hash(v);
}

TArray<FName> TG_MakeArrayOfArgumentNames(const FTG_Arguments& InArguments)
{
	TArray<FName> Names;
	Names.Empty(InArguments.Num());
	for (auto& Arg : InArguments)
	{
		Names.Add(Arg.Name);
	}
	return Names;
}

FTG_Hash FTG_ArgumentSet::Hash(const FTG_ArgumentSet& Set)
{
	FTG_Hash v = 0;
	for (const auto& it : Set.Arguments)
	{
		v += it.Hash();
	}
	return TG_Hash(v);
}

FTG_Index FTG_ArgumentSet::FindName(FTG_Name& Name) const
{
	FTG_Index i = 0;
	for (const auto& it : Arguments)
	{
		if (it.GetName().Compare(Name) == 0)
			return i;
		++i;
	}
	return FTG_Id::INVALID_INDEX;
}

FTG_Index FTG_ArgumentSet::FindHash(FTG_Hash Hash) const
{
	FTG_Index i = 0;
	for (const auto& it : Arguments)
	{
		if (it.Hash() == Hash)
			return i;
		++i;
	}
	return FTG_Id::INVALID_INDEX;
}

FTG_Signature::FTG_Signature(const FInit& Init) :
	Name(Init.Name)
{
	for (auto& Argument : Init.Arguments)
	{
		switch (Argument.GetType().GetAccess())
		{
		case ETG_Access::In:
		case ETG_Access::InSetting:
			InArgs.Arguments.Add(Argument);
			break;

		case ETG_Access::Out:
		case ETG_Access::OutSetting:
			OutArgs.Arguments.Add(Argument);
			break;

		case ETG_Access::InParam:
		case ETG_Access::InParamSetting:
			InArgs.Arguments.Add(Argument);
			bIsParam = true;
			break;

		case ETG_Access::OutParam:
		case ETG_Access::OutParamSetting:
			OutArgs.Arguments.Add(Argument);
			bIsParam = true;
			break;

		case ETG_Access::Private:
			PrivateArgs.Arguments.Add(Argument);
			break;
		}
	}
}

FTG_Hash FTG_Signature::Hash(const FTG_Signature& Type)
{
	FTG_Hash v = TG_HashName(Type.Name);
	v += Type.InArgs.Hash();
	v += Type.OutArgs.Hash();
	return TG_Hash(v);
};

const FTG_Argument& FTG_Signature::GetArgument(int32 ArgIdx)
{
	if (ArgIdx >= 0)
	{
		if (ArgIdx < InArgs.Arguments.Num())
			return InArgs.Arguments[ArgIdx];
		ArgIdx -= InArgs.Arguments.Num();

		if (ArgIdx < OutArgs.Arguments.Num())
			return OutArgs.Arguments[ArgIdx];
		ArgIdx -= OutArgs.Arguments.Num();

		if (ArgIdx < PrivateArgs.Arguments.Num())
			return PrivateArgs.Arguments[ArgIdx];
	}

	return FTG_Argument::Invalid;
}

FTG_Signature::IndexAccess FTG_Signature::FindArgumentIndexFromHash(FTG_Hash Hash) const
{
	FTG_Index i = InArgs.FindHash(Hash);
	if (i != FTG_Id::INVALID_INDEX)
		return { i, ETG_Access::In };

	i = OutArgs.FindHash(Hash);
	if (i != FTG_Id::INVALID_INDEX)
		return { i, ETG_Access::Out };

	i = PrivateArgs.FindHash(Hash);
	if (i != FTG_Id::INVALID_INDEX)
		return { i, ETG_Access::Private };

	return IndexAccess();
}

const FTG_Argument& FTG_Signature::FindArgumentFromHash(FTG_Hash Hash) const
{
	IndexAccess i = FindArgumentIndexFromHash(Hash);
	if (i.Index != FTG_Id::INVALID_INDEX)
	{
		switch (i.Access)
		{
		case ETG_Access::In:
		case ETG_Access::InParam:
		case ETG_Access::InSetting:
		case ETG_Access::InParamSetting:
			return InArgs.Get(i.Index);

		case ETG_Access::Out:
		case ETG_Access::OutParam:
		case ETG_Access::OutSetting:
		case ETG_Access::OutParamSetting:
			return OutArgs.Get(i.Index);

		case ETG_Access::Private:
			return PrivateArgs.Get(i.Index);
		}
	}
	return FTG_Argument::Invalid;
}

FTG_Signature::IndexAccess FTG_Signature::FindArgumentIndexFromName(FTG_Name& InName) const
{
	FTG_Index i = InArgs.FindName(InName);
	if (i != FTG_Id::INVALID_INDEX)
		return { i, ETG_Access::In };

	i = OutArgs.FindName(InName);
	if (i != FTG_Id::INVALID_INDEX)
		return { i, ETG_Access::Out };

	i = PrivateArgs.FindName(InName);
	if (i != FTG_Id::INVALID_INDEX)
		return { i, ETG_Access::Private };

	return IndexAccess();
}

const FTG_Argument& FTG_Signature::FindArgumentFromName(FTG_Name& InName) const
{
	IndexAccess i = FindArgumentIndexFromName(InName);
	if (i.Index != FTG_Id::INVALID_INDEX)
	{
		switch (i.Access)
		{
		case ETG_Access::In:
		case ETG_Access::InParam:
		case ETG_Access::InSetting:
		case ETG_Access::InParamSetting:
			return InArgs.Get(i.Index);

		case ETG_Access::Out:
		case ETG_Access::OutParam:
		case ETG_Access::OutSetting:
		case ETG_Access::OutParamSetting:
			return OutArgs.Get(i.Index);

		case ETG_Access::Private:
			return PrivateArgs.Get(i.Index);
		}
	}
	return FTG_Argument::Invalid;
}



FTG_Indices FTG_Signature::GenerateMappingArgIdxTable(const FTG_Signature& FromSignature, const FTG_Signature& ToSignature)
{
	FTG_Indices NewIdxToOld;
	FTG_Index OldArgIdxOffset = 0;

	for (const FTG_Argument& NewArg : ToSignature.GetInArguments())
	{
		FTG_Index OldArgIdx = FromSignature.FindInputArgument(NewArg.Name);
		// matching argument name
		if (OldArgIdx != FTG_Id::INVALID_INDEX)
		{
			// check the type is the same, if not then not a match
			if (FromSignature.GetInArguments()[OldArgIdx].CPPTypeName != NewArg.CPPTypeName)
				OldArgIdx = FTG_Id::INVALID_INDEX;
		}
		NewIdxToOld.Add((OldArgIdx != FTG_Id::INVALID_INDEX ? OldArgIdx + OldArgIdxOffset : FTG_Id::INVALID_INDEX));
	}
	OldArgIdxOffset += FromSignature.InArgs.Arguments.Num();

	for (const FTG_Argument& NewArg : ToSignature.GetOutArguments())
	{
		FTG_Index OldArgIdx = FromSignature.FindOutputArgument(NewArg.Name);
		// matching argument name
		if (OldArgIdx != FTG_Id::INVALID_INDEX)
		{
			// check the type is the same, if not then not a match
			if (FromSignature.GetOutArguments()[OldArgIdx].CPPTypeName != NewArg.CPPTypeName)
				OldArgIdx = FTG_Id::INVALID_INDEX;
		}
		NewIdxToOld.Add((OldArgIdx != FTG_Id::INVALID_INDEX ? OldArgIdx + OldArgIdxOffset : FTG_Id::INVALID_INDEX));
	}
	OldArgIdxOffset += FromSignature.OutArgs.Arguments.Num();

	for (const FTG_Argument& NewArg : ToSignature.GetPrivateArguments())
	{
		FTG_Index OldArgIdx = FromSignature.FindPrivateArgument(NewArg.Name);
		// matching argument name
		if (OldArgIdx != FTG_Id::INVALID_INDEX)
		{
			// check the type is the same, if not then not a match
			if (FromSignature.GetPrivateArguments()[OldArgIdx].CPPTypeName != NewArg.CPPTypeName)
				OldArgIdx = FTG_Id::INVALID_INDEX;
		}
		NewIdxToOld.Add((OldArgIdx != FTG_Id::INVALID_INDEX ? OldArgIdx + OldArgIdxOffset : FTG_Id::INVALID_INDEX));
	}

	return NewIdxToOld;
}
