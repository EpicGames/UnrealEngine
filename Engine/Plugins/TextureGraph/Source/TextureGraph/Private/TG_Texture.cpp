// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_Texture.h"

#include "TG_CustomVersion.h"
#include "TG_Var.h"
#include "Model/StaticImageResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Texture)

FArchive& operator<<(FArchive& Ar, FTG_TextureDescriptor& D)
{
	int32 Version = Ar.CustomVer(FTG_CustomVersion::GUID);
	Ar << D.Width;
	Ar << D.Height;
	Ar << D.TextureFormat;

	if (Ar.IsLoading())
	{
		/// Older versions had this defaulted to false
		if (Version < FTG_CustomVersion::TGTextureDescAdded_bSRGB)
			D.bIsSRGB = false;
		else
			Ar << D.bIsSRGB;
	}
	else
		Ar << D.bIsSRGB;

	return Ar;
}

template <> FString TG_Var_LogValue(FTG_Texture& Value)
{
	if (!Value.RasterBlob)
		return TEXT("FTG_Texture nullptr");

	return FString::Printf(TEXT("FTG_Texture [%s] %dx%d"), *Value.RasterBlob->DisplayName(), Value.RasterBlob->GetWidth(), Value.RasterBlob->GetHeight());

}

bool FTG_Texture::operator==(const FTG_Texture& RHS) const
{
	return (this->RasterBlob == RHS.RasterBlob &&
			this->TexturePath == RHS.TexturePath);
}

void FTG_Texture::ResetTexturePath()
{
	TexturePath.Empty();
}

FArchive& operator<<(FArchive& Ar, FTG_Texture& T)
{
	Ar.UsingCustomVersion(FTG_CustomVersion::GUID);
	int32 Version = Ar.CustomVer(FTG_CustomVersion::GUID);
		
	if (Ar.IsSaving() ||
		(Ar.IsLoading() && Version >= FTG_CustomVersion::TGTextureAddedTexturePath))
	{
		Ar << T.TexturePath;
	}
	Ar << T.Descriptor;
	return Ar;
}
