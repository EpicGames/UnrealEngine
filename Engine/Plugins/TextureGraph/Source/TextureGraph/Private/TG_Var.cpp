// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_Var.h"

#include <functional>

#include "TG_CustomVersion.h"
#include "TG_Texture.h"
#include "TG_OutputSettings.h"
#include "TG_Variant.h"
#include "TG_Material.h"
#include "Model/ModelObject.h"

#include "Expressions/TG_Expression.h"
#include "Expressions/Procedural/TG_Expression_Pattern.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Var)

template <> FString TG_Var_LogValue(uint8& Value)
{
	FString LogMessage = FString::FromInt(Value);
	return LogMessage;
}

template <> FString TG_Var_LogValue(bool& Value)
{
	FString LogMessage = (Value ? TEXT("true") : TEXT("false"));
	return LogMessage;
}

template <> FString TG_Var_LogValue(int& Value)
{
	FString LogMessage = FString::FromInt(Value);
	return LogMessage;
}

template <> FString TG_Var_LogValue(float& Value)
{
	FString LogMessage = FString::SanitizeFloat(Value);
	return LogMessage;
}

template <> FString TG_Var_LogValue(FName& Value)
{
	FString LogMessage = Value.ToString();
	return LogMessage;
}
template <> FString TG_Var_LogValue(FString& Value)
{
	return Value;
}
template <> FString TG_Var_LogValue(FLinearColor& Value)
{
	FString LogMessage = Value.ToString();
	return LogMessage;
}
template <> FString TG_Var_LogValue(FVector4f& Value)
{
	//Formating the string into comma seperated numbers
	FString LogMessage = FString::Printf(TEXT("%3.3f,%3.3f,%3.3f,%3.3f"), Value.X, Value.Y, Value.Z, Value.W);
	return LogMessage;
}
template <> FString TG_Var_LogValue(FVector2f& Value)
{
	FString LogMessage = Value.ToString();
	return LogMessage;
}
template <> FString TG_Var_LogValue(TObjectPtr<UObject>& Value)
{
	if (Value.Get())
	{
		auto Name = Value->GetClass()->GetName();
		return FString::Printf(TEXT("%s [%s]"), *Name, *Value->GetName());
	}
	else
		return TEXT("nullptr");
}

template <> FString TG_Var_LogValue(FTG_OutputSettings& Value)
{
	FString LogMessage = Value.ToString();
	return LogMessage;
}

template <> FString TG_Var_LogValue(FTG_TextureDescriptor& Value)
{
	FString LogMessage = Value.ToString();
	return LogMessage;
}

template <> FString TG_Var_LogValue(FPatternMaskPlacement_TS& Value)
{
	FString LogMessage = Value.ToString();
	return LogMessage;
}

template <> FString TG_Var_LogValue(FPatternMaskJitter_TS& Value)
{
	FString LogMessage = Value.ToString();
	return LogMessage;
}

template <> FString TG_Var_LogValue(FPatternMaskBevel_TS& Value)
{
	FString LogMessage = Value.ToString();
	return LogMessage;
}

template <> FString TG_Var_LogValue(FPatternMaskCutout_TS& Value)
{
	FString LogMessage = Value.ToString();
	return LogMessage;
}

template <> FString TG_Var_LogValue(FGradientDir_TS& Value)
{
	FString LogMessage = Value.ToString();
	return LogMessage;
}

template <> void TG_Var_SetValueFromString(int& Value, const FString& StrVal)
{
	Value = FCString::Atoi(*StrVal);
}
template <> void TG_Var_SetValueFromString(bool& Value, const FString& StrVal)
{
	Value = FCString::ToBool(*StrVal);
}

template <> void TG_Var_SetValueFromString(float& Value, const FString& StrVal)
{
	Value = FCString::Atof(*StrVal);
}
template <> void TG_Var_SetValueFromString(uint8& Value, const FString& StrVal)
{
	Value = static_cast<uint8>(FCString::Atoi(*StrVal));
}
template <> void TG_Var_SetValueFromString(FName& Value, const FString& StrVal)
{
	Value = FName(StrVal);
}
template <> void TG_Var_SetValueFromString(FString& Value, const FString& StrVal)
{
	Value = StrVal;
}
template <> void TG_Var_SetValueFromString(FLinearColor& Value, const FString& StrVal)
{
	Value.InitFromString(StrVal);
}
template <> void TG_Var_SetValueFromString(FVector4f& Value, const FString& StrVal)
{
	TArray<FString> StringArray;
	StrVal.ParseIntoArray(StringArray, TEXT(","), true);

	//should get the 4 comma seperated values from string
	check(StringArray.Num() == 4);

	Value.X = FCString::Atof(*StringArray[0]);
	Value.Y = FCString::Atof(*StringArray[1]);
	Value.Z = FCString::Atof(*StringArray[2]);
	Value.W = FCString::Atof(*StringArray[3]);
}
template <> void TG_Var_SetValueFromString(FVector2f& Value, const FString& StrVal)
{
	Value.InitFromString(StrVal);
}
template <> void TG_Var_SetValueFromString(TObjectPtr<UObject>& Value, const FString& StrVal)
{
	FSoftObjectPath objRef(StrVal);
	Value = Cast<UObject>(objRef.TryLoad());
}


template <> void TG_Var_SetValueFromString(FTG_OutputSettings& Value, const FString& StrVal)
{
	Value.InitFromString(StrVal);
}

template <> void TG_Var_SetValueFromString(FTG_TextureDescriptor& Value, const FString& StrVal)
{
	Value.InitFromString(StrVal);
}

template <> void TG_Var_SetValueFromString(FTG_Material& Value, const FString& StrVal)
{
	Value.InitFromString(StrVal);
}

template <> void TG_Var_SetValueFromString(FPatternMaskPlacement_TS& Value, const FString& StrVal)
{
	Value.InitFromString(StrVal);
}

template <> void TG_Var_SetValueFromString(FPatternMaskJitter_TS& Value, const FString& StrVal)
{
	Value.InitFromString(StrVal);
}

template <> void TG_Var_SetValueFromString(FPatternMaskBevel_TS& Value, const FString& StrVal)
{
	Value.InitFromString(StrVal);
}

template <> void TG_Var_SetValueFromString(FPatternMaskCutout_TS& Value, const FString& StrVal)
{
	Value.InitFromString(StrVal);
}

template <> void TG_Var_SetValueFromString(FGradientDir_TS& Value, const FString& StrVal)
{
	Value.InitFromString(StrVal);
}


FString FTG_Var::LogHead() const
{
	return FString::Printf(TEXT("v%s"), *GetId().ToString());
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
#define VAR_PROPERTY_SERIALIZER(name, function)	{ TEXT(name), &function }
#define VAR_PROPERTY_SERIALIZER_DEF(name)		VAR_PROPERTY_SERIALIZER(#name, VarPropertySerializer_##name)

template <typename T_PropertyType, typename T_ValueType>
void Generic_Simple_Serializer(FTG_Var::VarPropertySerialInfo& Info)
{
	FProperty* Property = Info.Owner->GetClass()->FindPropertyByName(Info.Arg.GetName());

	const T_PropertyType* TProperty = CastField<T_PropertyType>(Property);
	const T_ValueType VarValue = Info.Var->GetAs<T_ValueType>();

	if (Info.CopyVarToProperty)
	{
		// This calls the Setter method if the UProperty has a Setter
		TProperty->SetValue_InContainer(Info.Owner, VarValue);
	}
	else
		Info.Var->EditAs<T_ValueType>() = TProperty->GetPropertyValue(TProperty->template ContainerPtrToValuePtr<T_ValueType>(Info.Owner, Info.ClampedIndex()));
}

template <typename T_ValueType>
void Generic_Struct_Serializer(FTG_Var::VarPropertySerialInfo& Info)
{
	FProperty* Property = Info.Owner->GetClass()->FindPropertyByName(Info.Arg.GetName());


	const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
	T_ValueType* PropertyValue = StructProperty->ContainerPtrToValuePtr<T_ValueType>(Info.Owner, Info.ClampedIndex());

	if (Info.CopyVarToProperty)
	{
		if (!Info.Var->IsEmpty() && Info.Var->IsValid())
		{
			T_ValueType* VarValue = &Info.Var->GetAs<T_ValueType>();
			StructProperty->SetValue_InContainer(Info.Owner, VarValue);
		}
		else
		{
			T_ValueType* DefaultValue = StructProperty->ContainerPtrToValuePtrForDefaults<T_ValueType>(StructProperty->Struct, Info.Owner, Info.ClampedIndex());
			if (DefaultValue)
			{
				StructProperty->SetValue_InContainer(Info.Owner, DefaultValue);
			}
		}
	}
	else
	{
		Info.Var->EditAs<T_ValueType>() = *PropertyValue;
	}
}

FTG_Var::FTG_Var()
{
}

FTG_Var::FTG_Var(FTG_Id InPinUuid) 
	: PinId(InPinUuid) 
{
}

FTG_Var::FTG_Var(const FTG_Var& InVar) 
	: PinId(InVar.PinId)
	, Concept(InVar.Concept) 
{
}

void VarPropertySerializer_FTG_Texture(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Struct_Serializer<FTG_Texture>(Info);
}

void VarPropertySerializer_FTG_Material(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Struct_Serializer<FTG_Material>(Info);
}

void VarPropertySerializer_FTG_VariantArray(FTG_Var::VarPropertySerialInfo& Info)
{
	/// Just call the base generic serializer and set the var to array
	Generic_Struct_Serializer<FTG_VariantArray>(Info);

	if (!Info.CopyVarToProperty)
		Info.Var->SetArray();
}

void VarPropertySerializer_FVector4f(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Struct_Serializer<FVector4f>(Info);
}

void VarPropertySerializer_FVector2f(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Struct_Serializer<FVector2f>(Info);
}

void VarPropertySerializer_FLinearColor(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Struct_Serializer<FLinearColor>(Info);
}

void VarPropertySerializer_int32(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Simple_Serializer<FIntProperty, int32>(Info);
}

void VarPropertySerializer_uint32(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Simple_Serializer<FUInt32Property, int32>(Info);
}

void VarPropertySerializer_float(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Simple_Serializer<FFloatProperty, float>(Info);
}

void VarPropertySerializer_bool(FTG_Var::VarPropertySerialInfo& Info)
{
	//Generic_Simple_Serializer<FBoolProperty, bool>(Info);
	FProperty* Property = Info.Owner->GetClass()->FindPropertyByName(Info.Arg.GetName());

	const FBoolProperty* TProperty = CastField<FBoolProperty>(Property);
	const bool VarValue = Info.Var->GetAs<bool>();

	if (Info.CopyVarToProperty)
	{
		TProperty->SetPropertyValue(TProperty->template ContainerPtrToValuePtr<bool>(Info.Owner, Info.ClampedIndex()), VarValue);
		//TProperty->SetValue_InContainer(Info.Owner, VarValue);
	}
	else
		Info.Var->EditAs<bool>() = TProperty->GetPropertyValue(TProperty->template ContainerPtrToValuePtr<bool>(Info.Owner, Info.ClampedIndex()));
}

void VarPropertySerializer_FName(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Simple_Serializer<FNameProperty, FName>(Info);
}

void VarPropertySerializer_FString(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Simple_Serializer<FStrProperty, FString>(Info);
}

void VarPropertySerializer_UObjectPtr(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Struct_Serializer<FTG_Texture>(Info);
}

void VarPropertySerializer_FTG_Variant(FTG_Var::VarPropertySerialInfo& Info)
{
	FProperty* Property = Info.Owner->GetClass()->FindPropertyByName(Info.Arg.GetName());
	const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
	FTG_Variant* PropertyValue = StructProperty->ContainerPtrToValuePtr<FTG_Variant>(Info.Owner, Info.ClampedIndex());

	if (Info.CopyVarToProperty)
	{
		if (!Info.Var->IsEmpty() && Info.Var->IsValid())
		{
			if (!Info.Var->IsArray())
				*PropertyValue = Info.Var->GetAs<FTG_Variant>();
			else
			{
				const FTG_VariantArray& VarArray = Info.Var->GetAs<FTG_VariantArray>();
				int Index = Info.ClampedIndex();
				check(Index < VarArray.Num());
				*PropertyValue = VarArray.GetArray()[Index];
			}
		}
	}
	else
	{
		if (!Info.Var->IsArray())
			Info.Var->SetAs<FTG_Variant>(*PropertyValue);
		else
		{
			FTG_VariantArray& VarArray = Info.Var->GetAs<FTG_VariantArray>();
			int Index = Info.ClampedIndex();
			check(Index < VarArray.Num());
			VarArray.Set(Index, *PropertyValue);
		}
	}
}

void VarPropertySerializer_ObjectProperty(FTG_Var::VarPropertySerialInfo& Info)
{
	FProperty* Property = Info.Owner->GetClass()->FindPropertyByName(Info.Arg.GetName());

	FObjectProperty* ObjectPtrProperty = CastField<FObjectProperty>(Property);
	if (ObjectPtrProperty)
	{
		if (Info.CopyVarToProperty)
		{
			TObjectPtr<UObject> ObjectPtr = Info.Var->GetAs<TObjectPtr<UObject>>();
			ObjectPtrProperty->SetObjectPropertyValue_InContainer(Info.Owner, ObjectPtr.Get(), Info.ClampedIndex());

			// If UObject is going through a setter then make sure to feedback the true end value in the var
			if (Property->HasSetter())
			{
				Info.Var->EditAs<TObjectPtr<UObject>>() = ObjectPtrProperty->GetObjectPropertyValue_InContainer(Info.Owner, Info.ClampedIndex());
			}
		}
		else
		{
			Info.Var->EditAs<TObjectPtr<UObject>>() = ObjectPtrProperty->GetObjectPropertyValue_InContainer(Info.Owner, Info.ClampedIndex());
		}

	}


}

void VarPropertySerializer_StructProperty(FTG_Var::VarPropertySerialInfo& Info)
{
	FProperty* Property = Info.Owner->GetClass()->FindPropertyByName(Info.Arg.GetName());

	const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
	// const UClass* StructClass = StructProperty->Struct->GetClass();
	const FName TypeName = FName(StructProperty->Struct->GetStructCPPName());
	
	const auto WriterIt = FTG_Var::DefaultPropertySerializers.Find(TypeName);
	if (!WriterIt)
	{
		/// TODO: Perhaps think about doing a simple memcpy?
		UE_LOGF(LogTextureGraph, Warning, "Fails serialize Var %ls - Property %ls FPClass %ls CPPType %ls",
			*Info.Var->LogHead(),
			*Info.Arg.GetName().ToString(),
			*TypeName.ToString(),
			*Info.Arg.CPPTypeName.ToString());
		return;
	}

	(*WriterIt)(Info);
}


void VarPropertySerializer_ByteProperty(FTG_Var::VarPropertySerialInfo& Info)
{
	FProperty* Property = Info.Owner->GetClass()->FindPropertyByName(Info.Arg.GetName());

	const FByteProperty* ByteProperty = CastField<FByteProperty>(Property);

	uint8 PropValue = ByteProperty->GetPropertyValue_InContainer(Info.Owner);
	
	const uint8 VarValue = Info.Var->GetAs<uint8>();

	if (Info.CopyVarToProperty)
	{
		ByteProperty->SetPropertyValue_InContainer(Info.Owner, VarValue, 0);
	}
	else
	{
		Info.Var->EditAs<uint8>() = PropValue;
	}
}

void VarPropertySerializer_EnumProperty(FTG_Var::VarPropertySerialInfo& Info)
{
	FProperty* Property = Info.Owner->GetClass()->FindPropertyByName(Info.Arg.GetName());

	const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property);

	uint32 PropValue = 0;
	EnumProperty->GetValue_InContainer(Info.Owner, &PropValue);
	auto enumVal = EnumProperty->GetEnum();

	const int32 VarValue = Info.Var->GetAs<int32>();

	if (Info.CopyVarToProperty)
	{
		EnumProperty->SetValue_InContainer(Info.Owner, &VarValue);
	}
	else
	{
		Info.Var->EditAs<int32>() = PropValue;
	}
}

void VarPropertySerializer_FTG_OutputSettings(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Struct_Serializer<FTG_OutputSettings>(Info);
}

void VarPropertySerializer_FTG_TextureDescriptor(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Struct_Serializer<FTG_TextureDescriptor>(Info);
}

void VarPropertySerializer_FGradientDir_TS(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Struct_Serializer<FGradientDir_TS>(Info);
}
void VarPropertySerializer_FPatternMaskCutout_TS(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Struct_Serializer<FPatternMaskCutout_TS>(Info);
}
void VarPropertySerializer_FPatternMaskBevel_TS(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Struct_Serializer<FPatternMaskBevel_TS>(Info);
}
void VarPropertySerializer_FPatternMaskJitter_TS(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Struct_Serializer<FPatternMaskJitter_TS>(Info);
}
void VarPropertySerializer_FPatternMaskPlacement_TS(FTG_Var::VarPropertySerialInfo& Info)
{
	Generic_Struct_Serializer<FPatternMaskPlacement_TS>(Info);
}
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
FTG_Var::VarPropertySerializerMap FTG_Var::DefaultPropertySerializers
(
	{
		VAR_PROPERTY_SERIALIZER_DEF(StructProperty),
		VAR_PROPERTY_SERIALIZER_DEF(ObjectProperty),
		VAR_PROPERTY_SERIALIZER_DEF(ByteProperty),
		VAR_PROPERTY_SERIALIZER_DEF(EnumProperty),
		VAR_PROPERTY_SERIALIZER_DEF(bool),
		VAR_PROPERTY_SERIALIZER_DEF(int32),
		VAR_PROPERTY_SERIALIZER_DEF(uint32),
		VAR_PROPERTY_SERIALIZER_DEF(float),
		VAR_PROPERTY_SERIALIZER_DEF(FName),
		VAR_PROPERTY_SERIALIZER_DEF(FString),
		VAR_PROPERTY_SERIALIZER_DEF(FTG_Texture),
		VAR_PROPERTY_SERIALIZER_DEF(FVector4f),
		VAR_PROPERTY_SERIALIZER_DEF(FVector2f),
		VAR_PROPERTY_SERIALIZER_DEF(FLinearColor),
		VAR_PROPERTY_SERIALIZER_DEF(FTG_OutputSettings),
		VAR_PROPERTY_SERIALIZER_DEF(FTG_TextureDescriptor),
		VAR_PROPERTY_SERIALIZER_DEF(FTG_Variant),
		VAR_PROPERTY_SERIALIZER_DEF(FTG_Material),
		VAR_PROPERTY_SERIALIZER_DEF(FGradientDir_TS),
		VAR_PROPERTY_SERIALIZER_DEF(FPatternMaskCutout_TS),
		VAR_PROPERTY_SERIALIZER_DEF(FPatternMaskBevel_TS),
		VAR_PROPERTY_SERIALIZER_DEF(FPatternMaskJitter_TS),
		VAR_PROPERTY_SERIALIZER_DEF(FPatternMaskPlacement_TS),
		VAR_PROPERTY_SERIALIZER_DEF(FTG_VariantArray),
	}
);


void FTG_Var::RegisterDefaultSerializers()
{
}

void FTG_Var::RegisterVarPropertySerializer(FName CPPTypeName, FTG_Var::VarPropertySerializer Serializer)
{
	DefaultPropertySerializers.Add(CPPTypeName, Serializer);
}
void FTG_Var::UnregisterVarPropertySerializer(FName CPPTypeName)
{
	DefaultPropertySerializers.Remove(CPPTypeName);
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
bool FTG_Var::CopyGeneric(UTG_Expression* Owner, const FTG_Argument& Arg, bool CopyVarToProperty, int Index)
{
	FProperty* Property = Owner->GetClass()->FindPropertyByName(Arg.GetName());
	if (Property)
	{
		const FFieldClass* PropertyClass = Property->GetClass();
		check(PropertyClass);
		FName PropertyClassName = PropertyClass->GetFName();
		// Filter first for a serializer  against the FPropertyClass name:
		auto SerializerIt = FTG_Var::DefaultPropertySerializers.Find(PropertyClassName);

		// If not found one:
		if (!SerializerIt)
		{
			// Let's try with the simpler Property's cpp name (same as the argument type)
			FName ArgTypeName = FName(Property->GetCPPType());
			SerializerIt = FTG_Var::DefaultPropertySerializers.Find(ArgTypeName);

			if (!SerializerIt)
			{
				UE_LOGF(LogTextureGraph, Warning, "Fails serialize Var %ls - Property %ls FPClass %ls CPPType %ls",
					*LogHead(),
					*Arg.GetName().ToString(),
					*PropertyClassName.ToString(),
					*ArgTypeName.ToString());
				return false;
			}
		}

		VarPropertySerialInfo Info = {
			this,
			Owner,
			Arg,
			Index,
			CopyVarToProperty
		};

		(*SerializerIt)(Info);

		return true;
	}

	// No Property cannot copy with the FProperty infrastructure
	return false;
}

void FTG_Var::CopyTo(UTG_Expression* Owner, FTG_Argument& Arg, int Index)
{
	Owner->CopyVarToExpressionArgument(Arg, this);
}

void FTG_Var::CopyFrom(UTG_Expression* Owner, FTG_Argument& Arg, int Index)
{
	Owner->CopyVarFromExpressionArgument(Arg, this);
}

void FTG_Var::CopyTo(FTG_Var* InVar)
{
	if (Concept.IsValid() && InVar)
	{
		InVar->Concept = Concept->Clone();
	}
}

void FTG_Var::CopyFrom(FTG_Var* InVar)
{
	if (InVar && InVar->Concept)
	{
		InVar->CopyTo(this);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
#define VAR_ARCHIVE_SERIALIZER(name, function)	{ TEXT(name), &function }
#define VAR_ARCHIVE_SERIALIZER_DEF(name)		VAR_ARCHIVE_SERIALIZER(#name, VarArchiveSerializer_##name)

template <typename T_ValueType>
void Generic_Simple_ArSerializer(FTG_Var::VarArchiveSerialInfo& Info)
{
	if (Info.Var->IsEmpty())
	{
		Info.Var->ResetAs<T_ValueType>();
	}
	if (Info.Ar.IsSaving())
		UE_LOGF(LogTextureGraph, VeryVerbose, "        Save Var %ls: %ls", *Info.Var->GetId().ToString(), *Info.Var->LogValue());
	T_ValueType& Value = Info.Var->EditAs<T_ValueType>();
	Info.Ar << Value;
	if (Info.Ar.IsLoading())
		UE_LOGF(LogTextureGraph, VeryVerbose, "        Loaded Var %ls: %ls", *Info.Var->GetId().ToString(), *Info.Var->LogValue());

}

void VarArchiveSerializer_FTG_Texture(FTG_Var::VarArchiveSerialInfo& Info)
{
	Info.Ar.UsingCustomVersion(FTG_CustomVersion::GUID);
	if (Info.Var->IsEmpty())
	{
		Info.Var->ResetAs<FTG_Texture>();
	}
	FTG_Texture& Texture = Info.Var->EditAs<FTG_Texture>();
	
	if (Info.Ar.IsSaving())
	{
		UE_LOGF(LogTextureGraph, VeryVerbose, "        Save Var %ls: %ls", *Info.Var->GetId().ToString(), *Info.Var->LogValue());
		// Convert the TObjectPtr<UTexture> to FSoftObjectPath to save the asset path
		FSoftObjectPath AssetPath = FSoftObjectPath(Texture.TexturePath);
		AssetPath.SerializePath(Info.Ar);
	}
	
	if (Info.Ar.IsLoading())
	{
		UE_LOGF(LogTextureGraph, VeryVerbose, "        Loaded Var %ls: %ls", *Info.Var->GetId().ToString(), *Info.Var->LogValue());
		int32 Version = Info.Ar.CustomVer(FTG_CustomVersion::GUID);
		if (Version < FTG_CustomVersion::TGTextureAddedTexturePath)
		{
			Texture.TexturePath = FString(); // Default value for older assets
		}
		else
		{
			// Load the asset path and resolve it back to a UTexture pointer
			FSoftObjectPath AssetPath;
			AssetPath.SerializePath(Info.Ar);
			Texture.TexturePath = AssetPath.GetAssetPathString();
		}
	}
}

void VarArchiveSerializer_FTG_VariantArray(FTG_Var::VarArchiveSerialInfo& Info)
{
	//Generic_Simple_ArSerializer<FTG_TextureArray>(Info);
}

void VarArchiveSerializer_FVector4f(FTG_Var::VarArchiveSerialInfo& Info)
{
	Generic_Simple_ArSerializer<FVector4f>(Info);
}
void VarArchiveSerializer_FVector2f(FTG_Var::VarArchiveSerialInfo& Info)
{
	Generic_Simple_ArSerializer<FVector2f>(Info);
}
void VarArchiveSerializer_FLinearColor(FTG_Var::VarArchiveSerialInfo& Info)
{
	Generic_Simple_ArSerializer<FLinearColor>(Info);
}

void VarArchiveSerializer_int32(FTG_Var::VarArchiveSerialInfo& Info)
{
	Generic_Simple_ArSerializer<int32>(Info);
}

void VarArchiveSerializer_uint32(FTG_Var::VarArchiveSerialInfo& Info)
{
	Generic_Simple_ArSerializer<int32>(Info);
}

void VarArchiveSerializer_float(FTG_Var::VarArchiveSerialInfo& Info)
{
	Generic_Simple_ArSerializer<float>(Info);
}
void VarArchiveSerializer_bool(FTG_Var::VarArchiveSerialInfo& Info)
{
	Generic_Simple_ArSerializer<bool>(Info);
}
void VarArchiveSerializer_UTexture(FTG_Var::VarArchiveSerialInfo& Info)
{
	//Generic_Simple_ArSerializer<TObjectPtr<UTexture>>(Info);
	if (Info.Var->IsEmpty())
	{
		Info.Var->ResetAs<TObjectPtr<UTexture>>();
	}
	TObjectPtr<UTexture>& Texture = Info.Var->EditAs<TObjectPtr<UTexture>>();
	
	if (Info.Ar.IsSaving())
	{
		UE_LOGF(LogTextureGraph, VeryVerbose, "        Save Var %ls: %ls", *Info.Var->GetId().ToString(), *Info.Var->LogValue());
		// Convert the TObjectPtr<UTexture> to FSoftObjectPath to save the asset path
		FSoftObjectPath AssetPath = Texture != nullptr ? FSoftObjectPath(Texture.Get()) : FSoftObjectPath();
		AssetPath.SerializePath(Info.Ar);
	}
	
	// Info.Ar << Value;
	// TSoftObjectPtr SoftValue = Value.Get();
	// SoftValue->Serialize(Info.Ar);
	if (Info.Ar.IsLoading())
	{
		UE_LOGF(LogTextureGraph, VeryVerbose, "        Loaded Var %ls: %ls", *Info.Var->GetId().ToString(), *Info.Var->LogValue());
		// Load the asset path and resolve it back to a UTexture pointer
		FSoftObjectPath AssetPath;
		AssetPath.SerializePath(Info.Ar);

		// Resolve the asset path into a UTexture object (loaded asynchronously or synchronously)
		if (!AssetPath.IsNull())
		{
			// Load the texture synchronously (can be async depending on your needs)
			Texture = Cast<UTexture>(AssetPath.TryLoad());
		}
		else
		{
			Texture = nullptr;
		}
	}

}
void VarArchiveSerializer_FTG_OutputSettings(FTG_Var::VarArchiveSerialInfo& Info)
{
	Generic_Simple_ArSerializer<FTG_OutputSettings>(Info);
}

void VarArchiveSerializer_FTG_TextureDescriptor(FTG_Var::VarArchiveSerialInfo& Info)
{
	Generic_Simple_ArSerializer<FTG_TextureDescriptor>(Info);
}
void VarArchiveSerializer_FTG_Material(FTG_Var::VarArchiveSerialInfo& Info)
{
	Generic_Simple_ArSerializer<FTG_Material>(Info);
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
FTG_Var::VarArchiveSerializerMap FTG_Var::DefaultArchiveSerializers
(
	{
		VAR_ARCHIVE_SERIALIZER_DEF(bool),
		VAR_ARCHIVE_SERIALIZER_DEF(int32),
		VAR_ARCHIVE_SERIALIZER_DEF(uint32),
		VAR_ARCHIVE_SERIALIZER_DEF(float),
		VAR_ARCHIVE_SERIALIZER_DEF(FTG_Texture),
		VAR_ARCHIVE_SERIALIZER("TObjectPtr<UTexture>", VarArchiveSerializer_UTexture),
		VAR_ARCHIVE_SERIALIZER_DEF(FVector4f),
		VAR_ARCHIVE_SERIALIZER_DEF(FVector2f),
		VAR_ARCHIVE_SERIALIZER_DEF(FLinearColor),
		VAR_ARCHIVE_SERIALIZER_DEF(FTG_VariantArray),
		VAR_ARCHIVE_SERIALIZER_DEF(FTG_OutputSettings),
		VAR_ARCHIVE_SERIALIZER_DEF(FTG_TextureDescriptor),
		VAR_ARCHIVE_SERIALIZER_DEF(FTG_Material)
	}
); 

void FTG_Var::Serialize(FArchive& Ar, FTG_Id InPinId, const FTG_Argument& InArgument)
{
	// noop for private field
	if (InArgument.IsPrivate())
		return;

	// Init Var transient fields
	if (!PinId.IsValid())
		PinId = InPinId;
	assert(PinId == InPinId);

	if (InArgument.IsPersistentSelfVar())
	{
		auto SerializerIt = FTG_Var::DefaultArchiveSerializers.Find(InArgument.GetCPPTypeName());
		if (SerializerIt)
		{
			VarArchiveSerialInfo Info{
				.Var = this,
				.Ar = Ar };
			(*SerializerIt)(Info);
		}
		else
		{
			UE_LOGF(LogTextureGraph, Warning, "serialize Var %ls: NOT FOUND for %ls", *LogHead(), *InArgument.GetCPPTypeName().ToString());
		}
	}
}


