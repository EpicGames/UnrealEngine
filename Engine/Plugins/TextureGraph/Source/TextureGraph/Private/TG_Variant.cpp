// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_Variant.h"
#include "TG_Var.h"
#include "TG_GraphEvaluation.h"
#include "Helper/ColorUtil.h"
#include "Transform/Expressions/T_FlatColorTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Variant)

FTG_Variant::FTG_Variant()
{
	EditScalar() = 0.0f;
}

FTG_Variant::FTG_Variant(const FTG_Variant& RHS) :
	Data(RHS.Data)
{
}

FTG_Variant::FTG_Variant(float RHS)
{
	EditScalar() = RHS;
}

FTG_Variant::FTG_Variant(FVector4f RHS)
{
	EditVector() = RHS;
}

FTG_Variant::FTG_Variant(FLinearColor RHS)
{
	EditColor() = RHS;
}

FTG_Variant::FTG_Variant(FTG_Texture RHS)
{
	EditTexture() = RHS;
}

bool FTG_Variant::Serialize(FArchive& Ar)
{
	Ar << *this;
	UE_LOGF(LogTextureGraph, VeryVerbose, "FTG_Variant::Serialize");
	return true;
}


bool FTG_Variant::ResetTypeAs(EType InType)
{
	if (InType != GetType())
	{
		switch (InType)
		{
		case EType::Scalar:
			Data.Set<float>(0);
			break;
		case EType::Color:
			Data.Set<FLinearColor>(FLinearColor());
			break;
		case EType::Vector:
			Data.Set<FVector4f>(FVector4f{});
			break;
		case EType::Texture:
			Data.Set<FTG_Texture>(FTG_Texture());
			break;
		}
		return true;
	}
	return false;
}

const float& FTG_Variant::GetScalar() const
{
	check(IsScalar());
	return Data.Get<float>();
}

float& FTG_Variant::EditScalar()
{
	ResetTypeAs(EType::Scalar);
	return Data.Get<float>();
}

const FLinearColor& FTG_Variant::GetColor() const
{
	check(IsColor());
	return Data.Get<FLinearColor>();
}

FLinearColor FTG_Variant::GetColor(FLinearColor Default)
{
	if (IsColor())
		return Data.Get<FLinearColor>();
	else if (IsVector())
	{
		FVector4f Vec = Data.Get<FVector4f>();
		return FLinearColor((float)Vec.X, (float)Vec.Y, (float)Vec.Z, (float)Vec.W);
	}
	else if (IsScalar())
	{
		float Value = Data.Get<float>();
		return FLinearColor(Value, Value, Value, Value);
	}

	// Return the default
	return Default;
}

FVector4f FTG_Variant::GetVector(FVector4f Default)
{
	if (IsVector())
		return Data.Get<FVector4f>();
	else if (IsColor())
	{
		FLinearColor Color = Data.Get<FLinearColor>();
		FVector4f Vec = FVector4f{ Color.R, Color.G, Color.B, Color.A };
		return Vec;
	}
	else if (IsScalar())
	{
		float Value = Data.Get<float>();
		return FVector4f{ Value, Value, Value, Value };
	}

	// Return the default
	return Default;
}

FTG_Texture FTG_Variant::GetTexture(FTG_EvaluationContext* InContext, FTG_Texture Default, const BufferDescriptor* DesiredDesc)
{
	// If it's actually a texture type then just return that
	if (IsTexture())
	{
		// However if no pin is actually connected, then we can return the default value
		FTG_Texture Texture = Data.Get<FTG_Texture>();
		if (!Texture)
			return Default;
		return Texture;
	}

	FLinearColor Color = FLinearColor::Black;
	FString Name;
	BufferFormat TexelFormat = BufferFormat::Byte;

	if (IsColor())
	{
		Name = FTG_Evaluation::GColorToTextureAutoConv_Name;
		Color = Data.Get<FLinearColor>();
		TexelFormat = BufferFormat::Byte;
	}
	else if (IsVector())
	{
		Name = FTG_Evaluation::GColorToTextureAutoConv_Name;
		FVector4f Vec = Data.Get<FVector4f>();
		Color = FLinearColor((float)Vec.X, (float)Vec.Y, (float)Vec.Z, (float)Vec.W);
		TexelFormat = BufferFormat::Half;
	}
	else if (IsScalar())
	{
		Name = FTG_Evaluation::GFloatToTextureAutoConv_Name;
		float Value = Data.Get<float>();
		Color = FLinearColor(Value, Value, Value, Value);
		TexelFormat = BufferFormat::Half;
	}
	else
	{
		/// This needs explicit support in the future
		check(false);
	}

	/// Special handling for black, white and gray textures as they are quite common in the system
	/// both in vector, color as well as scalar forms
	if (!DesiredDesc || (DesiredDesc->Width <= 1 && DesiredDesc->Height <= 1))
	{
		if (ColorUtil::IsColorBlack(Color))
			return FTG_Texture::GetBlack();
		else if (ColorUtil::IsColorWhite(Color))
			return FTG_Texture::GetWhite();
		else if (ColorUtil::IsColorGray(Color))
			return FTG_Texture::GetGray();
		else if (ColorUtil::IsColorRed(Color)) 
			return FTG_Texture::GetRed();
		else if (ColorUtil::IsColorGreen(Color)) 
			return FTG_Texture::GetGreen();
		else if (ColorUtil::IsColorBlue(Color)) 
			return FTG_Texture::GetBlue();
		else if (ColorUtil::IsColorYellow(Color)) 
			return FTG_Texture::GetYellow();
		else if (ColorUtil::IsColorMagenta(Color)) 
			return FTG_Texture::GetMagenta();
	}

	BufferDescriptor Desc = !DesiredDesc ? T_FlatColorTexture::GetFlatColorDesc(Name, TexelFormat) : *DesiredDesc;
	TiledBlobPtr Output = T_FlatColorTexture::Create(InContext->Cycle, Desc, Color, InContext->TargetId);

	return Output;
}

FLinearColor& FTG_Variant::EditColor()
{
	ResetTypeAs(EType::Color);
	return Data.Get<FLinearColor>();
}

const FVector4f& FTG_Variant::GetVector() const
{
	check(IsVector());
	return Data.Get<FVector4f>();
}
FVector4f& FTG_Variant::EditVector()
{
	ResetTypeAs(EType::Vector);
	return Data.Get<FVector4f>();
}

const FTG_Texture& FTG_Variant::GetTexture() const
{
	check(IsTexture());
	return Data.Get<FTG_Texture>();
}

FTG_Texture& FTG_Variant::GetTexture() 
{
	check(IsTexture());
	return Data.Get<FTG_Texture>();
}

FTG_Texture& FTG_Variant::EditTexture()
{
	ResetTypeAs(EType::Texture);
	return Data.Get<FTG_Texture>();
}

FTG_Variant& FTG_Variant::operator = (const FTG_Variant& RHS)
{
	Data = RHS.Data;
	return *this;
}

FTG_Variant& FTG_Variant::operator = (const float RHS)
{
	EditScalar() = RHS;
	return *this;
}

FTG_Variant& FTG_Variant::operator = (const FVector4f RHS)
{
	EditVector() = RHS;
	return *this;
}

FTG_Variant& FTG_Variant::operator = (const FLinearColor RHS)
{
	EditColor() = RHS;
	return *this;
}

FTG_Variant& FTG_Variant::operator = (const FTG_Texture RHS)
{
	EditTexture() = RHS;
	return *this;
}

bool FTG_Variant::operator==(const FTG_Variant& RHS) const
{
	if (GetType() == RHS.GetType())
	{
		switch(GetType())
		{
		default:
		case EType::Scalar:
			return Data.Get<float>() == RHS.Data.Get<float>();
		case EType::Color:
			return Data.Get<FLinearColor>() == RHS.Data.Get<FLinearColor>();
		case EType::Vector:
			return Data.Get<FVector4f>() == RHS.Data.Get<FVector4f>();
		case EType::Texture:
			return Data.Get<FTG_Texture>() == RHS.Data.Get<FTG_Texture>();
		}
	}
	return false;
}

FTG_Variant::operator bool() const
{
	if (IsScalar())
		return !!GetScalar();
	else if (IsTexture())
		return (bool)GetTexture();
	return true;
}

template <> FString TG_Var_LogValue(FTG_Variant& Value)
{
	auto Type = Value.GetType();
	FString TypeName = FTG_Variant::GetNameFromType(Type).ToString();
	switch (Type)
	{
	case FTG_Variant::EType::Scalar:
		return FString::Printf(TEXT("%s %f"), *TypeName, Value.GetScalar());
	case FTG_Variant::EType::Color:
		return FString::Printf(TEXT("%s %s"), *TypeName, *Value.Data.Get<FLinearColor>().ToString());
	case FTG_Variant::EType::Vector:
		return FString::Printf(TEXT("%s %s"), *TypeName, *Value.Data.Get<FVector4f>().ToString());
	case FTG_Variant::EType::Texture:
		return FString::Printf(TEXT("%s"), *TypeName);
	default:
		return FString::Printf(TEXT("%s"), *TypeName);
	}
}

//////////////////////////////////////////////////////////////////////////

template <> void TG_Var_SetValueFromString(FTG_Variant& Value, const FString& StrVal)
{
	// TODO : Implement!
}


void FTG_VariantArray::SetNum(int Count)
{
	int CurrentCount = Array.Num();
	Array.SetNum(Count);

	/// Fill the new items with black
	for (int Index = CurrentCount; Index < Count; Index++)
	{
		Array[Index] = FTG_Texture::GetBlack();
	}
}

void FTG_VariantArray::Set(int Index, const FTG_Variant& Var)
{
	check(Index >= 0 && Index < Array.Num());
	Array[Index] = Var;
}

void FTG_VariantArray::Set(int Index, const FTG_Texture& Tex)
{
	check(Index >= 0 && Index < Array.Num());
	Array[Index].EditTexture() = Tex;
}

void FTG_VariantArray::CopyFrom(const FTG_VariantArray& RHS)
{
	Array = RHS.GetArray();
}

void FTG_VariantArray::SetViewIndex(int Index)
{
	check(Index >= 0 && Index < Array.Num());
	ViewIndex = Index;
}
