// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureGraph.h"
#include "Expressions/Output/TG_Expression_Output.h"
#include "TG_Graph.h"
#include "TG_GraphEvaluation.h"
#include "TG_CustomVersion.h"

#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"

#include "2D/TextureHelper.h"
#include "Expressions/Input/TG_Expression_Graph.h"
#include "FxMat/MaterialManager.h"
#include "Model/Mix/MixManager.h"
#include "Model/Mix/MixSettings.h"
#include "Model/Mix/MixUpdateCycle.h"
#include "Model/ModelObject.h"
#include "Model/Mix/ViewportSettings.h"
#include "Transform/Mix/T_InvalidateTiles.h"
#include "Transform/Mix/T_UpdateTargets.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextureGraph)

/** ---------------------------------------
 *	Texture Graph Instance implementation
 * ---------------------------------------**/

void UTextureGraphInstance::Construct(FString InName)
{
	Super::Construct(InName);
	RuntimeGraph = NewObject<UTG_Graph>(GetTransientPackage(), NAME_None,RF_Transactional);
	RuntimeGraph->Construct(InName);
	
	const UTG_Node* OutputNode = RuntimeGraph->CreateExpressionNode(UTG_Expression_Output::StaticClass());

	UTG_Expression_Output* OutputExpression = Cast<UTG_Expression_Output>(OutputNode->GetExpression());
	OutputExpression->InitializeOutputSettings();

	Settings->GetViewportSettings().InitDefaultSettings(OutputNode->GetNodeName());
}

void UTextureGraphInstance::CopyParamsToRuntimeGraph()
{
	RuntimeGraph->SetInputParamsFromVarMap(InputParams);

	if (!OutputSettingsMap.IsEmpty())
	{
		RuntimeGraph->SetOutputSettings(OutputSettingsMap);
	}
	else
	{
		UpdateOutputSettingsFromGraph();
	}
}

void UTextureGraphInstance::SetParent(const TObjectPtr<UTextureGraphBase>& Parent)
{
	this->ParentTextureGraph = Parent;
	if (Parent == nullptr)
	{
		Construct("");
		InputParams.Empty();
	}
	else
	{
		RuntimeGraph = DuplicateObject(Parent->Graph(), GetTransientPackage());
		Settings = DuplicateObject(Parent->GetSettings(), this);

		// copy InputParams and ExportSettings from Parent if it is an instance
		if (Parent.IsA<UTextureGraphInstance>())
		{
			InputParams = Cast<UTextureGraphInstance>(Parent)->InputParams;
			OutputSettingsMap = Cast<UTextureGraphInstance>(Parent)->OutputSettingsMap;
		}
		else
		{
			InputParams = Graph()->GetInputParamsVarMap();
			
			// recreate fresh output settings from graph
			UpdateOutputSettingsFromGraph();
		}
	}
}

void UTextureGraphInstance::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);

	// save local param changes from graph updates before saving
	InputParams = Graph()->GetInputParamsVarMap();
	UpdateOutputSettingsFromGraph();
}

bool UTextureGraphInstance::CheckOutputSettingsMatch(const TObjectPtr<UTextureGraphBase>& Parent) const
{
	if (OutputSettingsMap.IsEmpty())
		return false;

	FTG_Ids OutputIds = Parent->Graph()->GetOutputParamIds();

	if (OutputIds.Num() != OutputSettingsMap.Num())
		return false;

	bool returnValue = true;
	for(FTG_Id OutputId : OutputIds)
	{
		returnValue &= OutputSettingsMap.Contains(OutputId);
	}
	return returnValue;
}
void UTextureGraphInstance::UpdateOutputSettingsFromGraph()
{
	OutputSettingsMap.Reset();
	Graph()->CollectOutputSettings(OutputSettingsMap);
	//TODO: Keep values if names match
}

UTG_Graph* UTextureGraphInstance::Graph()
{
	if (!RuntimeGraph && ParentTextureGraph)
	{
		RuntimeGraph = DuplicateObject(ParentTextureGraph->Graph(), GetTransientPackage());
	}
	return RuntimeGraph.Get();
}

void UTextureGraphInstance::Initialize()
{
	if (ParentTextureGraph)
	{
		RuntimeGraph = DuplicateObject(ParentTextureGraph->Graph(), GetTransientPackage());
		CopyParamsToRuntimeGraph();
	}
}

/** ---------------------------------------
 *	Texture Graph implementation
 * ---------------------------------------**/

bool UTextureGraph::CheckRecursiveDependency(const UTextureGraph* InTextureGraph) const
{
	TArray<UTextureGraph*> DependentGraphs;
	GatherAllDependentGraphs(DependentGraphs);

	// if after exhausting all nodes and their dependent graphs recursively, we found our source graph, we have a dependency
	return DependentGraphs.ContainsByPredicate([&InTextureGraph](const UTextureGraph* CurrentTextureGraph)
	{
		const UPackage* Package = CurrentTextureGraph->GetPackage();  
		const UPackage* SecondPackage = InTextureGraph->GetPackage();  
		const bool bIsTransientPackage = Package->HasAnyFlags(RF_Transient) || Package == GetTransientPackage();
		return CurrentTextureGraph->GetOutermostObject() == InTextureGraph->GetOutermostObject() ||
				(Package == SecondPackage && !bIsTransientPackage);
	});
}

void UTextureGraph::Construct(FString InName)
{
	Super::Construct(InName);
	TextureGraph = NewObject<UTG_Graph>(this, NAME_None,RF_Transactional);
	TextureGraph->Construct(InName);

	const UTG_Node* OutputNode = TextureGraph->CreateExpressionNode(UTG_Expression_Output::StaticClass());

	UTG_Expression_Output* OutputExpression = Cast<UTG_Expression_Output>(OutputNode->GetExpression());
	OutputExpression->InitializeOutputSettings();

	Settings->GetViewportSettings().InitDefaultSettings(OutputNode->GetNodeName());
}

bool UTextureGraph::HasCyclicDependency() const
{
	return CheckRecursiveDependency(this);
}

void UTextureGraph::GatherAllDependentGraphs(TArray<UTextureGraph*>& DependentGraphs) const
{
	Graph()->ForEachNodes(
			[&](const UTG_Node* Node, uint32 Index)
		{
				if (Node)
				{
					if (UTG_Expression_TextureGraph* TextureGraphExpr = Cast<UTG_Expression_TextureGraph>(Node->GetExpression()))
					{
						const auto OriginalAssetForGraphInExpression = TextureGraphExpr->TextureGraph;

						if (OriginalAssetForGraphInExpression != nullptr)
						{
							// save it in the list
							if(!DependentGraphs.Contains(OriginalAssetForGraphInExpression))
								DependentGraphs.Add(OriginalAssetForGraphInExpression);

							// recursively gather graphs for all GraphExpressions encountered
							OriginalAssetForGraphInExpression->GatherAllDependentGraphs(DependentGraphs);
						}
					}
				}
		});
}
bool UTextureGraph::IsDependentOn(const UTextureGraph* InTextureGraph) const
{
	
	// check if we're trying to assign our own TextureGraph to this expression
	
	if (this->GetOutermostObject() == InTextureGraph->GetOutermostObject())
	{
		return true;
	}
			
	// check for cyclic dependency
	if (CheckRecursiveDependency(InTextureGraph))
	{
		return true;
	}

	// default to false (Not dependent)
	return false;
}

/** ---------------------------------------
 *	Texture Graph Base implementation
 * ---------------------------------------**/

void UTextureGraphBase::Construct(FString InName)
{
	// On the first new texture script we set the engine in run mode
	TextureGraphEngine::SetRunEngine();

	Settings = NewObject<UMixSettings>(this);
	bInvalidateTextures = false;
}


void UTextureGraphBase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FTG_CustomVersion::GUID);

	int32 Version = Ar.CustomVer(FTG_CustomVersion::GUID);

	UE_LOGF(LogTextureGraph, Verbose, "%ls TextureGraph: %ls >>>> %ls",
		(Ar.IsSaving() ? TEXT("Saved") : TEXT("Loaded")),
		*GetName(),
		*FString::FromInt(Version));
}

void UTextureGraphBase::PostLoad()
{
	// On the first script load we set the engine in run mode as well
	TextureGraphEngine::SetRunEngine();

	Super::PostLoad();
	bInvalidateTextures = false;

	// Settings must exist in case it wasn't saved properly
	if (!Settings)
	{
		Settings = NewObject<UMixSettings>(this);
	}

	// Fallback to default material.
	if(!Settings->GetViewportSettings().Material)
	{
		FTG_Ids OutputIds = Graph()->GetOutputParamIds();
		if (!OutputIds.IsEmpty())
		{
			Settings->GetViewportSettings().InitDefaultSettings(Graph()->GetNode(OutputIds[0])->GetNodeName());
		}
		else // the TG doesn't have any output? that's weird but could happen, let's simply create the material regardless targeting a name
		{
			Settings->GetViewportSettings().InitDefaultSettings("Output");
		}
	}
}

void UTextureGraphBase::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);
	UE_LOGF(LogTextureGraph, Verbose, "PreSave Script: %ls", *GetName());
}

void UTextureGraphBase::Update(MixUpdateCyclePtr InCycle)
{
	// Graph Evaluate
	SceneTargetUpdatePtr Target = std::make_shared<MixTargetUpdate>(InCycle->GetMix(), 0);
	InCycle->AddTarget(Target);
	
	T_InvalidateTiles::Create(InCycle, 0);

	// Now Evaluate the Graph!
	FTG_EvaluationContext EvaluationContext;
	EvaluationContext.Cycle = InCycle;

	EvaluationContext.Cycle->PushMix(this);
	if (Graph()->Validate(InCycle))
		Graph()->Evaluate(&EvaluationContext);

	//TODO: fetch the outputs and assign textures to the target texture set here
	// if(Output.IsTexture() && Output.GetTexture())
	// {
	// 	InContext->Cycle->GetTarget(0)->GetLastRender().SetTexture(GetTitleName(), Output.GetTexture().RasterBlob);
	// }
	// else if (Output.IsColor())
	// {
	// 	BufferDescriptor DesiredDesc = T_FlatColorTexture::GetFlatColorDesc("OutputFlat");			
	// 	auto OutputFlatTexture = Source.GetTexture(InContext, FTG_Texture::GetBlack(), &DesiredDesc);
	// 	InContext->Cycle->GetTarget(0)->GetLastRender().SetTexture(GetTitleName(), OutputFlatTexture.RasterBlob);
	// }
	
	EvaluationContext.Cycle->PopMix();
	
	/// This will be the final result of the rendering
	T_UpdateTargets::Create(InCycle, 0, true);
}

void UTextureGraphBase::PostMeshLoad()
{
	FModelInvalidateInfo InvalidateInfo;
	Invalidate(InvalidateInfo);
}

void UTextureGraphBase::FlushInvalidations()
{
	TextureGraphEngine::GetMixManager()->FlushMix(this);
}

void UTextureGraphBase::TriggerUpdate(bool Tweaking)
{
	FModelInvalidateInfo InvalidateInfo;

	InvalidateInfo.Details.All();
	InvalidateInfo.Details.bTweaking = Tweaking;
	InvalidateInfo.Details.Mix = this;

	TextureGraphEngine::GetMixManager()->InvalidateMix(this, InvalidateInfo.Details);
}

void UTextureGraphBase::InvalidateAll()
{
	FInvalidationDetails Details = FInvalidationDetails().All();
	Details.Mix = this;

	InvalidationFrameId = TextureGraphEngine::GetFrameId();

	TextureGraphEngine::GetMixManager()->InvalidateMix(this, Details);
}

void UTextureGraphBase::UpdateGlobalTGSettings()
{
	check(GetSettings());
	GetSettings()->SetWidth(GetMaxWidth());
	GetSettings()->SetHeight(GetMaxHeight());

	int32 Channels = GetMaxBufferChannels();
	BufferFormat Format = GetMaxBufferFormat();
	const ETG_TextureFormat TextureFormat = TextureHelper::GetTGTextureFormatFromChannelsAndFormat(Channels, Format);
	GetSettings()->SetTextureFormat(TextureFormat);
}

EResolution UTextureGraphBase::GetMaxWidth()
{
	EResolution MaxWidth = EResolution::Auto;
	Graph()->ForEachOutputSettings( [MaxWidth](const FTG_OutputSettings& OutSettings) mutable
	{
		EResolution ItemWidth = OutSettings.Width;
		MaxWidth = static_cast<EResolution>(FMath::Max(static_cast<int32>(MaxWidth), static_cast<int32>(ItemWidth)));
	});
	return MaxWidth;
}

EResolution UTextureGraphBase::GetMaxHeight()
{
	EResolution MaxHeight = EResolution::Auto;
	Graph()->ForEachOutputSettings( [MaxHeight](const FTG_OutputSettings& OutSettings) mutable
	{
		EResolution ItemHeight = OutSettings.Height;
		MaxHeight = static_cast<EResolution>(FMath::Max(static_cast<int32>(MaxHeight), static_cast<int32>(ItemHeight)));
	});
	return MaxHeight;
}

int32 UTextureGraphBase::GetMaxBufferChannels()
{
	uint32 MaxBufferChannels = 0;
	Graph()->ForEachOutputSettings( [MaxBufferChannels](const FTG_OutputSettings& OutSettings) mutable
	{
		uint32 Channels = 0;
		BufferFormat Format = BufferFormat::Auto;
		TextureHelper::GetBufferFormatAndChannelsFromTGTextureFormat(OutSettings.TextureFormat, Format, Channels);
		MaxBufferChannels = FMath::Max(MaxBufferChannels, Channels);
	});
	return MaxBufferChannels;
}

BufferFormat UTextureGraphBase::GetMaxBufferFormat()
{
	BufferFormat MaxBufferFormat = BufferFormat::Auto;
	Graph()->ForEachOutputSettings( [MaxBufferFormat](const FTG_OutputSettings& OutSettings) mutable
	{
		uint32 Channels = 0;
		BufferFormat Format = BufferFormat::Auto;
		TextureHelper::GetBufferFormatAndChannelsFromTGTextureFormat(OutSettings.TextureFormat, Format, Channels);
		MaxBufferFormat = static_cast<BufferFormat>(FMath::Max(static_cast<int32>(MaxBufferFormat), static_cast<int32>(Format)));
	});
	return MaxBufferFormat;
}

void UTextureGraphBase::Log() const
{
	Graph()->Log();
}
