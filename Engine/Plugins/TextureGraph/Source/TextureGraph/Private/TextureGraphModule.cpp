// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureGraphModule.h"
#include "TG_Var.h"
#include "Expressions/Filter/TG_Expression_Levels.h"

#define LOCTEXT_NAMESPACE "FTextureGraphModule"

void FTextureGraphModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	FTG_Var::RegisterVarPropertySerializer(TEXT("FTG_LevelsSettings"), FTG_LevelsSettings_VarPropertySerialize);
}

void FTextureGraphModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	FTG_Var::UnregisterVarPropertySerializer(TEXT("FTG_LevelsSettings"));

}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FTextureGraphModule, TextureGraph)

