// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "TG_SystemTypes.h"
#include "TextureGraph.h"
#include "TG_HelperFunctions.h"
#include "Export/TextureExporter.h"
#include "TG_AsyncTask.h"
#include "TG_AsyncRenderTask.h"
#include "TG_AsyncExportTask.generated.h"

#define UE_API TEXTUREGRAPH_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FTSExportTaskDelegate);

UCLASS(MinimalAPI)
class UTG_AsyncExportTask : public UTG_AsyncRenderTaskBase
{
	GENERATED_UCLASS_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "TextureGraph" , meta=(DisplayName="Texture Graph Export (Async)", BlueprintInternalUseOnly = "true"))
	static UE_API UTG_AsyncExportTask* TG_AsyncExportTask(UTextureGraphBase* TextureGraph, const bool OverwriteTextures, const bool bSave, const bool bExportAll, const bool bDisableCache = false);

	UE_API virtual void Activate() override;
	UE_API virtual const TArray<UTextureRenderTarget2D*>& ActivateBlocking(JobBatchPtr Batch);

	UE_API virtual void FinishDestroy() override;

	UPROPERTY(BlueprintAssignable, Category = "TextureGraph")
	FTSExportTaskDelegate OnDone;

private:

	bool bBlobberCachingState;
	UFUNCTION()
	UE_API void OnExportDone();
	bool bOverwriteTextures = true;
	bool bSave = false;
	bool bExportAll = false;
	FExportSettings TargetExportSettings;
};

#undef UE_API
