// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "TG_SystemTypes.h"
#include "TextureGraph.h"
#include "TG_HelperFunctions.h"
#include "Data/Blob.h"
#include "Data/TiledBlob.h"
#include "TG_AsyncTask.h"
#include "Engine/World.h"
#include "TG_AsyncRenderTask.generated.h"

#define UE_API TEXTUREGRAPH_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTSRenderTaskDelegate, const TArray<UTextureRenderTarget2D*>&, OutputRts);

UCLASS(MinimalAPI)
class UTG_AsyncRenderTaskBase : public UTG_AsyncTask
{
	GENERATED_UCLASS_BODY()

public:
	UE_API virtual const TArray<UTextureRenderTarget2D*>& ActivateBlocking(JobBatchPtr Batch);
	UE_API virtual void FinishDestroy() override;
	UE_API virtual void SetReadyToDestroy() override;

	UE_API void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool CleanupResources);

protected:
	UE_API AsyncBool FinalizeAllOutputBlobs();
	UE_API JobBatchPtr PrepareActivate(JobBatchPtr Batch, bool bIsAsync);
	UE_API void GatherAllOutputBlobs();
	UE_API void GatherAllRenderTargets();
	UE_API AsyncBool GetRenderTextures();

	TArray<UTextureRenderTarget2D*> OutputRts;
	TArray<BlobPtr> OutputBlobs;
	UTextureGraphBase* OriginalTextureGraphPtr;
	UTextureGraphBase* TextureGraphPtr;
	bool bShouldDestroyOnRenderComplete = true;
	bool bRenderComplete = false;
};

/////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_AsyncRenderTask : public UTG_AsyncRenderTaskBase
{
	GENERATED_UCLASS_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "TextureGraph" , meta=(DisplayName="Texture Graph Render (Async)" , BlueprintInternalUseOnly = "true"))
	static UE_API UTG_AsyncRenderTask* TG_AsyncRenderTask(UTextureGraphBase* InTextureGraph);

	UE_API virtual void Activate() override;

	UPROPERTY(BlueprintAssignable, Category = "TextureGraph")
	FTSRenderTaskDelegate OnDone;
};

#undef UE_API
