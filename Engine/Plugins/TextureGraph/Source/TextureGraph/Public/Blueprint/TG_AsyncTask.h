// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "TG_AsyncTask.generated.h"

#define UE_API TEXTUREGRAPH_API

/**
* BlueprintCallable factory functions for Async TG_Script Actions. Helps regiser with TG_AsyncTaskManager to keep Task alive and manage its life cycle 
*/

UCLASS(MinimalAPI)
class UTG_AsyncTask : public UBlueprintAsyncActionBase
{
	GENERATED_UCLASS_BODY()

public:

	UE_API virtual void Activate() override;
	
	/**
	 * Call to register this task with a AsyncTaskManager, it will not be destroyed until SetReadyToDestroy is called
	 * This allows having an action stay alive until SetReadyToDestroy is manually called, allowing it to be used inside loops or if the calling BP goes away
	 */
	UE_API virtual void RegisterWithTGAsyncTaskManger();

	/** Call when the action is completely done, this makes the action free to delete, and will unregister it with the game instance and AsyncTaskManager  */
	UE_API virtual void SetReadyToDestroy() override;
};

#undef UE_API
