// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "TG_AsyncTask.h"
#include "TG_SystemTypes.h"
#include "Containers/Array.h"

/**
* Keeps reference to the TG_AsyncTasks to manage their life cycle
*/

class FTG_AsyncTaskManager
{
private:
	FTG_AsyncTaskManager() {}  // Private constructor to enforce singleton pattern.
	TArray<UTG_AsyncTask*> Tasks;
public:
	static FTG_AsyncTaskManager& GetInstance()
	{
		static FTG_AsyncTaskManager Instance;
		return Instance;
	}

	void RegisterTask(UTG_AsyncTask* Task)
	{
		if (!Tasks.Contains(Task))
		{
			Tasks.Add(Task);
		}
		else
		{
			UE_LOGF(LogTextureGraph, Log, "FTG_AsyncTaskManager::RegisterTask: Task Already added RegisterTask function should only be called once for each task");
		}
	}

	void UnRegisterTask(UTG_AsyncTask* Task)
	{
		if (Tasks.Contains(Task))
		{
			Tasks.Remove(Task);
		}
		else
		{
			UE_LOGF(LogTextureGraph, Log, "FTG_AsyncTaskManager::UnRegisterTask: Task not registerd with FTG_AsyncTaskManager consider calling RegisterTask");
		}
	}
};

