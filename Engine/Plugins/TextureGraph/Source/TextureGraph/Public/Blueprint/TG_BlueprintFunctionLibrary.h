// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "TextureGraph.h"
#include "TG_OutputSettings.h"
#include "TG_AsyncRenderTask.h"
#include "TG_AsyncExportTask.h"
#include "TG_BlueprintFunctionLibrary.generated.h"

//DECLARE_DYNAMIC_DELEGATE_OneParam(FOnTextureGraphRenderDone, UTextureRenderTarget2D*, Results);
//DECLARE_DELEGATE_TwoParams(FOnGroomBindingAssetBuildCompleteNative, UGroomBindingAsset*, EGroomBindingAssetBuildResult);

UCLASS( meta=(ScriptName="TextureScreptingLibrary"))
class UTG_BlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Sets a Texture parameter value on the TextureGraph instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category="TextureGraph", meta=(Keywords="SetTextureParameterValue", WorldContext="WorldContextObject"))
	static void SetTextureParameterValue(UObject* WorldContextObject, UTextureGraph* TextureGraph, FName ParameterName, UTexture* ParameterValue);

	/** Gets a texture parameter value from the TextureGraph instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category="TextureGraph", meta=(Keywords="GetTextureParameterValue", WorldContext="WorldContextObject"))
	static UTexture* GetTextureParameterValue(UObject* WorldContextObject, UTextureGraph* TextureGraph, FName ParameterName);

	/** Sets a scalar parameter value on the TextureGraph instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category = "TextureGraph", meta = (Keywords = "SetScalarParameterValue", WorldContext = "WorldContextObject"))
	static void SetScalarParameterValue(UObject* WorldContextObject, UTextureGraph* TextureGraph, FName ParameterName, float ParameterValue);

	/** Gets a scalar parameter value from the TextureGraph instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category = "TextureGraph", meta = (Keywords = "GetScalarParameterValue", WorldContext = "WorldContextObject"))
	static float GetScalarParameterValue(UObject* WorldContextObject, UTextureGraph* TextureGraph, FName ParameterName);

	/** Sets a boolean parameter value on the TextureGraph instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category = "TextureGraph", meta = (Keywords = "SetBoolParameterValue", WorldContext = "WorldContextObject"))
	static void SetBoolParameterValue(UObject* WorldContextObject, UTextureGraph* TextureGraph, FName ParameterName, bool ParameterValue);

	/** Gets a string parameter value from the TextureGraph instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category = "TextureGraph", meta = (Keywords = "GetBoolParameterValue", WorldContext = "WorldContextObject"))
	static bool GetBoolParameterValue(UObject* WorldContextObject, UTextureGraph* TextureGraph, FName ParameterName);

	/** Sets a string parameter value on the TextureGraph instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category = "TextureGraph", meta = (Keywords = "SetBoolParameterValue", WorldContext = "WorldContextObject"))
	static void SetStringParameterValue(UObject* WorldContextObject, UTextureGraph* TextureGraph, FName ParameterName, FString ParameterValue);

	/** Gets a String parameter value from the TextureGraph instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category = "TextureGraph", meta = (Keywords = "GetBoolParameterValue", WorldContext = "WorldContextObject"))
	static FString GetStringParameterValue(UObject* WorldContextObject, UTextureGraph* TextureGraph, FName ParameterName);

	/** Sets a Vector parameter value on the TextureGraph instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category = "TextureGraph", meta = (Keywords = "SetVectorParameterValue", WorldContext = "WorldContextObject"))
	static void SetVectorParameterValue(UObject* WorldContextObject, UTextureGraph* TextureGraph, FName ParameterName, FVector4f ParameterValue);

	/** Gets a Vector parameter value from the TextureGraph instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category = "TextureGraph", meta = (Keywords = "GetVectorParameterValue", WorldContext = "WorldContextObject"))
	static FVector4f GetVectorParameterValue(UObject* WorldContextObject, UTextureGraph* TextureGraph, FName ParameterName);

	/** Sets a color parameter value on the TextureGraph instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category = "TextureGraph", meta = (Keywords = "SetColorParameterValue", WorldContext = "WorldContextObject"))
	static void SetColorParameterValue(UObject* WorldContextObject, UTextureGraph* TextureGraph, FName ParameterName, FLinearColor ParameterValue);

	/** Gets a color parameter value from the TextureGraph instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category = "TextureGraph", meta = (Keywords = "GetColorParameterValue", WorldContext = "WorldContextObject"))
	static FLinearColor GetColorParameterValue(UObject* WorldContextObject, UTextureGraph* TextureGraph, FName ParameterName);

	/** Sets a FTG_OutputSettings parameter value on the TextureGraph instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category = "TextureGraph", meta = (Keywords = "SetSettingsParameterValue", WorldContext = "WorldContextObject"))
	static void SetSettingsParameterValue(UObject* WorldContextObject, UTextureGraph* TextureGraph, FName ParameterName, int Width,int Height, FName FileName = "None",
		FName Path = "None", ETG_TextureFormat Format = ETG_TextureFormat::BGRA8 , ETG_TexturePresetType TextureType = ETG_TexturePresetType::None,
		TextureGroup LODTextureGroup = TextureGroup::TEXTUREGROUP_World, TextureCompressionSettings Compression = TextureCompressionSettings::TC_Default, bool SRGB = false);

	/** Gets a FTG_OutputSettings parameter value from the TextureGraph instance. Logs if ParameterName is invalid. */
	UFUNCTION(BlueprintCallable, Category = "TextureGraph", meta = (Keywords = "GetOutputSettingsParameterValue", WorldContext = "WorldContextObject"))
	static FTG_OutputSettings GetSettingsParameterValue(UObject* WorldContextObject, UTextureGraph* TextureGraph, FName ParameterName , int& Width, int& Height);

	/** Render the texture graph and return an array of texture render targets. */
	UFUNCTION(BlueprintCallable, Category = "TextureGraph", meta = (Keywords = "RenderTextureGraph", WorldContext = "WorldContextObject"))
	static const TArray<UTextureRenderTarget2D*>& RenderTextureGraph(UObject* WorldContextObject, UTextureGraphBase* InTextureGraph);

	/** Export texture graph. */
	UFUNCTION(BlueprintCallable, Category = "TextureGraph", meta = (Keywords = "ExportTextureGraph", WorldContext = "WorldContextObject"))
	static void ExportTextureGraph(UObject* WorldContextObject, UTextureGraphBase* InTextureGraph, bool bOverwriteTextures = true, bool bSave = false, bool bExportAll = false);

	static void AddParamWarning(FName ParamName, UObject* ObjectPtr, FString FunctionName);
	static void AddError(UObject* ObjectPtr, FString FunctionName, FString Error);
};
