// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"

#include "TG_Expression_NormalFromHeightMap.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_NormalFromHeightMap : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Adjustment);
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;
	UE_API virtual bool						Validate(MixUpdateCyclePtr Cycle) override;

	// Normals from height are generated by taking nearby points from the texture having some offset. Using smaller offset values means we are getting samples closer to the surface from the texture. 
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0.001", ClampMin = "0.001", UIMax = "0.1", ClampMax = "0.1"))
	float								Offset = 0.002f;

	// Strength of the normals
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "2", ClampMax = "2"))
	float								Strength = 1;
	
	// The input image to be converted to Normal from height map
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Texture							Input;

	// The output image which is a single channel grayscale version of the input image 
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;

	virtual FTG_Name					GetDefaultName() const override { return TEXT("NormalFromHeightMap");}
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Converts the input into normal.")); }
};

#undef UE_API
