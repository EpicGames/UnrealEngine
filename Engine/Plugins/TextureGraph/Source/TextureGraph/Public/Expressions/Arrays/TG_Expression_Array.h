// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "TG_Expression_Array.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_Array4 : public UTG_Expression
{
	 GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Arrays);
	UE_API virtual void					Evaluate(FTG_EvaluationContext* InContext) override;
	
	/// Texture at the first index of the output texture array
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Variant						Input1;

	/// Texture at the second index of the output texture array
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Variant						Input2;

	/// Texture at the third index of the output texture array
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Variant						Input3;

	/// Texture at the fourth index of the output texture array
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Variant						Input4;

	/// All the input textures organized into an array
	UPROPERTY(meta = (TGType = "TG_Output", PinDisplayName = "", HideInnerPropertiesInNode))
	FTG_VariantArray				Output;

	virtual FTG_Name				GetDefaultName() const override { return TEXT("Array4");}
	virtual FText					GetTooltipText() const override { return FText::FromString(TEXT("Combines 4 inputs of any type and constructs an array out of it.")); } 
 };

#undef UE_API
