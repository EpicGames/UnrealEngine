// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "TG_Expression_ArrayGrid.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_ArrayGrid : public UTG_Expression
{
	 GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Arrays);
	UE_API virtual void					Evaluate(FTG_EvaluationContext* InContext) override;
	
	/// The input texture array. Must be an array object of type textures
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "", HideInnerPropertiesInNode))
	FTG_VariantArray				Input;

	/// The number of rows in the output grid. 0 or Negative values mean automatically distribute
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	int32							Rows = 0;

	/// The number of colums in the output grid. 0 or Negative values mean automatically distribute
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	int32							Columns = 0;

	/// The background color to use for the output texture
	UPROPERTY(meta = (TGType = "TG_Input"))
	FLinearColor					BackgroundColor = FLinearColor::Transparent;

	/// A single output texture
	UPROPERTY(meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture						Output;

	virtual FTG_Name				GetDefaultName() const override { return TEXT("ArrayGrid");}
	virtual FText					GetTooltipText() const override { return FText::FromString(TEXT("Arranges an input texture array into an MxN sized grid.")); } 
 };

#undef UE_API
