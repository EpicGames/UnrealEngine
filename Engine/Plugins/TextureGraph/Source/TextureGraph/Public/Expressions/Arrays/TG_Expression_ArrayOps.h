// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "TG_Expression_ArrayOps.generated.h"

#define UE_API TEXTUREGRAPH_API

/////////////////////////////////////////////////////////////
/// Array concatenation
/////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_ArrayConcat : public UTG_Expression
{
	 GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Arrays);
	UE_API virtual void					Evaluate(FTG_EvaluationContext* InContext) override;
	
	/// The first array as part of the concatenation process
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "Input-1"))
	FTG_VariantArray				Input1;

	/// The index at which Input1 array is going to start. Input 2 is going occupy the rest of the places within the array
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "Input-1 Out Index"))
	int32							StartIndex = 0;

	/// The second array as part of the concatenation process
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "Input-2"))
	FTG_VariantArray				Input2;

	/// Concatenated array
	UPROPERTY(meta = (TGType = "TG_Output", PinDisplayName = "", HideInnerPropertiesInNode))
	FTG_VariantArray				Output;

	virtual FTG_Name				GetDefaultName() const override { return TEXT("Concatenate Array");}
	virtual FText					GetTooltipText() const override { return FText::FromString(TEXT("Combines two arrays into a single one.")); } 
 };

/////////////////////////////////////////////////////////////
/// Split array
/////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_ArraySplit: public UTG_Expression
{
	 GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Arrays);
	UE_API virtual void					Evaluate(FTG_EvaluationContext* InContext) override;
	
	/// The array to slice
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "", MinValue))
	FTG_VariantArray				Input;

	/// The starting index of the slicing operation. 0-indexed meaning that 0 indicates the first element of the input array
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	int32							StartIndex = 0;

	/// The ending index of the slicing operation (non-inclusive). Negative values mean the end of the array.
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	int32							EndIndex = -1;

	/// The ending index of the slicing operation (non-inclusive). Negative values mean the end of the array.
	UPROPERTY(meta = (TGType = "TG_Setting", PinDisplayName = "Splice"))
	bool							bSplice = false;

	/// Sliced part of the array
	UPROPERTY(meta = (TGType = "TG_Output", PinDisplayName = "Sliced", HideInnerPropertiesInNode))
	FTG_VariantArray				Sliced;

	/// Spliced part of thea array
	UPROPERTY(meta = (TGType = "TG_Output", PinDisplayName = "Spliced", HideInnerPropertiesInNode))
	FTG_VariantArray				Spliced;

	virtual FTG_Name				GetDefaultName() const override { return TEXT("Modify Array (Slice/Splice)");}
	virtual FText					GetTooltipText() const override { return FText::FromString(TEXT("Slice a portion of an input array and put that as the output of this array. Alternatively splice an array and remove some elements from the input array")); } 
 };

#undef UE_API
