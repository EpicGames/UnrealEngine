// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"

#include "TG_Expression_ChannelCombiner.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_ChannelCombiner : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Channel);
	UE_API virtual void			Evaluate(FTG_EvaluationContext* InContext) override;

	// The image that will end up in the red channel of the output image
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "R"))
	FTG_Texture				Red;

	// The image that will end up in the green channel of the output image
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "G"))
	FTG_Texture				Green;

	// The image that will end up in the blue channel of the output image
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "B"))
	FTG_Texture				Blue;

	// The image that will end up in the alpha channel of the output image
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "A"))
	FTG_Texture				Alpha;

	// The output image that combines the red, green, blue and alpha images into a single one
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = "RGBA"))
	FTG_Texture				Output = FTG_Texture::GetBlack();

	virtual FTG_Name		GetDefaultName() const override { return TEXT("CombineChannels");}
	virtual FText			GetTooltipText() const override { return FText::FromString(TEXT("Combines individual R, G, B, A channels into an RGBA image.")); } 
};

#undef UE_API
