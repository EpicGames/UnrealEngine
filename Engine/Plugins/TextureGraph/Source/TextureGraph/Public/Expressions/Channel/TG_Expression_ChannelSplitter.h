// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"

#include "TG_Expression_ChannelSplitter.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_ChannelSplitter : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Channel);
	UE_API virtual void			Evaluate(FTG_EvaluationContext* InContext) override;

	// The input texture
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "RGBA"))
	FTG_Texture				Input;

	// Red channel of the input
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = "R", HideInnerPropertiesInNode))
	FTG_Texture				Red = FTG_Texture::GetBlack();

	// Green channel of the input
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = "G", HideInnerPropertiesInNode))
	FTG_Texture				Green = FTG_Texture::GetBlack();

	// Blue channel of the input
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = "B", HideInnerPropertiesInNode))
	FTG_Texture				Blue = FTG_Texture::GetBlack();

	// Alpha channel of the input
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = "A", HideInnerPropertiesInNode))
	FTG_Texture							Alpha = FTG_Texture::GetBlack();

	virtual FTG_Name		GetDefaultName() const override { return TEXT("SplitChannels");}
	virtual FText			GetTooltipText() const override { return FText::FromString(TEXT("Splits an RGBA image into individual R, G, B, A channels.")); } 
};

#undef UE_API
