// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "Transform/Expressions/T_Channel.h"

#include "TG_Expression_ChannelSwizzle.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_ChannelSwizzle : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Channel);
	UE_API virtual void			Evaluate(FTG_EvaluationContext* InContext) override;

	// The input texture
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Texture				Input;

	// The output in the red channel
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", DisplayName = "Red"))
	EColorChannel			RedChannel = EColorChannel::Red;

	// The output in the Green channel
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", DisplayName = "Green"))
	EColorChannel			GreenChannel = EColorChannel::Green;

	// The output in the Blue channel
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", DisplayName = "Blue"))
	EColorChannel			BlueChannel = EColorChannel::Blue;

	// The output in the Alpha channel
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", DisplayName = "Alpha"))
	EColorChannel			AlphaChannel = EColorChannel::Alpha;

	// The output texture after swizzle operation
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture				Output;


	virtual FTG_Name		GetDefaultName() const override { return TEXT("Swizzle");}
	virtual FText			GetTooltipText() const override { return FText::FromString(TEXT("Performs a swizzle operation.")); } 
};

#undef UE_API
