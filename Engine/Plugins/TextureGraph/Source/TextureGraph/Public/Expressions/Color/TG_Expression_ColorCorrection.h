// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "TG_Expression_ColorCorrection.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_ColorCorrection : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Adjustment);
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

	// The input image to be color corrected
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Texture							Input;

	// Adjusts the overall lightness or darkness of an image. Increasing brightness makes it brighter, while decreasing brightness makes it darker.
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "-1", ClampMin = "-1", UIMax = "1", ClampMax = "1"))
	float								Brightness = 0;

	// Modifies the difference between light and dark areas within an image. Increasing contrast makes light areas lighter and dark areas darker, while decreasing contrast reduces this difference, resulting in a more uniform appearance.
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "10", ClampMax = "10"))
	float								Contrast = 1;

	// Gamma can be used to manipulate the curve
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "10", ClampMax = "10"))
	float								Gamma = 0.5f;

	// Saturation is how intense and pure a color appears. 
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "10", ClampMax = "10"))
	float								Saturation = 1;

	// The temperature adjustment for the image
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "1000", ClampMin = "1000", UIMax = "20000", ClampMax = "20000"))
	float								Temperature = 6500.0;

	// The strength of the temperature application. It just gives more control over mixing the original image with the temperature applied image
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1"))
	float								TemperatureStrength = 0;

	// How to normalize the original brightness of the image with the new brigthness (after temperature adjustment)
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1"))
	float								TemperatureBrightnessNormalization = 0;

	// The output image 
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;

	virtual FTG_Name					GetDefaultName() const override { return TEXT("Color Correction");}
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Provides color tweaking and correction options.")); } 

};

#undef UE_API
