// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"

#include "TG_Expression_Grayscale.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_Grayscale : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Adjustment);
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

	// The input image to be converted to grayscale
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Texture							Input;

	// The output image which is a single channel grayscale version of the input image 
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;

	virtual FTG_Name					GetDefaultName() const override { return TEXT("ConvertToGrayscale");}
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Converts the input into grayscale.")); } 

};

#undef UE_API
