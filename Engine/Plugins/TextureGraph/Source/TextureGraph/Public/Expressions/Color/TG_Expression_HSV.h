// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "Expressions/Maths/TG_Expression_Maths_OneInput.h"

#include "TG_Expression_HSV.generated.h"

#define UE_API TEXTUREGRAPH_API

//////////////////////////////////////////////////////////////////////////
/// RGB2HSV Correction
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_RGB2HSV : public UTG_Expression_OneInput
{
	GENERATED_BODY()

public:
	virtual FTG_Name					GetDefaultName() const override { return TEXT("RGBtoHSV");}
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Interprets the input as RGB and converts it to HSV.")); } 
	virtual FName						GetCategory() const override { return TG_Category::Adjustment; } 

protected:
	/// This is unsupported because it only makes sense on a vector input
	virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override { return 0.0f; }
	UE_API virtual FVector4f					EvaluateVector_WithValue(FTG_EvaluationContext* InContext, const FVector4f* const Value, size_t Count);
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
};

//////////////////////////////////////////////////////////////////////////
/// HSV2RGB Correction
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_HSV2RGB : public UTG_Expression_OneInput
{
	GENERATED_BODY()

public:
	virtual FTG_Name					GetDefaultName() const override { return TEXT("HSVtoRGB");}
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Interprets the input as HSV and converts it to RGB.")); } 
	virtual FName						GetCategory() const override { return TG_Category::Adjustment; } 

protected:
	/// This is unsupported because it only makes sense on a vector input
	virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override { return 0.0f; }
	UE_API virtual FVector4f					EvaluateVector_WithValue(FTG_EvaluationContext* InContext, const FVector4f* const Value, size_t Count);
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
};

//////////////////////////////////////////////////////////////////////////
/// HSV Correction
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_HSV : public UTG_Expression_OneInput
{
	GENERATED_BODY()

public:
	// Defines the basic color tone, such as red, green, or blue, without considering brightness or intensity. Adjusting the hue changes the overall color appearance while maintaining its saturation and brightness.
	// The normalized hue. Please divide your [0, 359] hue values by 359 for this input
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", PinDisplayName = "Hue (Normalized)", DisplayName = "Hue (Normalized)"))
	float								Hue = 1.0f;
	
	// Controls the intensity of the color. Higher values represent more vivid colors, while lower values produce muted tones.
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1"))
	float								Saturation = 1.0f;

	// Specifies the brightness or darkness of the color. Higher values correspond to brighter colors, while lower values result in darker shades.
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1"))
	float								Value = 1.0f;

	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Converts Hue, Saturation and Value inputs to an RGB Image.")); } 
	virtual FName						GetCategory() const override { return TG_Category::Adjustment; } 

protected:
	/// This is unsupported because it only makes sense on a vector input
	virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const ValuePtr, size_t Count) override { return 0.0f; }
	UE_API virtual FVector4f					EvaluateVector_WithValue(FTG_EvaluationContext* InContext, const FVector4f* const ValuePtr, size_t Count);
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
};

#undef UE_API
