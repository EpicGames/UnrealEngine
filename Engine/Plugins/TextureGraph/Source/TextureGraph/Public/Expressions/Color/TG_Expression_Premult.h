// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "Expressions/Maths/TG_Expression_Maths_OneInput.h"

#include "TG_Expression_Premult.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_Premult : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Adjustment);
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

	// The input image to be used
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Texture							Input;

	// The output of the image with the premultiplied alpha
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;

	virtual FTG_Name					GetDefaultName() const override { return TEXT("Premultiply Alpha");}
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Premultiply alpha with the RGB color values at each pixel. The alpha remains unchanged.")); } 
};

#undef UE_API
