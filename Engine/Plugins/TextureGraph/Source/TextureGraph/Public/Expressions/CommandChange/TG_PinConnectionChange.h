// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TG_SystemTypes.h"  
#include "Misc/Change.h"

#define UE_API TEXTUREGRAPH_API

class UTG_Graph;
class UTG_Node;
class UTG_Pin;

/** This class helps to Undo the pin connection correctly
as the Blob is not serialized so if the expression property is not connected and is holding 
a reference of blob it is not rest when undoing the connection*/
class FTG_PinConnectionChange : public FCommandChange 
{
public:
	static UE_API void StoreChange(UTG_Graph* InGraph, UTG_Node& NodeFrom, FTG_Name& PinFromName);
	UE_API FTG_PinConnectionChange(UTG_Graph* InGraph, UTG_Node& NodeFrom, FTG_Name& PinFromName);

	virtual FString ToString() const override
	{
		return FString(TEXT("Pin Connection"));
	}

	virtual void Apply(UObject* Object) override {};
	UE_API virtual void Revert(UObject* Object) override;

private:
	UTG_Graph* Graph;
	UTG_Pin* PinTo;
};

/** This class helps to Redo the pin connection Break correctly
as the Blob is not serialized so if the expression property is not connected and is holding
a reference of blob it is not rest when redoing the connection*/
class FTG_PinConnectionBreakChange : public FCommandChange
{
public:
	static UE_API void StoreChange(UTG_Graph* InGraph, UTG_Node& NodeFrom, FTG_Name& PinFromName);
	UE_API FTG_PinConnectionBreakChange(UTG_Graph* InGraph, UTG_Node& NodeFrom, FTG_Name& PinFromName);

	virtual FString ToString() const override
	{
		return FString(TEXT("Pin Connection Break"));
	}

	UE_API virtual void Apply(UObject* Object) override;
	virtual void Revert(UObject* Object) override {};

private:
	UTG_Graph* Graph;
	UTG_Pin* PinTo;
};

#undef UE_API
