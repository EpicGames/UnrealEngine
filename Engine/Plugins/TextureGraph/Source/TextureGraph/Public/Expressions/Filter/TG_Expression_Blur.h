// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Expressions/TG_Expression_MaterialBase.h"

#include "TG_Expression_Blur.generated.h"

#define UE_API TEXTUREGRAPH_API

UENUM(BlueprintType)
enum class EBlurType : uint8
{
	Gaussian = 0		UMETA(DisplayName = "Gaussian"),
	Directional	= 1		UMETA(DisplayName = "Directional"),
	Radial = 2			UMETA(DisplayName = "Radial")
};

UCLASS(MinimalAPI)
class UTG_Expression_Blur : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Filter);
	
#if WITH_EDITOR
	// Used to implement EditCondition logic for both Node UI and Details View
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif

	// Controls how much an texture is blurred
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = 0, ClampMax = 100, EditConditionHides))
	int32 Radius = 1;

	// Adjusting the angle changes the direction at which the blur effect is applied." 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = -180, ClampMax = 180,EditConditionHides))
	float Angle = 0.0f;

	// Determines how intense the blurring effect is applied. Higher values result in stronger blur, while lower values produce a milder effect.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = 0, ClampMax = 1, EditConditionHides))
	float Strength = 0.1f;

	// Style of blurring applied to a texture. 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", RegenPinsOnChange))
	EBlurType BlurType = EBlurType::Gaussian;

	// The output texture having blurred effect
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture Output;

	// The input texture to apply the blur effect
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Texture Input;

	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Applies a blur filter with a given strength.")); }
	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;
};

#undef UE_API
