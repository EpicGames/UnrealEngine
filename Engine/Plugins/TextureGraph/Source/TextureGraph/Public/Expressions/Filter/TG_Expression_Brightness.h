// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "FxMat/FxMaterial.h"

#include "TG_Expression_Brightness.generated.h"

#define UE_API TEXTUREGRAPH_API

//////////////////////////////////////////////////////////////////////////
/// Expression
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Brightness : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Adjustment);
	UE_API virtual void							Evaluate(FTG_EvaluationContext* InContext) override;

	// Adjusts the overall lightness or darkness of an image. Increasing brightness makes it brighter, while decreasing brightness makes it darker.
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "-1", ClampMin = "-1", UIMax = "1", ClampMax = "1"))
		float								Brightness = 0;

	// Modifies the difference between light and dark areas within an image. Increasing contrast makes light areas lighter and dark areas darker, while decreasing contrast reduces this difference, resulting in a more uniform appearance.
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "10", ClampMax = "10"))
		float								Contrast = 1;

	// The input texture to adjust the brightness/contrast for
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
		FTG_Texture							Input;

	// The output image
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
		FTG_Texture							Output;
	virtual FTG_Name						GetDefaultName() const override { return TEXT("BrightnessContrast");}
	virtual FText							GetTooltipText() const override { return FText::FromString(TEXT("Changes the brightness and contrast of the input.")); } 
};

#undef UE_API
