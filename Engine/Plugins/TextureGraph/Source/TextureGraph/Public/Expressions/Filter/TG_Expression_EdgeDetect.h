// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Expressions/TG_Expression_MaterialBase.h"

#include "TG_Expression_EdgeDetect.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_EdgeDetect : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Filter);
	
	// What is the thickness of the edge
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = 0.1, ClampMax = 30, UIMin = 0.1, UIMax = 1, EditConditionHides))
	float Thickness = 1;

	// The output texture having blurred effect
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture Output;

	// The input texture to apply the blur effect
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Texture Input;

	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Detects edges (areas of sharp color change) from the input.")); }
	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;
};

#undef UE_API
