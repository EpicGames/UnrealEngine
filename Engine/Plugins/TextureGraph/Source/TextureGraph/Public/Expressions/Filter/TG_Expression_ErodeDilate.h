// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "Expressions/Maths/TG_Expression_Maths_OneInput.h"

#include "TG_Expression_ErodeDilate.generated.h"

#define UE_API TEXTUREGRAPH_API

//////////////////////////////////////////////////////////////////////////
/// Type
//////////////////////////////////////////////////////////////////////////
UENUM(BlueprintType)
enum class EErodeOrDilate : uint8
{
	Erode = 0		UMETA(DisplayName = "Erode"),
	Dilate = 1		UMETA(DisplayName = "Dilate"),
};

//////////////////////////////////////////////////////////////////////////
/// Type
//////////////////////////////////////////////////////////////////////////
UENUM(BlueprintType)
enum class EErodeDilateKernelType : uint8
{
	Box = 0			UMETA(DisplayName = "Box"),
	Circular = 1 	UMETA(DisplayName = "Circular"),
	Diamond = 2		UMETA(DisplayName = "Diamond"),
};

//////////////////////////////////////////////////////////////////////////
/// Type
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_ErodeDilate : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Filter);
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

	/// The input image to 
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Texture							Input;

	///// The minimum brightness level to use. Stick with 0 for non-hdr images
	//UPROPERTY(meta = (TGType = "TG_Input"))
	//float								MinBrightness = 0.0f;

	///// The maximum brightness level to use. Stick with 1 for non-hdr images
	//UPROPERTY(meta = (TGType = "TG_Input"))
	//float								MinBrightness = 0.0f;

	/// The size of the kernel to use
	UPROPERTY(meta = (TGType = "TG_Setting", ClampMin = 0, ClampMax = 64, UIMin = 0, UIMax = 64))
	int32								Size = 2;

	/// The type of the kernel to use
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting"))
	EErodeDilateKernelType				Kernel = EErodeDilateKernelType::Box;

	/// Whether to erode or dilate
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting"))
	EErodeOrDilate						Type = EErodeOrDilate::Erode;

	/// The output image which is a single channel grayscale version of the input image 
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;

	virtual FTG_Name					GetDefaultName() const override { return TEXT("Erode/Dilate");}
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Erode or dilate a particular image.")); } 
};

#undef UE_API
