// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"

#include "TG_Expression_Levels.generated.h"

#define UE_API TEXTUREGRAPH_API



UENUM(BlueprintType)
enum class ELevelsExpressionType : uint8
{
	LowMidHigh = 0			UMETA(DisplayName = "Manual"),
	AutoLowHigh				UMETA(DisplayName = "Auto Levels"),
};

struct FLevels;

USTRUCT(BlueprintType)
struct FTG_LevelsSettings
{
	GENERATED_USTRUCT_BODY()

	// The Low value of the Levels adjustment, any pixel under that value is set to black. Default is 0.
	UPROPERTY(EditAnywhere, Category = "Levels", meta = (PinDisplayName = "Low", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1"))
		float								Low = 0;

	// The mid value of the Levels adjustment, must be in the range [Min, Max] and the Default is 0.5.
	// The mid value determine where the smoothing curve applying the midpoint filter is crossing 0.5.
	UPROPERTY(EditAnywhere, Category = "Levels", meta = (PinDisplayName = "Mid", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1"))
		float								Mid = 0.5f; // Midtones

	// The High value of the Levels adjustment, any pixel above that value is set to white. Default is 1.
	UPROPERTY(EditAnywhere, Category = "Levels", meta = (PinDisplayName = "High", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1"))
		float								High = 1;

	UE_API bool SetLow(float InValue);
	UE_API bool SetMid(float InValue);
	UE_API bool SetHigh(float InValue);

	// Eval Low-High range mapping on value
	UE_API float EvalRange(float Val) const;
	// Eval reverse Low-High range mapping on value
	UE_API float EvalRangeInv(float Val) const;

	// Evaluate the MidExponent of the power curve applying the midpoint filter
	UE_API float EvalMidExponent() const;
	UE_API bool SetMidFromMidExponent(float InExponent);

	void InitFromString(const FString& StrVal)
	{
		FTG_LevelsSettings::StaticStruct()->ImportText(*StrVal, this, /*OwnerObject*/nullptr, 0, nullptr, FTG_LevelsSettings::StaticStruct()->GetName(), /*bAllowNativeOverride*/true);
	}

	FString ToString() const
	{
		FString ExportString;
		FTG_LevelsSettings::StaticStruct()->ExportText(ExportString, this, this, /*OwnerObject*/nullptr, /*PortFlags*/0, /*ExportRootScope*/nullptr);
		return ExportString;
	}


};

void FTG_LevelsSettings_VarPropertySerialize(FTG_Var::VarPropertySerialInfo&);
template <> FString TG_Var_LogValue(FTG_LevelsSettings& Value);
template <> void TG_Var_SetValueFromString(FTG_LevelsSettings& Value, const FString& StrVal);

UCLASS(MinimalAPI)
class UTG_Expression_Levels : public UTG_Expression
{
	GENERATED_BODY()

	FTG_LevelsSettings					Levels;

	TSharedPtr<FLevels> 				LevelsControl;

public:	
	UE_API virtual void						PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void 						PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// Used to implement EditCondition logic for both Node UI and Details View
	UE_API virtual bool						CanEditChange(const FProperty* InProperty) const override;
#endif



	TG_DECLARE_EXPRESSION(TG_Category::Adjustment);
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", PinDisplayName = "Type", PinNotConnectable = true, RegenPinsOnChange))
	ELevelsExpressionType				LevelsExpressionType = ELevelsExpressionType::LowMidHigh;

	// The input image to adjust the levels for
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Texture							Input;

	// The Low value of the Levels adjustment, any pixel under that value is set to black. Default is 0.
	UPROPERTY(EditAnywhere, Setter,  Category = NoCategory, meta = (TGType = "TG_Setting", PinDisplayName = "Low", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", EditConditionHides, MD_ScalarEditor))
	float								LowValue = 0;
	UE_API void SetLowValue(float InValue);

	// The mid value of the Levels adjustment, must be in the range [Min, Max] and the Default is 0.5.
	// The mid value determine where the smoothing curve applying the midpoint filter is crossing 0.5.
	UPROPERTY(EditAnywhere, Setter, Category = NoCategory, meta = (TGType = "TG_Setting", PinDisplayName = "Mid", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", EditConditionHides, MD_ScalarEditor))
	float								MidValue = 0.5f; // Midtones
	UE_API void SetMidValue(float InValue);

	// The High value of the Levels adjustment, any pixel above that value is set to white. Default is 1.
	UPROPERTY(EditAnywhere, Setter, Category = NoCategory, meta = (TGType = "TG_Setting", PinDisplayName = "High", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", EditConditionHides, MD_ScalarEditor))
	float								HighValue = 1;
	UE_API void SetHighValue(float InValue);

	// The black point of the output. Moving this will remap the dark point to this value. Default is 0
	UPROPERTY(EditAnywhere, Setter, Category = NoCategory, meta = (TGType = "TG_Setting", PinDisplayName = "Out Low", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", EditConditionHides, MD_ScalarEditor))
	float								OutLowValue = 0;
	UE_API void SetOutLowValue(float InValue);

	// The white point of the output. Moving this will remap the white point to this value. Default is 1
	UPROPERTY(EditAnywhere, Setter, Category = NoCategory, meta = (TGType = "TG_Setting", PinDisplayName = "Out High", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", EditConditionHides, MD_ScalarEditor))
	float								OutHighValue = 1;
	UE_API void SetOutHighValue(float InValue);

	// The High value of the Levels adjustment, any pixel above that value is set to white. Default is 1.
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", PinDisplayName = "Mid Auto Levels", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", EditConditionHides, MD_ScalarEditor))
	float								MidAutoLevels = 0.5;


	// The output image filtered result of the Levels operator 
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;

	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Remaps shadows and highlights of the input. Any values less or equal to Low are mapped to black, any values, greater or equal to High are mapped to white, and any values inbetween have Gamma applied as an exponent.")); } 

	bool IsAutoLevel() const { return LevelsExpressionType == ELevelsExpressionType::AutoLowHigh; }
};

/////////////////////////////////////////////////////////////////
/// Histogram scan
/////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_HistogramScan : public UTG_Expression
{
	GENERATED_BODY()

	TSharedPtr<FLevels> 				LevelsControl;

public:

	TG_DECLARE_EXPRESSION(TG_Category::Adjustment);
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

	// The input image to adjust the levels for
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Input", PinDisplayName = "", MD_HistogramLuminance))
	FTG_Texture							Input;

	// Drives the position of the histogram of the input image
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", MD_ScalarEditor))
	float 								Position = 0.5f;

	// Drives the contrast of the histogram of the input image
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", MD_ScalarEditor))
	float 								Contrast = 0.0f; 

	// The output image filtered result of the Levels operator 
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;

	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Lets you drive the contrast and position of the histogram. Input must be a grayscale image.")); } 

};

/////////////////////////////////////////////////////////////////
/// Histogram range
/////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_HistogramRange : public UTG_Expression
{
	GENERATED_BODY()

	TSharedPtr<FLevels> 				LevelsControl;

public:

	TG_DECLARE_EXPRESSION(TG_Category::Adjustment);
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

	// The input image to adjust the levels for
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Input", PinDisplayName = "", MD_HistogramLuminance))
	FTG_Texture							Input;

	// Drives the center of the levels out
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", MD_ScalarEditor))
	float 								Position = 0.5f;

	// Drives the range of the levels out
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", MD_ScalarEditor))
	float 								Range = 1.0f; 

	// The output image filtered result of the Levels operator 
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;

	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Reduce and/or move the range of a grayscale input. Can be used to remap grayscale images.")); } 

};

/////////////////////////////////////////////////////////////////
/// Histogram select
/////////////////////////////////////////////////////////////////
#if 0 /// TODO
UCLASS(MinimalAPI)
class UTG_Expression_HistogramSelect : public UTG_Expression
{
	GENERATED_BODY()

	TSharedPtr<FLevels> 				LevelsControl;

public:

	TG_DECLARE_EXPRESSION(TG_Category::Adjustment);
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

	// The input image to adjust the levels for
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Input", PinDisplayName = "", MD_HistogramLuminance))
	FTG_Texture							Input;

	// Drives the position of the histogram of the input image
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", MD_ScalarEditor))
	float 								Position = 0.5f;

	// Drives the range of the levels out
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", MD_ScalarEditor))
	float 								Range = 1.0f; 

	// Drives the contrast of the histogram of the input image
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", MD_ScalarEditor))
	float 								Contrast = 0.5f; 

	// The output image filtered result of the Levels operator 
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;

	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Reduce and/or move the range of a grayscale input. Can be used to remap grayscale images.")); } 

};
#endif 

#undef UE_API
