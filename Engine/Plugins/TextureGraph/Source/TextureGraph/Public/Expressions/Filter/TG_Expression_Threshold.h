// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"

#include "TG_Expression_Threshold.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_Threshold : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Adjustment);
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

	// The input image to adjust the levels for
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Input", PinDisplayName = "", MD_HistogramLuminance))
	FTG_Texture							Input;

	// The min value to be used. Can be an image (Defaults to 0/black)
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", PinDisplayName = "Threshold", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1", MD_ScalarEditor))
	float								Threshold;

	// The output image which is a single channel grayscale version of the input image 
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;

	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Apply a threshold filter to the input image, output is white if the luminance of the pixel is greater or equal of the threshold, black otherwise")); } 
};

#undef UE_API
