// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Expressions/TG_Expression_MaterialBase.h"
#include "Transform/Expressions/T_Filter.h"

#include "TG_Expression_Warp.generated.h"

#define UE_API TEXTUREGRAPH_API

//////////////////////////////////////////////////////////////////////////
/// Base warp class with common options
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Warp : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Filter);

	/// The type of warp that we want on the input image	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", RegenPinsOnChange))
	TEnumAsByte<EWarp::Type> Type = EWarp::Directional;

	// What is the intensity of the warp. Warp of 1 is 10% (1/10th) of the width of the input image
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = 0.0, ClampMax = 20, UIMin = 0.0, UIMax = 1, EditConditionHides))
	float Intensity = 1;

	// Angle of the directional warp 0 - 360 degrees
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = 0.0, ClampMax = 360, UIMin = 0.0, UIMax = 360, EditConditionHides))
	float Angle = 0;

	// Phase X for the Sine wave warp (for U coordinate in UV space)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = -1, ClampMax = 1, UIMin = -1, UIMax = 1, EditConditionHides))
	float PhaseU = 0.0f;

	// Phase Y for the Sine wave warp (for V coordinate in UV space)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = -1, ClampMax = 1, UIMin = -1, UIMax = 1, EditConditionHides))
	float PhaseV = 0.0f;

	// The input texture to apply the warp
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Texture Input;

	// The mask used for the warp effect
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Texture Mask;

	// The output texture having blurred effect
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture Output;

public:
#if WITH_EDITOR
	// Used to implement EditCondition logic for both Node UI and Details View
	UE_API virtual bool						CanEditChange(const FProperty* InProperty) const override;
#endif

	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Evalute different kinds of warp depending on the Mask and Intensity settings.")); }
	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;
};

#undef UE_API
