// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TG_Expression_InputParam.h"
#include "TG_Expression_Bool.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_Bool : public UTG_Expression_InputParam
{
	GENERATED_BODY()
	TG_DECLARE_INPUT_PARAM_EXPRESSION(TG_Category::Input);

public:

	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_InputParam", PinDisplayName = ""))
    bool Bool = false;
    
	// The output of the node, which is the color value
    UPROPERTY(meta = (TGType = "TG_Output", PinDisplayName = ""))
	bool ValueOut = false;
	
	virtual FTG_Name GetDefaultName() const override { return TEXT("Bool"); }
	UE_API virtual void SetTitleName(FName NewName) override;
	UE_API virtual FName GetTitleName() const override;
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Makes a boolean input parameter. It is automatically exposed as a graph parameter.")); } 
};

#undef UE_API
