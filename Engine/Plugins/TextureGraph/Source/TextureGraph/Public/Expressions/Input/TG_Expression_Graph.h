// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"

#include "TG_Expression_Graph.generated.h"

#define UE_API TEXTUREGRAPH_API

class UTextureGraph;

//////////////////////////////////////////////////////////////////////////
//// Generic subgraph
//////////////////////////////////////////////////////////////////////////

UCLASS(MinimalAPI, Hidden)
class UTG_Expression_Graph : public UTG_Expression
{
	GENERATED_BODY()
public:
	TG_DECLARE_DYNAMIC_EXPRESSION(TG_Category::Input);
	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

protected:
	
	mutable TArray<FTG_Id> InParamIds;
	mutable TArray<FTG_Id> OutParamIds;

	virtual UTG_Graph* GetGraph() const { return nullptr; }
	UE_API void NotifyGraphChanged();

	UE_API virtual void SetupAndEvaluate(FTG_EvaluationContext* InContext) override;
};

//////////////////////////////////////////////////////////////////////////
// TextureGraph Expression
//////////////////////////////////////////////////////////////////////////

UCLASS(MinimalAPI)
class UTG_Expression_TextureGraph : public UTG_Expression_Graph
{
	GENERATED_BODY()
public:

	UE_API UTG_Expression_TextureGraph();
	UE_API virtual ~UTG_Expression_TextureGraph();
	

#if WITH_EDITOR
protected:
	// Only in editor mode, in the case the TextureGraph is assigned from Details panel we need to remember the previous version
	TWeakObjectPtr<UTextureGraph>	PreEditTextureGraph; 
public:
	UE_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
#endif

	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;
	UE_API virtual bool Validate(MixUpdateCyclePtr	Cycle) override;

	UPROPERTY(EditAnywhere, Setter, Category = NoCategory, meta = (TGType = "TG_Setting"))
	TObjectPtr<UTextureGraph>	TextureGraph;				// The TextureGraph to use for the sub-graph

	UE_API void SetTextureGraph(UTextureGraph* InTextureGraph);

	UE_API virtual bool CanHandleAsset(UObject* Asset) override;
	UE_API virtual void SetAsset(UObject* Asset) override;
	
protected:
	UE_API virtual void Initialize() override;
	UE_API virtual void CopyVarGeneric(const FTG_Argument& Arg, FTG_Var* InVar, bool CopyVarToArg) override;

	UE_API virtual UTG_Graph* GetGraph() const override;

	UE_API void OnTextureGraphPreSave(UObject* Object, FObjectPreSaveContext SaveContext);
private:
	UPROPERTY(Transient)
	TObjectPtr<UTG_Graph>	RuntimeGraph;		// The runtime graph instance to use for the sub-graph

	FDelegateHandle PreSaveHandle;				// delegate handle for global ObjectPreSave event 
	
	UE_API bool CheckDependencies(const UTextureGraph* InTextureGraph) const;
	UE_API void SetTextureGraphInternal(UTextureGraph* InTextureGraph);

public:

	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Encapsulates another graph as a node with the graph's input and output parameters exposed as the node input and output pins.")); }

};

#undef UE_API
