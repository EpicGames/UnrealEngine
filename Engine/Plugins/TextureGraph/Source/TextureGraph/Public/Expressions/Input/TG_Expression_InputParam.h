// Copyright Epic Games, Inc. All Rights Reserve

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"

#include "TG_Expression_InputParam.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI, Abstract)
class UTG_Expression_InputParam : public UTG_Expression
{
	GENERATED_BODY()

protected:
	// these 2 methods need to be implemented to provide concrete signatures for derived InputParam expression classes
	UE_API virtual FTG_SignaturePtr BuildInputParameterSignature() const;
	UE_API virtual FTG_SignaturePtr BuildInputConstantSignature() const;
	virtual bool ShouldShowSettings() const override { return false; }
public:

#if WITH_EDITOR
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	// Node can be converted from Param to Constant
	UPROPERTY(Setter)
	bool bIsConstant = false;
	UE_API void SetbIsConstant(bool InIsConstant);
	UE_API void ToggleIsConstant();

};


#define TG_DECLARE_INPUT_PARAM_EXPRESSION(Category) \
	public:	virtual FTG_SignaturePtr GetSignature() const override { if (bIsConstant) { \
			static FTG_SignaturePtr ConstantSignature = BuildInputConstantSignature(); return ConstantSignature; \
		} else { \
			static FTG_SignaturePtr ParameterSignature = BuildInputParameterSignature(); return ParameterSignature; \
		} } \
	public: virtual FName GetCategory() const override { return Category; }

#undef UE_API
