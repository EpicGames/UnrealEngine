// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression_MaterialBase.h"
#include "Materials/Material.h"
#include "TG_Texture.h"
#include "TG_Material.h"

#include "TG_Expression_Material.generated.h"

#define UE_API TEXTUREGRAPH_API

typedef std::weak_ptr<class Job>		JobPtrW;

UCLASS(MinimalAPI)
class UTG_Expression_Material : public UTG_Expression_MaterialBase
{
	GENERATED_BODY()

public:
	UE_API UTG_Expression_Material();
	UE_API virtual ~UTG_Expression_Material();
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
#endif

	UE_API virtual void PostLoad() override;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "Material field has been deprecated.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use the InputMaterial to specify the Material asset referenced"))
	TObjectPtr<UMaterialInterface> Material_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	// The input material referenced by this Material node
	UPROPERTY(EditAnywhere, Setter, Category = NoCategory, meta = (TGType = "TG_InputParam", DisplayName = "Material", PinDisplayName = "Material"))
	FTG_Material InputMaterial;
	UE_API void SetInputMaterial(const FTG_Material& InMaterial);

	// The Material attribute identifier among all the attributes of the material that is rendered in the output
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Category = NoCategory, meta = (TGType = "TG_Setting", GetOptions = "GetRenderAttributeOptions"))
	FName RenderedAttribute;
	UE_API void SetRenderedAttribute(FName InRenderedAttribute);

	// THe list of Rendered attribute options available 
	UFUNCTION(CallInEditor)
	UE_API TArray<FName> GetRenderAttributeOptions() const;

	// When Material references VT, can specify the number of warmup frames.
	// Default value is 0 meaning that the CVar <TG.VirtualTexture.NumWarmupFrames> is used instead 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = 0, ClampMax = 32, PinDisplayName = "Num Warmup Frames") )
	int32 NumWarmupFrames = 0;
	virtual int32 GetNumVirtualTextureWarmupFrames() const override { return NumWarmupFrames; }

	UE_API virtual bool CanHandleAsset(UObject* Asset) override;
	UE_API virtual void SetAsset(UObject* Asset) override;
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Renders a material into a quad and makes it available. It is automatically exposed as a graph input parameter.")); } 

protected:
	// Transient and per instance Data, recreated on every new instance from the reference material
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UMaterialInterface> MaterialCopy = nullptr;

	// As the "Material" or the "InputMaterial" properties change
	// we update the current active "MaterialCopy" and pass on internally as the active "GetMaterial()"
	UE_API virtual void SetMaterialInternal(UMaterialInterface* InMaterial) override;

	// Title name is still used mostly for legacy but not exposed anymore in the details.
	// This is changed on the node itself and then call SetTitleName and rename the InnputMaterial Alias name
	UPROPERTY()
	FName TitleName = TEXT("Material");

	UE_API virtual void Initialize() override;

public:
	
	UE_API virtual void SetTitleName(FName NewName) override;
	UE_API virtual FName GetTitleName() const override;

	virtual FName GetCategory() const override { return TG_Category::Input;}
	
protected:
	virtual TObjectPtr<UMaterialInterface> GetMaterial() const override { return MaterialCopy;};
	UE_API virtual EDrawMaterialAttributeTarget GetRenderedAttributeId()  override;

#if WITH_EDITOR // Listener for referenced material being saved to update the integration in TG 
	FDelegateHandle PreSaveHandle; 
	UE_API void OnReferencedObjectPreSave(UObject* Object, FObjectPreSaveContext SaveContext);
#endif

};

#undef UE_API
