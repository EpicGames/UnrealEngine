// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression_MaterialBase.h"
#include "Materials/MaterialFunction.h"
#include "TG_Texture.h"

#include "TG_Expression_MaterialFunction.generated.h"

#define UE_API TEXTUREGRAPH_API


UCLASS(MinimalAPI)
class UTG_Expression_MaterialFunction : public UTG_Expression_MaterialBase
{
	GENERATED_BODY()
public:
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
#endif
	
	// The material function to employ for rendering
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Category = NoCategory, meta = (TGType = "TG_Setting", TGPinNotConnectable))
	TObjectPtr<UMaterialFunctionInterface> MaterialFunction;
	UE_API void SetMaterialFunction(UMaterialFunctionInterface* InMaterialFunction);

	// When MaterialFunction references VT, can specify the number of warmup frames.
	// Default value is 0 meaning that the CVar <TG.VirtualTexture.NumWarmupFrames> is used instead 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = 0, ClampMax = 32, PinDisplayName = "Num Warmup Frames") )
	int32 NumWarmupFrames = 0;
	virtual int32 GetNumVirtualTextureWarmupFrames() const override { return NumWarmupFrames; }

	UE_API virtual bool CanHandleAsset(UObject* Asset) override;
	UE_API virtual void SetAsset(UObject* Asset) override;

	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Renders a material function output into a quad and makes it available for the texture graph.")); } 

protected:
	UE_API virtual void Initialize() override;

	UE_API UMaterialInterface* CreateMaterialReference();
	TObjectPtr<UMaterialInterface> ReferenceMaterial = nullptr;
	virtual TObjectPtr<UMaterialInterface> GetMaterial() const { return ReferenceMaterial; }
	virtual EDrawMaterialAttributeTarget GetRenderedAttributeId() { return EDrawMaterialAttributeTarget::Emissive; }

private:
	UE_API void SetMaterialFunctionInternal(UMaterialFunctionInterface* InMaterialFunction);

public:
	
	virtual FTG_Name GetDefaultName() const override { return TEXT("MaterialFunction");}
	UE_API virtual bool CanRenameTitle() const override;

	virtual FName GetCategory() const override { return TG_Category::Input; }

};

#undef UE_API
