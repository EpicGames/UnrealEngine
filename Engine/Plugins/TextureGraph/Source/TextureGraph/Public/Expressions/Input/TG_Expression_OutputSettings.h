// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_OutputSettings.h"

#include "TG_Expression_OutputSettings.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_OutputSettings : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Input);
	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	UE_API virtual bool Validate(MixUpdateCyclePtr	Cycle) override;
	
	// The output of the node
	UPROPERTY(meta = (TGType = "TG_Output", PinDisplayName = ""))
		FTG_OutputSettings Output;

	// Input is the one that recieves the Settings from another node
	UPROPERTY(meta = (TGType = "TG_Input", HideChildProperties))
		FTG_OutputSettings Input;

	//Settings will overwrite the Input values from another node 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_InputParam", PinDisplayName = ""))
		FTG_OutputSettings Settings;

	class ULayerChannel* Channel;
	virtual FTG_Name GetDefaultName() const override { return TEXT("Settings"); }
	UE_API virtual void SetTitleName(FName NewName) override;
	UE_API virtual FName GetTitleName() const override;
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Makes the settings for the output.")); }

private:
	FTG_OutputSettings PreviousInput;
};

#undef UE_API
