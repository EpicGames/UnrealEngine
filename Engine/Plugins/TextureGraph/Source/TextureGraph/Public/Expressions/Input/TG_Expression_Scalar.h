// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TG_Expression_InputParam.h"

#include "TG_Expression_Scalar.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_Scalar : public UTG_Expression_InputParam
{
	GENERATED_BODY()
	TG_DECLARE_INPUT_PARAM_EXPRESSION(TG_Category::Input);

public:

	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	// The floating point constant
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_InputParam", PinDisplayName = ""))
		float Scalar = 1.0f;

	// The output of the node, which is the floating point constant
	UPROPERTY(meta = (TGType = "TG_Output", PinDisplayName = ""))
		float ValueOut = 1.0f;

	virtual FTG_Name GetDefaultName() const override { return TEXT("Scalar"); }
	UE_API virtual void SetTitleName(FName NewName) override;
	UE_API virtual FName GetTitleName() const override;
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Makes a single floating point value available. It is automatically exposed as a graph input parameter.")); } 
};

#undef UE_API
