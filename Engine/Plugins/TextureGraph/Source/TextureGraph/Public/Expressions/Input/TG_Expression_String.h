// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TG_Expression_InputParam.h"
#include "TG_Expression_String.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_String : public UTG_Expression_InputParam
{
	GENERATED_BODY()
	TG_DECLARE_INPUT_PARAM_EXPRESSION(TG_Category::Input);

public:

	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	// The string constant
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_InputParam", PinDisplayName = ""))
		FString String;

	// The output of the node, which is the string constant
	UPROPERTY(meta = (TGType = "TG_Output", PinDisplayName = ""))
		FString ValueOut;

	virtual FTG_Name GetDefaultName() const override { return TEXT("String"); }
	UE_API virtual void SetTitleName(FName NewName) override;
	UE_API virtual FName GetTitleName() const override;
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("String parameter that can be used as input in various places.")); } 
};

#undef UE_API
