// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TG_Expression_InputParam.h"
#include "Engine/Texture.h"
#include "TG_Texture.h"

#include "TG_Expression_Texture.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_Texture : public UTG_Expression_InputParam
{
	GENERATED_BODY()
	TG_DECLARE_INPUT_PARAM_EXPRESSION(TG_Category::Input);

public:

	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	UE_API virtual bool Validate(MixUpdateCyclePtr	Cycle) override;
	UE_API void SetSourceInternal();

	// The output of the node, which is the loaded texture asset
	UPROPERTY(meta = (TGType = "TG_Output", PinDisplayName = "", HideInnerPropertiesInNode))
	FTG_Texture Output;
	FString OutputPath;

	// The source asset to be used to generate the Output
	UPROPERTY(EditAnywhere, Setter, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Input", TGPinNotConnectable) )
	TObjectPtr<UTexture> Source;

	// The input texture that was loaded from the asset
	UPROPERTY(EditAnywhere, Setter, Category = NoCategory, meta = (TGType = "TG_InputParam"))
	FTG_Texture Texture;

	UE_API void SetSource(UTexture* InSource);
	UE_API void SetTextureInternal();
	UE_API void SetTexture(const FTG_Texture& InTexture);
	
#if WITH_EDITOR
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	class ULayerChannel* Channel;
	virtual FTG_Name GetDefaultName() const override { return TEXT("Texture"); }
	UE_API virtual void SetTitleName(FName NewName) override;
	UE_API virtual FName GetTitleName() const override;
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Makes an existing texture asset available. It is automatically exposed as a graph input parameter.")); }
	virtual bool IgnoreInputTextureOnUndo() const override { return false; }

	UE_API virtual void SetAsset(UObject* Asset) override;
	UE_API virtual bool CanHandleAsset(UObject* Asset) override;
};

#undef UE_API
