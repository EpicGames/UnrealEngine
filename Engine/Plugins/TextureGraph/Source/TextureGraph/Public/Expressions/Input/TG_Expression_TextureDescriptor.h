// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TG_Expression_InputParam.h"
#include "TG_Expression_TextureDescriptor.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_TextureDescriptor : public UTG_Expression_InputParam
{
	GENERATED_BODY()
	TG_DECLARE_INPUT_PARAM_EXPRESSION(TG_Category::Input);

public:

	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	/// The width of the texture. 0 and negative values mean "Auto"
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Input"))
    int32 Width = (int32)EResolution::Resolution2048;
    
	/// The height of the texture. 0 and negative values mean "Auto"
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Input"))
    int32 Height = (int32)EResolution::Resolution2048;
    
	/// The height of the texture. 0 and negative values mean "Auto"
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Input", PinDisplayName = "sRGB"))
    bool bIsSRGB = false;
    
	/// The height of the texture. 0 and negative values mean "Auto"
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting"))
    ETG_TextureFormat Format = ETG_TextureFormat::Auto;
    
	// The output of the node, which is the color value
    UPROPERTY(meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_TextureDescriptor Output;
	
	virtual FTG_Name GetDefaultName() const override { return TEXT("Texture Settings"); }
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Allows the user to customize texture settings like resolution, format etc.")); } 
};

#undef UE_API
