// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TG_Expression_InputParam.h"
#include "Engine/Texture.h"
#include "TG_Texture.h"

#include "TG_Expression_TexturePath.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_TexturePath : public UTG_Expression_InputParam
{
	GENERATED_BODY()
	TG_DECLARE_INPUT_PARAM_EXPRESSION(TG_Category::Input);

protected:
	// Special case for TexturePath Constant signature, we want to keep the Path Input connectable in that case
	// so do this in the override version of BuildInputConstantSignature()
	UE_API virtual FTG_SignaturePtr BuildInputConstantSignature() const override;

	// Validate the input path, returning the actual path to use
	// empty if the input path is NOT valid
	UE_API bool ValidateInputPath(FString& ValidatedPath) const;

	UE_API void ReportError(MixUpdateCyclePtr Cycle);
	static UE_API TiledBlobPtr LoadStaticImage(FTG_EvaluationContext* InContext, const FString& Filename, BufferDescriptor* DesiredDesc);
public:

	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;
	UE_API virtual bool Validate(MixUpdateCyclePtr	Cycle) override;
	
	// All the output textures from this node. Note that it outputs an array of textures.
	UPROPERTY(meta = (TGType = "TG_Output", PinDisplayName = "", HideInnerPropertiesInNode))
	FTG_VariantArray Output;
	FString OutputPath;

	// Input file path of the texture. If the path is a directory then all files from that directory are loaded
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_InputParam", NoResetToDefault) )
	FString Path;

	class ULayerChannel* Channel;
	virtual FTG_Name GetDefaultName() const override { return TEXT("TexturePath"); }
	UE_API virtual void SetTitleName(FName NewName) override;
	UE_API virtual FName GetTitleName() const override;
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Loads texture(s) from a path. If the path is a single file name then one texture is loaded. If it's a directory then all textures are loaded from that directory.")); }
};

#undef UE_API
