// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TG_Expression_InputParam.h"

#include "TG_Expression_Vector.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_Vector : public UTG_Expression_InputParam
{
	GENERATED_BODY()
	TG_DECLARE_INPUT_PARAM_EXPRESSION(TG_Category::Input);

public:

	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	// The 4 dimensional vector X, Y, Z and W.
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_InputParam", PinDisplayName = ""))
    FVector4f Vector = FVector4f(0);
    
	// The output of the node, which is the color value
    UPROPERTY(meta = (TGType = "TG_Output", PinDisplayName = ""))
	FVector4f ValueOut = FVector4f(0);
	
	virtual FTG_Name GetDefaultName() const override { return TEXT("Vector"); }
	UE_API virtual void SetTitleName(FName NewName) override;
	UE_API virtual FName GetTitleName() const override;
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Makes an XYZ vector available. It is automatically exposed as a graph input parameter.")); } 
};

#undef UE_API
