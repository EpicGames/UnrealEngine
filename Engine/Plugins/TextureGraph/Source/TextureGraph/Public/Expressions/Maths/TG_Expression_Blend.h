// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "Engine/Texture.h"
#include "TG_Texture.h"
#include "2D/BlendModes.h"
#include "TG_Expression_Blend.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_Blend : public UTG_Expression
{
	GENERATED_BODY()
public:
	TG_DECLARE_EXPRESSION(TG_Category::Maths)
	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	// Blend mode determine how inputs texture mix together.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting"))
	TEnumAsByte<EBlendModes::Type> BlendMode = EBlendModes::Normal;

	// The opacity to be used in conjunction with the mask. The final blend value = 'Mask[RedChannel] * Opacity'
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1"))
	float Opacity = 1.0;

	// If true, it will ignore the alpha channel blending and keep the background alpha.  
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", DisplayName = "Ignore Alpha"))
	bool bIgnoreAlpha = true;

	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", DisplayName = "Clamp"))
	bool bClamp = true;
	
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture Output;

	// The first input (Foreground) to the blend function 
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Texture Foreground;

	// The second input (Background) to the blend function
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Texture Background;
	
	// The mask used to calculate the final blend value which is, 'Mask[RedChannel] * Opacity'. Default value is white. 
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Texture Mask;
	

	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Combines two inputs based on a chosen blend type. The second input is multiplied with the Opacity, and if available the Mask as well, before applying the blend.")); } 
};

#undef UE_API
