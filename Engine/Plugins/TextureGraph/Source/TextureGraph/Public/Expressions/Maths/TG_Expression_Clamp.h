// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression_Variant.h"
#include "TG_Texture.h"

#include "TG_Expression_Clamp.generated.h"

#define UE_API TEXTUREGRAPH_API

//////////////////////////////////////////////////////////////////////////
/// Clamp
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Clamp : public UTG_Expression_Variant
{
	GENERATED_BODY()

public:
	// The input image to clamp
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Variant							Input;

	// The desired min value (defaults to 0/black)
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "Min"))
	FTG_Variant							MinValue = 0;

	// The desired max value (defaults to 1/white)
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "Max"))
	FTG_Variant							MaxValue = 1;

	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Clamps the input to the given minimum and maximum values, Min \u2264 A \u2264 Max.")); } 
	virtual FName						GetCategory() const override { return TG_Category::Maths; } 

protected:
	virtual std::vector<FTG_Variant>	GetEvaluateArgs() override { return std::vector<FTG_Variant>({ Input, MinValue, MaxValue }); }
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
	virtual bool						ErrorCheckInputTextures() const { return false; }
};

//////////////////////////////////////////////////////////////////////////
/// Smooth step
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_SmoothStep : public UTG_Expression_Variant
{
	GENERATED_BODY()

public:

	// The angle in radians to run the trigonometric function on
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Variant							Input = 0;

	UPROPERTY(meta = (TGType = "TG_Input", UIMin = "0", UIMax = "100"))
	FTG_Variant							Min = 0;

	UPROPERTY(meta = (TGType = "TG_Input", UIMin = "0", UIMax = "100"))
	FTG_Variant							Max = 1;

	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Returns a smooth Hermite interpolation between 0 and 1, if x is in the range [min, max].")); } 
	virtual FName						GetCategory() const override { return TG_Category::Maths; } 

protected:
	virtual std::vector<FTG_Variant>	GetEvaluateArgs() override { return std::vector<FTG_Variant>({ Input, Min, Max}); }
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
	virtual bool						ErrorCheckInputTextures() const { return false; }
};

//////////////////////////////////////////////////////////////////////////
/// Min
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Min : public UTG_Expression_Variant
{
	GENERATED_BODY()

public:
	// The first input for Output = min(Input1, Input2)
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "A"))
		FTG_Variant						Input1;

	// The second input for Output = min(Input1, Input2)
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "B"))
		FTG_Variant						Input2;

	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Selects the minimum of the two inputs.")); } 
	virtual FName						GetCategory() const override { return TG_Category::Maths; } 

protected:
	virtual std::vector<FTG_Variant>	GetEvaluateArgs() override { return std::vector<FTG_Variant>({ Input1, Input2 }); }
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
};

//////////////////////////////////////////////////////////////////////////
/// Max
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Max : public UTG_Expression_Variant
{
	GENERATED_BODY()

public:
	// The first input for Output = max(Input1, Input2)
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "A"))
		FTG_Variant						Input1;

	// The second input for Output = max(Input1, Input2)
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "B"))
		FTG_Variant						Input2;

	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Selects the maximum of the two inputs.")); } 
	virtual FName						GetCategory() const override { return TG_Category::Maths; } 

protected:
	virtual std::vector<FTG_Variant>	GetEvaluateArgs() override { return std::vector<FTG_Variant>({ Input1, Input2 }); }
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
};

#undef UE_API
