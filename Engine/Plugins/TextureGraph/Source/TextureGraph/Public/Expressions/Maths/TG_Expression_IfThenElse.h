// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression_Variant.h"
#include "TG_Texture.h"
#include "Transform/Expressions/T_Maths_TwoInputs.h"

#include "TG_Expression_IfThenElse.generated.h"

#define UE_API TEXTUREGRAPH_API

//////////////////////////////////////////////////////////////////////////
/// Expression
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_IfThenElse : public UTG_Expression_Variant
{
	GENERATED_BODY()

public:
	// The LHS value for the comparison (LHS <operator> RHS)
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "A"))
	FTG_Variant							LHS;

	// The type of operator in LHS <operator> RHS
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting"))
	TEnumAsByte<EIfThenElseOperator>	Operator = EIfThenElseOperator::GT;

	// The RHS value for the comparison (LHS <operator> RHS)
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "B"))
	FTG_Variant							RHS;

	// The value to return if the comparison is successful if (<true>) <Then> else <Else>
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Variant							Then;

	// The value to return if the comparison is unsuccessful if (<false>) <Then> else <Else>
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Variant							Else;

	// The type of comparison to use. Individual = IndividualComponent of the components are individually passed through the comparison. AllComponents = All components must succeed check to trigger the Then part of the statement. Magnitude = Grayscale value of the color is used in comparison 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting"))
	TEnumAsByte<EIfThenElseType>		ComparisonType = EIfThenElseType::IndividualComponent;

	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Compares input A with B and depending on the result selects either the Then input or the Else input.")); } 
	virtual FName						GetCategory() const override { return TG_Category::Maths; } 

protected:
	virtual std::vector<FTG_Variant>	GetEvaluateArgs() override { return std::vector<FTG_Variant>({ LHS, RHS, Then, Else }); }
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FVector4f					EvaluateVector_WithValue(FTG_EvaluationContext* InContext, const FVector4f* const Value, size_t Count);
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
};

#undef UE_API
