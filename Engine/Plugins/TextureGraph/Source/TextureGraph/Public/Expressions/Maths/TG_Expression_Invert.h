// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "FxMat/FxMaterial.h"

#include "TG_Expression_Invert.generated.h"

#define UE_API TEXTUREGRAPH_API

//////////////////////////////////////////////////////////////////////////
/// Expression
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Invert : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Maths);
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

	// The max value to use. This is used as MaxValue - Input for every pixel in every channel to calculate the output
	// NB: Hiding this property for the time being as requested
	//UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1", PinDisplayName = "Max"))
	float								MaxValue = 1;

	// The input texture to be inverted
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Texture							Input;

	// Whether to invert the Alpha channel as well?
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", PinNotConnectable = true))
	bool								IncludeAlpha = false;

	// Saturate/clamp values in the [0, 1] range or not
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", PinNotConnectable = true))
	bool								Clamp = false;

	// The inverted output. This is calculated as MaxValue - Input for every pixel in every channel
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;
	
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Inverts the input, i.e. computes one minus the pixel value. Alpha is only inverted if the flag is checked.")); } 
};

#undef UE_API
