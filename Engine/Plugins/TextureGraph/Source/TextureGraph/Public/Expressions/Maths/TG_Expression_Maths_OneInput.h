// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression_Variant.h"
#include "Transform/Expressions/T_Maths_OneInput.h"

#include "TG_Expression_Maths_OneInput.generated.h"

#define UE_API TEXTUREGRAPH_API

//////////////////////////////////////////////////////////////////////////
/// Trigonometry functions
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Trigonometry : public UTG_Expression_Variant
{
	GENERATED_BODY()

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
	virtual std::vector<FTG_Variant>	GetEvaluateArgs() override { return std::vector<FTG_Variant>({ Angle }); }

public:
	// The angle in radians to run the trigonometric function on
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Variant							Angle;

	// What trigonometric function to apply at the angle
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", RegenPinsOnChange))
	ETrigFunction						Function = ETrigFunction::Sin;

	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("A variety of Trigonometric functions.")); } 
	virtual FName						GetCategory() const override { return TG_Category::Maths; } 
};

//////////////////////////////////////////////////////////////////////////
/// Generic single input math function
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI, Abstract)
class UTG_Expression_OneInput : public UTG_Expression_Variant
{
	GENERATED_BODY()

protected:
	virtual std::vector<FTG_Variant>	GetEvaluateArgs() override { return std::vector<FTG_Variant>({ Input }); }

public:
	// The input to the node
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Variant							Input;

	virtual FName						GetCategory() const override { return TG_Category::Maths; } 
};

//////////////////////////////////////////////////////////////////////////
/// Absolute value of a floating point number
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Abs : public UTG_Expression_OneInput
{
	GENERATED_BODY()

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;

public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Absolute value of a floating point number.")); } 
};

//////////////////////////////////////////////////////////////////////////
/// Square root of a floating point number
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Sqrt : public UTG_Expression_OneInput
{
	GENERATED_BODY()

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;

public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Square root of a number.")); } 
};

//////////////////////////////////////////////////////////////////////////
/// Square of a floating point number
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Square : public UTG_Expression_OneInput
{
	GENERATED_BODY()

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;

public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Square of a number.")); } 
};

//////////////////////////////////////////////////////////////////////////
/// Cube of a floating point number
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Cube : public UTG_Expression_OneInput
{
	GENERATED_BODY()

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;

public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Cube of a number.")); } 
};

//////////////////////////////////////////////////////////////////////////
/// Cube root of a floating point number
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Cbrt : public UTG_Expression_OneInput
{
	GENERATED_BODY()

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;

public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Cubic root of a number.")); } 
};

//////////////////////////////////////////////////////////////////////////
/// Base-e exponential function
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Exp : public UTG_Expression_OneInput
{
	GENERATED_BODY()

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;

public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Base-e exponential function [Exp(Input)].")); } 
};

//////////////////////////////////////////////////////////////////////////
/// Log base-2
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Log2 : public UTG_Expression_OneInput
{
	GENERATED_BODY()

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;

public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Log base-2 of input [Log2(Input)].")); } 
};

//////////////////////////////////////////////////////////////////////////
/// Log base-10
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Log10 : public UTG_Expression_OneInput
{
	GENERATED_BODY()

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;

public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Log base-10 of input [Log10(Input)].")); } 
};

//////////////////////////////////////////////////////////////////////////
/// Log base-e
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Log : public UTG_Expression_OneInput
{
	GENERATED_BODY()

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;

public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Log natural (Log base-e) of input [Log(Input)].")); } 
};

//////////////////////////////////////////////////////////////////////////
/// Floor
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Floor : public UTG_Expression_OneInput
{
	GENERATED_BODY()

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;

public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Floor of a number.")); } 
};

//////////////////////////////////////////////////////////////////////////
/// Ceiling
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Ceil : public UTG_Expression_OneInput
{
	GENERATED_BODY()

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;

public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Ceiling of a number.")); } 
};

//////////////////////////////////////////////////////////////////////////
/// Round
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Round : public UTG_Expression_OneInput
{
	GENERATED_BODY()

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;

public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Round a number.")); } 
};

#undef UE_API
