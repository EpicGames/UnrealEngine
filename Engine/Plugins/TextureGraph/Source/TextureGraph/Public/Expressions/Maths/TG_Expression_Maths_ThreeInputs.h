// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression_Variant.h"

#include "TG_Expression_Maths_ThreeInputs.generated.h"

#define UE_API TEXTUREGRAPH_API

//////////////////////////////////////////////////////////////////////////
/// MAD
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_MAD : public UTG_Expression_Variant
{
public:
	GENERATED_BODY()

	// First input to the MAD operation [Input1 * Input2 + Input3]
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "A"))
	FTG_Variant							Input1 = 0;

	// Second input to the MAD operation [Input1 * Input2 + Input3]
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "B"))
	FTG_Variant							Input2 = 0;

	// Third input to the MAD operation [Input1 * Input2 + Input3]
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "C"))
	FTG_Variant							Input3 = 0;

public:
	virtual FName						GetCategory() const override { return TG_Category::Maths; } 
	virtual FTG_Name					GetDefaultName() const override { return TEXT("MultiplyAdd");}
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Multiplies the first two inputs and then adds the third input, A \u2192 B \u002b C.")); } 

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
	virtual std::vector<FTG_Variant>	GetEvaluateArgs() override { return std::vector<FTG_Variant>({ Input1, Input2, Input3 }); }
};

//////////////////////////////////////////////////////////////////////////
/// Lerp
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Lerp : public UTG_Expression_Variant
{
public:
	GENERATED_BODY()
	
	// First input to lerp
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "X"))
	FTG_Variant							Input1 = 0;

	// Second input to lerp
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "Y"))
	FTG_Variant							Input2 = 0;

	// A value that linearly interpolates between X and Y. This can be an image or a Scalar node connected for easier control
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "S"))
	FTG_Variant							LerpValue = 0;

public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Calculates Lerp(X, Y, S).")); } 
	virtual FName						GetCategory() const override { return TG_Category::Maths; } 

protected:
	virtual std::vector<FTG_Variant>	GetEvaluateArgs() override { return std::vector<FTG_Variant>({ Input1, Input2, LerpValue }); }
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
};

#undef UE_API
