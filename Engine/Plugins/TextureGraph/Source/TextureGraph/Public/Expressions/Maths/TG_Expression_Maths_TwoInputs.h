// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression_Variant.h"

#include "TG_Expression_Maths_TwoInputs.generated.h"

#define UE_API TEXTUREGRAPH_API

//////////////////////////////////////////////////////////////////////////
/// Basic math op expression
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI, Abstract)
class UTG_Expression_BasicMath_Op : public UTG_Expression_Variant
{
	GENERATED_BODY()

public:
	// First input to the math operation [Input1 <operation> Input2]
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "A"))
	FTG_Variant							Input1 = 0;

	// Second input to the math operation [Input1 <operation> Input2]
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "B"))
	FTG_Variant							Input2 = 0;

	virtual FName						GetCategory() const override { return TG_Category::Maths; } 

protected:
	virtual std::vector<FTG_Variant>	GetEvaluateArgs() override { return std::vector<FTG_Variant>({ Input1, Input2 }); }
};

//////////////////////////////////////////////////////////////////////////
/// Multiply
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Multiply : public UTG_Expression_BasicMath_Op
{
	GENERATED_BODY()
	
public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Multiplies the two inputs, A \u2192 B.")); } 

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
};

//////////////////////////////////////////////////////////////////////////
/// Divide
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Divide : public UTG_Expression_BasicMath_Op
{
	GENERATED_BODY()
	
public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Divides the two inputs, A \u00f7 B.")); } 

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
};

//////////////////////////////////////////////////////////////////////////
/// Add
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Add : public UTG_Expression_BasicMath_Op
{
	GENERATED_BODY()
	
public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Adds up the two inputs, A \u002b B.")); } 

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
};

//////////////////////////////////////////////////////////////////////////
/// Subtract
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Subtract : public UTG_Expression_BasicMath_Op
{
	GENERATED_BODY()
	
public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Subtracts the two inputs, A \u2212 B.")); } 

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
};

//////////////////////////////////////////////////////////////////////////
/// Dot product of two vectors
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Dot : public UTG_Expression_BasicMath_Op
{
	GENERATED_BODY()
	
public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Dot product of inputs. A <dot> B.")); } 

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FVector4f					EvaluateVector_WithValue(FTG_EvaluationContext* InContext, const FVector4f* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;

	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

	UE_API virtual FTG_Variant::EType			EvalExpressionCommonOutputVariantType() const override; // specify the logic to decide the output variant type
};

//////////////////////////////////////////////////////////////////////////
/// Cross product of two vectors
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Cross : public UTG_Expression_BasicMath_Op
{
	GENERATED_BODY()
	
public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Cross product of inputs. A <cross> B.")); } 

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FVector4f					EvaluateVector_WithValue(FTG_EvaluationContext* InContext, const FVector4f* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
};

//////////////////////////////////////////////////////////////////////////
/// Step function: Usual HLSL step style function
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Step : public UTG_Expression_BasicMath_Op
{
	GENERATED_BODY()
	
public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("step(y, x) returns 1 if the x parameter is greater than or equal to the y parameter; otherwise, 0.")); } 

protected:
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
};

//////////////////////////////////////////////////////////////////////////
/// Power
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Pow : public UTG_Expression_Variant
{
	GENERATED_BODY()
	
	// The base of the power function
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Variant							Base = 0;

	// The exponent
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Variant							Exponent = 1; // Default exponent to 1 for a noop behavior

public:
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Calculates Pow(Base, Exponent).")); } 
	virtual FName						GetCategory() const override { return TG_Category::Maths; } 

protected:
	virtual std::vector<FTG_Variant>	GetEvaluateArgs() override { return std::vector<FTG_Variant>({ Base, Exponent }); }
	UE_API virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) override;
	UE_API virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) override;
};

#undef UE_API
