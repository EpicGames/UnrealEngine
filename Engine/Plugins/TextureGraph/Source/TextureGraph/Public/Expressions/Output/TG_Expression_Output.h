// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "TG_OutputSettings.h"

#include "TG_Expression_Output.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_Output : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_VARIANT_EXPRESSION(TG_Category::Output);
	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	UE_API virtual bool Validate(MixUpdateCyclePtr Cycle) override;
	
	// The final generated output
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Variant Source = 0;

	// The final generated output value
	UPROPERTY(meta = (TGType = "TG_OutputParam", PinDisplayName = ""))
	FTG_Variant Output = 0;

	//When we will work on Node UI for FOutputSettings we will set the category as TG_Setting
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Input", CollapsableChildProperties,ShowOnlyInnerProperties, FullyExpand, NoResetToDefault, PinDisplayName = "Settings") )
	FTG_OutputSettings OutputSettings;

	UE_DEPRECATED(5.6, "Use OutputSettings.bShouldExport instead")
	bool bShouldExport = true;

	virtual FTG_Name GetDefaultName() const override { return TEXT("Output");}
	UE_API virtual void SetTitleName(FName NewName) override;
	UE_API virtual FName GetTitleName() const override;
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Uses the node input as a graph output. It can be exported as a texture asset and is automatically exposed as a graph output parameter. ")); } 
	virtual bool ShouldShowSettings() const { return false; }

	UE_API void UpdateBufferDescriptorValues();
	UE_API void InitializeOutputSettings();

	UE_API void SetShouldExport(bool InShouldExport);

	bool GetShouldExport() const { return OutputSettings.bShouldExport; }
};

#undef UE_API
