// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "Transform/Mask/T_Gradient.h"

#include "TG_Expression_Gradient.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_Gradient : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Procedural)
	
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

#if WITH_EDITOR
	// Used to implement EditCondition logic for both Node UI and Details View
	UE_API virtual bool						CanEditChange(const FProperty* InProperty) const override;
#endif


	// The type of the gradient function
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", RegenPinsOnChange))
	EGradientType						Type = EGradientType::GT_Linear_1;

	// Type of interpolation to use
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", EditConditionHides))
	EGradientInterpolation				Interpolation = EGradientInterpolation::GTI_Linear;

	// Rotation of the gradient
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", EditConditionHides))
	EGradientRotation					Rotation = EGradientRotation::GTR_0;

	// Rotation of the gradient
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", DisplayName = "Rotation", EditConditionHides, PinDisplayName = "Rotation"))
	EGradientRotationLimited			RotationLimited = EGradientRotationLimited::GTRL_0;

	// The center of the radial gradient
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Input", EditConditionHides, UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1"))
	FVector2f							Center = { 0.5f, 0.5f };

	// The center of the radial gradient
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Input", EditConditionHides, UIMin = "0.001", ClampMin = "0.001", UIMax = "1", ClampMax = "1"))
	float								Radius = 0.25f;

	// First point of the line for axial gradients
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Input", EditConditionHides, UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1"))
	FVector2f							Point1 = { 0.25f, 0.25f };

	// Second point of the line for axial gradients
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Input", EditConditionHides, UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1"))
	FVector2f							Point2 = { 0.75f, 0.75f };

	// The generated gradient texture
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;
	
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Generates different types of gradients.")); } 
};

#undef UE_API
