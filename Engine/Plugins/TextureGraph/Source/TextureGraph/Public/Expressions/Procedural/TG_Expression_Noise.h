// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"

#include "TG_Expression_Noise.generated.h"

#define UE_API TEXTUREGRAPH_API

UENUM(BlueprintType)
enum ENoiseType : int
{
	NOISETYPE_Simplex = 0				UMETA(DisplayName = "Simplex"),
	NOISETYPE_Perlin					UMETA(DisplayName = "Perlin"),
	NOISETYPE_Worley1					UMETA(DisplayName = "Worley1"),
	NOISETYPE_Worley2					UMETA(DisplayName = "Worley2"),
	NOISETYPE_Worley3					UMETA(DisplayName = "Worley3"),
};

UCLASS(MinimalAPI)
class UTG_Expression_Noise : public UTG_Expression
{
	GENERATED_BODY()

	static constexpr int				DefaultSize = 1024;
public:
	TG_DECLARE_EXPRESSION(TG_Category::Procedural)
	
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

	// The type of the noise function
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting"))
	TEnumAsByte<ENoiseType>				NoiseType = NOISETYPE_Simplex;

	// A value used to initialize or "seed" a random number generator when generating procedural noise. Changing the noise seed produces different patterns or variations in the generated noise.
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "-100", ClampMin = "-100", UIMax = "100", ClampMax = "100"))
	int32								Seed = 75;

	// The magnitude or strength of the variations introduced by procedural noise. Adjusting this parameter can control the intensity or impact of procedural noise on textures or effects.
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "10", ClampMax = "10"))
	float								Amplitude = 1;

	// Determines how quickly or slowly the patterns change within the generated noise
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "20", ClampMax = "20"))
	float								Frequency = 5;

	// The number of levels of details that you want from the noise signal
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "10", ClampMax = "10"))
	int32								Octaves = 1;

	// Number that determines how much detail is added to or removed from the noise signal (adjusts frequency)
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "1.5", ClampMin = "1.5", UIMax = "3.5", ClampMax = "3.5"))
	float								Lacunarity = 2;

	// Number that determines how much each octave contributes to the overall shape of the noise signal (adjusts amplitude)
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting"))
	float								Persistence = 0.5f;

	// The generated noise texture
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;
	
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Generates different types of image noise.")); } 
};

#undef UE_API
