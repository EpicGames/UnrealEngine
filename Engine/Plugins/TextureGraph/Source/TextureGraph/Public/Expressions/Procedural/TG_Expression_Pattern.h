// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "Misc/OutputDeviceNull.h"

#include "TG_Expression_Pattern.generated.h"

#define UE_API TEXTUREGRAPH_API

UENUM(BlueprintType)
enum EPatternType : int 
{
	Square = 0			UMETA(DisplayName = "Square"),
	Circle				UMETA(DisplayName = "Circle"),
	Checker				UMETA(DisplayName = "Checker"),
	Gradient			UMETA(DisplayName = "Gradient"),
};

USTRUCT(BlueprintType)
struct FPatternMaskPlacement_TS
{
	GENERATED_BODY()
	
	// The number of repetitions along the X-axis
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "1", ClampMin = "1", UIMax = "24", ClampMax = "24"))
	int32								RepeatX = 4;

	// The number of repetitions along the Y-axis
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "1", ClampMin = "1", UIMax = "24", ClampMax = "24"))
	int32								RepeatY = 4;

	// The spacing between each pattern along X-axis
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1"))
	float								SpacingX = 0.01f;

	// The spacing between each pattern along Y-axis
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1"))
	float								SpacingY = 0.01f;

	// The offset offset of the pattern
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "-1", ClampMin = "-1", UIMax = "1", ClampMax = "1"))
	float								Offset = 0.0f;

	// Whether it's a horizontal or a vertical offset 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting"))
	bool								OffsetHorizontal = true;
	
	Vector2								Repeat() const { return Vector2(RepeatX, RepeatY); }
	Vector2								Spacing() const { return Vector2(SpacingX, SpacingY); }
	Vector2								OffsetValue() const { return (OffsetHorizontal ? Vector2( Offset, 0.0f ) : Vector2( 0.0f, Offset )); }

	void InitFromString(const FString& StrVal)
	{
		FOutputDeviceNull NullOut;
		FPatternMaskPlacement_TS::StaticStruct()->ImportText(*StrVal, this, /*OwnerObject*/nullptr, 0, &NullOut, FPatternMaskPlacement_TS::StaticStruct()->GetName(), /*bAllowNativeOverride*/true);
	}

	FString ToString() const
	{
		FString ExportString;
		FPatternMaskPlacement_TS::StaticStruct()->ExportText(ExportString, this, this, /*OwnerObject*/nullptr, /*PortFlags*/0, /*ExportRootScope*/nullptr);
		return ExportString;
	}
};

USTRUCT(BlueprintType)
struct FPatternMaskJitter_TS
{
	GENERATED_BODY()

	// Jitter brightness amount
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1.0", ClampMax = "1.0" ))
	float								BrightnessAmount = 1;

	// Jitter brightness threshold
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1.0", ClampMax = "1.0" ))
	float								BrightnessThreshold = 0;

	// Seed to control the brightness randomness. 0 means no randomness
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "100", ClampMax = "100" ))
	int32								BrightnessSeed = 0;

	// Size of the jitter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1.0", ClampMax = "1.0" ))
	float								SizeAmount = 1;

	// Size threshold 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1.0", ClampMax = "1.0" ))
	float								SizeThreshold = 0;

	// Seed to control the size randomness. 0 means no randomness
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "100", ClampMax = "100" ))
	int32								SizeSeed = 0;

	// Amount of tilt along the X-axis
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "-1", ClampMin = "-1", UIMax = "1.0", ClampMax = "1.0" ))
	float								TiltXAmount = 0;

	// Amount of tilt along the Y-axis
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "-1", ClampMin = "-1", UIMax = "1.0", ClampMax = "1.0" ))
	float								TiltYAmount = 0;

	// Tilt threshold
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1.0", ClampMax = "1.0" ))
	float								TiltThreshold = 0;

	// Seed to control the tilt randomness. 0 means no randomness 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "100", ClampMax = "100" ))
	int32								TiltSeed = 0;

	// Amount of angle along the X-axis
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "-1", ClampMin = "-1", UIMax = "1.0", ClampMax = "1.0" ))
	float								AngleXAmount = 0;

	// Amount of angle along the Y-axis
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "-1", ClampMin = "-1", UIMax = "1.0", ClampMax = "1.0" ))
	float								AngleYAmount = 0;

	// Seed to control the angle randomness. 0 means no randomness 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "100", ClampMax = "100" ))
	int32								AngleSeed = 0;

	Vector2								Angle() const { return Vector2(AngleXAmount, AngleYAmount); }
	Vector2								Tilt() const { return Vector2(TiltXAmount, TiltYAmount); }

	void InitFromString(const FString& StrVal)
	{
		FOutputDeviceNull NullOut;
		FPatternMaskJitter_TS::StaticStruct()->ImportText(*StrVal, this, /*OwnerObject*/nullptr, 0, &NullOut, FPatternMaskJitter_TS::StaticStruct()->GetName(), /*bAllowNativeOverride*/true);
	}

	FString ToString() const
	{
		FString ExportString;
		FPatternMaskJitter_TS::StaticStruct()->ExportText(ExportString, this, this, /*OwnerObject*/nullptr, /*PortFlags*/0, /*ExportRootScope*/nullptr);
		return ExportString;
	}
};

USTRUCT(BlueprintType)
struct FPatternMaskBevel_TS
{
	GENERATED_BODY()
	
	// The bevel value
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1.0", ClampMax = "1.0" ))
	float								Bevel = 0;

	// The bevel curve
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "-1", ClampMin = "-1", UIMax = "1.0", ClampMax = "1.0" ))
	float								BevelCurve = 0;
	
	void InitFromString(const FString& StrVal)
	{
		FOutputDeviceNull NullOut;
		FPatternMaskBevel_TS::StaticStruct()->ImportText(*StrVal, this, /*OwnerObject*/nullptr, 0, &NullOut, FPatternMaskBevel_TS::StaticStruct()->GetName(), /*bAllowNativeOverride*/true);
	}

	FString ToString() const
	{
		FString ExportString;
		FPatternMaskBevel_TS::StaticStruct()->ExportText(ExportString, this, this, /*OwnerObject*/nullptr, /*PortFlags*/0, /*ExportRootScope*/nullptr);
		return ExportString;
	}
};

USTRUCT(BlueprintType)
struct FPatternMaskCutout_TS
{
	GENERATED_BODY()
	
	// The cutoff value
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1.0", ClampMax = "1.0" ))
	float								CutoffThreshold = 0;

	// The seed to control the cutoff randomness. 0 means no randomness
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "100", ClampMax = "100" ))
	float								CutoffSeed = 0;
	
	void InitFromString(const FString& StrVal)
	{
		FOutputDeviceNull NullOut;
		FPatternMaskCutout_TS::StaticStruct()->ImportText(*StrVal, this, /*OwnerObject*/nullptr, 0, &NullOut, FPatternMaskCutout_TS::StaticStruct()->GetName(), /*bAllowNativeOverride*/true);
	}

	FString ToString() const
	{
		FString ExportString;
		FPatternMaskCutout_TS::StaticStruct()->ExportText(ExportString, this, this, /*OwnerObject*/nullptr, /*PortFlags*/0, /*ExportRootScope*/nullptr);
		return ExportString;
	}
};

USTRUCT(BlueprintType)
struct FGradientDir_TS
{
	GENERATED_BODY()
	
	// Gradient value along X-axis in degrees [0, 90]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "90", ClampMax = "90" ))
	float								X = 90;

	// Gradient value along Y-axis in degrees [-180, 180]
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "-180", ClampMin = "-180", UIMax = "180", ClampMax = "180" ))
	float								Y = 0;

	Vector2								Value() const { return Vector2(X, Y); }

	void InitFromString(const FString& StrVal)
	{
		FOutputDeviceNull NullOut;
		FGradientDir_TS::StaticStruct()->ImportText(*StrVal, this, /*OwnerObject*/nullptr, 0, &NullOut, FGradientDir_TS::StaticStruct()->GetName(), /*bAllowNativeOverride*/true);
	}

	FString ToString() const
	{
		FString ExportString;
		FGradientDir_TS::StaticStruct()->ExportText(ExportString, this, this, /*OwnerObject*/nullptr, /*PortFlags*/0, /*ExportRootScope*/nullptr);
		return ExportString;
	}
};

UCLASS(MinimalAPI)
class UTG_Expression_Pattern : public UTG_Expression
{
	GENERATED_BODY()

	static constexpr int				DefaultSize = 1024;
public:
	TG_DECLARE_EXPRESSION(TG_Category::Procedural)
	
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

	// The type of pattern
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting"))
	TEnumAsByte<EPatternType>			PatternType = EPatternType::Square;

	// Placement
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, Meta = (TGType = "TG_Setting", HideNodeUI))
	FPatternMaskPlacement_TS			Placement;

	// Jitter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, Meta = (TGType = "TG_Setting", HideNodeUI))
	FPatternMaskJitter_TS				Jitter;

	// Bevel
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, Meta = (TGType = "TG_Setting", HideNodeUI))
	FPatternMaskBevel_TS				Bevel;

	// Cut out
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, Meta = (TGType = "TG_Setting", HideNodeUI))
	FPatternMaskCutout_TS				Cutout;

	// Gradient direction
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, Meta = (TGType = "TG_Setting", HideNodeUI))
	FGradientDir_TS						GradientDirection;

	// The output generated by the pattern
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;
	
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Generates different types of geometric patterns.")); } 
};

#undef UE_API
