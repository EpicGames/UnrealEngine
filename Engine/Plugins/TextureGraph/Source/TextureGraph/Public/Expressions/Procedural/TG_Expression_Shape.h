// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"

#include "TG_Expression_Shape.generated.h"

#define UE_API TEXTUREGRAPH_API

UENUM(BlueprintType)
enum class EShapeType : uint8
{
	Circle = 0			UMETA(DisplayName = "Circle"),
	Segment				UMETA(DisplayName = "Segment"),
	Rectangle			UMETA(DisplayName = "Rectangle"),
	Triangle			UMETA(DisplayName = "Triangle"),
	Ellipse				UMETA(DisplayName = "Ellipse"),
	Pentagon = 5		UMETA(DisplayName = "Pentagon"),
	Hexagon				UMETA(DisplayName = "Hexagon"),
	Polygon				UMETA(DisplayName = "Polygon"),
};

UCLASS(MinimalAPI)
class UTG_Expression_Shape : public UTG_Expression
{
	GENERATED_BODY()

	static constexpr int				DefaultSize = 1024;
public:
	TG_DECLARE_EXPRESSION(TG_Category::Procedural)
	
	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;

#if WITH_EDITOR
	// Used to implement EditCondition logic for both Node UI and Details View
	UE_API virtual bool						CanEditChange(const FProperty* InProperty) const override;
#endif
	
	// Select the geometric primitive shape to generate
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", RegenPinsOnChange))
	EShapeType							ShapeType = EShapeType::Circle;

	// The number of sides of the Polygon (only relevant when using the <Polygon> type)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = 7, ClampMax = 32, EditConditionHides))
	int32								PolygonNumSides = 8;

	// Width representing the relative size of the shape in the generated texture. For Circle/Triangle/Pentagon hexagon and Polygon it is the radius of the shape relative to the uv domain of the generated texture
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = 0, ClampMax =1))
	float 								Width = 1.0f;

	// Height representing the relative size of the shape in the generate texture. Height is used in Rectangle and ellipse.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = 0, ClampMax = 1, EditConditionHides))
	float 								Height = 1.0f;

	// Orientation is the 2D rotation of the shape around the center
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "-180", ClampMin = "-180", UIMax = "180", ClampMax = "180", Delta = "1", Units = "Degrees", EditConditionHides))
	float								Orientation = 0.0f;

	// Rounding is about the 2D footprint of the shape generated, 0 means the perfect shape, 1 means the most rounded version of the shape
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = 0, ClampMax = 1, EditConditionHides))
	float								Rounding = 0.0f;

	// Bevel width is the distance from the outline of the shape on which the bevelling function is applied to the generated mask
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = 0, ClampMax = 1))
	float								BevelWidth = 0.0f;

	// Bevel curve is controlling the shape of bevelling varying from concave to linear to convex
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = -1, ClampMax = 1))
	float								BevelCurve = 0.0f;

	// Signed distance function the mathematical function defining the distance to the edge of the shape
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = NoCategory, meta = (TGType = "TG_Setting", ClampMin = 0, ClampMax = 1))
	float								ShowSDF = 0.0f;
	
	// The output generated by the pattern
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture							Output;
	
	virtual FText						GetTooltipText() const override { return FText::FromString(TEXT("Generates different types of geometric shapes.")); } 
};



#undef UE_API
