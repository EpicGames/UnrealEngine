// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"

#include "TG_Expression_Transform.generated.h"

#define UE_API TEXTUREGRAPH_API


//////////////////////////////////////////////////////////////////////////
/// Expression
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Transform : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_EXPRESSION(TG_Category::Procedural);
	UE_API virtual void							Evaluate(FTG_EvaluationContext* InContext) override;

	// Output image
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output") )
	FTG_Texture								Output;

	// The Input image to transform
	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Texture								Input;

	// Filter mode is wrap or clamped
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", DisplayName = "Wrap Mapping"))
	bool									WrapMode = true;

	// Filter mode is wrap or clamped
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", DisplayName = "Mirror X"))
	bool									MirrorX = false;

	// Filter mode is wrap or clamped
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", DisplayName = "Mirror Y"))
	bool									MirrorY = false;

	// The coverage of the transform in range [0, 1]. Defaults to 1,1
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0.0", ClampMin = "0.0", UIMax = "1.0", ClampMax = "1.0"))
	FVector2f								Coverage = { 1, 1 };

	// The translation offset along xy in range [0, 1]. Defaults to 0
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "-1.0", ClampMin = "-1.0", UIMax = "1.0", ClampMax = "1.0"))
	FVector2f								Offset = { 0, 0 };

	// The pivot XY coord in range [0, 1]. Defaults to 0.5
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0.0", ClampMin = "0.0", UIMax = "1.0", ClampMax = "1.0", Multiple = "0.01"))
	FVector2f								Pivot = { 0.5, 0.5 };

	// The XY rotation expressed in degrees
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "-180", ClampMin = "-180", UIMax = "180", ClampMax = "180", Delta = "1", Units="Degrees"))
	float									Rotation = 0;

	// Uniform scaling of the image within the tiled cell.
	// Value is a percentage, default value is 100% meaning the image fully fill the cell:
	// Zoom out with value in the range [0,100]
	// Zoom in with value above 100%
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "100", ClampMax = "100000", Multiple = "1", Delta = "1", Units="Percent"))
	float									Zoom = 100.0f;

	// Keep the AspectRatio of the source image (0) or stretch to fit the cell rectangle (1)
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", DisplayName = "Stretch to fit", UIMin = "0.0", ClampMin = "0.0", UIMax = "1.0", ClampMax = "1.0", Multiple = "0.01"))
	float									StretchToFit = 1.0f;
	
	// The repetitions along rotated X & Y axes. Defaults to 1
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "1.0", ClampMin = "0.01", UIMax = "32", ClampMax = "32", Delta = "1"))
	FVector2f								Repeat = { 1, 1 };

	// Filling color for non texture source
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting"))
	FLinearColor 	FillColor = { 0, 0, 0, 0 };

	// Staggering is horizontal ? or false for vertical
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", DisplayName = "Stagger Horizontally"))
	bool							    StaggerHorizontally = true;

	// The staggering offset from one row to the next
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "-1", ClampMin = "-1", UIMax = "1", ClampMax = "1", Multiple = "0.005"))
	float									StaggerOffset = 0;

	// The skipping offset from one tile to the next
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "31", ClampMax = "31", Delta = "1"))
	FVector2f								Stride = { 0, 0 };

	// [DEBUG ONLY OPTION] Output blended with the debugging grid showing the transformation applied. This option WILL NOT get applied while exporting!
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", UIMin = "0", ClampMin = "0", UIMax = "1", ClampMax = "1"))
	float									 ShowDebugGrid = 0;

	virtual FText							GetTooltipText() const override { return FText::FromString(TEXT("Performs translation, rotation and repetition of an image")); } 
};

#undef UE_API
