// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TG_Node.h"
#include "TG_Var.h"
#include "TG_GraphEvaluation.h"

#include "TG_Expression.generated.h"

#define UE_API TEXTUREGRAPH_API

#if WITH_EDITOR
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTSExpressionChanged, UTG_Expression*);
#endif

class TG_Category
{
public:
	static UE_API const FName Default;
	static UE_API const FName Output;
	static UE_API const FName Input;
	static UE_API const FName Adjustment;
	static UE_API const FName Channel;
	static UE_API const FName DevOnly;
	static UE_API const FName Procedural;
	static UE_API const FName Maths;
	static UE_API const FName Utilities;
	static UE_API const FName Filter;
	static UE_API const FName Arrays;
	static UE_API const FName Custom;
};

UCLASS(MinimalAPI, abstract)
class UTG_Expression : public UObject
{
	GENERATED_BODY()

protected:
	// UTG_Expression classes have a per instance expression class version number recovered from serialization
	UPROPERTY() // Only to retreive the version of a loaded expression and compare it against the 
	int32 InstanceExpressionClassVersion = 0;

	// Method returning this Expression class version that can be overwritten in a sub class requiring to
	// support version change
	virtual int32		GetExpressionClassVersion() const { return 0; }
	virtual bool		ShouldShowSettings() const { return true; }

public:
	//When we will work on Node UI for FOutputSettings we will set the category as TG_Setting
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Setting", CollapsableChildProperties, ShowOnlyInnerProperties, FullyExpand, NoResetToDefault, PinDisplayName = "Settings", EditConditionHides))
	FTG_TextureDescriptor BaseOutputSettings;

	// Override serialization to save Class Serialization version
	UE_API virtual	void		Serialize(FArchive& Ar) override;


	UE_API virtual FTG_Name GetDefaultName() const;
	virtual FName GetCategory() const { return TG_Category::Default; }
	virtual FTG_SignaturePtr GetSignature() const { return nullptr; }
	virtual FText GetTooltipText() const { return FText::FromString(TEXT("Texture Graph Node")); } 

	virtual void SetTitleName(FName NewName) {}
	virtual FName GetTitleName() const { return GetDefaultName(); }
	UE_API virtual bool CanRenameTitle() const;

	virtual bool CanHandleAsset(UObject* Asset) { return false; };
	virtual void SetAsset(UObject* Asset) { check(CanHandleAsset((Asset))); };

	// This is THE evaluation call to overwrite
	virtual void Evaluate(FTG_EvaluationContext* InContext) {}

#if WITH_EDITOR
	UE_API void PropertyChangeTriggered(FProperty* Property, EPropertyChangeType::Type ChangeType);
	UE_API virtual bool CanEditChange( const FProperty* InProperty ) const override;	
#endif
	
	// Validate internal checks, warnings and errors
	virtual bool Validate(MixUpdateCyclePtr	Cycle) { return true; }

	// Validate that the expression conforms to a conformant function (e.g. Clamp etc.)
	UE_API void ValidateGenerateConformer(UTG_Pin* InPin);

	// Expression notifies its parent node on key events
	UE_API virtual	UTG_Node* GetParentNode() const final;
protected:
	friend class UTG_Graph;
	friend class UTG_Node;
	friend struct FTG_Var;
	friend struct FTG_Evaluation;

	// Initialize the expression in cascade from the node allowing it to re create transient data
	// This is called in the PostLoad of the Graph
	virtual void Initialize() {}

	// Called first from the Graph::Evaluate/Traverse which then call the Evaluate function.
	// This is where the Var values are copied over to the matching Expression's properties
	// This is NOT the function you want to derive, unless you know exactly what you are doing.
	// Instead overwrite the Execute function.
	UE_API virtual void SetupAndEvaluate(FTG_EvaluationContext* InContext);

	UE_API virtual void CopyVarToExpressionArgument(const FTG_Argument& Arg, FTG_Var* InVar);
	UE_API virtual void CopyVarFromExpressionArgument(const FTG_Argument& Arg, FTG_Var* InVar);
	virtual void CopyVarGeneric(const FTG_Argument& Arg, FTG_Var* InVar, bool CopyVarToArg) {}

	// Log the actual values for vars and the expression evaluation
	// Called from SetupAndEvaluate call if Context ask for it
	UE_API virtual void LogEvaluation(FTG_EvaluationContext* InContext);

	UE_API virtual FTG_Signature::FInit GetSignatureInitArgsFromClass() const final;


	// Build the signature of the expression by collecting the FTG_ExpressionXXX UProperties of the class
	UE_API virtual FTG_SignaturePtr BuildSignatureFromClass() const final;

	// Build Signature in derived classes dynamically
	virtual FTG_SignaturePtr BuildSignatureDynamically() const { return nullptr; }


public:
	// If some state has changed in the expression that affects its representation
	// triggered when a property has changed and needs to be copied over to its corresponding Var
	// NB: required to be public for calling from TG_Variant customization
	UE_API virtual void NotifyExpressionChanged(const FPropertyChangedEvent& PropertyChangedEvent) const final;
protected:

	// If the signature changes and the node need to regenerate its own signature.
	// Only concrete implementation for Dynamic Expression
	UE_API virtual void NotifySignatureChanged() const;

	// Variant expression API: specialized when using the macro TG_DECLARE_VARIANT_EXPRESSION: 
	// When input connection are changing, the graph / node calls ResetCommonInputVariantType in order for the expression
	// to reeval its CommonInputVariant type as well as it s signature if it changes 
	virtual bool ResetCommonInputVariantType(FTG_Variant::EType InType = FTG_Variant::EType::Invalid) const { return false; }
	virtual void NotifyCommonInputVariantTypeChanged(FTG_Variant::EType NewType = FTG_Variant::EType::Invalid) const final {
		if (ResetCommonInputVariantType(NewType))
			NotifySignatureChanged();
	}
	virtual FTG_Variant::EType GetCommonInputVariantType() const { return FTG_Variant::EType::Scalar; }

	// Variant expression API:
	// Eval the common input variant type used by the expression (only if it is variant)
	// by looking at the various input pins connected or default types and come up with a "common input" type used for 
	// how to interpret all the input variants
	// Default implementation: The common variant type is found as the type that superseeds all the variant input arguments
	//           currently fed from other nodes 
	// This call can be modified in a specific class to get a different behavior.
	UE_API virtual FTG_Variant::EType EvalExpressionCommonInputVariantType() const;

	// Variant expression API:
	// Similarly, this call eval the common output variant type used by the expression (only if it is variant)
	// Default implementation: the common output variant is the same as the common input variant.
	// this call can be modified in a specific class to get a differnet behavior, see Expression_Dot for an example.
	UE_API virtual FTG_Variant::EType EvalExpressionCommonOutputVariantType() const;

	virtual bool IgnoreInputTextureOnUndo() const { return true; }

	// In some cases, evaluation or change in the Expression needs to be feedback to the matching pin's value
	// This is not needed for the standard flow of evaluation but is sometime required for coupled member.
	template<typename T> bool FeedbackPinValue(const FName& InPinName, const T& InValue)
	{
		UTG_Node* ParentNode = GetParentNode();
		if (ParentNode)
		{
			UTG_Pin* Pin = ParentNode->GetPin(InPinName);
			if (Pin)
			{
				return Pin->SetValue(InValue);
			}
		}
		return false;
	}

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
#endif
};



#define TG_DECLARE_EXPRESSION(Category) \
		FTG_SignaturePtr GetSignature() const override { static FTG_SignaturePtr Signature = BuildSignatureFromClass(); return Signature; } \
		virtual FName GetCategory() const override { return Category; } 

#define TG_DECLARE_DYNAMIC_EXPRESSION(Category) \
	protected: mutable FTG_SignaturePtr DynSignature = nullptr; \
	virtual FTG_SignaturePtr BuildSignatureDynamically() const override; \
	virtual void NotifySignatureChanged() const final { DynSignature.Reset(); /*Where the DynSynature is reset!*/ Super::NotifySignatureChanged(); } \
    public: FTG_SignaturePtr GetSignature() const override { if (!DynSignature) DynSignature = BuildSignatureDynamically(); return DynSignature; } \
	public: virtual FName GetCategory() const override { return Category; } 

#define TG_DECLARE_VARIANT_EXPRESSION(Category) \
	protected: mutable FTG_Variant::EType CommonInputVariantType = FTG_Variant::EType::Scalar; \
	virtual void Initialize() override { \
		Super::Initialize(); \
		CommonInputVariantType = EvalExpressionCommonInputVariantType(); \
	} \
	virtual FTG_Variant::EType GetCommonInputVariantType() const final { return CommonInputVariantType; } \
	virtual bool ResetCommonInputVariantType(FTG_Variant::EType InType = FTG_Variant::EType::Invalid) const final { \
		if (InType == FTG_Variant::EType::Invalid) { \
			InType = EvalExpressionCommonInputVariantType(); \
		} \
		if (CommonInputVariantType != InType) { \
			CommonInputVariantType = InType; \
			return true; \
		} \
		return false; \
	} \
	protected: mutable FTG_SignaturePtr DynSignature = nullptr; \
	virtual FTG_SignaturePtr BuildSignatureDynamically() const final { \
		FTG_Signature::FInit SignatureInit = GetSignatureInitArgsFromClass(); \
		for (auto& a : SignatureInit.Arguments) { \
			if (a.IsOutput()) { \
				a.CPPTypeName = FTG_Variant::GetArgNameFromType(EvalExpressionCommonOutputVariantType()); \
			} \
		} \
		return MakeShared<FTG_Signature>(SignatureInit); \
	} \
	virtual void NotifySignatureChanged() const final { DynSignature.Reset(); /*Where the DynSynature is reset!*/ Super::NotifySignatureChanged(); } \
	public: FTG_SignaturePtr GetSignature() const override { if (!DynSignature) DynSignature = BuildSignatureDynamically(); return DynSignature; } \
	public: virtual FName GetCategory() const override { return Category; } 
	
	

UCLASS(MinimalAPI, Hidden)
class UTG_Expression_Null : public UTG_Expression
{
	GENERATED_BODY()

protected:
	TG_DECLARE_EXPRESSION(TG_Category::Default);
};

#undef UE_API
