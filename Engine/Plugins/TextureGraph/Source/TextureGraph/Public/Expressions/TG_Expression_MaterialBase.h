// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TG_Expression.h"
#include "TG_Texture.h"
#include "TG_Expression_MaterialBase.generated.h"

#define UE_API TEXTUREGRAPH_API

class RenderMaterial_BP;
typedef std::shared_ptr<RenderMaterial_BP> RenderMaterial_BPPtr;


// Describe the possible attributes extracted from a MAterial during a DrawMaterial call
// this is used as high level data.
UENUM(BlueprintType)
enum class EDrawMaterialAttributeTarget : uint8
{
	BaseColor = 0,
	Metallic,
	Specular,
	Roughness,
	Anisotropy,
	Emissive,
	Opacity,
	OpacityMask,
	Normal,
	Tangent,

	/// Always has to be the last
	Count UMETA(Hidden),
};

struct FMaterialInstanceStaticSwitchPermutationMap;

UCLASS(MinimalAPI, abstract)
class UTG_Expression_MaterialBase : public UTG_Expression
{
	GENERATED_BODY()

public:
	TG_DECLARE_DYNAMIC_EXPRESSION(TG_Category::Default)

	// Zohaib: Right now, we only support tile mode.
	// IMO, We dont need to show it user.
	// Disabling it for now.

	// Whether to run the material in tiled mode (Tiles in output settings)
	// UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = NoCategory, meta = (TGType = "TG_Setting")
	bool TiledMode = true;

	// The output of the material expressed as a texture
	UPROPERTY(meta = (TGType = "TG_Output") )
	FTG_Texture Output;

	mutable TArray<FTG_Texture> Outputs;

	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	// Validate internal checks, warnings and errors
	UE_API virtual bool Validate(MixUpdateCyclePtr	Cycle) override;

	// Access the list of attributes available for rendering from the current Material
	const TArray<EDrawMaterialAttributeTarget>& GetAvailableMaterialAttributeIds() const { return AvailableMaterialAttributeIds; }
	const TArray<FName>& GetAvailableMaterialAttributeNames() const { return AvailableMaterialAttributeNames; }

	static UE_API FName CPPTypeNameFromMaterialParamType(EMaterialParameterType InMatType);

	// 0 will fallback to CVar <TG.VirtualTexture.NumWarmupFrames> value.
	virtual int32 GetNumVirtualTextureWarmupFrames() const { return 0; }

protected:
	// A local Material Instance Constant is recreated from the reference material assigned through SetMaterialInternal
	// All static switches permutations are cached in this map and only created on demand
	UPROPERTY(Transient)
	FMaterialInstanceStaticSwitchPermutationMap MaterialPermutations;

	// The list of the material attributes declared in the signature as Output of the expression,
	TArray<EDrawMaterialAttributeTarget>	AvailableMaterialAttributeIds; // The set of material properties available for rendering
	TArray<FName>							AvailableMaterialAttributeNames; // same with the attribute names
	mutable TArray<FName>					MatAttributesOutputArgNames; // Their argument names (because they may have to be different from the true attribute material name)

	// Arg to Material Param Info array records the map of arg name to the corresponding Material Param
	struct FArgToMaterialParamInfo
	{
		FName ArgName;
		FName MatParamName;
		FGuid MatParamGuid;
		EMaterialParameterType MatType = EMaterialParameterType::None;

		FMaterialParameterValue Value;

		bool operator== (const FName& InArgName) const { return ArgName == InArgName; }
	};
	mutable TArray<FArgToMaterialParamInfo> ArgToMatParams;					// mutable because it is populated in the BuildSignatureDynamically()



	UE_API virtual void Initialize() override;

	UE_API void GenerateMaterialAttributeOptions(); // Based on the current Material, list of material attributes available

	// During evaluation, pick the MIC from the input static switch(es) combination
	UE_API UMaterialInstanceConstant* GetEvaluationMaterialInstanceConstant(FTG_EvaluationContext* InContext);

	UE_API TiledBlobPtr CreateRenderMaterialJob(FTG_EvaluationContext* InContext, const FString& InName, const FString& InMaterialPath, const BufferDescriptor& InDescriptor, EDrawMaterialAttributeTarget InDrawMaterialAttributeTarget);
	UE_API TiledBlobPtr CreateRenderMaterialJob(FTG_EvaluationContext* InContext, const RenderMaterial_BPPtr& InRenderMaterial, const BufferDescriptor& InDescriptor, EDrawMaterialAttributeTarget InDrawMaterialAttributeTarget);
	UE_API void LinkMaterialParameters(FTG_EvaluationContext* InContext, JobUPtr& InMaterialJob, const UMaterialInterface* InMaterial, BufferDescriptor InDescriptor);

	UE_API virtual void CopyVarGeneric(const FTG_Argument& Arg, FTG_Var* InVar, bool CopyVarToArg);
	UE_API virtual void SetMaterialInternal(UMaterialInterface* InMaterial);

	virtual EDrawMaterialAttributeTarget GetRenderedAttributeId() { return EDrawMaterialAttributeTarget::Emissive;}
	virtual TObjectPtr<UMaterialInterface> GetMaterial() const { return nullptr;}

private:
	UE_API void AddSignatureParam(const TArray<FMaterialParameterInfo>& OutParameterInfo, const TArray<FGuid>& OutParameterIds, EMaterialParameterType MatType, FTG_Signature::FInit& SignatureInit) const;

	static UE_API EDrawMaterialAttributeTarget ConvertEMaterialPropertyToEDrawMaterialAttributeTarget(EMaterialProperty InMaterialProperty);
};

 

#undef UE_API
