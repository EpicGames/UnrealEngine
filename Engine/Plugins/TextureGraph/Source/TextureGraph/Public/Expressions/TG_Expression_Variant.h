// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TG_Expression.h"
#include "TG_Texture.h"
#include "FxMat/FxMaterial.h"
#include "TG_Variant.h"
#include "Transform/Expressions/T_Maths_OneInput.h"

#include "TG_Expression_Variant.generated.h"

#define UE_API TEXTUREGRAPH_API

//////////////////////////////////////////////////////////////////////////
/// An abstract class to help with the boilerplate of handling
/// variants in general
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI, Abstract)
class UTG_Expression_Variant : public UTG_Expression
{
	GENERATED_BODY()

protected:
	/// Most of the time these are the two functions that expressions will need to implement. 
	/// These are for scalar and texture handling respectively
	virtual float						EvaluateScalar_WithValue(FTG_EvaluationContext* InContext, const float* const Value, size_t Count) { return 0.0f; }
	virtual FTG_Texture					EvaluateTexture(FTG_EvaluationContext* InContext) { return FTG_Texture::GetBlack(); }

	/// If you use the basic structure of variant expression then you'll need to implement
	/// this correctly. There are plenty of examples. This simplifies a lot of error
	/// checking and most of the error checking gets handled in the base class
	virtual std::vector<FTG_Variant>	GetEvaluateArgs() { return std::vector<FTG_Variant>(); }

	/// Evaluate vector only needs to be implemented if there vector handling isn't just a 
	/// component-wise scalar handling. In our current implementation for instance
	/// Dot and cross product is a good example where special vector handling is required
	UE_API virtual FVector4f					EvaluateVector_WithValue(FTG_EvaluationContext* InContext, const FVector4f* const Value, size_t Count);

	/// Only override this if you don't want use the base class error handling (which is
	/// quite strict out of the box). Clamp expression is one which does it's own error
	/// checking and avoids using the strict handling of the base class
	virtual bool						ErrorCheckInputTextures() const { return true; }

	/// This only need to be overridden if want to have your own variant implementation
	/// along with the EvaluateTexture(...) method defined above. Most of the time 
	/// overriding these will not be required.
	UE_API virtual float						EvaluateScalar(FTG_EvaluationContext* InContext);
	UE_API virtual FVector4f					EvaluateVector(FTG_EvaluationContext* InContext);
	UE_API virtual FLinearColor				EvaluateColor(FTG_EvaluationContext* InContext);

public:
	TG_DECLARE_VARIANT_EXPRESSION(TG_Category::Default);

	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Variant							Output;

	UE_API virtual void						Evaluate(FTG_EvaluationContext* InContext) override;
};

#undef UE_API
