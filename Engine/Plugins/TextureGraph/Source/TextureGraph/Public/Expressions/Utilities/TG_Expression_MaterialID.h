// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "Transform/Expressions/T_ExtractMaterialIds.h"

#include "TG_Expression_MaterialID.generated.h"

#define UE_API TEXTUREGRAPH_API

UCLASS(MinimalAPI)
class UTG_Expression_MaterialID : public UTG_Expression
{
	GENERATED_BODY()
public:
	TG_DECLARE_EXPRESSION(TG_Category::Utilities)
	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	UPROPERTY(meta = (TGType = "TG_Input"))
	FTG_Texture MaterialIDMap;

	// List of available colors extracted from the input texture. Selected colors will be shown as white in the output texture. 
	UPROPERTY(EditAnywhere, EditFixedSize, Category = NoCategory, meta = (TGType = "TG_Input", NoResetToDefault, TGPinNotConnectable = "true"))
	TArray<FMaterialIDMaskInfo> MaterialIDMaskInfos;

	UPROPERTY()
	FMaterialIDCollection MaterialIDInfoCollection;

	UPROPERTY(Transient)
	TArray<FLinearColor> ActiveColors;

	UPROPERTY(Transient)
	int32 ActiveColorsCount;
	
	// The output of the node, which is the loaded texture asset
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture Output;
	
protected:
    UE_API virtual void Initialize() override;

private:
	UE_API void UpdateActiveColors();
public:

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
	virtual FTG_Name GetDefaultName() const override { return TEXT("MaterialID"); }
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Extracts colors from the texture and creates a mask with selected colors")); }
	UE_API void SetMaterialIdMask(int32 Id, bool bEnable);
	
};

#undef UE_API
