// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Expressions/TG_Expression.h"
#include "TG_Texture.h"
#include "Transform/Expressions/T_ExtractMaterialIds.h"

#include "TG_Expression_Utils.generated.h"

#define UE_API TEXTUREGRAPH_API

//////////////////////////////////////////////////////////
#if 0
UCLASS(MinimalAPI)
class UTG_Expression_Utils_GetWidth : public UTG_Expression
{
	GENERATED_BODY()
public:
	TG_DECLARE_EXPRESSION(TG_Category::Utilities)
	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Texture Input;

	/// The width of the texture
	UPROPERTY(meta = (TGType = "TG_Output", HideInnerPropertiesInNode))
	float Width;
public:
	virtual FTG_Name GetDefaultName() const override { return TEXT("GetWidth"); }
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Extracts the width from an input texture. Useful for doing maths on image dimensions.")); }
};

//////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Utils_GetHeight : public UTG_Expression
{
	GENERATED_BODY()
public:
	TG_DECLARE_EXPRESSION(TG_Category::Utilities)
	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Texture Input;

	/// The height of the texture
	UPROPERTY(meta = (TGType = "TG_Output", HideInnerPropertiesInNode))
	float Height;
public:
	virtual FTG_Name GetDefaultName() const override { return TEXT("GetHeight"); }
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Extracts the height from an input texture. Useful for doing maths on image dimensions.")); }
};
#endif 
//////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Utils_MakeVector4 : public UTG_Expression
{
	GENERATED_BODY()
public:
	TG_DECLARE_EXPRESSION(TG_Category::Utilities)
	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	UPROPERTY(meta = (TGType = "TG_Input"))
	float X = 0;

	UPROPERTY(meta = (TGType = "TG_Input"))
	float Y = 0;

	UPROPERTY(meta = (TGType = "TG_Input"))
	float Z = 0;

	UPROPERTY(meta = (TGType = "TG_Input"))
	float W = 0;

	/// The height of the texture
	UPROPERTY(meta = (TGType = "TG_Output"))
	FVector4f Output;

public:
	virtual FTG_Name GetDefaultName() const override { return TEXT("MakeVector4"); }
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Converts a bunch of floating point values to a vector.")); }
};

//////////////////////////////////////////////////////////
UCLASS(MinimalAPI)
class UTG_Expression_Utils_Resize : public UTG_Expression
{
	GENERATED_BODY()
public:
	TG_DECLARE_EXPRESSION(TG_Category::Utilities)
	UE_API virtual void Evaluate(FTG_EvaluationContext* InContext) override;

	/// The input image to resize
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = ""))
	FTG_Texture Input;

	/// Whether to maintain aspect ratio for the output
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "Maintain Aspect Ratio"))
	bool bMaintainAspectRatio = true;

	/// The background color to use for the output texture
	UPROPERTY(meta = (TGType = "TG_Input", PinDisplayName = "Background Color"))
	FLinearColor BackgroundColor = FLinearColor(0, 0, 0, 0);

	// The output texture which should be resized to the desired dimensions
	UPROPERTY(EditAnywhere, Category = NoCategory, meta = (TGType = "TG_Output", PinDisplayName = ""))
	FTG_Texture Output;

public:
	virtual FTG_Name GetDefaultName() const override { return TEXT("Resize Texture"); }
	virtual FText GetTooltipText() const override { return FText::FromString(TEXT("Resizes an input texture to the given dimensions.")); }
};

#undef UE_API
