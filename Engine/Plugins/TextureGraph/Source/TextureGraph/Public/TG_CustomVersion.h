// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// Custom serialization version for Texture Graph asset config
struct FTG_CustomVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		// Added TexturePath to FTG_Texture
		TGTextureAddedTexturePath,

		// Added BaseOutputSettings to TG_Expression
		TGExpressionAddedBaseOutputSettings,

		// Added bSRGB to FTG_TextureDescriptor
		TGTextureDescAdded_bSRGB,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FTG_CustomVersion() {}
};
