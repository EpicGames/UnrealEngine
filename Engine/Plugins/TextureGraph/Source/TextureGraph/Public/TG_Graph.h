// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TG_Pin.h"
#include "TG_Var.h"
#include "TG_Node.h"
#include <functional>
#include "TG_Graph.generated.h"

#define UE_API TEXTUREGRAPH_API

class UTG_ExportSettings;

// Graph Traversal
// A Graph internal struct populated at runtime used to Traverse it.
// This data is built by the Graph itself.
// 
struct FTG_GraphTraversal
{
	FTG_Ids InPins;
	FTG_Ids OutPins;

	FTG_Ids TraverseOrder;

	int32 InNodesCount = 0; // number of starting nodes in the traverse order
	int32 OutNodesCount = 0; // number of ending nodes in the traverse order
	int32 NodeWavesCount = 0; // number of waves to go through the graph
};

class UTG_Expression;
struct FTG_EvaluationContext;
#if WITH_EDITOR
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTGNodeSignatureChanged, UTG_Node*);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnTextureGraphChanged, UTG_Graph*, UTG_Node*, bool);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnNodeEvaluation, UTG_Node*, const FTG_EvaluationContext*);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTGNodeAdded, UTG_Node*);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTGNodeRemoved, UTG_Node*,FName);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTGNodeRenamed, UTG_Node*, FName);

#endif

UCLASS(MinimalAPI)
class UTG_Graph : public UObject
{
    GENERATED_BODY()

private:
    // Inner setup node, used when a new node is created in this graph
	// The node is initialized, assigned a uuid in the graph
	// The node pins are created from the expression signature
	// Notifiers are installed
    UE_API void SetupNode(UTG_Node* Node);

	// Inner recreate node, used when the node's signature has changed,
	// All the existing pins of the node are killed
	// The new signature is obtained from the Expression
	// the new pins are created
	UE_API void RegenerateNode(UTG_Node* InNode);

	// Inner allocate node pins, used when a node is setup or when a node is regenerated
	UE_API void AllocateNodePins(UTG_Node* Node);
	// Inner kill node pins, used when a node is removed or when a node is regenerated
	UE_API void KillNodePins(UTG_Node* Node);

	// Inner allocate pin
	UE_API FTG_Id AllocatePin(UTG_Node* Node, const FTG_Argument& SourceArg, int32 PinIdx = -1);
	// Inner kill pin
	UE_API void KillPin(FTG_Id InPin);

	// The array of nodes indexed by their uuid.NodeIdx()
	// An element can be null if the node has been removed during the authoring
	UPROPERTY()
    TArray<TObjectPtr<UTG_Node>> Nodes;

	UPROPERTY()
	FString Name = TEXT("noname");

	UPROPERTY(EditAnywhere, Transient, Category = "TextureGraphParams")
	TMap<FName, FTG_Id> Params; // The Parameter pins in the graph

	UE_API void NotifyGraphChanged(UTG_Node* InNode = nullptr, bool bIsTweaking = false);
	friend struct FTG_Evaluation; // NodePostEvaluate notifier is triggered from the FTG_Evaluation::EvaluateNode call
	UE_API void NotifyNodePostEvaluate(UTG_Node* InNode, const FTG_EvaluationContext* InContext);

	bool IsValidNode(FTG_Id NodeId) const { return (NodeId != FTG_Id::INVALID && Nodes.IsValidIndex(NodeId.NodeIdx())); }
	bool IsValidPin(FTG_Id PinId) const { return IsValidNode(PinId) && Nodes[PinId.NodeIdx()] && Nodes[PinId.NodeIdx()]->Pins.IsValidIndex(PinId.PinIdx()); }
	
protected:
	friend class UTG_Node;
	friend class UTG_Pin;
	UE_API void OnNodeChanged(UTG_Node* InNode, bool Tweaking);
	UE_API void OnNodeSignatureChanged(UTG_Node* InNode);
	UE_API void OnNodePinChanged(FTG_Id InPinId, UTG_Node* InNode);
	UE_API void OnNodeAdded(UTG_Node* InNode);
	UE_API void OnNodeRemoved(UTG_Node* InNode,FName InName);
	UE_API void OnNodeRenamed(UTG_Node* InNode,FName OldName);

	// Rename existing param
	UE_API bool RenameParam(FName OldName, FName NewName);

#if WITH_EDITOR
protected:
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	// Override PostEditUndo method of UObject
	UE_API virtual void PostEditUndo() override;

public:
	// Overwrite the modify to also mark the transient states as dirty
	UE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
#endif

public:
	//////////////////////////////////////////////////////////////////////////
	//// Constructing the Graph and UObject overrides
	//////////////////////////////////////////////////////////////////////////

	// Construct the Graph, can only be done once before "using" the graph
	UE_API virtual void Construct(FString InName);

	// Override Serialize method of UObject
	UE_API virtual void Serialize(FArchive& Ar) override;

    // Override the PostLoad method of UObject to finalize the un-serialization
    // and fill in the runtime fields of the graph sub objects
	UE_API virtual void PostLoad() override;

	// Override PreSave method of UObject
	UE_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;

	//////////////////////////////////////////////////////////////////////////
	//// Graph methods to build the graph!
	//////////////////////////////////////////////////////////////////////////

	// Create a node from an Expression Class
	// A new Expression instance is created along with the Node instance
	// The Node is setup with its input and output pins reflecting
	// the Signature of the Expression.
    UE_API UTG_Node* CreateExpressionNode(const UClass* ExpressionClass);

	// Create a node from an initialized Expression
	// A new Node instance is created which uses initialized expression
	// The Node is setup with its input and output pins reflecting
	// the Signature of the Expression.
    UE_API UTG_Node* CreateExpressionNode(UTG_Expression* NewExpression);
	
	// Remove the node
	// The node is destroyed, its Uuid becomes invalid
	// The associated Expression is dereferenced and potentially GCed
	// All the associated Pins are destroyed
	// As a consequence, each pin's
	//     associated connection Edge(s) are destroyed
	//	   associated Var is dereferenced and potentially GCed
	UE_API void RemoveNode(UTG_Node* InNode);

	// Accessed from TG_EdGraphNode after a paste of a new node
	UE_API void AddPostPasteNode(UTG_Node* NewNode);

    // Connect a output pin from a node to the specified input pin of the to node
    // Return true if the edge is created or false if couldn't create the edge
	UE_API bool Connect(UTG_Node& NodeFrom, FTG_Name& PinFrom, UTG_Node& NodeTo, FTG_Name& PinTo);
	
    // Check that the connection from PinFrom to PInTO doesn't create a loop
	static UE_API bool ConnectionCausesLoop(const UTG_Pin* PinFrom, const UTG_Pin* PinTo);

	// Are 2 Pins compatible ?
	// This function check that the 2 pins can be connected to each other in terms of their argument types
	// A non empty ConverterKey is also returned if required to achieve a conversion of the pin A to the pin B
	// The converterKey is then used during the evaluation to find the proper Converter
	static UE_API bool ArePinsCompatible(const UTG_Pin* PinFrom, const UTG_Pin* PinTo, FName& ConverterKey);

	// Remove all the edges connected to the specified Pin
	UE_API void RemovePinEdges(UTG_Node& InNode, FTG_Name& InPin);

	// Remove a particular edge
	UE_API void RemoveEdge(UTG_Node& NodeFrom, FTG_Name& PinFrom, UTG_Node& NodeTo, FTG_Name& PinTo);

	// Reset the graph empty, destroy any nodes or associated resources
	UE_API void Reset();

	// Create a Signature from the current graph topology declaring the current Params as Arguments
	// The signature list the input and output pins of the graph
	UE_API void AppendParamsSignature(FTG_Arguments& InOutArguments, TArray<FTG_Id>& InParams, TArray<FTG_Id>& OutParams) const;

	//////////////////////////////////////////////////////////////////////////
	// Accessors for nodes, pins, params
	//////////////////////////////////////////////////////////////////////////

	// Access a Node from Id
	UTG_Node*		GetNode(FTG_Id NodeId) { return IsValidNode(NodeId) ? Nodes[NodeId.NodeIdx()] : nullptr; }
	const UTG_Node* GetNode(FTG_Id NodeId) const { return IsValidNode(NodeId) ? Nodes[NodeId.NodeIdx()] : nullptr; }
    
	// Access a Pin from Id
	UTG_Pin*		GetPin(FTG_Id PinId) { return IsValidPin(PinId) ? Nodes[PinId.NodeIdx()]->Pins[PinId.PinIdx()].Get() : nullptr; }
	const UTG_Pin*	GetPin(FTG_Id PinId) const { return IsValidPin(PinId) ? Nodes[PinId.NodeIdx()]->Pins[PinId.PinIdx()].Get() : nullptr; }
	
	// Access the Node owning a Pin from the Pin Id
	UTG_Node*		GetNodeFromPinId(FTG_Id PinId) { return GetNode( FTG_Id(PinId.NodeIdx()) ); }
	
	// Access a Var from Id (the Id of a Var is similar to the Id of the Pin that owns it
	FTG_Var*		GetVar(FTG_Id VarId) { UTG_Pin* Pin = GetPin(VarId); return (Pin ? Pin->EditSelfVar() : nullptr); }

	// Accessing the params
	// A param is simply the Id of a pin exposed as a param of the graph, the pin can be retrieved using GetPin()
	// The Param name is the Alias Name of the associated TG_Pin

	FTG_Ids			GetParamIds() const { FTG_Ids Array;  Params.GenerateValueArray(Array); return Array; }
	TArray<FName>	GetParamNames() const { TArray<FName> Array;  Params.GenerateKeyArray(Array); return Array; }

	UE_API FName			GetParamName(const FTG_Id& InParamId) const;

	UE_API FTG_Ids			GetInputParamIds() const;
	UE_API FTG_Ids			GetOutputParamIds() const;

	// Find a param pin from its name
	FTG_Id			FindParamPinId(const FName& InName) const		{ auto* PinId = Params.Find(InName); return (PinId ? (*PinId) : FTG_Id()); }
	const UTG_Pin*	FindParamPin(const FName& InName) const			{ return GetPin(FindParamPinId(InName)); }
	UTG_Pin*		FindParamPin(const FName& InName)				{ return GetPin(FindParamPinId(InName)); }

	// Get the names and ids of the output params which are Textures
	UE_API int				GetOutputParamTextures(TArray<FName>& OutNames, FTG_Ids& OutPinIds) const;

	// Iterate through all the VALID nodes
    UE_API void ForEachNodes(std::function<void(const UTG_Node* /*node*/, uint32 /*index*/)> visitor) const;

	// Iterate through all the VALID pins
	UE_API void ForEachPins(std::function<void(const UTG_Pin* /*pin*/, uint32 /*index*/, uint32 /*node_index*/)> visitor) const;

	// Iterate through all the VALID vars
	UE_API void ForEachVars(std::function<void(const FTG_Var* /*var*/, uint32 /*index*/, uint32 /*node_index*/)> visitor) const;

	// Iterate through all the VALID Param pins
	UE_API void ForEachParams(std::function<void(const UTG_Pin* /*pin*/, uint32 /*index*/)> visitor) const;

	// Iterate through all the VALID edges
	UE_API void ForEachEdges(std::function<void(const UTG_Pin* /*pinFrom*/, const UTG_Pin* /*pinTo*/)> visitor) const;

	// Iterate through all the output settings
	UE_API void ForEachOutputSettings( std::function<void(const FTG_OutputSettings& /*settings*/)> visitor);

	//////////////////////////////////////////////////////////////////////////
	// Accessors for output param values after evaluation
	// Only valid AFTER Evaluation of the graph
	//////////////////////////////////////////////////////////////////////////

	// Get the nammed <InName> output param value as FTG_Variant if it exists.
	// The value is collected in the <OutVariant> parameter if found.
	// return true if the correct nammed & typed param was found, false otherwise
	UE_API bool GetOutputParamValue(const FName& InName, FTG_Variant& OutVariant) const;

	// Get all the output param's value as FTG_Variant
	// along with their names optionally
	UE_API int GetAllOutputParamValues(TArray<FTG_Variant>& OutVariants, TArray<FName>* OutNames = nullptr) const;

	// Get a map of input parameter vars from the graph
	UE_API FTG_VarMap GetInputParamsVarMap();

	// Apply an override map of input parameter vars to the graph
    UE_API void SetInputParamsFromVarMap(FTG_VarMap InMap);

	// get a map of output settings from the graph
	UE_API void CollectOutputSettings(TMap<FTG_Id, FTG_OutputSettings>& OutExportSettingsMap) const;

	// Apply an override map of output settings to the graph
	UE_API void SetOutputSettings(const TMap<FTG_Id, FTG_OutputSettings>& OutputSettingsMap);

    //////////////////////////////////////////////////////////////////////////
	// Traverseal of the graph and evaluation 
	//////////////////////////////////////////////////////////////////////////

	// Useful function to navigate the Node
	// Collect all the DIRECT nodes whose output pins which are
	// connected to the input pins of this node
    UE_API TArray<FTG_Id> GatherSourceNodes(const UTG_Node* Node) const;

	// Useful function to navigate the Nodes
	// Collect all the nodes which are connecting directly or indirectly into the specified node
	UE_API TArray<FTG_Id> GatherAllSourceNodes(const UTG_Node* Node) const;

	//// Graph methods to traverse the graph!
	
	// Validate internal checks, warnings and errors, Returns true if Graph is valid and has no validation errors
	UE_API bool Validate(MixUpdateCyclePtr	Cycle);

	// Access the traversal  (update it if needed)
	UE_API const FTG_GraphTraversal& GetTraversal() const;

    // Traversal Visitor function type
    // Node visited
    // Index of the node in the traverse order
    // Subgraph level we are currently visiting
    using NodeVisitor = void (*) (UTG_Node*, int32, int32);
    using NodeVisitorFunction = std::function<void (UTG_Node*, int32, int32)>;

    // Traverse!
    UE_API void Traverse(NodeVisitorFunction visitor, int32 graph_depth = 0) const;

	// Evaluate the graph
	// Traverse every nodes in the traversal order and call the node's expression evaluate method
	// A configured evaluation context with a valid Cycle is required
	// Evaluation is concretely implemented in FTG_Evaluation struct
	UE_API void Evaluate(FTG_EvaluationContext* InContext);

#if WITH_EDITOR
	const TArray<TObjectPtr<UObject>>& GetExtraEditorNodes() const { return ExtraEditorNodes; }
	UE_API void SetExtraEditorNodes(const TArray<TObjectPtr<const UObject>>& InNodes);
#endif
	
protected:
	// Utility functions used to establish the traversal order
	UE_API void GatherOuterNodes(const TArray<FTG_Ids>& sourceNodesPerNode, TSet<FTG_Id>& nodeReservoirA, TSet<FTG_Id>& nodeReservoirB) const;
	UE_API void EvalInOutPins() const;
	UE_API void EvalTraverseOrder() const;

#if WITH_EDITORONLY_DATA
	// Extra data to hold information that is useful only in editor (like comments)
	UPROPERTY()
	TArray<TObjectPtr<UObject>> ExtraEditorNodes;
#endif // WITH_EDITORONLY_DATA

	
	// Pure data struct evaluated at runtime when traversal is required
	// And a dirty flag
	UPROPERTY(Transient)
	mutable bool					bIsGraphTraversalDirty = true;
	mutable FTG_GraphTraversal		Traversal;
	
public:
	// Notifiers
#if WITH_EDITOR
	FOnTGNodeSignatureChanged OnNodeSignatureChangedDelegate;
	FOnTextureGraphChanged OnGraphChangedDelegate;
	FOnNodeEvaluation OnNodePostEvaluateDelegate;
	FOnTGNodeAdded OnTGNodeAddedDelegate;
	FOnTGNodeRemoved OnTGNodeRemovedDelegate;
	FOnTGNodeRenamed OnTGNodeRenamedDelegate;
#endif

	//// Logging Utilities

	// Log the graph description to the log output buffer
	UE_API void Log() const;

	static constexpr int32 LogHeaderWidth = 32; // width in number of chars for any header token during logging
	UE_API FString LogNodes(FString InTab = TEXT("")) const;
	UE_API FString LogParams(FString InTab = TEXT("")) const;
	UE_API FString LogTraversal(FString InTab = TEXT("")) const;
	UE_API FString LogVars(FString InTab = TEXT("")) const;
	static UE_API FString LogCall(const TArray<FTG_Id>& PinInputs, const  TArray<FTG_Id>& PinOutputs, int32 InputLogWidth = 10);

	//// Testing zone
	
#ifdef UE_BUILD_DEBUG
	// 0 = DiskVersion, 
	// 1 = Runtime copy made by TG_Editor, 
	// 2 = Runtime copy made by TG_Expression_Graph
	int                 IsRuntime = 0;
#endif
private:
};




#undef UE_API
