// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Model/Mix/MixInterface.h"
#include "TG_Var.h"
#include "TG_GraphEvaluation.generated.h"

#define UE_API TEXTUREGRAPH_API
class UTG_Graph;
class UTG_Node;
	
USTRUCT()
struct FVarArgument
{
	GENERATED_BODY()

	UPROPERTY()
	FTG_Var Var;

	UPROPERTY()
	FTG_Argument Argument;

	// Custom Serialize method for FVarArgument
	bool Serialize(FArchive& Ar)
	{
		FTG_Argument::StaticStruct()->SerializeItem(Ar, ((void*) &Argument), nullptr);
		// Argument.Serialize(Ar);
		Argument.SetPersistentSelfVar();

		Var.Serialize(Ar, FTG_Id(), Argument);
		
		return true;
	}
};
template<>
struct TStructOpsTypeTraits<FVarArgument> : public TStructOpsTypeTraitsBase2<FVarArgument>
{
	enum
	{
		WithSerializer = true,  // Enables the use of a custom Serialize method
	};
};

struct FTG_OutputVarMap
{
	struct FOutputVarArgument
	{
		FTG_Var* Var;
		FTG_Argument Argument;
	};
	
	TMap<FName, FOutputVarArgument> VarArguments;

	FTG_Var* GetVar(FName Name)
	{
		auto Element = VarArguments.Find(Name);
		
		if (Element)
		{
			return Element->Var;
		}

		return nullptr;
	}

	const FTG_Argument* GetVarArgument(FName Name)
	{
		auto Element = VarArguments.Find(Name);
		
		if (Element)
		{
			return &Element->Argument;
		}

		return nullptr;
	}

	void Empty()
	{
		VarArguments.Empty();
	}
};

USTRUCT()
struct FTG_VarMap
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FName, FVarArgument> VarArguments;

	FTG_Var* GetVar(FName Name)
	{
		auto Element = VarArguments.Find(Name);
		
		if (Element)
		{
			return &Element->Var;
		}

		return nullptr;
	}

	const FTG_Argument* GetVarArgument(FName Name)
	{
		auto Element = VarArguments.Find(Name);
		
		if (Element)
		{
			return &Element->Argument;
		}

		return nullptr;
	}

	void Empty()
	{
		VarArguments.Empty();
	}
};

// 
struct FTG_EvaluationContext
{
	// The cycle used to call in the concrete TextureGraph engine system
	MixUpdateCyclePtr	Cycle;

	// in and out buckets of vars passed as the arguments to the expression evaluate call
	// For each node/expression, these are populated in the traverse/evaluate call in TG_Graph
	FTG_VarMap			Inputs;
	FTG_OutputVarMap	Outputs;
	
	// These are the Ids of the Vars connected as Param to an upper Graph
	// These are persistent in the scope of a graph evaluation
	TArray<FTG_Id>		ConnectedInputParamIds;
	TArray<FTG_Id>		ConnectedOutputParamIds;

	UTG_Graph*			Graph = nullptr;
	UTG_Node*			CurrentNode = nullptr;

	int32				GraphDepth = 0;
	
	int32				TargetId = 0;

	// For debug, let's log the evaluation call sequence while it happens
	bool				bDoLog = true;

	FORCEINLINE bool	IsTweaking() const
	{
		return Cycle ? Cycle->IsTweaking() : false;
	}
};

class UTG_Pin;
class UTG_Node;
struct FTG_Evaluation
{
	static UE_API const FString GVectorToTextureAutoConv_Name;
	static UE_API const FString GColorToTextureAutoConv_Name;
	static UE_API const FString GFloatToTextureAutoConv_Name;

	static UE_API void EvaluateGraph(UTG_Graph* InGraph, FTG_EvaluationContext* Context);
	static UE_API void EvaluateNodeArray(UTG_Node* InNode, const TArray<TObjectPtr<UTG_Pin>>& ArrayInputs, 
		const TArray<TObjectPtr<UTG_Pin>>& NonArrayPins, int MaxCount, FTG_EvaluationContext* Context);
	static UE_API void EvaluateNode(UTG_Node* InNode, FTG_EvaluationContext* Context);

	static UE_API void TransferVarToPin(UTG_Pin* InPin, FTG_EvaluationContext* Context, int Index);
	static UE_API int FilterArrayInputs(FTG_EvaluationContext* InContext, const TArray<TObjectPtr<UTG_Pin>>& InPins, 
		TArray<TObjectPtr<UTG_Pin>>& ArrayPins, TArray<TObjectPtr<UTG_Pin>>& NonArrayPins);

	// Converter used to introduce transformation from one var to another var when the arguments are compatible but require to be converted
	struct VarConverterInfo
	{
		FTG_Var* InVar = nullptr;
		FTG_Var* OutVar = nullptr;
		int Index = -1;
		FTG_EvaluationContext* Context = nullptr;
	};
	typedef TFunction<void(VarConverterInfo& Info)> VarConverter;

	typedef TMap				<FName, VarConverter> ConverterMap;
	static UE_API ConverterMap			DefaultConverters;
	static UE_API FName				MakeConvertKey(FName From, FName To);
	static UE_API FName				MakeConvertKey(const FTG_Argument& ArgFrom, const FTG_Argument& ArgTo);

	// Conformer functors used to conform pin values
	struct VarConformerInfo
    {
    	FTG_Var* InVar = nullptr;
    	FTG_Var* OutVar = nullptr;
		int Index = -1;
		FTG_EvaluationContext* Context = nullptr;
    };
    typedef TFunction<bool(VarConformerInfo& Info)> VarConformer;
	
	static UE_API bool AreArgumentsCompatible(const FTG_Argument& ArgFrom, const FTG_Argument& ArgTo, FName& ConverterKey);

};

#undef UE_API
