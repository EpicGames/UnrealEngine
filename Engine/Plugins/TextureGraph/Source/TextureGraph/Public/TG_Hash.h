// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TG_SystemTypes.h"
#include <algorithm>

FTG_Hash TG_Hash(FTG_Hash H);
FTG_Hash TG_HashName(FTG_Name& Name);

 