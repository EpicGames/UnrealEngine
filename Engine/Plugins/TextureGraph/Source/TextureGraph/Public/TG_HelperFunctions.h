// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Job/JobBatch.h"
#include "Export/TextureExporter.h"
#include "Data/Blob.h"
#include "TG_Graph.h"
#include "2D/TextureHelper.h"

#define UE_API TEXTUREGRAPH_API

class UTextureGraphBase;

class FTG_HelperFunctions
{

public:
	static UE_API void	InitTargets(UTextureGraphBase* InTextureGraph);
	static UE_API TArray<BlobPtr> GetTexturedOutputs(const UTG_Node* Node, FTG_EvaluationContext* TextureConversionContext = nullptr);

	static UE_API void EnsureOutputIsTexture(MixUpdateCyclePtr Cycle, UTG_Node* OutputNode);

	static UE_API JobBatchPtr InitExportBatch(UTextureGraphBase* InTextureGraph, FString ExportPath, FString AssetName, FExportSettings& TargetExportSettings, bool OverrideExportPath, bool OverwriteTextures, bool ExportAllOutputs, bool bSave);
	static UE_API AsyncInt ExportAsync(UTextureGraphBase* InTextureGraph, FString ExportPath, FString AssetName, FExportSettings& TargetExportSettings, bool OverrideExportPath, bool OverwriteTextures = true, bool ExportAllOutputs = false, bool bSave = true);

	static UE_API JobBatchPtr InitRenderBatch(UTextureGraphBase* InTextureGraph, JobBatchPtr ExistingBatch = nullptr);
	static UE_API AsyncBool RenderAsync(UTextureGraphBase* InTextureGraph, JobBatchPtr ExistingBatch = nullptr);

	template <typename T_Type>
	static TArray<T_Type> GetOutputsOfType(const UTG_Node* Node)
	{
		TArray<T_Type> Outputs;

		if (Node)
		{
			auto OutPinIds = Node->GetOutputPinIds();
			for (auto Id : OutPinIds)
			{
				//This is a work around for checking the type of the output
				//Probably we need to have a better solution for checking output type
				auto Pin = Node->GetGraph()->GetPin(Id);
				T_Type Output;
				Pin->GetValue(Output);

				Outputs.Add(Output);
			}
		}

		return Outputs;
	}
};

#undef UE_API
