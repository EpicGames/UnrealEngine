// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TG_SystemTypes.h"
#include "Materials/Material.h"
#include "Misc/OutputDeviceNull.h"
#include "TG_Material.generated.h"

#define UE_API TEXTUREGRAPH_API

USTRUCT(BlueprintType)
struct FTG_Material
{
	GENERATED_USTRUCT_BODY()
	
public:	
	UPROPERTY(Transient, EditAnywhere, Category = "Material", meta = (AllowedClasses = "/Script/Engine.MaterialInterface"))
	FSoftObjectPath AssetPath;

	FTG_Material() = default;
	FTG_Material(const FTG_Material& InSrc) = default;
	FTG_Material& operator = (const FTG_Material& RHS) = default;

	// Check that the referenced asset is valid.
	// If true, the GetMaterial call will NOT return nullptr
	UE_API bool IsValid() const;

	// Access the referenced Material asset
	// return a valid MaterialInterface object or null if the reference is invalid.
	UE_API UMaterialInterface* GetMaterial() const;

	// Assign the referenced asset from an actual UMaterialInsterface* live object
	UE_API void SetMaterial(UMaterialInterface* InMaterial);

	UE_API bool Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FTG_Material& InMaterial);
	bool operator==(const FTG_Material& RHS) const = default;
	UE_API void ResetTexturePath();

	void InitFromString(const FString& StrVal)
	{
		FOutputDeviceNull NullOut;
		FTG_Material::StaticStruct()->ImportText(*StrVal, this, /*OwnerObject*/nullptr, 0, &NullOut, FTG_Material::StaticStruct()->GetName(), /*bAllowNativeOverride*/true);
	}
	FString ToString() const
	{
		FString ExportString;
		FTG_Material::StaticStruct()->ExportText(ExportString, this, this, /*OwnerObject*/nullptr, /*PortFlags*/0, /*ExportRootScope*/nullptr);
		return ExportString;
	}

};


template<>
struct TStructOpsTypeTraits<FTG_Material>
	: public TStructOpsTypeTraitsBase2<FTG_Material>
{
	enum
	{
		WithSerializer = true,
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};

#undef UE_API
