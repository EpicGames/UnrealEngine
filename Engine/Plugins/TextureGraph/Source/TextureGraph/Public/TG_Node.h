// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TG_SystemTypes.h"
#include "TG_Signature.h"
#include "TG_Pin.h"
#include "TG_Variant.h"

#include "TG_Node.generated.h"

#define UE_API TEXTUREGRAPH_API

class UTG_Graph;
class UTG_Expression;

USTRUCT()
struct FTG_NodeEditorData
{
	GENERATED_BODY()

	UPROPERTY()
	int32 PosX = 0;
	UPROPERTY()
	int32 PosY = 0;

	UPROPERTY()
	FString NodeComment;

	UPROPERTY()
	bool bCommentBubblePinned = false;

	UPROPERTY()
	bool bCommentBubbleVisible = false;
};

// A Node in the graph
// Created from the graph
// Expose the Input and Output pins
//
// UTG_Node is created from a UTG_Graph
//
UCLASS(MinimalAPI)
class UTG_Node : public UObject
{
    GENERATED_BODY()

    friend class UTG_Graph;
	TSharedPtr<FTG_Signature> Signature;

	// Note: This property is assigned at construction of the node in the Construct call
	UPROPERTY()
	TObjectPtr<UTG_Expression> Expression = nullptr;

	// Assigned by Graph and restored postLoad
	UPROPERTY(Transient, TextExportTransient)
	FTG_Id Id; // invalid by default

private:
    UE_API void Construct(UTG_Expression* InExpression);

	// Initialize the node after being created from the graph or unserialized
	// This is when the Graph assign itself and the Id to the node
	// The expression member must be valid:
	//		- assigned with the Construct call above for a brand new node
	//		- or recovered from serialization
	UE_API void Initialize(FTG_Id Id);

	// Evaluate the arguments array deduced from the current array of Pins and the arguments
	// This is used to compare the validity of the PIn's against the Expression signature
	UE_API FTG_Arguments GetPinArguments() const;

	// Check that the node signature deduced from the pin's array is the same as the signature from the Expression
	// return false if it is not the case, true otherwise
	UE_API bool CheckPinSignatureAgainstExpression() const;

public:

	// Override Serialize method of UObject
	UE_API virtual void Serialize(FArchive& Ar) override;

	// The Graph in which this node exists
	UE_API UTG_Graph* GetGraph() const;

	// The expression owned by this node
	UTG_Expression* GetExpression() const { return Expression; }

	// Get the Uuid for this node
	FTG_Id GetId() const { return Id; }

	// Validate internal checks, warnings and errors
    UE_API virtual bool Validate(MixUpdateCyclePtr	Cycle);

	// Node name is the title name of the inner expression
	UE_API FTG_Name GetNodeName() const;

	//////////////////////////////////////////////////////////////////////////
	// Signature of the node and predicates about the pins of the node
	//////////////////////////////////////////////////////////////////////////

	UE_API const FTG_Signature& GetSignature() const;

	bool HasInputs() const { return GetSignature().HasInputs(); }
	bool HasOutputs() const { return GetSignature().HasOutputs(); }
	bool HasPrivates() const { return GetSignature().HasPrivates(); }

	bool IsParam() const { return GetSignature().IsParam(); }
	bool IsInputParam() const { return GetSignature().IsInputParam(); }
	bool IsOutputParam() const { return GetSignature().IsOutputParam(); }

	//////////////////////////////////////////////////////////////////////////
	// Accessors for pins of the node
	// All the pins are stored packed in the <Pins> member
	//////////////////////////////////////////////////////////////////////////

    UPROPERTY()
	TArray<TObjectPtr<UTG_Pin>> Pins;

	// Iterate through all the Input OR Output pins in this node, return the number traversed
	UE_API int32 ForEachInputPins(std::function<void(const UTG_Pin* /*pin*/, uint32 /*index*/)> visitor) const;
	UE_API int32 ForEachOutputPins(std::function<void(const UTG_Pin* /*pin*/, uint32 /*index*/)> visitor) const;

	// Collect the input / output pins
	UE_API int32 GetInputPins(TArray<const UTG_Pin*>& OutPins) const;
	UE_API int32 GetOutputPins(TArray<const UTG_Pin*>& OutPins) const;
	UE_API int32 GetInputPins(TArray<UTG_Pin*>& OutPins) const;
	UE_API int32 GetOutputPins(TArray<UTG_Pin*>& OutPins) const;

	// Get pin Id from index in each scope
	UE_API FTG_Id GetInputPinIdAt(FTG_Index inIndex) const;
	UE_API FTG_Id GetOutputPinIdAt(FTG_Index outIndex) const;
	UE_API FTG_Id GetPrivatePinIdAt(FTG_Index privateIndex) const;

	UE_API TArray<FTG_Id> GetInputPinIds() const;
	UE_API TArray<FTG_Id> GetOutputPinIds() const;
	UE_API TArray<FTG_Id> GetPrivatePinIds() const;

	// Get/Find the Pin Id for Pins of this Node from its Argument Name
	UE_API FTG_Id GetPinId(FName Name) const;
	UE_API FTG_Id GetInputPinID(FTG_Name& Name) const;
	UE_API FTG_Id GetOutputPinID(FTG_Name& Name) const;
	UE_API FTG_Id GetPrivatePinID(FTG_Name& Name) const;

    UE_API UTG_Pin* GetInputPinAt(FTG_Index inIndex) const;
	UE_API UTG_Pin* GetOutputPinAt(FTG_Index outIndex) const;
	UE_API UTG_Pin* GetPrivatePinAt(FTG_Index privateIndex) const;

	UE_API UTG_Pin* GetPin(FName Name) const;
	UE_API UTG_Pin* GetInputPin(FTG_Name& name) const;
	UE_API UTG_Pin* GetOutputPin(FTG_Name& name) const;
	UE_API UTG_Pin* GetPrivatePin(FTG_Name& name) const;

	UE_API UTG_Pin* GetPin(FTG_Id pinID) const;

	UE_API TArray<FTG_Id> GetInputVarIds() const;
	UE_API TArray<FTG_Id> GetOutputVarIds() const;

	//////////////////////////////////////////////////////////////////////////
	// Accessors for output values after evaluation
	// Only valid AFTER Evaluation of the graph
	//////////////////////////////////////////////////////////////////////////

	// Get all the output's values which are compatible with FTG_Variant
	// along with their names optionally
	UE_API int GetAllOutputValues(TArray<FTG_Variant>& OutVariants, TArray<FName>* OutNames = nullptr) const;

	// Get all the output's values which are compatible with FTG_Texture
	// along with their names optionally
	UE_API int GetAllOutputValues(TArray<FTG_Texture>& OutTextures, TArray<FName>* OutNames = nullptr) const;

	// Get the current common input variant type of the expression
	// only valid for expression with FTG_Variant arguments
	UE_API FTG_Variant::EType GetExpressionCommonInputVariantType() const;

	UE_API void OnPinConnectionUndo(FTG_Id InPinId);

protected:

	// Return the array of alias name of all the pins
	// Can be filtered based on the argument access type bitfield
	// Default is not filtered, aka AccessFilter is 0
	UE_API TArray<FName> GetPinAliasNames(uint8 AccessFilter = 0) const;

	// Check that a candidate Pin's alias name is unique in the Node's Pins
	// And in the Graph's Param Pins if it is a Param.
	// If not edit it to propose a unique name that could work
	// return the CandidateName as is or the modified version
	// This is meant to be used from within the Pin::SetAliasName call
	// or when a new pin is allocated to make sure its default alias name doesn't collide in
	// the scope of the node or of the graph if it is a param
	UE_API FName ValidateGeneratePinAliasName(FName CandidateName, const FTG_Id& PinId) const;

	// Validate that the node conforms to a conformant function coming from the Expression
    UE_API void ValidateGenerateConformer(UTG_Pin* NewPin);

	// Trigger notification to the graph that the node pin's value (default input or output) has changed  
	UE_API void NotifyGraphOfNodeChange(bool bIsTweaking);

	// Embedded expression is able to notify state changes:
	friend class UTG_Expression;
	friend class UTG_Pin;
	UE_API void OnExpressionChangedWithoutVar(const FPropertyChangedEvent& PropertyChangedEvent);
	UE_API void OnSignatureChanged();
	UE_API void OnPinRenamed(FTG_Id InPinId, FName OldName);

	UE_API void OnPinConnectionChanged(FTG_Id InPinId, FTG_Id OldPinId, FTG_Id NewPinId);

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
	UE_API void OnPostUndo();
#endif
	
	TSet<FName> WarningStack;

public:

#if WITH_EDITORONLY_DATA
	UPROPERTY()
		FTG_NodeEditorData EditorData;
#endif

	//// Logging Utilities
	UE_API FString LogHead() const;
	UE_API FString LogPins(FString Tab) const;
};

using FTG_Nodes = TArray<UTG_Node*>;


#undef UE_API
