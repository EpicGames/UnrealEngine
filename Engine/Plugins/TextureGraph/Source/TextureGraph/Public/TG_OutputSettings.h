// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TG_SystemTypes.h"
#include "Model/Mix/MixSettings.h"
#include "Misc/OutputDeviceNull.h"
#include "TG_OutputSettings.generated.h"

#define UE_API TEXTUREGRAPH_API

class UTG_Expression_Output;
class UTG_Node;
class UTextureGraph;

USTRUCT()
struct FOutputExpressionInfo
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FName OutputName;

	FTG_Id OutputNodeID;

	bool operator==(const FOutputExpressionInfo& Other) const
	{
		return OutputName == Other.OutputName;
	}
};

UCLASS(MinimalAPI)
class UTG_ExportSettings : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, EditFixedSize, Category = NoCategory, meta = (HideItemCount, NoResetToDefault, EditFixedOrder, ShowOnlyInnerProperties, FullyExpand))
	TArray<FOutputExpressionInfo>	OutputExpressionsInfos;
};

USTRUCT(BlueprintType)
struct FTG_OutputSettings
{
	GENERATED_USTRUCT_BODY()

	// Export name of the textured asset.
	UPROPERTY(BlueprintReadWrite,EditAnywhere, Category = "Basic", DisplayName = "File Name", Meta = (NoResetToDefault))
	FName BaseName;

	UPROPERTY()
	FName OutputName;

	// Export path for the textured asset.
	UPROPERTY(BlueprintReadWrite,EditAnywhere, Category = "Basic", DisplayName = "Path", Meta = (NoResetToDefault))
	FName FolderPath;

	// Width of the texture in pixels. Auto means system will detect automatically based on other images
	UPROPERTY(BlueprintReadWrite,EditAnywhere, Category = "Advanced", Meta = (NoResetToDefault))
	EResolution Width = EResolution::Auto;

	// Height of the texture in pixels. Auto means system will detect automatically based on other images
	UPROPERTY(BlueprintReadWrite,EditAnywhere, Category = "Advanced", Meta = (NoResetToDefault))
	EResolution Height = EResolution::Auto;

	// List of available texture formats. Auto means system will detect automatically based on the input
	UPROPERTY(BlueprintReadWrite,EditAnywhere, Category = "Advanced", DisplayName = "Texture Format", Meta = (NoResetToDefault))
	ETG_TextureFormat TextureFormat = ETG_TextureFormat::BGRA8;

	// List of available texture presets available for export. 
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Advanced", DisplayName = "Texture Type", Meta = (NoResetToDefault))
	ETG_TexturePresetType TexturePresetType = ETG_TexturePresetType::None;

	// The Level of detail group of the texture
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Advanced", DisplayName = "LOD Texture Group", Meta = (NoResetToDefault, EditCondition = "TexturePresetType == ETG_TexturePresetType::None"))
	TEnumAsByte<enum TextureGroup> LODGroup = TextureGroup::TEXTUREGROUP_World;

	// Compression methods available for exporting textured asset.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Advanced", DisplayName = "Compression", Meta = (NoResetToDefault, EditCondition = "TexturePresetType == ETG_TexturePresetType::None") )
	TEnumAsByte <enum TextureCompressionSettings> Compression = TextureCompressionSettings::TC_Default;

	// Adjust the color space of exporting textured asset. Can be in Linear or Gamma color space. 
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Advanced", DisplayName = "sRGB", Meta = (NoResetToDefault, EditCondition = "TexturePresetType == ETG_TexturePresetType::None"))
	bool bSRGB = false;

	UPROPERTY()
	bool bShouldExport = true;

	FString GetFullOutputName() { return  FString::Format(TEXT("{0}"), { BaseName.ToString()});}
	friend FArchive& operator<<(FArchive& Ar, FTG_OutputSettings& T);
	
	bool operator==(const FTG_OutputSettings& Other) const
	{
		return OutputName == Other.OutputName && BaseName == Other.BaseName;
	}

	UE_API void Initialize(FString PathName, FName InName = "Output");

	void InitFromString(const FString& StrVal)
	{
		FOutputDeviceNull NullOut;
		FTG_OutputSettings::StaticStruct()->ImportText(*StrVal, this, /*OwnerObject*/nullptr, 0, &NullOut, FTG_OutputSettings::StaticStruct()->GetName(), /*bAllowNativeOverride*/true);
	}

	FString ToString() const
	{
		FString ExportString;
		FTG_OutputSettings::StaticStruct()->ExportText(ExportString, this, this, /*OwnerObject*/nullptr, /*PortFlags*/0, /*ExportRootScope*/nullptr);
		return ExportString;
	}

	UE_API void Set(int InWidth, int InHeight, FName Name = "None", FName Path = "None", ETG_TextureFormat Format = ETG_TextureFormat::BGRA8, ETG_TexturePresetType InTextureType = ETG_TexturePresetType::None,
		TextureCompressionSettings InCompression = TextureCompressionSettings::TC_Default, TextureGroup InLodGroup = TextureGroup::TEXTUREGROUP_World, bool InbSRGB = false);

	UE_API void OnSetTexturePresetType(ETG_TexturePresetType Type);
	UE_API bool Validate(FString& OutErrors) const;
};

#undef UE_API
