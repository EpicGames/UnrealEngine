// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TG_GraphEvaluation.h"
#include "TG_SystemTypes.h"
#include "TG_Signature.h"
#include "TG_Var.h"
#include "TG_Texture.h"
#include "TG_Variant.h"

#include "TG_Pin.generated.h"

#define UE_API TEXTUREGRAPH_API

// Pin are simple structs connecting the Nodes in the context of a Graph.
// They are not intended to be derived.


class UTG_Node;
class UTG_Graph;
class FAssetThumbnail;
// Pin is a struct referencing:
// the Node where it belongs and the ArgumentHash to the corresponding Argument in the Signature
// The Edge(s) connecting to this pin, aka the other pinId to which this pin is connected.
// The Var containing the default value for this pin
//
// Pins are stored in the Graph.
// Created when instancing a Node.
UCLASS(MinimalAPI)
class UTG_Pin : public UObject {
	GENERATED_BODY()

		friend class UTG_Node;
	friend class UTG_Graph;

	// Only accessible by UTG_Graph when allocating a new Pin.
	UE_API void Construct(FTG_Id InId, const FTG_Argument& InArgument);

	// Initialize the self var with the matching expression argument
	UE_API void InitSelfVar();

	UE_API void ResetSelfVarDefault();

	// Used when remapping a pin to a different id, update the sefl Var if as well
	UE_API void RemapId(FTG_Id InId);



	// Should be protected, Used by TG_Graph when updating connections
	// These have NO side effects, they just update the Pin members.
	UE_API void				RemoveAllEdges();
	UE_API void				RemoveEdge(FTG_Id OtherPinId);
	UE_API void				AddEdge(FTG_Id OtherPinId);
	UE_API void				RemapEdgeId(FTG_Id OldId, FTG_Id NewId);

	UE_API void				InstallInputVarConverterKey(FName ConverterKey);
	UE_API void				RemoveInputVarConverterKey();

	// Id member is assigned by TG_Graph at creation, potentially modified during a remap
	UPROPERTY()
		FTG_Id				Id;

	UPROPERTY()
		FTG_Argument		Argument; // The argument copied from the signature of the node

	UPROPERTY(Setter, Getter)
		FName				AliasName; // The Alias name for this pin that can be customized and overwrite the Argument name

	UPROPERTY()
		TArray<FTG_Id>		Edges; // The edge array contains the Pin Id of the other side of the connection

	// SelfVar is the Var owned by this pin containing the value.
	// For an input pin, SelfVar is containing the value of the pin without any connection
	// For an output pin, SelfVar is The var shared across all the connections with the other pins.
	// For a setting pin SelfVar is not valid, the value is accessed directly through the UProperty in the Expression
	UPROPERTY()
		FTG_Var				SelfVar;

	// Input var ConverterKey is indicating that the Pin is fed from a Pin/Var
	// which is compatible BUT requires a conversion during evaluation.
	UPROPERTY()
		FName				InputVarConverterKey;

	// When an InputVarConverterKey is installed, then we host the result of the conversion as var value here
	// this member is not saved and updated on every evaluation
	// It is only useful during the evaluation to capture the value passed in and converted from a connected pin
	UPROPERTY(Transient)
		FTG_Var				ConvertedVar;

protected:
	UE_API void				NotifyPinSelfVarChanged(bool bIsTweaking = false);

#if WITH_EDITOR
	UE_API virtual void		PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
public:
	// Override serialization for conditional serialization of SelfVar
	UE_API virtual	void		Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	UE_API virtual bool		Modify(bool bAlwaysMarkDirty = true) override;
#endif

	bool				IsValid() const { return Id != FTG_Id::INVALID; }

	UE_API UTG_Node*			GetNodePtr() const;
	UE_API FTG_Id				GetNodeId() const;

	FTG_Id				GetId() const { return Id; }

	FTG_Argument		GetArgument() const { return Argument; }
	FTG_Name			GetArgumentName() const { return GetArgument().GetName(); }

	FName				GetAliasName() const { return AliasName; }
	UE_API void				SetAliasName(FName InAliasName); // Assign the alias name and notify the graph for change
	bool				HasAliasName() const { return GetArgumentName() != AliasName; }

	FTG_ArgumentType	GetArgumentType() const { return GetArgument().GetType(); }
	bool				IsInput() const { return GetArgument().IsInput(); }
	bool				IsOutput() const { return GetArgument().IsOutput(); }
	bool				IsParam() const { return GetArgument().IsParam(); }
	bool				IsSetting() const { return GetArgument().IsSetting(); }
	bool				IsPrivate() const { return GetArgument().IsPrivate(); }
	bool				IsNotConnectable() const { return GetArgument().IsNotConnectable(); }

	FName				GetArgumentCPPTypeName() const { return GetArgument().GetCPPTypeName(); }

	const TArray<FTG_Id>& GetEdges() const { return Edges; }
	bool				IsConnected() const { return !Edges.IsEmpty(); }

	// Access the SelfVar of the Pin containing the currrent default value for the pin
	// if an output pin, SelfVar contains the current result value produced by the node's expression by last evaluation
	// if an input pin, SelfVar contains the current 'default' value of the pin used as expression input IF the pin is not connected.		
	bool				IsValidSelfVar() const { return SelfVar.IsValid(); }
	const FTG_Var*		GetSelfVar() const { return &SelfVar; }
	FTG_Var*			EditSelfVar() { return &SelfVar; }
	FTG_Id				GetSelfVarId() const { return GetId(); } // SelfVarID is the same as this pin's id

	// GetVarId returns the Id of the current valid Var for this Pin.
	// if an output pin, it is SelfVar Id aka this pin Id
	// if an input pin:
	//		if NOT connected, it is SelfVar ID
	//		if connected, it is the SelfVar Id of the Connected Pin
	FTG_Id				GetVarId() const { return ((IsOutput() || Edges.IsEmpty()) ? GetId() : Edges[0]); }

	// In the case the pin is an input, and is connected and the feeding pin var needs a conversion
	// This implies that during the evaluation, the feeding Var is Converted into the SelfVar which is then input in the expression
	bool				ConnectionNeedsConversion() const { return ((IsInput() || IsSetting()) && IsConnected() && !InputVarConverterKey.IsNone()); }
	FName				GetInputVarConverterKey() const { return InputVarConverterKey; }

	// EditConvertedVar returns the internal Var used to input its value in the pin's expression AFTER conversion
	// Should only be accessed by the Graph Evaluation
	FTG_Var*			EditConvertedVar() { return &ConvertedVar; }

	UE_API FString				LogHead() const;
	UE_API FString				Log(FString Tab) const;
	UE_API FString				LogTooltip() const;

	// Access the FProperty associated to this pin in the Expression::Class of the node
	// A valid FProperty is returned if it exists, null otherwise
	UE_API FProperty*			GetExpressionProperty() const;

	FTG_Evaluation::VarConformer ConformerFunctor;
	bool				NeedsConformance() const { return ConformerFunctor != nullptr; }

	// Hashing
	static UE_API FTG_Hash		Hash(const UTG_Pin& Pin);
	FTG_Hash			Hash() const { return Hash(*this); }


	// Check the type of the argument's Var expected to be contained in the Pin's Var
	bool IsArgScalar() const { return Argument.IsScalar(); }
	bool IsArgBool() const { return Argument.IsBool(); }
	bool IsArgString() const { return Argument.IsString(); }
	bool IsArgColor() const { return Argument.IsColor(); }
	bool IsArgVector() const { return Argument.IsVector(); }
	bool IsArgArray() const { return Argument.IsArray() || (GetSelfVar() && GetSelfVar()->IsArray()); }
	bool IsArgTextureArray() const { return IsArgArray() && GetSelfVar() && GetSelfVar()->GetAs<FTG_VariantArray>().IsTexture(); }
	bool IsArgTexture() const { return Argument.IsTexture(); }
	bool IsArgVariant() const { return Argument.IsVariant(); }
	bool IsArgObject(UClass* InClass) const { return Argument.IsObject(InClass); }

	// Set/Get default value of the SelfVar of the Pin
	// The methods check that the type requested is compatible with the actual cpp type of the Pin's argument to proceed
	// The function return true if valid, false otherwise, if false they are NO OP

	// Scalar getter only on a scalar argument
	UE_API bool GetValue(float& OutValue) const;

	// bool getter only on a boolean argument
	UE_API bool GetValue(bool& OutValue) const;

	// String getter only on a FString argument
	UE_API bool GetValue(FString& OutValue) const;

	// Color or Vector getter only on a vector or color argument
	UE_API bool GetValue(FLinearColor& OutValue) const;
	UE_API bool GetValue(FVector4f& OutValue) const;

	// Texture value getter only on a Texture argument
	UE_API bool GetValue(FTG_Texture& OutValue) const;

	// Texture value getter only on a Texture argument
	UE_API bool GetValue(FTG_VariantArray& OutValue) const;

	// Variant value getter only if a Variant argument OR if the argument is one of the supported type by Variant (Scalar/Color/Vector/Texture)
	// After getting the value, the next step is to check the type contained in the variant.
	UE_API bool GetValue(FTG_Variant& OutValue) const;

	// Useful for the UI:
	// access the current value of this pin as passed to the evaluation if it is an input
	// or received from the evaluation if it is an output 
	// if output pin: the selfVar
	// else if input pin:
	//		if NOT connected: the selfVar
	//		else if connected with a conversion: the convertedVar
	//		else if connected without conversion: the Var from the connected pin
	// This is MEANT TO BE USED for UI purpose !!!!
	UE_API FString				GetEvaluatedVarValue() const;

	// Set the self var value from string
	// Used by UI to assign pin default value
	// This is the same as the selfvar setter
	UE_API void				FromString(const FString& InValueStr, bool bIsTweaking = false);

	// Setters for Scalar, Color, Vector, bool and String  arguments
	// These check that the arg is of the correct type
	UE_API bool SetValue(float Value);
	bool SetValue(double Value) { return SetValue((float)Value); }
	bool SetValue(uint32 Value) { return SetValue((float)Value); }
	bool SetValue(int32 Value) { return SetValue((float)Value); }
	UE_API bool SetValue(bool bValue);
	UE_API bool SetValue(FString Value); // FName pin is assigned by the templated version, not FString
	UE_API bool SetValue(const FLinearColor& Value);
	UE_API bool SetValue(const FVector4f& Value);

	UE_API bool SetValue(const float* Value, size_t Count);
	bool SetValue(FTG_Texture&) { return false; } // this is not a possible case
	bool SetValue(FTG_VariantArray&) { return false; } // this is not a possible case
	UE_API bool SetValue(const FTG_TextureDescriptor& Value); 

	// Template version of the setter works correctly as long as the selfvar contains exactly the specified type
	// Use with care
	template <typename T>
	bool SetValue(const T& Value)
	{
		Modify();
		EditSelfVar()->EditAs<T>() = Value;
		NotifyPinSelfVarChanged();
		return true;
	}



};




#undef UE_API
