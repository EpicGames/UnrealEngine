// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TG_SystemTypes.h"       

#include "TG_Hash.h"


#include "TG_Signature.generated.h"

#define UE_API TEXTUREGRAPH_API


// Pin access either a in(put) or out(put) or ?
UENUM()
enum class ETG_Access : uint8
{
	In = 0,						// A regular Input access (read only in the expression)
	Out = 1,					// A regular Output access (write only in the expression)
	InParam = 2,				// An Input access also exported as a Param of the Graph
	OutParam = 3,				// An Output access also exported as a Param of the Graph
	InSetting = 4,				// A Setting is a regular input field of the expression, there is a default value that persists serialization
	OutSetting = 5,				// A Setting is a regular output field of the expression, there is a default value that persists serialization
	InParamSetting = 6,			// A Setting is a regular input field of the expression, also exported as a Param of the Graph
	OutParamSetting = 7,		// A Setting is a regular output field of the expression, also exported as a Param of the Graph
	Private = 8,				// A private field is a regular UProperty of the expression not exposed for connection

	OutputBitMask = 0x01,		// the bit mask for just the IsOutput flag (or not aka is Input)
	ParamBitMask = 0x02,		// the bit mask for just the Param flag
	SettingBitMask = 0x04,		// the bit mask for just the Setting flag
	PrivateBitMask = 0x08,	// the bit mask for just the PrivateField flag

	AccessBitMask = 0x0F,		// The bit mask for ALL the flags defining the Access

	PersistentSelfVarFlag = 0x10,	// the bit mask for just the Persistent Selfvar qualifier flag
	NotConnectableFlag = 0x20,      // The bit mask for the Not Connectable qualifier flag
	EnumFlag = 0x40,				// the bit mask for telling whether an argument is an enum or not
};

#define AccessToUint8(AccessEnum) static_cast<uint8>(ETG_Access::AccessEnum)

// Descriptor struct of an argument core desc
// all the typing information for a pin EXCEPT its name
USTRUCT()
struct FTG_ArgumentType
{
	GENERATED_BODY();

	FTG_ArgumentType() {}
	FTG_ArgumentType(ETG_Access InAccess) : Flags(static_cast<uint8>(InAccess)) {}
	FTG_ArgumentType(ETG_Access InAccess, uint8 Flags) : Flags(static_cast<uint8>(InAccess) | Flags) {}
	
	UPROPERTY()
	uint32			Flags = static_cast<uint8>(ETG_Access::In);

	ETG_Access		GetAccess() const	{ return static_cast<ETG_Access>(AccessToUint8(AccessBitMask) & Flags); }
	bool			IsInput() const		{ return !IsOutput(); }
	bool			IsOutput() const	{ return ((AccessToUint8(AccessBitMask) & Flags) & AccessToUint8(OutputBitMask)); }
	bool			IsParam() const		{ return ((AccessToUint8(AccessBitMask) & Flags) & AccessToUint8(ParamBitMask)); }
	bool			IsSetting() const	{ return ((AccessToUint8(AccessBitMask) & Flags) & AccessToUint8(SettingBitMask)); }
	bool			IsPrivate() const	{ return ((AccessToUint8(AccessBitMask) & Flags) & AccessToUint8(PrivateBitMask)); }
	bool			MatchAccessFilter(uint8 InFilter) const { return ((AccessToUint8(AccessBitMask) & Flags) & (InFilter == 0 ? AccessToUint8(AccessBitMask) : InFilter));  }

	bool			IsPersistentSelfVar() const { return (Flags & AccessToUint8(PersistentSelfVarFlag)) != 0; }
	void			SetPersistentSelfVar() { Flags = Flags | AccessToUint8(PersistentSelfVarFlag); }

	bool			IsNotConnectable() const { return (Flags & AccessToUint8(NotConnectableFlag)) != 0; }
	bool			IsEnum() const { return (Flags & AccessToUint8(EnumFlag)) != 0; }
	void			SetNotConnectable() { Flags = Flags | AccessToUint8(NotConnectableFlag); }


	FTG_ArgumentType	Unparamed() const
	{
		// Same Argument type without the Param type
		if (IsParam())
		{
			if (IsOutput())
				return { ETG_Access::Out };
			else
				return { ETG_Access::In };
		}
			
		return *this;
	}

	FString ToString() const
	{
		switch (GetAccess())
		{
		case ETG_Access::In:
			return (IsPersistentSelfVar() ? TEXT("in persistent") : TEXT("in"));
		case ETG_Access::Out:
			return (IsPersistentSelfVar() ? TEXT("out persistent") : TEXT("out"));
		case ETG_Access::InParam:
			return (IsPersistentSelfVar() ? TEXT("in param persistent") : TEXT("in param"));
		case ETG_Access::OutParam:
			return (IsPersistentSelfVar() ? TEXT("out param persistent") : TEXT("out param"));
		case ETG_Access::InSetting:
			return (IsPersistentSelfVar() ? TEXT("in setting persistent") : TEXT("in setting"));
		case ETG_Access::OutSetting:
			return (IsPersistentSelfVar() ? TEXT("out setting persistent") : TEXT("out setting"));
		case ETG_Access::InParamSetting:
			return (IsPersistentSelfVar() ? TEXT("in param setting persistent") : TEXT("in param setting"));
		case ETG_Access::OutParamSetting:
			return (IsPersistentSelfVar() ? TEXT("out param setting persistent") : TEXT("out param setting"));
		case ETG_Access::Private:
			return (IsPersistentSelfVar() ? TEXT("private persistent") : TEXT("private"));\
		};
		return FString::Printf(TEXT("undefined"));
	}

	friend bool operator== (const FTG_ArgumentType& lhs, const FTG_ArgumentType& rhs) {
		return (lhs.Flags == rhs.Flags);
	}
};
using FTG_ArgumentTypes = TArray<FTG_ArgumentType>;


// Argument of a signature:
// Combining a Name with an Argument type makes a unique Argument
// where the name is combined for hashing with
// the rest of the argument type fields
USTRUCT()
struct FTG_Argument
{
	GENERATED_BODY();

	UPROPERTY()
	FName Name;

	UPROPERTY()
	FName  CPPTypeName;

	UPROPERTY()
	FTG_ArgumentType ArgumentType;

	UPROPERTY()
	TMap<FName, FString> MetaDataMap; // We can add any additional information required here.
public:
	FTG_Name			GetName() const { return Name; }

	FTG_ArgumentType	GetType() const { return ArgumentType; }
	bool				IsInput() const { return ArgumentType.IsInput(); }
	bool				IsOutput() const { return ArgumentType.IsOutput(); }
	bool				IsParam() const { return ArgumentType.IsParam(); }
	bool				IsSetting() const { return ArgumentType.IsSetting(); }
	bool				IsPrivate() const { return ArgumentType.IsPrivate(); }

	bool				IsPersistentSelfVar() const { return ArgumentType.IsPersistentSelfVar(); }
	void				SetPersistentSelfVar() { ArgumentType.SetPersistentSelfVar(); }

	bool				IsNotConnectable() const { return ArgumentType.IsNotConnectable(); }
	void				SetNotConnectable() { return ArgumentType.SetNotConnectable(); }

	FName				GetCPPTypeName() const { return CPPTypeName; }

	UE_API bool				IsScalar() const; // float or int
	UE_API bool				IsBool() const; // boolean
	UE_API bool				IsString() const; // string  but NOT FName
	UE_API bool				IsColor() const; // FlinearColor
	UE_API bool				IsVector() const; // FLinearColor or ...
	UE_API bool				IsTexture() const; // FTG_Texture or ...
	UE_API bool				IsArray() const; // FTG_TextureArray or ...
	UE_API bool				IsVariant() const; // FTG_Variant
	UE_API bool				IsObject(UClass* InClass) const; // check that the Argument is a UObject and of baseType InClass
	UE_API bool				GetMetaData(const FName KeyToFind, FString& OutValue);
	
    // Hashing
    static UE_API FTG_Hash Hash(const FTG_Argument& Argument);
    FTG_Hash Hash() const { return Hash(*this); }

    UE_API const static FTG_Argument Invalid;

	friend bool operator== (const FTG_Argument& lhs, const FTG_Argument& rhs) {
		return (lhs.Name == rhs.Name) && (lhs.CPPTypeName == rhs.CPPTypeName) && (lhs.ArgumentType == rhs.ArgumentType);
	}

	// Comparison operator against the Arg Name, useful for FindByKey on TArray of Arguments
	bool operator== (const FName& InArgName) {
		return (Name == InArgName);
	}
};
using FTG_Arguments = TArray<FTG_Argument>;

TArray<FName> TG_MakeArrayOfArgumentNames(const FTG_Arguments& InArguments);

// Argument Set:
// A container for several Arguments which is searchable and hashable
// The arguments all have an index (their position in the set).
// All the argument names in the set MUST be different.
// THe hashes of each arguments are also different as a consequence.
struct FTG_ArgumentSet
{
private:
public:
    FTG_Arguments Arguments;

    // Access the Argument at the Index in the set.
    //const FTG_Argument& operator[] (const FTG_Index& Index) const
    const FTG_Argument&  Get(FTG_Index Index) const
    {
        if (Index >= 0 && Index < Arguments.Num())
            return Arguments[Index];
        return FTG_Argument::Invalid;
    }

    // Find the Index of an argument at a particular name.
    // Return INVALID_INDEX if not found
    UE_API FTG_Index FindName(FTG_Name& Name) const;

    // Find the Index of a argument at a particular hash.
    // Return INVALID_INDEX if not found
    UE_API FTG_Index FindHash(FTG_Hash Hash) const;

    // Hashing
    static UE_API FTG_Hash Hash(const FTG_ArgumentSet& Set);
    FTG_Hash Hash() const { return Hash(*this); }

};

// The Signature describe a full function signature
// a name + collection of arguments
USTRUCT()
struct FTG_Signature {

    GENERATED_BODY();

    struct FInit {
        FTG_Name Name;
        FTG_Arguments Arguments;
    };

protected:
    FTG_Name Name;
    FTG_ArgumentSet InArgs;
	FTG_ArgumentSet OutArgs;
	FTG_ArgumentSet PrivateArgs;
    bool bIsParam = false;

public:

	UE_API FTG_Signature(const FInit& Init);
    FTG_Signature() {}

	// Added an defaulted copy constructor here because Clang 16 expects either both or neither copy constructor
	// and copy assignment operator to be implemented.  But it's pretty weird for a type to do nothing
	// on copy.  UStructs need an assignment operator so this would be surprising behavior.
	FTG_Signature(const FTG_Signature& Other) = default;
	FTG_Signature& operator= (const FTG_Signature& type)
	{
		return *this;
	}

	FTG_Name GetName() const { return Name; }

	// Accessor to the argument set container's array
	// One can get a particular Argument retrieved from the Index
	const FTG_Arguments& GetInArguments() const { return InArgs.Arguments; }
	const FTG_Arguments& GetOutArguments() const { return OutArgs.Arguments; }
	const FTG_Arguments& GetPrivateArguments() const { return PrivateArgs.Arguments; }

	// Arguments accessed by linear index
	FORCEINLINE int32 GetNumArguments() const { return InArgs.Arguments.Num()
    												+ OutArgs.Arguments.Num()
													+ PrivateArgs.Arguments.Num(); }
	UE_API const FTG_Argument& GetArgument(int32 ArgIdx);

	FTG_Arguments GetArguments() const {	FTG_Arguments Args(InArgs.Arguments);
												Args.Append(OutArgs.Arguments);
												Args.Append(PrivateArgs.Arguments);
												return Args; }

	// Find the Index of an input argument at a particular name.
	// Return INVALID_INDEX if not found
	FTG_Index FindInputArgument(FTG_Name& InName) const { return InArgs.FindName(InName); }

	// Find the Index of an output argument at a particular name.
	// Return INVALID_INDEX if not found
	FTG_Index FindOutputArgument(FTG_Name& InName) const { return OutArgs.FindName(InName); }

	// Find the Index of a Private argument at a particular name.
	// Return INVALID_ID if not found
	FTG_Index FindPrivateArgument(FTG_Name& InName) const { return PrivateArgs.FindName(InName); }

    bool HasInputs() const { return !InArgs.Arguments.IsEmpty(); }
	bool HasOutputs() const { return !OutArgs.Arguments.IsEmpty(); }
	bool HasPrivates() const { return !PrivateArgs.Arguments.IsEmpty(); }

	bool IsParam() const { return bIsParam; }
    bool IsInputParam() const { return IsParam() && HasOutputs(); } // Input PARAM has Output
    bool IsOutputParam() const { return IsParam() && HasInputs(); } // Output PARAM has Input

	struct IndexAccess
	{
		FTG_Index Index = FTG_Id::INVALID_INDEX;
		ETG_Access Access = ETG_Access::In;
	};
	using IndexAccessArray = TArray<IndexAccess>;

	UE_API IndexAccess FindArgumentIndexFromName(FTG_Name& InName) const;
	UE_API const FTG_Argument& FindArgumentFromName(FTG_Name& InName) const;

    // Hashing
    static UE_API FTG_Hash Hash(const FTG_Signature& Type);
    FTG_Hash Hash() const { return Hash(*this); }

	UE_API const FTG_Argument& FindArgumentFromHash(FTG_Hash Hash) const;
	UE_API IndexAccess FindArgumentIndexFromHash(FTG_Hash Hash) const;

	// Generate a mapping table of the argument indices which is an array of index elements 
	// The element at poisiton i indicate the mapping argument index in the FromSignature corresponding to index i in the ToSignature
	// if the ToSignature argument doesn't have an equivalent, then INVALID_INDEX is at this element
	static UE_API FTG_Indices GenerateMappingArgIdxTable(const FTG_Signature& FromSignature, const FTG_Signature& ToSignature);
};
using FTG_SignaturePtr = TSharedPtr< FTG_Signature >;


#define TG_STR1(x)  #x
#define TG_STR(x)  TG_STR1(x)
#define TG_ARGUMENT_IN(name, t) { TEXT(TG_STR(name)), { ETG_Access::In }}
#define TG_ARGUMENT_OUT(name, t) { TEXT(TG_STR(name)), { ETG_Access::Out }}

#undef UE_API
