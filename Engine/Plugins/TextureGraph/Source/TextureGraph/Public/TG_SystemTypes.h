// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TG_SystemTypes.generated.h"

#define UE_API TEXTUREGRAPH_API

DECLARE_LOG_CATEGORY_EXTERN(LogTextureGraph, Log, All);

using FTG_Index = int32;
using FTG_Indices = TArray<FTG_Index>;

using ErrorReportMap = TMap<int32, TArray<struct FTextureGraphErrorReport>>;
USTRUCT()
struct FTG_Id
{
	GENERATED_BODY()

	static constexpr FTG_Index INVALID_INDEX = -1;

	static UE_API const FTG_Id INVALID;

	UPROPERTY()
	uint32 _IndexRaw = 0xFFFFFFFF; // The saved value for the FTG_Id is a single uint32 containing the 2 sub indices as in16 packed.

	uint32 IndexRaw()const { return _IndexRaw; }
	
	int16 NodeIdx() const // the main index for the node stored in the 4 top bytes of _IndexRaw
	{
		return (_IndexRaw >> 16) & 0x0000FFFF;
	}
	int16 PinIdx() const // the secondary index for the pin in the node stored in the 4 lower bytes of _IndexRaw
	{
		return (_IndexRaw & 0xFFFF);
	}

	FTG_Id() {}

	FTG_Id(int32 InNodeId, int32 InPinId = INVALID_INDEX)
		: _IndexRaw ( ((InNodeId << 16) & 0xFFFF0000) | (InPinId & 0x0000FFFF) )
	{
	}

	FORCEINLINE bool IsValid() const
	{
		return (NodeIdx() != INVALID_INDEX);
	}

	FORCEINLINE bool IsNodeId() const
	{
		return  IsValid() && (PinIdx() == INVALID_INDEX);
	}

	FORCEINLINE bool IsPinId() const
	{
		return  IsValid() && (PinIdx() != INVALID_INDEX);
	}

	FORCEINLINE bool operator==(const FTG_Id& Other) const
	{
		return IndexRaw() == Other.IndexRaw();
	}

	FORCEINLINE bool operator!=(const FTG_Id& Other) const
	{
		return IndexRaw() != Other.IndexRaw();
	}

	FORCEINLINE bool operator>=(const FTG_Id& Other) const
	{
		return IndexRaw() >= Other.IndexRaw();
	}

	FORCEINLINE bool operator>(const FTG_Id& Other) const
	{
		return IndexRaw() > Other.IndexRaw();
	}

	FORCEINLINE bool operator<=(const FTG_Id& Other) const
	{
		return IndexRaw() <= Other.IndexRaw();
	}

	FORCEINLINE bool operator<(const FTG_Id& Other) const
	{
		return IndexRaw() < Other.IndexRaw();
	}

	FORCEINLINE FString ToString() const
	{
		if (PinIdx() == INVALID_INDEX)
			return FString::FromInt(NodeIdx());
		else
			return FString::FromInt(NodeIdx()) + TEXT(".") + FString::FromInt(PinIdx());
	}

	// Specialize the UE function GetTypeHash for the type FTG_Id
	// Implemented in TG_Hash.cpp
	friend FORCEINLINE uint32 GetTypeHash(const FTG_Id& InID)
	{
		return GetTypeHash(InID.IndexRaw());
	}
};


using FTG_Ids = TArray<FTG_Id>;


using FTG_Name = const FName;

// Util function to validate that a name is unique in a collection and make a new candidate name if not
// It returns a FName equal to the Candidate if unique in the collection or a proposed edited candidate name which is unique
FName TG_MakeNameUniqueInCollection(FName CandidateName, const TArray<FName>& Collection, FName RecursionPreFix = TEXT("_"), int32 RecursionCount = 0);


using FTG_Hash = uint64;

namespace TG_TypeUtils
{
	const FString TEnumAsBytePrefix = TEXT("TEnumAsByte");

	const FString BoolType = TEXT("bool");
	const FString FloatType = TEXT("float");
	const FString DoubleType = TEXT("double");
	const FString Int32Type = TEXT("int32");
	const FString Int64Type = TEXT("int64");
	const FString UInt8Type = TEXT("uint8");
	const FString FNameType = TEXT("FName");
	const FString FStringType = TEXT("FString");
	const FString BoolArrayType = TEXT("TArray<bool>");
	const FString FloatArrayType = TEXT("TArray<float>");
	const FString DoubleArrayType = TEXT("TArray<double>");
	const FString Int32ArrayType = TEXT("TArray<int32>");
	const FString UInt8ArrayType = TEXT("TArray<uint8>");
	const FString FNameArrayType = TEXT("TArray<FName>");
	const FString FStringArrayType = TEXT("TArray<FString>");

	const FName BoolTypeName = *BoolType;
	const FName FloatTypeName = *FloatType;
	const FName DoubleTypeName = *DoubleType;
	const FName Int32TypeName = *Int32Type;
	const FName Int64TypeName = *Int64Type;
	const FName UInt8TypeName = *UInt8Type;
	const FName FNameTypeName = *FNameType;
}

namespace TG_MetadataSpecifiers
{
	// Assign to a FTG_Texture UProperty to represent its luminance histogram in the detail view
	const FName MD_HistogramLuminance = TEXT("MD_HistogramLuminance");
	
	// Assign to a float UProperty to go through the widget customizer
	const FName MD_ScalarEditor = TEXT("MD_ScalarEditor");

	// Assign to Levels Settings FVector UProperty to go through the widget customizer
	const FName MD_LevelsSettings = TEXT("MD_LevelsSettings");
}

#undef UE_API
