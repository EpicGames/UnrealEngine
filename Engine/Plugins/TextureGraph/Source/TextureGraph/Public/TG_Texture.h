// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Data/TiledBlob.h"
#include "Data/RawBuffer.h"
#include "Model/Mix/MixSettings.h"
#include "2D/TextureHelper.h"
#include "Misc/OutputDeviceNull.h"
#include "TG_Texture.generated.h"

#define UE_API TEXTUREGRAPH_API

USTRUCT(BlueprintType)
struct FTG_TextureDescriptor
{
	GENERATED_USTRUCT_BODY()
	FTG_TextureDescriptor()
	{
	}

	FTG_TextureDescriptor(BufferDescriptor Desc)
	{
		Width = static_cast<EResolution>(Desc.Width);
		Height = static_cast<EResolution>(Desc.Height);
		TextureFormat = TextureHelper::GetTGTextureFormatFromChannelsAndFormat(Desc.ItemsPerPoint, Desc.Format);
		bIsSRGB = Desc.bIsSRGB;
	}

public:
	// Width of the texture in pixels. Auto means system will detect automatically based on other images
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Advanced", Meta = (NoResetToDefault))
		EResolution Width = EResolution::Auto;		

	// Height of the texture in pixels. Auto means system will detect automatically based on other images
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Advanced", Meta = (NoResetToDefault))
		EResolution Height = EResolution::Auto;	

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Texture Format instead"))
		ETSBufferChannels NumChannels_DEPRECATED = ETSBufferChannels::Auto; /// How many items of type BufferFormat per point

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Texture Format instead"))
		ETSBufferFormat Format_DEPRECATED = ETSBufferFormat::Auto; /// What is the type of each item in the buffer
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	
	//UPROPERTY(EditAnywhere, Category = "TextureDescriptor", DisplayName = "Texture Format")
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Advanced", DisplayName = "Texture Format", Meta = (NoResetToDefault))
		ETG_TextureFormat TextureFormat = ETG_TextureFormat::Auto;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Advanced", DisplayName = "sRGB", Meta = (NoResetToDefault))
		bool bIsSRGB = false;

	bool operator==(const FTG_TextureDescriptor& RHS) const
	{
		return Width == RHS.Width && Height == RHS.Height && TextureFormat == RHS.TextureFormat && bIsSRGB == RHS.bIsSRGB;
	}

	friend FArchive& operator<<(FArchive& Ar, FTG_TextureDescriptor& D);

	FORCEINLINE BufferDescriptor ToBufferDescriptor() const
	{
		uint32 NumChannels = 0;
		BufferFormat Format = BufferFormat::Auto;
		TextureHelper::GetBufferFormatAndChannelsFromTGTextureFormat(TextureFormat, Format, NumChannels);
		BufferDescriptor Desc = BufferDescriptor(
			static_cast<int32>(Width),
			static_cast<int32>(Height),
			NumChannels,
			Format
		);

		Desc.bIsSRGB = bIsSRGB;
		return Desc;
	}

	FORCEINLINE operator BufferDescriptor() const { return ToBufferDescriptor(); }
	FORCEINLINE bool IsDefault() const
	{
		return Width == EResolution::Auto && Height == EResolution::Auto && TextureFormat == ETG_TextureFormat::Auto;
	}

	void InitFromString(const FString& StrVal)
	{
		FOutputDeviceNull NullOut;
		FTG_TextureDescriptor::StaticStruct()->ImportText(*StrVal, this, /*OwnerObject*/nullptr, 0, &NullOut, FTG_TextureDescriptor::StaticStruct()->GetName(), /*bAllowNativeOverride*/true);
	}

	FString ToString() const
	{
		FString ExportString;
		FTG_TextureDescriptor::StaticStruct()->ExportText(ExportString, this, this, /*OwnerObject*/nullptr, /*PortFlags*/0, /*ExportRootScope*/nullptr);
		return ExportString;
	}
};

USTRUCT()
struct FTG_Texture
{
	GENERATED_BODY()
	
public:
	TiledBlobPtr RasterBlob = nullptr;
	
	UPROPERTY(EditAnywhere, Category = "TextureDescriptor")
	FString TexturePath;
	
	UPROPERTY(EditAnywhere, Category = "TextureDescriptor")
	FTG_TextureDescriptor Descriptor;

	FTG_Texture() = default;
	FTG_Texture(TiledBlobPtr RHS) : RasterBlob(RHS) { }
	
	operator TiledBlobPtr() { return RasterBlob; }
	operator TiledBlobRef() { return RasterBlob; }
	operator bool() const { return !!RasterBlob; }
	friend FArchive& operator<<(FArchive& Ar, FTG_Texture& T);
	FTG_Texture& operator = (TiledBlobRef RHS) { RasterBlob = RHS; ResetTexturePath(); return *this; }
	FTG_Texture& operator = (TiledBlobPtr RHS) { RasterBlob = RHS; ResetTexturePath(); return *this; }
	UE_API bool operator == (const FTG_Texture& RHS) const;
	FORCEINLINE TiledBlob* operator -> () const { return RasterBlob.get(); }
	UE_API void ResetTexturePath();
	
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static FTG_Texture GetBlack() { return { TextureHelper::GBlack }; } 
	static FTG_Texture GetWhite() { return { TextureHelper::GWhite }; } 
	static FTG_Texture GetGray() { return { TextureHelper::GGray }; } 
	static FTG_Texture GetRed() { return { TextureHelper::GRed}; } 
	static FTG_Texture GetGreen() { return { TextureHelper::GGreen}; } 
	static FTG_Texture GetBlue() { return { TextureHelper::GBlue}; } 
	static FTG_Texture GetYellow() { return { TextureHelper::GYellow}; } 
	static FTG_Texture GetMagenta() { return { TextureHelper::GMagenta}; } 
	static FTG_Texture GetWhiteMask() { return { TextureHelper::GWhiteMask}; } 
	static FTG_Texture GetBlackMask() { return { TextureHelper::GBlackMask}; } 
	

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE BufferDescriptor GetBufferDescriptor() const { return Descriptor.ToBufferDescriptor(); }
};

#undef UE_API
