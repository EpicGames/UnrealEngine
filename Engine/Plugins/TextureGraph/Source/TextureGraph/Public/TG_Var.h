// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <functional>

#include "TG_Signature.h"

#include "TG_Var.generated.h"

#define UE_API TEXTUREGRAPH_API

class UTG_Expression;

template <typename T>
void TG_Var_SetValueFromString(T& Value, const FString& StrVal)
{
	UE_LOGF(LogTemp, Warning, "Need To Implement TG_Var_SetValueFromString");
}


template <typename T>
FString TG_Var_LogValue(T& Value)
{
	FString LogMessage = TEXT("Default Var Log : Need to define a type specific TG_Var_LogValue function");
	return LogMessage;
}

USTRUCT()
struct FTG_Var
{
	GENERATED_BODY()

	friend class UTG_Pin;
	friend class UTG_Expression;

	// Map of per type serializer for the var allowing to copy the value in the Var to a matching UObject's FProperty
	struct VarPropertySerialInfo
	{
		FTG_Var*				Var = nullptr;
		UObject*				Owner = nullptr;
		FTG_Argument			Arg;
		int32					Index = 0;
		bool					CopyVarToProperty = false;

		FORCEINLINE int32		ClampedIndex() const { return Index >= 0 ? Index : 0; }
	};
	//typedef std::function<void(VarPropertySerialInfo&)>		VarPropertySerializer;
	typedef void (*VarPropertySerializer) (VarPropertySerialInfo&)		;
	typedef TMap<FName, VarPropertySerializer>				VarPropertySerializerMap;
	static UE_API VarPropertySerializerMap							DefaultPropertySerializers;
	static UE_API void RegisterDefaultSerializers();
	static UE_API void RegisterVarPropertySerializer(FName CPPTypeName, VarPropertySerializer Serializer);
	static UE_API void UnregisterVarPropertySerializer(FName CPPTypeName);

	UE_API bool CopyGeneric(UTG_Expression* Owner, const FTG_Argument& Arg, bool CopyVarToProperty, int Index = -1);

	// Map of per type serializer for the var to/from an Archive
	struct VarArchiveSerialInfo
	{
		FTG_Var* Var = nullptr;
		FArchive& Ar;
	};
	typedef TFunction<void(VarArchiveSerialInfo&)>		VarArchiveSerializer;
	typedef TMap<FName, VarArchiveSerializer>			VarArchiveSerializerMap;
	static UE_API VarArchiveSerializerMap				DefaultArchiveSerializers;


	// Serialize the var value
	// Called from the Pin serialize()
	UE_API void Serialize(FArchive& Ar, FTG_Id PinId, const FTG_Argument& Argument);
	
	UPROPERTY(Transient)
	FTG_Id						PinId;



	template <typename T_ValueType>
	static void FGeneric_Struct_Serializer(FTG_Var::VarPropertySerialInfo& Info)
	{
		FProperty* Property = Info.Owner->GetClass()->FindPropertyByName(Info.Arg.GetName());


		const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
		T_ValueType* PropertyValue = StructProperty->ContainerPtrToValuePtr<T_ValueType>(Info.Owner, Info.ClampedIndex());

		if (Info.CopyVarToProperty)
		{
			if (!Info.Var->IsEmpty() && Info.Var->IsValid())
				*PropertyValue = Info.Var->GetAs<T_ValueType>();
			else
			{
				T_ValueType* DefaultValue = StructProperty->ContainerPtrToValuePtrForDefaults<T_ValueType>(StructProperty->Struct, Info.Owner, Info.ClampedIndex());
				if (DefaultValue)
					*PropertyValue = *DefaultValue;
			}
		}
		else
		{
			Info.Var->EditAs<T_ValueType>() = *PropertyValue;
		}
	}

public:
	UE_API FTG_Var();
	UE_API FTG_Var(FTG_Id InPinUuid);
	UE_API FTG_Var(const FTG_Var& InVar);

	FTG_Var& operator = (const FTG_Var& InVar) { PinId = InVar.PinId; Concept = InVar.Concept; return *this; }

	void ShareData(const FTG_Var& InVar)
	{
		Concept = InVar.Concept;
	}

	FTG_Id			GetId() const { return PinId; }

	// Check that the Var is valid meaning that it is correctly allocated in the Graph and reference its Pin Owner
	bool IsValid() const { return PinId != FTG_Id::INVALID; }

	// Check if the Var has No Data allocated
	bool IsEmpty() const { return !Concept; } 
	bool IsArray() const { return Concept && Concept->IsArray(); }
	void SetArray() { check(Concept); Concept->SetArray(); }

	FString LogValue() const { if (Concept) return Concept->LogValue(); else return TEXT("nullptr"); }

	void SetValueFromStr(FString StrVal) {	if (Concept) Concept->SetValueFromString(StrVal); }
	
	// Manage data Arg provide the type name and rely on the VarAllocator map to create the concrete cpp type
	// If Owner is provided then Var can be initialized from the matching owner's property if found
	UE_API void CopyTo(UTG_Expression* Owner, FTG_Argument& Arg, int Index = -1);
	UE_API void CopyFrom(UTG_Expression* Owner, FTG_Argument& Arg, int Index = -1);

	UE_API void CopyTo(FTG_Var* InVar);
	UE_API void CopyFrom(FTG_Var* InVar);

	// Manage the Var Data with the knowledge of the Type
	void Reset() { Concept = nullptr; }
	template <class T> void ResetAs() const { Concept = MakeShared<FModel<T>>(); }
	template <class T> T& GetAs() const
	{
		if (IsEmpty())
			ResetAs<T>();
		return static_cast<FModel<T>*>(Concept.Get())->Value;
	}
	
	template <class T> T& GetAsWithDefault(T& Default)
	{
		if (IsEmpty())
			return Default;
		return static_cast<FModel<T>*>(Concept.Get())->Value;
	}
	
	template <class T> void SetAs(const T& InValue)
	{
		if (IsEmpty())
			ResetAs<T>();
		static_cast<FModel<T>*>(Concept.Get())->Value = InValue;
	}
	
	template <class T> T& EditAs()
	{
		if (IsEmpty())
			ResetAs<T>();
		return static_cast<FModel<T>*>(Concept.Get())->Value;
	}

	UE_API FString LogHead() const;

private:

	struct FConcept
	{
		virtual ~FConcept() = default;

		virtual FString LogValue() = 0;
		virtual void SetValueFromString(const FString& String) = 0;
		virtual TSharedPtr<FConcept> Clone() = 0;
		virtual void SetArray() = 0;
		virtual bool IsArray() const = 0;
	};

	template <class T>
	struct FModel : public FConcept
	{
		T Value;
		bool bIsArray = false;

		FString LogValue() override 
		{ 
			return TG_Var_LogValue(Value); 
		}
	
		void SetValueFromString(const FString& StrValue) override 
		{ 
			TG_Var_SetValueFromString(Value, StrValue); 
		}

		TSharedPtr<FConcept> Clone() override 
		{
			auto Ptr = MakeShared<FModel<T>>();
			Ptr->Value = Value;
			Ptr->bIsArray = bIsArray;
			return Ptr;
		}

		virtual void SetArray() 
		{ 
			bIsArray = true; 
		}
		virtual bool IsArray() const 
		{ 
			return bIsArray; 
		}
	};

	mutable TSharedPtr<FConcept> Concept;
};

template <> UE_API FString TG_Var_LogValue(bool& Value);
template <> UE_API FString TG_Var_LogValue(int& Value);
template <> UE_API FString TG_Var_LogValue(float& Value);
template <> UE_API FString TG_Var_LogValue(uint8& Value);
template <> UE_API FString TG_Var_LogValue(FLinearColor& Value);
template <> UE_API FString TG_Var_LogValue(FVector4f& Value);
template <> UE_API FString TG_Var_LogValue(FVector2f& Value);
template <> UE_API FString TG_Var_LogValue(FName& Value);
template <> UE_API FString TG_Var_LogValue(FString& Value);
template <> UE_API FString TG_Var_LogValue(TObjectPtr<UObject>& Value);
template <> UE_API FString TG_Var_LogValue(struct FTG_Texture& Value);
template <> UE_API FString TG_Var_LogValue(struct FTG_Scalar& Value);
template <> UE_API FString TG_Var_LogValue(struct FTG_OutputSettings& Value);
template <> UE_API FString TG_Var_LogValue(struct FTG_TextureDescriptor& Value);
template <> UE_API FString TG_Var_LogValue(struct FTG_Variant& Value);
template <> UE_API FString TG_Var_LogValue(struct FTG_Material& Value);


template <> UE_API void TG_Var_SetValueFromString(int& Value, const FString& StrVal);
template <> UE_API void TG_Var_SetValueFromString(bool& Value, const FString& StrVal);
template <> UE_API void TG_Var_SetValueFromString(float& Value, const FString& StrVal);
template <> UE_API void TG_Var_SetValueFromString(uint8& Value, const FString& StrVal);
template <> UE_API void TG_Var_SetValueFromString(TObjectPtr<UObject>& Value, const FString& StrVal);

template <> UE_API void TG_Var_SetValueFromString(FLinearColor& Value, const FString& StrVal);
template <> UE_API void TG_Var_SetValueFromString(FVector4f& Value, const FString& StrVal);
template <> UE_API void TG_Var_SetValueFromString(FVector2f& Value, const FString& StrVal);
template <> UE_API void TG_Var_SetValueFromString(FName& Value, const FString& StrVal);
template <> UE_API void TG_Var_SetValueFromString(FString& Value, const FString& StrVal);

//template <> UE_API void TG_Var_SetValueFromString(struct FTG_Texture& Value, const FString& StrVal);
template <> UE_API void TG_Var_SetValueFromString(struct FTG_Scalar& Value, const FString& StrVal);
template <> UE_API void TG_Var_SetValueFromString(struct FTG_OutputSettings& Value, const FString& StrVal);
template <> UE_API void TG_Var_SetValueFromString(struct FTG_TextureDescriptor& Value, const FString& StrVal);
template <> UE_API void TG_Var_SetValueFromString(struct FTG_Variant& Value, const FString& StrVal);
template <> UE_API void TG_Var_SetValueFromString(struct FTG_Material& Value, const FString& StrVal);


#undef UE_API
