// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Variant.h"
#include "TG_Texture.h"

#include "TG_Variant.generated.h"

#define UE_API TEXTUREGRAPH_API

// Inner data type of the variant relies on the TVariant
using FTG_VariantInnerData = TVariant<float, FLinearColor, FVector4f, FTG_Texture>;
struct FTG_Texture;
struct FTG_EvaluationContext;

// Types of the variant are organized in increasing complexity, so a compatible type for 2 variants is always the highest 
UENUM()
enum class ETG_VariantType : int8
{
	Invalid = -1												UMETA(DisplayName = "Invalid"),
	Scalar = FTG_VariantInnerData::IndexOfType<float>()			UMETA(DisplayName = "Scalar"),
	Color = FTG_VariantInnerData::IndexOfType<FLinearColor>()	UMETA(DisplayName = "Color"),
	Vector = FTG_VariantInnerData::IndexOfType<FVector4f>()		UMETA(DisplayName = "Vector"),
	Texture = FTG_VariantInnerData::IndexOfType<FTG_Texture>()	UMETA(DisplayName = "Texture"),
};
ENUM_RANGE_BY_FIRST_AND_LAST(ETG_VariantType, ETG_VariantType::Scalar, ETG_VariantType::Texture);

//////////////////////////////////////////////////////////////////////////
/// Base class for making working with Variants easier and less repetitive
//////////////////////////////////////////////////////////////////////////
USTRUCT()
struct FTG_Variant
{
	GENERATED_BODY()

public:
	using Variant = FTG_VariantInnerData;
	using EType = ETG_VariantType;

	UE_API FTG_Variant();
	UE_API FTG_Variant(const FTG_Variant& RHS);
	UE_API FTG_Variant(float RHS);
	UE_API FTG_Variant(FVector4f RHS);
	UE_API FTG_Variant(FLinearColor RHS);
	UE_API FTG_Variant(FTG_Texture RHS);

	UE_API FTG_Variant& operator = (const FTG_Variant& RHS);
	UE_API FTG_Variant& operator = (const float RHS);
	UE_API FTG_Variant& operator = (const FVector4f RHS);
	UE_API FTG_Variant& operator = (const FLinearColor RHS);
	UE_API FTG_Variant& operator = (const FTG_Texture RHS);
	UE_API bool operator == (const FTG_Variant& RHS) const;

	// Retrieve the FName corresponding to a variant type
	static FName GetNameFromType(ETG_VariantType InType)
	{
		/// Invalid defaults to type scalar
		if (InType == ETG_VariantType::Invalid)
			return TEXT("Scalar");

		// We could use this: 
		// return StaticEnum<ETG_VariantType>()->GetName(static_cast<uint32>(InType));
		// But we want simpler names
		static FName VariantType_Names[] = {
			TEXT("Scalar"),
			TEXT("Color"),
			TEXT("Vector"),
			TEXT("Texture"),
		};
		return VariantType_Names[static_cast<uint32>(InType)];
	}

	// Retrieve the FName associated to a varaint type used for Arg cpoptypename
	static FName GetArgNameFromType(ETG_VariantType InType)
	{
		/// Invalid defaults to type scalar
		if (InType == ETG_VariantType::Invalid)
			return TEXT("FTG_Variant.Scalar");

		static FName ArgVariantType_Names[] = {
			TEXT("FTG_Variant.Scalar"),
			TEXT("FTG_Variant.Color"),
			TEXT("FTG_Variant.Vector"),
			TEXT("FTG_Variant.Texture"),
		};
		return ArgVariantType_Names[static_cast<uint32>(InType)];
	}


	// Retrieve the variant type value matching a FName
	// Return Scalar if couldn't find a match
	static ETG_VariantType GetTypeFromName(const FName& InTypeName)
	{
		// We could use this: 
		// int32 Index = StaticEnum<EType>()->GetIndexByName(InTypeName, EGetByNameFlags::CaseSensitive);
		// But we want to support many more aliases:
		static TMap<FName, ETG_VariantType> VariantType_NameToEnumTable = {
			{TEXT("Scalar"), EType::Scalar},	{TEXT("float"), EType::Scalar},			{TEXT("FTG_Variant.Scalar"), EType::Scalar},
			{TEXT("Color"), EType::Color},		{TEXT("FLinearColor"), EType::Color},	{TEXT("FTG_Variant.Color"), EType::Color},
			{TEXT("Vector"), EType::Vector},	{TEXT("FVector4f"), EType::Vector},		{TEXT("FTG_Variant.Vector"), EType::Vector},
			{TEXT("Texture"), EType::Texture},	{TEXT("FTG_Texture"), EType::Texture},	{TEXT("FTG_Variant.Texture"), EType::Texture},
		};
		auto FoundType = VariantType_NameToEnumTable.Find(InTypeName);
		if (FoundType)
			return (*FoundType);
		else
			return  EType::Scalar;
	}

	// Predicates
	static bool IsScalar(EType InType) { return (InType == EType::Scalar); }
	static bool IsColor(EType InType) { return (InType == EType::Color); }
	static bool IsVector(EType InType) { return (InType == EType::Vector); }
	static bool IsTexture(EType InType) { return (InType == EType::Texture); }
	static bool IsInvalid(EType InType) { return (InType == EType::Invalid); }

	// Find the common type between 2
	static EType WhichCommonType(EType T0, EType T1)
	{
		if (T0 >= T1)
			return T0;
		else
			return T1;
	}
	friend FArchive& operator<<(FArchive& Ar, FTG_Variant& D)
	{
		return Ar << D.Data;
	}
	UE_API bool Serialize( FArchive& Ar );
	
	// FTG_Variant struct members and methods

	// The concrete data
	Variant Data;

	// Get the Type of the Variant, Scalar by default
	EType GetType() const { return (EType) Data.GetIndex(); }

	// Reset the Variant to the specified type,
	// If the type is changed, the value is reset to 0
	// return true if the type as mutated
	UE_API bool ResetTypeAs(EType InType);


	// Helper predicates to narrow down the type of the variant
	bool IsScalar() const { return IsScalar(GetType()); }
	bool IsColor() const { return IsColor(GetType()); }
	bool IsVector() const { return IsVector(GetType()); }
	bool IsTexture() const { return IsTexture(GetType()); }


	UE_API operator bool() const;

	// Getter and Editor
	// The getter methods are valid ONLY if the type matches
	UE_API const float& GetScalar() const;
	UE_API const FLinearColor& GetColor() const;
	UE_API const FVector4f& GetVector() const;
	UE_API const FTG_Texture& GetTexture() const;
	UE_API FTG_Texture& GetTexture();

	UE_API FLinearColor GetColor(FLinearColor Default = FLinearColor::Black);
	UE_API FVector4f GetVector(FVector4f Default = FVector4f { 0, 0, 0, 0 });
	UE_API FTG_Texture GetTexture(FTG_EvaluationContext* InContext, FTG_Texture Default = { TextureHelper::GBlack }, const BufferDescriptor* DesiredDesc = nullptr);

	// The editor methods are calling ResetTypeAs(expectedType) 
	// and thus the Variant will mutate to the expected type and assigned 0
	UE_API float& EditScalar();
	UE_API FLinearColor& EditColor();
	UE_API FVector4f& EditVector();
	UE_API FTG_Texture& EditTexture();
};

template<>
struct TStructOpsTypeTraits<FTG_Variant>
	: public TStructOpsTypeTraitsBase2<FTG_Variant>
{
	enum
	{
		WithSerializer = true,
		WithCopy = true,
		WithIdenticalViaEquality = true
	};
};

//////////////////////////////////////////////////////////////////////////
/// Variant Arrays
//////////////////////////////////////////////////////////////////////////
USTRUCT()
struct FTG_VariantArray
{
	GENERATED_BODY()

private: 
	TArray<FTG_Variant> Array;			/// An array of textures
	int ViewIndex = 0;


public:
	UE_API void SetNum(int Count);
	UE_API void Set(int Index, const FTG_Variant& Var);
	UE_API void Set(int Index, const FTG_Texture& Tex);
	UE_API void CopyFrom(const FTG_VariantArray& RHS);
	UE_API void SetViewIndex(int Index);

	FORCEINLINE bool IsTexture()
	{
		return !Array.IsEmpty() && Array[0].IsTexture();
	}

	FORCEINLINE int Num() const
	{
		return Array.Num();
	}

	FORCEINLINE FTG_Variant& Get(int Index)
	{
		check(Index >= 0 && Index < Array.Num());
		return Array[Index];
	}

	FORCEINLINE const FTG_Variant& Get(int Index) const
	{
		check(Index >= 0 && Index < Array.Num());
		return Array[Index];
	}

	FORCEINLINE const TArray<FTG_Variant>& GetArray() const
	{
		return Array;
	}

	FORCEINLINE FTG_Texture* GetViewTexturePtr() 
	{
		if (ViewIndex >= 0 && ViewIndex < Array.Num() && Array[ViewIndex].IsTexture())
			return &(Array[ViewIndex].GetTexture());

		return nullptr;
	}

	FORCEINLINE FTG_Texture GetViewTexture() const
	{
		if (ViewIndex >= 0 && ViewIndex < Array.Num() && Array[ViewIndex].IsTexture())
			return Array[ViewIndex].GetTexture();

		return FTG_Texture::GetBlack();
	}
};

#undef UE_API
