// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Model/Mix/MixInterface.h"
#include "TG_OutputSettings.h"
#include "TG_GraphEvaluation.h"
#include "TextureGraph.generated.h"

#define UE_API TEXTUREGRAPH_API

class UTG_Graph;
using ErrorReportMap = TMap<int32, TArray<FTextureGraphErrorReport>>;

UCLASS(abstract, BlueprintType, MinimalAPI)
class UTextureGraphBase : public UMixInterface
{
	GENERATED_BODY()

protected:
	
	EResolution GetMaxWidth();

	EResolution GetMaxHeight();

	int32 GetMaxBufferChannels();

	BufferFormat GetMaxBufferFormat();
	
	// Override Serialize method of UObject
	virtual void Serialize(FArchive& Ar) override;

	// Override the PostLoad method of UObject to allocate settings
	virtual void PostLoad() override;

	// Override PreSave method of UObject
	virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	void PostMeshLoad() override;

public:
	virtual const UTG_Graph* Graph() const { return nullptr; }
	virtual UTG_Graph* Graph() { return nullptr; }
	
	// Construct the script giving it its name Initialize to a default one output script
	virtual void Construct(FString Name);
	
	void InvalidateAll() override;
	void Update(MixUpdateCyclePtr InCycle) override;
	virtual void Initialize() {}; 

	TEXTUREGRAPH_API void TriggerUpdate(bool Tweaking);
	TEXTUREGRAPH_API void FlushInvalidations();
	TEXTUREGRAPH_API void UpdateGlobalTGSettings();
	TEXTUREGRAPH_API void Log() const;
};

UCLASS(MinimalAPI, BlueprintType)
class UTextureGraph: public UTextureGraphBase
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<class UTG_Graph> TextureGraph;
	
protected:
	UE_API void GatherAllDependentGraphs(TArray<UTextureGraph*>& DependentGraphs) const;
	
	UE_API bool CheckRecursiveDependency(const UTextureGraph* InTextureGraph) const;
	
public:
	UE_API virtual void Construct(FString Name) override;
	virtual const UTG_Graph* Graph() const { return TextureGraph; }
	virtual UTG_Graph* Graph() { return TextureGraph; }
	UE_API bool IsDependentOn(const UTextureGraph* TextureGraph) const;
	UE_API bool HasCyclicDependency() const;
};

UCLASS(MinimalAPI, BlueprintType)
class UTextureGraphInstance: public UTextureGraphBase
{
	friend struct FTG_InstanceImpl;
	GENERATED_BODY()
	
private:
	UPROPERTY(transient)
	TObjectPtr<UTG_Graph>	RuntimeGraph;		// The runtime graph instance to use for the sub-graph
public:
	
	UPROPERTY()
	FTG_VarMap InputParams;

	UPROPERTY()
	TMap<FTG_Id, FTG_OutputSettings> OutputSettingsMap;
	
	/** Parent Texture Graph. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=TextureGraphInstance, AssetRegistrySearchable)
	TObjectPtr<UTextureGraphBase> ParentTextureGraph; 

	UE_API virtual void Construct(FString Name) override;

	UE_API void CopyParamsToRuntimeGraph();
	UE_API bool CheckOutputSettingsMatch(const TObjectPtr<UTextureGraphBase>& Parent) const;
	UE_API void UpdateOutputSettingsFromGraph();
	
	UE_API void SetParent(const TObjectPtr<UTextureGraphBase>& Parent);

	// Override PreSave method of UObject
	UE_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	
	virtual const UTG_Graph* Graph() const override { return RuntimeGraph.Get(); }
	UE_API virtual UTG_Graph* Graph() override;
	UE_API virtual void Initialize() override;
};

#undef UE_API
