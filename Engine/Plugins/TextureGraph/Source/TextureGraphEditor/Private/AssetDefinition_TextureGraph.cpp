// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_TextureGraph.h"
#include "TextureGraphEditorModule.h"
#include "TextureGraph.h"

#include "AssetRegistry/AssetIdentifier.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "IAssetTools.h"
#include "ToolMenuSection.h"
#include "TG_InstanceFactory.h"
#include "AssetRegistry/AssetRegistryHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_TextureGraph)

#define LOCTEXT_NAMESPACE "UAssetDefinition_TextureGraph"

EAssetCommandResult UAssetDefinition_TextureGraph::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UTextureGraph* TextureGraph : OpenArgs.LoadObjects<UTextureGraph>())
	{
		FTextureGraphEditorModule* TextureEditorModule = &FModuleManager::LoadModuleChecked<FTextureGraphEditorModule>("TextureGraphEditor");
		TextureEditorModule->CreateTextureGraphEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, TextureGraph);
	}
	
	return EAssetCommandResult::Handled;
}

FAssetOpenSupport UAssetDefinition_TextureGraph::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(OpenSupportArgs.OpenMethod,OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit || OpenSupportArgs.OpenMethod == EAssetOpenMethod::View); 
}


// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_TextureGraph
{

	static void ExecuteExportTextureGraph(const FToolMenuContext& InContext)
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			const TArray<UTextureGraph*> &TextureGraphs = CBContext->LoadSelectedObjects<UTextureGraph>();
			const FTextureGraphEditorModule& TextureGraphEditorModule = FModuleManager::Get().LoadModuleChecked<FTextureGraphEditorModule>(TEXT("TextureGraphEditor"));
			
			for (auto TextureGraphIt = TextureGraphs.CreateConstIterator(); TextureGraphIt; ++TextureGraphIt)
			{
				TextureGraphEditorModule.GetTextureExporter()->SetTextureGraphToExport(*TextureGraphIt);
			}
			
		}
	}
	static void ExecuteNewTextureGraphInstance(const FToolMenuContext& InContext)
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			
			IAssetTools::Get().CreateAssetsFrom<UTextureGraph>(
				CBContext->LoadSelectedObjects<UTextureGraph>(), UTextureGraphInstance::StaticClass(), TEXT("_Inst"), [](UTextureGraph* SourceObject)
				{
					UTG_InstanceFactory* Factory = NewObject<UTG_InstanceFactory>();
					Factory->InitialParent = SourceObject;
					return Factory;
				}
			);
			
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

			{
				UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UTextureGraph::StaticClass());
		
				FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
				Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
					{
						{
							const TAttribute<FText> Label = LOCTEXT("TextureGraph_Export", "Export Texture Graph");
							const TAttribute<FText> ToolTip = LOCTEXT("Texture_ExportTextureGraphTooltip", "Allows Exporting Texture Graph with parameter changes.");
							const FSlateIcon Icon = FSlateIcon();
							const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteExportTextureGraph);
							InSection.AddMenuEntry("Texture_Export", Label, ToolTip, Icon, UIAction);
						}
					}
				}));

				Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
					{
						{
							const TAttribute<FText> Label = LOCTEXT("TextureGraph_NewInstance", "Create Texture Graph Instance");
							const TAttribute<FText> ToolTip = LOCTEXT("TextureGraph_NewInstanceTooltip", "Creates a parameterized Texture Graph Instance using this Texture Graph as a base");
							const FSlateIcon Icon = FSlateIcon();
							const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewTextureGraphInstance);
							InSection.AddMenuEntry("TextureGraph_NewInstance", Label, ToolTip, Icon, UIAction);
						}
					}
				}));
			}

			
		}));
	});
}

#undef LOCTEXT_NAMESPACE
