// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TextureGraph.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_TextureGraph.generated.h"

UCLASS()
class UAssetDefinition_TextureGraph : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_TextureGraph", "Texture Graph"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor::Emerald); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UTextureGraph::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Texture, NSLOCTEXT("AssetDefinition", "TextureGraph_SubMenu", "Texture Graph"), ECategoryMenuType::Section) };
		return Categories;
	}

	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};
