// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_TextureGraphInstance.h"
#include "TextureGraphEditorModule.h"
#include "TextureGraph.h"

#include "AssetRegistry/AssetIdentifier.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "IAssetTools.h"
#include "ToolMenuSection.h"
#include "TG_InstanceFactory.h"
#include "AssetRegistry/AssetRegistryHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_TextureGraphInstance)

#define LOCTEXT_NAMESPACE "UAssetDefinition_TextureGraphInstance"

EAssetCommandResult UAssetDefinition_TextureGraphInstance::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	for (UTextureGraphInstance* TextureGraphInstance : OpenArgs.LoadObjects<UTextureGraphInstance>())
	{
		FTextureGraphEditorModule* TextureEditorModule = &FModuleManager::LoadModuleChecked<FTextureGraphEditorModule>("TextureGraphEditor");
		TextureEditorModule->CreateTextureGraphInstanceEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, TextureGraphInstance);
	}
	
	return EAssetCommandResult::Handled;
}

FAssetOpenSupport UAssetDefinition_TextureGraphInstance::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(OpenSupportArgs.OpenMethod,OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit || OpenSupportArgs.OpenMethod == EAssetOpenMethod::View); 
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_TextureGraphInstance
{

	static void ExecuteExportTextureGraphInstance(const FToolMenuContext& InContext)
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			const TArray<UTextureGraphInstance*> &TextureGraphs = CBContext->LoadSelectedObjects<UTextureGraphInstance>();
			const FTextureGraphEditorModule& TextureGraphEditorModule = FModuleManager::Get().LoadModuleChecked<FTextureGraphEditorModule>(TEXT("TextureGraphEditor"));
			
			for (auto TextureGraphIt = TextureGraphs.CreateConstIterator(); TextureGraphIt; ++TextureGraphIt)
			{
				TextureGraphEditorModule.GetTextureExporter()->SetTextureGraphToExport((*TextureGraphIt));
			}
			
		}
	}
	static void ExecuteNewTextureGraphInstance(const FToolMenuContext& InContext)
	{
		if (const UContentBrowserAssetContextMenuContext* CBContext = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			
			IAssetTools::Get().CreateAssetsFrom<UTextureGraphInstance>(
				CBContext->LoadSelectedObjects<UTextureGraphInstance>(), UTextureGraphInstance::StaticClass(), TEXT("_Inst"), [](UTextureGraphInstance* SourceObject)
				{
					UTG_InstanceFactory* Factory = NewObject<UTG_InstanceFactory>();
					Factory->InitialParent = SourceObject;
					return Factory;
				}
			);
			
		}
	}
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{ 
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

			{
				UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UTextureGraphInstance::StaticClass());
		
				FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
				Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
					{
						{
							const TAttribute<FText> Label = LOCTEXT("TextureGraphInstance_Export", "Export Texture Graph Instance");
							const TAttribute<FText> ToolTip = LOCTEXT("Texture_ExportTextureGraphInstanceTooltip", "Allows Exporting Texture Graph Instance.");
							const FSlateIcon Icon = FSlateIcon();
							const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteExportTextureGraphInstance);
							InSection.AddMenuEntry("Texture_Export", Label, ToolTip, Icon, UIAction);
						}
					}
				}));

				Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InSection))
					{
						{
							const TAttribute<FText> Label = LOCTEXT("TextureGraph_NewInstance", "Create Texture Graph Instance");
							const TAttribute<FText> ToolTip = LOCTEXT("TextureGraph_NewInstanceTooltip", "Creates a parameterized Texture Graph Instance using this Texture Graph as a base");
							const FSlateIcon Icon = FSlateIcon();
							const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateStatic(&ExecuteNewTextureGraphInstance);
							InSection.AddMenuEntry("TextureGraph_NewInstance", Label, ToolTip, Icon, UIAction);
						}
					}
				}));
			}
		}));
	});
}

#undef LOCTEXT_NAMESPACE
