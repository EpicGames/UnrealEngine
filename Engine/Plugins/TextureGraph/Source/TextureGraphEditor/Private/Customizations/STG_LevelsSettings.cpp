// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_LevelsSettings.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"



void STG_LevelsSettings::Construct(const FArguments& InArgs)
{
	Levels = InArgs._Levels;
	OnValueChanged = InArgs._OnValueChanged;
	FName Low = (TEXT("Low"));
	FName Mid = (TEXT("Mid"));
	FName High = (TEXT("High"));

	ChildSlot
	[
		SNew(SVerticalBox)
		+
		SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(FText::FromName(Low))
		]
		+
		SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(LowLevelsSlider, SSlider)
			.OnValueChanged(this, &STG_LevelsSettings::OnLowChanged)
		]
		+
		SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(FText::FromName(Mid))
		]
		+
		SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(MidLevelsSlider, SSlider)
			.OnValueChanged(this, &STG_LevelsSettings::OnMidChanged)
		]
		+
		SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(FText::FromName(High))
		]
		+
		SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(HighLevelsSlider, SSlider)
			.OnValueChanged(this, &STG_LevelsSettings::OnHighChanged)
		]
	];
	UpdateLevelsWidget();
}

void STG_LevelsSettings::UpdateLevelsWidget()
{
/*	LowLevelsSlider->SetMinAndMaxValues(0, Levels.High);
	MidLevelsSlider->SetMinAndMaxValues(Levels.Low, Levels.High);
	HighLevelsSlider->SetMinAndMaxValues(Levels.Low, 1);
*/
	LowLevelsSlider->SetValue(Levels.Low);
	MidLevelsSlider->SetValue(Levels.Mid);
	HighLevelsSlider->SetValue(Levels.High);

	OnValueChanged.ExecuteIfBound(Levels);
}

void STG_LevelsSettings::OnLowChanged(float InValue)
{
	if (Levels.SetLow(InValue))
		UpdateLevelsWidget();
}
void STG_LevelsSettings::OnMidChanged(float InValue)
{
	if (Levels.SetMid(InValue))
		UpdateLevelsWidget();
}
void STG_LevelsSettings::OnHighChanged(float InValue)
{
	if (Levels.SetHigh(InValue))
		UpdateLevelsWidget();
}

