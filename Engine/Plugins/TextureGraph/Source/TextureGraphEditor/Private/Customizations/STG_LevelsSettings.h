// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <Expressions/Filter/TG_Expression_Levels.h>
#include <Widgets/SCompoundWidget.h>
#include <Widgets/Input/SSlider.h>

/** Notification for float value change */
DECLARE_DELEGATE_OneParam(FTG_OnLevelsSettingsValueChanged, const FTG_LevelsSettings&);

class STG_LevelsSettings : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(STG_LevelsSettings) :
		_Levels(),
		_OnValueChanged()
	{}
	SLATE_ARGUMENT(FTG_LevelsSettings, Levels)
	SLATE_ARGUMENT(FTG_OnLevelsSettingsValueChanged, OnValueChanged)

	SLATE_END_ARGS()

	void Construct(const FArguments& Args);

	void OnLowChanged(float InValue);
	void OnMidChanged(float InValue);
	void OnHighChanged(float InValue);
	void UpdateLevelsWidget();
protected:
	TSharedPtr<SSlider> LowLevelsSlider;
	TSharedPtr<SSlider> MidLevelsSlider;
	TSharedPtr<SSlider> HighLevelsSlider;
	FTG_LevelsSettings	Levels;

	FTG_OnLevelsSettingsValueChanged OnValueChanged;
};
