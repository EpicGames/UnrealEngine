// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_TextureHistogram.h"

#include "Transform/Utility/T_TextureHistogram.h"
#include "TG_Graph.h"
#include "TextureGraph.h"
#include <Widgets/Layout/SBorder.h>
#include "Widgets/Layout/SBox.h"

const float STG_TextureHistogram::PreferredWidth = 300.f;

void STG_TextureHistogram::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SBox)
		.MinDesiredHeight(InArgs._Height)
		[
			SNew(SBorder)
			.ColorAndOpacity(InArgs._BackgroundColor)
			.Content()
			[
				SAssignNew(HistogramBars, STG_HistogramBlob)
				.Curves(ETG_HistogramCurves::Luma)
				.Height(InArgs._Height)
			]
		]
	];
}

void STG_TextureHistogram::SetTexture(FTG_Texture& Source, UTextureGraph* InTextureGraph)
{
	if (Source.RasterBlob && InTextureGraph)
	{
		if (Source.RasterBlob->IsTransient())
			return;

		Source.RasterBlob->OnFinalise()
			.then([this, InTextureGraph, Source]() mutable
			{
				// OnFinalise can sometimes occur after the editor is closed and thus can potentially
				// deallocate all corresponding slate objects
				if (DoesSharedInstanceExist() && !Source.RasterBlob->IsTransient())
				{
					T_TextureHistogram::CreateOnService(InTextureGraph, std::static_pointer_cast<TiledBlob>(Source.RasterBlob), 0);
					return Source->GetHistogram()->OnFinalise();
				}
				return (AsyncBlobResultPtr)(cti::make_ready_continuable<const Blob*>(nullptr));
			})
			.then([this, Source]() mutable
			{
				// OnFinalise can sometimes occur after the editor is closed and thus can potentially
				// deallocate all corresponding slate objects
				if (DoesSharedInstanceExist() && !Source.RasterBlob->IsTransient())
				{
					TiledBlobPtr FinalizedHistogram = std::static_pointer_cast<TiledBlob>(Source.RasterBlob->GetHistogram());
					HistogramBars->Update(FinalizedHistogram);
				}
			});
	}
}