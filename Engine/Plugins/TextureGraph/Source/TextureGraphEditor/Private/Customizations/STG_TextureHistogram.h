// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <Widgets/SCompoundWidget.h>
#include "TG_Texture.h"
#include "STextureHistogram.h"
#include "Widgets/STG_HistogramBlob.h"

class UTextureGraph;

// This widget represents the Histogram of the source FTG_Texture assigned along with the UTG_Script where it belongs
class STG_TextureHistogram : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STG_TextureHistogram):
		_HistogramColor(TextureHistogramUtils::LinearWhite),
		_BackgroundColor(FLinearColor(0.08f, 0.08f, 0.08f, 1.f)),
		_Height(20.0)
	{}
	SLATE_ARGUMENT(FLinearColor, HistogramColor)
	SLATE_ARGUMENT(FLinearColor, BackgroundColor)
	SLATE_ARGUMENT(float, Height)

	SLATE_END_ARGS()

	void Construct(const FArguments& Args);

	void SetTexture(FTG_Texture& Source, UTextureGraph* InTextureGraph);

	static const float PreferredWidth;
protected:
	TSharedPtr<STG_HistogramBlob> HistogramBars;
	TiledBlobPtr HistogramBlob;
};

