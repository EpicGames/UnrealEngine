// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "Expressions/Filter/TG_Expression_Levels.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "STG_LevelsSettings.h"
#include "STG_TextureHistogram.h"

#define LOCTEXT_NAMESPACE "FTextureGraphEditorModule"

class FTG_LevelsSettingsCustomization : public IPropertyTypeCustomization
{
	TSharedPtr<IPropertyHandle> LevelsHandle;

public:
	static TSharedRef<IPropertyTypeCustomization> Create()
	{
		return MakeShareable(new FTG_LevelsSettingsCustomization);
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		// Hide the struct header
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		LevelsHandle = PropertyHandle;
		check(LevelsHandle.IsValid());

		TWeakPtr<IPropertyHandle> LevelsHandlePtr = LevelsHandle;

		void* LevelsValuePtr;
		LevelsHandle->GetValueData(LevelsValuePtr);
		FTG_LevelsSettings* LevelsValue = static_cast<FTG_LevelsSettings*>(LevelsValuePtr);

		FTG_OnLevelsSettingsValueChanged OnLevelsChanged = FTG_OnLevelsSettingsValueChanged::CreateRaw(this, &FTG_LevelsSettingsCustomization::OnValueChanged, LevelsHandlePtr);

		// Build the  ui
		TSharedPtr<STG_LevelsSettings>  LevelSettingsWidget;	
		ChildBuilder.AddCustomRow(LOCTEXT("Levels", "Levels"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(FText::FromName(TEXT("Levels")))
				.Font(CustomizationUtils.GetRegularFont())
			]
			.ValueContent()
			.MinDesiredWidth(STG_TextureHistogram::PreferredWidth)
			[
				SAssignNew(LevelSettingsWidget, STG_LevelsSettings)
				.Levels((*LevelsValue))
				.OnValueChanged(OnLevelsChanged)
			];
	}


	void OnValueChanged(const FTG_LevelsSettings& NewValue, TWeakPtr<IPropertyHandle> HandleWeakPtr)
	{
		auto HandleSharedPtr = HandleWeakPtr.Pin();
		//auto HandleSharedPtr = LevelsHandle;
		if (HandleSharedPtr.IsValid())
		{
			ensure(HandleSharedPtr->SetValueFromFormattedString(NewValue.ToString()) == FPropertyAccess::Success);
		}
	}
};

#undef LOCTEXT_NAMESPACE