// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "TextureGraph.h"
#include "TG_Graph.h"
#include "TG_Material.h"

class FTG_MaterialCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> Create()
	{
		return MakeShareable(new FTG_MaterialCustomization);
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		const FName TGType = FName(TEXT("TGType"));
		const FName CategoryName = PropertyHandle->GetDefaultCategoryName();
		const FString ParentTGType = PropertyHandle->GetParentHandle()->GetMetaData(TGType);

		// show the desc UI
		TSharedPtr<IPropertyHandle> DescPropertyHandle = PropertyHandle->GetChildHandle(TEXT("AssetPath"));
		FString DisplayName = PropertyHandle->GetPropertyDisplayName().ToString();
		{
			ChildBuilder.AddProperty(DescPropertyHandle.ToSharedRef())
				.DisplayName(FText::FromString(DisplayName))
				.ShouldAutoExpand(true);
		}
	}

};
