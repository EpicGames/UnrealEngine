// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TG_Parameter.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "IPropertyTypeCustomization.h"
//#include "IPropertyUtilities.h"
//#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
//#include "DetailLayoutBuilder.h"
//#include "ISinglePropertyView.h"
#include "IDetailChildrenBuilder.h"
//#include "IDetailCustomization.h"
#include "PropertyCustomizationHelpers.h"
#include "TG_Pin.h"
#include "Expressions/Output/TG_Expression_Output.h"
#include "TG_Node.h"
#include "TG_Graph.h"
#include "TextureGraph.h"
#include "Model/Mix/ViewportSettings.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FTextureGraphEditorModule"

class FTG_MaterialMappingInfoCustomization : public IPropertyTypeCustomization
{

private:
	UMixSettings* MixSettings;
	
	TSharedPtr<IPropertyHandle> MaterialInputPropertyHandle;
	TSharedPtr<IPropertyHandle> TargetPropertyHandle;
	
public:
	static TSharedRef<IPropertyTypeCustomization> Create()
	{
		return MakeShareable(new FTG_MaterialMappingInfoCustomization);
	}

	void GenerateMaterialInputsArray(TArray< TSharedPtr<FString> >& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems, bool bAllowClear, FName FilterStructName)
	{
		check(MixSettings)

		UTextureGraphBase* TextureGraph = Cast<UTextureGraphBase>(MixSettings->Mix());

		OutComboBoxStrings.Add(MakeShared<FString>("None"));		
		if (TextureGraph->Graph())
		{
			TextureGraph->Graph()->ForEachNodes(
			[&](const UTG_Node* node, uint32 index)
			{
				if(UTG_Expression_Output* OutputExpression = Cast<UTG_Expression_Output>(node->GetExpression()))
				{
					OutComboBoxStrings.Add(MakeShared<FString>(OutputExpression->GetTitleName().ToString()));			
				}
			});
		}
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		
	}

	FString OnGetTypeValueString()
	{
		FString TargetValue;
		TargetPropertyHandle->GetValue(TargetValue);
		
		return TargetValue;
	}

	void OnTargetSelected(const FString& UpdatedTargetValue)
	{
		TargetPropertyHandle->SetValue(UpdatedTargetValue);
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		
		MaterialInputPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMaterialMappingInfo, MaterialInput));
		TargetPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMaterialMappingInfo, Target));

		FName MaterialInputValue;
		MaterialInputPropertyHandle->GetValue(MaterialInputValue);

		TArray<UObject*> OuterObjects;
		MaterialInputPropertyHandle->GetOuterObjects(OuterObjects);

		MixSettings = Cast<UMixSettings>(OuterObjects[0]);		
		
		FName FilterStructName;
		
		FPropertyComboBoxArgs TypeArgs(TargetPropertyHandle,
			FOnGetPropertyComboBoxStrings::CreateSP(this,&FTG_MaterialMappingInfoCustomization::GenerateMaterialInputsArray, true, FilterStructName),
			FOnGetPropertyComboBoxValue::CreateSP(this, &FTG_MaterialMappingInfoCustomization::OnGetTypeValueString),
			FOnPropertyComboBoxValueSelected::CreateSP(this, &FTG_MaterialMappingInfoCustomization::OnTargetSelected));
		
		ChildBuilder.AddCustomRow(LOCTEXT("MaterialInput", "Material Input"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(FText::FromName(MaterialInputValue))
				.Font(CustomizationUtils.GetRegularFont())
			]
			.ValueContent()
			.MaxDesiredWidth(0.0f)
			[
				PropertyCustomizationHelpers::MakePropertyComboBox(TypeArgs)
			];
	}

	
};

#undef LOCTEXT_NAMESPACE
