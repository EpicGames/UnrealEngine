// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "TG_OutputSettings.h"
#include "DetailCategoryBuilder.h"
#include "Transform/Layer/T_Thumbnail.h"
#include "Expressions/Output/TG_Expression_Output.h"
#include "Expressions/Input/TG_Expression_OutputSettings.h"
#include "TG_Node.h"
#include "TG_Graph.h"
#include "TextureGraph.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "KismetPins/SGraphPinStructInstance.h"
#include "IDetailGroup.h"
#include "Misc/Paths.h"
#include <TG_Texture.h>

#define LOCTEXT_NAMESPACE "OutputSettingsCustomization"

class FTG_OutputSettingsCustomization : public IPropertyTypeCustomization
{
	TSharedPtr<SVerticalBox> PropertyLayout;
	TSharedPtr<IPropertyHandle> PathPropertyHandle;
	FString DefaultPath;
	
public:
	static TSharedRef<IPropertyTypeCustomization> Create()
	{
		return MakeShareable(new FTG_OutputSettingsCustomization);
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		FName OutputNameValue = "OutputSettings";
		
		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);

		check(OuterObjects[0]);

		auto Expression = Cast<UTG_Expression_OutputSettings>(OuterObjects[0]);

		if (Expression)
		{
			auto Node = Cast<UTG_Node>(Expression->GetOuter());

			auto Pin = Node->GetPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_OutputSettings, Settings));

			OutputNameValue = Pin->GetAliasName();
		}

		HeaderRow.WholeRowContent()
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			[
				//Show the Target Name and is not editable
				SNew(STextBlock)
				.Text(FText::FromName(OutputNameValue))
				.Font(CustomizationUtils.GetRegularFont())
			]
		];
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		PathPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTG_OutputSettings, FolderPath));
		auto OutputNamePropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTG_OutputSettings, OutputName));
		auto BaseNamePropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTG_OutputSettings, BaseName));

		FString PathValue;
		PathPropertyHandle->GetValue(PathValue);

		PathValue = "Enter Path";

		FMargin Margin = FMargin(2, 2);
		FMargin VMargin = FMargin(0, 2);
		//Since A margin of FMargin(12,0,2,0) is given by RightRowPadding Value Widget so we need to 
		//revert horizontal margins for separator so it has no space
		FMargin SeparatorMargin = FMargin(-12,2,-2,2);

		TArray<UObject*> OuterObjects;
		OutputNamePropertyHandle->GetOuterObjects(OuterObjects);

		check(OuterObjects[0]);

		auto Expression = Cast<UTG_Expression>(OuterObjects[0]);

		if (Expression)
		{
			auto Node = Cast<UTG_Node>(Expression->GetOuter());

			auto Pin = Node->GetOutputPinAt(0);

			auto TextureGraph = Cast<UTextureGraph>(Node->GetGraph()->GetOuter());//Expression->OutputSettings.DefaultPath;
			auto AssetPath = TextureGraph->GetPathName();
			DefaultPath = FPaths::GetPath(AssetPath);
		}

		//constexpr int ThumbSize = 64;
		// Get ThumbAsset from Pin's FTG_Texture
		/*auto& Texture = Pin->EditSelfVar()->EditAs<FTG_Texture>();
		auto thumbnailWidget = T_Thumbnail::GetThumbnailWidgetFromAsset(Texture.ThumbAsset, Pin, UThumbnailManager::Get().GetSharedThumbnailPool());
		
		TSharedPtr<SWidget> PinThumbHolder =
			SNew(SBox)
			.WidthOverride(ThumbSize)
			.HeightOverride(ThumbSize)
			.Padding(Margin)
			[
				SNew(SBox)
				[
					thumbnailWidget.ToSharedRef()
				]
			];*/

		auto& row = ChildBuilder.AddCustomRow(LOCTEXT("Output Settings", "Export Path"))
		.NameContent()
		[
			PathPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.FillWidth(1)
			[
				PathPropertyHandle->CreatePropertyValueWidget()
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &FTG_OutputSettingsCustomization::OnBrowseClick)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.ToolTipText(LOCTEXT("Browsepath_ToolTip", "Select the path for the output"))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush("Icons.BrowseContent"))
				]
			]
		];

		row.ShouldAutoExpand(true);

		ChildBuilder.AddProperty(BaseNamePropertyHandle.ToSharedRef());

		IDetailGroup& CollapsibleGroup = ChildBuilder.AddGroup("Advanced", FText::FromString("Advanced"));

		CollapsibleGroup.AddPropertyRow(PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTG_OutputSettings, Width)).ToSharedRef());
		CollapsibleGroup.AddPropertyRow(PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTG_OutputSettings, Height)).ToSharedRef());
		CollapsibleGroup.AddPropertyRow(PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTG_OutputSettings, TextureFormat)).ToSharedRef());
	}

	FReply OnBrowseClick()
	{
		FString PackagePath;
		BrowseFolderPath(DefaultPath, "/Output", PackagePath);
		PathPropertyHandle->SetValue(PackagePath);
		return FReply::Handled();
	}

	static void BrowseFolderPath(const FString& DefaultFolder, const FString& NewFolderName, FString& OutPackagePath)
	{
		FString TextureAssetName;
		FString PackagePathSuggestion = DefaultFolder.IsEmpty() ? GetCurrentFolderInContentBrowser() + NewFolderName : DefaultFolder + NewFolderName;
		FString Name;
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(PackagePathSuggestion, TEXT(""), PackagePathSuggestion, Name);

		TSharedPtr<SDlgPickAssetPath> PickAssetPathWidget =
			SNew(SDlgPickAssetPath)
			.Title(LOCTEXT("Output Path Selection", "Choose Output Path"))
			.DefaultAssetPath(FText::FromString(PackagePathSuggestion));

		if (PickAssetPathWidget->ShowModal() == EAppReturnType::Ok)
		{
			// Get the full name of where we want to create the Texture asset.
			OutPackagePath = PickAssetPathWidget->GetFullAssetPath().ToString();
			TextureAssetName = FPackageName::GetLongPackageAssetName(OutPackagePath);

			OutPackagePath = OutPackagePath.LeftChop(TextureAssetName.Len() + 1);
		}
	}

	static FString GetCurrentFolderInContentBrowser()
	{
		IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
		const FContentBrowserItemPath CurrentPath = ContentBrowser.GetCurrentPath();
		return CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : FString();
	}
};


class FTG_OutputExpressionInfoCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> Create()
	{
		return MakeShareable(new FTG_OutputExpressionInfoCustomization);
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		HeaderRow.Visibility(EVisibility::Collapsed);
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		TArray<void*> RawData;
		PropertyHandle->AccessRawData(RawData);

		FOutputExpressionInfo* OutputExpressionInfo = reinterpret_cast<FOutputExpressionInfo*>(RawData[0]);
		TArray<UObject*> ExternalObjects;

		PropertyHandle->GetOuterObjects(ExternalObjects);
		UObject* OutermostObject = ExternalObjects[0]->GetOutermostObject();
		UTextureGraphBase* TextureGraph = CastChecked<UTextureGraphBase>(OutermostObject);

		FTG_Id OutputNodeId = OutputExpressionInfo->OutputNodeID;

		UTG_Expression_Output* OutputExpression = Cast<UTG_Expression_Output>(TextureGraph->Graph()->GetNode(OutputNodeId)->GetExpression());
		
		ExternalObjects.Empty();
		ExternalObjects.Add(OutputExpression);
		
		TObjectPtr<UTG_Pin> TGPin = OutputExpression->GetParentNode()->GetPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Output, OutputSettings));
		FProperty* Property = TGPin->GetExpressionProperty();
		bool bCanEditChange = OutputExpression->CanEditChange(Property);
		FName NAME_EditCondition(TEXT("EditCondition"));
		Property->SetMetaData(NAME_EditCondition, bCanEditChange ? TEXT("true") : TEXT("false"));

		
		IDetailPropertyRow* Row = ChildBuilder.AddExternalObjectProperty(ExternalObjects, GET_MEMBER_NAME_CHECKED(UTG_Expression_Output, OutputSettings), FAddPropertyParams().HideRootObjectNode(true).CreateCategoryNodes(false));
		Row->DisplayName(FText::FromName(OutputExpression->GetTitleName()));
		
		// Custom edit condition for OutputSettings
		TAttribute<bool> OutputSettingsEditCondition = TAttribute<bool>::Create(
			[this, OutputExpression]()
			{
				return OutputExpression->GetShouldExport();
			});
		
		Row->EditCondition(OutputSettingsEditCondition, FOnBooleanValueChanged::CreateLambda([this, OutputExpression](bool NewValue)
			{
				OutputExpression->SetShouldExport(NewValue);
			}));
	}
};

#undef LOCTEXT_NAMESPACE
