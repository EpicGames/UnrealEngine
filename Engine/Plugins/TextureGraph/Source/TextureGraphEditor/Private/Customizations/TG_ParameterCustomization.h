// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TG_Parameter.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "IPropertyTypeCustomization.h"
//#include "IPropertyUtilities.h"
//#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
//#include "DetailLayoutBuilder.h"
//#include "ISinglePropertyView.h"
#include "IDetailChildrenBuilder.h"
//#include "IDetailCustomization.h"
#include "TG_Pin.h"
#include "TG_Graph.h"
#include "Expressions/TG_Expression.h"
#include "Expressions/Input/TG_Expression_Texture.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"


class FTG_ParameterInfoCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> Create()
	{
		return MakeShareable(new FTG_ParameterInfoCustomization);
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		TArray<void*> RawData;
		PropertyHandle->AccessRawData(RawData);

		FTG_ParameterInfo* ParameterInfoPtr = reinterpret_cast<FTG_ParameterInfo*>(RawData[0]);

		TArray<UObject*> OuterObjects;

		if (ParameterInfoPtr != nullptr)
		{
			FTG_Id id = ParameterInfoPtr->Id;
			PropertyHandle->GetOuterObjects(OuterObjects);

			UTG_Parameters* paramsOuter = Cast<UTG_Parameters>(OuterObjects[0]);

			UTG_Pin* PinPtr = paramsOuter->TextureGraph->GetPin(id);
			if (PinPtr != nullptr)
			{
				if (PinPtr->IsInput())
				{
					auto ExpressionPtr = PinPtr->GetNodePtr()->GetExpression();

					TArray<UObject*> ExternalObjects;
					ExternalObjects.Add(ExpressionPtr);

					FName PinName = PinPtr->GetArgumentName();

					// In case of texture expression
					// Param is TG_Texture but it cannot be modified here
					// use UTexture source because its the only way we can change the parameter value via parameter panel.
					UTG_Expression_Texture* TextureExpression = Cast<UTG_Expression_Texture>(ExpressionPtr);
					if(TextureExpression)
					{
						if (PinName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Texture, Source))
						{
							// skip the Source pin to be shown directly
							return;
						}
						else if (PinName == GET_MEMBER_NAME_CHECKED(UTG_Expression_Texture, Texture))
						{
							// allow the Texture pin instead with the correct parameter alias name but internally use the Source pin.
							PinName = GET_MEMBER_NAME_CHECKED(UTG_Expression_Texture, Source);
						}
					}
					
					IDetailPropertyRow* ParameterDetailsPropertyRow = ChildBuilder.AddExternalObjectProperty(ExternalObjects, PinName, FAddPropertyParams().HideRootObjectNode(true));
					if (ParameterDetailsPropertyRow)
					{
						ParameterDetailsPropertyRow->DisplayName(FText::FromName(ParameterInfoPtr->Name));
					}
				}
			}
		}
	}
};
