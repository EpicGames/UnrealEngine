// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SSlider.h"
#include "Editor.h"
#include "STG_TextureHistogram.h"
#include "TG_SystemTypes.h"

#define LOCTEXT_NAMESPACE "FTextureGraphEditorModule"

class FTG_ScalarTypeIdentifier : public IPropertyTypeIdentifier
{
public:
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& PropertyHandle) const override
	{
		return PropertyHandle.HasMetaData(TG_MetadataSpecifiers::MD_ScalarEditor);
	}
};

class FTG_ScalarCustomization : public IPropertyTypeCustomization
{
	/** Edited Property */
	TSharedPtr<IPropertyHandle> ScalarHandle;

	/** True if the slider is being used to change the value of the property */
	bool bIsUsingSlider = false;

public:
	static TSharedRef<IPropertyTypeCustomization> Create()
	{
		return MakeShareable(new FTG_ScalarCustomization);
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		ScalarHandle = PropertyHandle;

		// Get min/max metadata values if defined
		float MinAllowedValue = 0.0f;
		float MinUIValue = 0.0f;
		float MaxAllowedValue = 1.0f;
		float MaxUIValue = 1.0f;
		FText DisplayName;
		if (ScalarHandle.IsValid())
		{
			if (ScalarHandle->HasMetaData(TEXT("ClampMin")))
			{
				MinAllowedValue = ScalarHandle->GetFloatMetaData(TEXT("ClampMin"));
			}
			if (ScalarHandle->HasMetaData(TEXT("UIMin")))
			{
				MinUIValue = ScalarHandle->GetFloatMetaData(TEXT("UIMin"));
			}
			if (ScalarHandle->HasMetaData(TEXT("ClampMax")))
			{
				MaxAllowedValue = ScalarHandle->GetFloatMetaData(TEXT("ClampMax"));
			}
			if (ScalarHandle->HasMetaData(TEXT("UIMax")))
			{
				MaxUIValue = ScalarHandle->GetFloatMetaData(TEXT("UIMax"));
			}
			DisplayName = ScalarHandle->GetPropertyDisplayName();
		}

		// Build the  ui
		HeaderRow
			.NameContent()
			[
				SNew(STextBlock)
				.Text(DisplayName)
				.Font(CustomizationUtils.GetRegularFont())
			]
			.ValueContent()
			.MinDesiredWidth(STG_TextureHistogram::PreferredWidth)
			[
				SNew(SSlider)
				.Value(this, &FTG_ScalarCustomization::OnGetValue)
				.MinValue(MinAllowedValue)
				.MaxValue(MaxAllowedValue)
				.OnValueChanged(this, &FTG_ScalarCustomization::OnValueChanged)
				.OnMouseCaptureBegin(this, &FTG_ScalarCustomization::OnBeginSliderMovement)
				.OnMouseCaptureEnd(this, &FTG_ScalarCustomization::OnEndSliderMovement)
			];
	}

	float OnGetValue() const
	{
		float NumericVal = 0;
		if (ScalarHandle.IsValid())
		{
			if (ScalarHandle->GetValue(NumericVal) != FPropertyAccess::Fail)
			{
				return NumericVal;
			}
		}
		return NumericVal;
	}

	void OnValueChanged(float NewValue)
	{
		if (bIsUsingSlider)
		{
			if (ScalarHandle.IsValid())
			{
				float OrgValue(0);
				if (ScalarHandle->GetValue(OrgValue) != FPropertyAccess::Fail)
				{
					// Value hasn't changed, so lets return now
					if (OrgValue == NewValue)
					{
						return;
					}
				}

				// We don't create a transaction for each property change when using the slider.  Only once when the slider first is moved
				EPropertyValueSetFlags::Type Flags = (EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable);
				ScalarHandle->SetValue(NewValue, Flags);
			}
		}
	}

	void OnBeginSliderMovement()
	{
		bIsUsingSlider = true;
		if (ScalarHandle.IsValid())
		{
			GEditor->BeginTransaction(NSLOCTEXT("GraphEditor", "ChangeNumberPinValueSlider", "Change Number Pin Value slider"));
		}
	}

	void OnEndSliderMovement()
	{
		bIsUsingSlider = false;

		// set value once more with default flags so the TextureGraph system recognizes a non-interactive change as well.
		float OrgValue(0);
		if (ScalarHandle->GetValue(OrgValue) != FPropertyAccess::Fail)
		{
			ScalarHandle->SetValue(OrgValue);
		}
		GEditor->EndTransaction();
	}


	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
	}
};

#undef LOCTEXT_NAMESPACE