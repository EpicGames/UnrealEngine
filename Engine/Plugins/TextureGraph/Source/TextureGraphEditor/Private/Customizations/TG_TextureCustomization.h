// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "TextureGraph.h"
#include "TG_Graph.h"
#include "TG_Texture.h"
#include "Expressions/TG_Expression.h"
#include "STG_TextureHistogram.h"


class FTG_TextureCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> Create()
	{
		return MakeShareable(new FTG_TextureCustomization);
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
#if 0
		const FName TGType = FName(TEXT("TGType"));
		const FName CategoryName = PropertyHandle->GetDefaultCategoryName();
		const FString ParentTGType = PropertyHandle->GetParentHandle()->GetMetaData(TGType);
		
		const bool bShouldShowHistogram = PropertyHandle->HasMetaData(TG_MetadataSpecifiers::MD_HistogramLuminance);
		// Show header only for Output TextureDescriptors inner properties or Input Textures with histogram meta
		if (ParentTGType == TEXT("TG_Output") || bShouldShowHistogram)
		{
			HeaderRow.NameContent()
			[
				PropertyHandle->CreatePropertyNameWidget()
			];
		}

		if (bShouldShowHistogram)
		{
			TSharedPtr<STG_TextureHistogram> TextureHistogram;
			HeaderRow.ValueContent()
			.MinDesiredWidth(STG_TextureHistogram::PreferredWidth)
			[
				SAssignNew(TextureHistogram, STG_TextureHistogram)
			];

			// Get to the ftg_texture and root TextureGraph pointer
			TArray<void*> RawData;
			PropertyHandle->AccessRawData(RawData);
			FTG_Texture* TexturePtr = reinterpret_cast<FTG_Texture*>(RawData[0]);

			TArray<UObject*> OuterObjects;
			PropertyHandle->GetOuterObjects(OuterObjects);

			UTG_Expression* InExpressionPtr = nullptr;
			if (!OuterObjects.IsEmpty())
			{
				InExpressionPtr = Cast<UTG_Expression>(OuterObjects[0]);
			}
			if (InExpressionPtr)
			{
				UTextureGraph* TextureGraph = Cast<UTextureGraph>(Cast<UTG_Node>(InExpressionPtr->GetOuter())->GetGraph()->GetOuter());
				TextureHistogram->SetTexture(*TexturePtr, TextureGraph);
			}
		}
#endif 
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
#if 0
		const FString TGType = PropertyHandle->GetMetaData(FName(TEXT("TGType"))); 
		if (TGType == TEXT("TG_Output"))
		{
			// show the desc UI
			TSharedPtr<IPropertyHandle> DescPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTG_Texture, Descriptor));
			FString DisplayName = PropertyHandle->GetPropertyDisplayName().ToString() + TEXT(" Settings");
			{
				ChildBuilder.AddProperty(DescPropertyHandle.ToSharedRef())
					.DisplayName(FText::FromString(DisplayName))
					.ShouldAutoExpand(true);
			}
		}
#endif 
	}
};
