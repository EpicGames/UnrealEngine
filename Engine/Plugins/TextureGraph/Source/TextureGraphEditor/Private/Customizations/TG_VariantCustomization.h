// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "TG_Texture.h"
#include "Expressions/TG_Expression.h"
#include "STG_TextureHistogram.h"


class FTG_VariantCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> Create()
	{
		return MakeShareable(new FTG_VariantCustomization);
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		const FString TGType = PropertyHandle->GetMetaData(FName(TEXT("TGType"))); 
		if (TGType == TEXT("TG_Output"))
		{
			
			TArray<UObject*> OuterObjects;
			PropertyHandle->GetOuterObjects(OuterObjects);
			check(OuterObjects[0]);
			UTG_Expression* ParentExpression = Cast<UTG_Expression>(OuterObjects[0]);
			TArray<void*> VariantPtrs;
			PropertyHandle->AccessRawData(VariantPtrs);
			if (VariantPtrs.Num() == 0)
			{
				return;
			}
			FTG_Variant* VariantPtr = reinterpret_cast<FTG_Variant*>(VariantPtrs[0]);
			// check if texture
			if (VariantPtr->IsTexture())
			{
				TSharedRef<FStructOnScope> StructData = MakeShareable(new FStructOnScope(FTG_TextureDescriptor::StaticStruct(), (uint8*)&VariantPtr->GetTexture().Descriptor));
				IDetailPropertyRow* DetailPropertyRow = ChildBuilder.AddExternalStructureProperty(StructData, NAME_None, FAddPropertyParams().CreateCategoryNodes(false).HideRootObjectNode(true));
				DetailPropertyRow->ShouldAutoExpand(true);
				
				if (TSharedPtr<IPropertyHandle> Handle = DetailPropertyRow->GetPropertyHandle(); Handle.IsValid())
				{
					FString DisplayName = PropertyHandle->GetPropertyDisplayName().ToString() + TEXT(" Settings");
					DetailPropertyRow->DisplayName(FText::FromString(DisplayName));
					
					// Handle->SetInstanceMetaData(TEXT("TGType"), TEXT("TG_Output"));
					TSharedPtr<IPropertyUtilities> PropertyUtilities;
					PropertyUtilities = CustomizationUtils.GetPropertyUtilities();

					TDelegate<void(const FPropertyChangedEvent&)> OnPropertyChangeWithData = TDelegate<void(const FPropertyChangedEvent&)>::CreateLambda([=, this](const FPropertyChangedEvent& InEvent)
					{
						if (PropertyUtilities.IsValid())
						{
							void* StructData = nullptr;
							const FPropertyAccess::Result Result = Handle->GetValueData(StructData);
							if (Result == FPropertyAccess::Success)
							{
								check(StructData);
								FTG_TextureDescriptor* Descriptor = static_cast<FTG_TextureDescriptor*>(StructData);

								VariantPtr->EditTexture().Descriptor = *Descriptor;
								FPropertyChangedEvent ChangeEvent(InEvent);
								ChangeEvent.MemberProperty = PropertyHandle->GetProperty();
								ParentExpression->NotifyExpressionChanged(ChangeEvent);
								PropertyUtilities->RequestRefresh();
							}
						}
					});

					FSimpleDelegate OnPropertyValuePreChange = FSimpleDelegate::CreateLambda([=, this]()
					{
						ParentExpression->Modify();
					});

					//when property itself is modified like when reset button is clicked
					Handle->SetOnPropertyValuePreChange(OnPropertyValuePreChange);
					Handle->SetOnPropertyValueChangedWithData(OnPropertyChangeWithData);

					//when child properties are modified
					Handle->SetOnChildPropertyValuePreChange(OnPropertyValuePreChange);
					Handle->SetOnChildPropertyValueChangedWithData(OnPropertyChangeWithData);
				}
			}
		}
	}
};
