// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "IDetailChildrenBuilder.h"

#include "Model/Mix/ViewportSettings.h"
#include "Model/Mix/MixSettings.h"

#define LOCTEXT_NAMESPACE "FTextureGraphEditorModule"

class FTG_ViewportSettingsCustomization : public IPropertyTypeCustomization
{

private:
	UMixSettings* MixSettings;
	
	TSharedPtr<IPropertyHandle> MaterialPropertyHandle;
	TSharedPtr<IPropertyHandle> MaterialMappingInfosPropertyHandle;
	
public:
	static TSharedRef<IPropertyTypeCustomization> Create()
	{
		return MakeShareable(new FTG_ViewportSettingsCustomization);
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		MaterialPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FViewportSettings, Material));
		MaterialMappingInfosPropertyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FViewportSettings, MaterialMappingInfos));
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		const TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();
		if (PropertyHandle->IsValidHandle())
		{
			ChildBuilder.AddProperty(MaterialPropertyHandle.ToSharedRef());
			MaterialPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([=, this]()
			{
				// refresh details
				PropertyUtils->ForceRefresh();
			}));
			// In this case, we'll add the array elements directly without the header for the array name.
			uint32 NumChildren;
			MaterialMappingInfosPropertyHandle->GetNumChildren(NumChildren);

			for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
			{
				TSharedRef<IPropertyHandle> ChildHandle = MaterialMappingInfosPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
				if (ChildHandle->IsValidHandle())
				{
					ChildBuilder.AddProperty(ChildHandle);
				}
			}
		}
	}
};

#undef LOCTEXT_NAMESPACE
