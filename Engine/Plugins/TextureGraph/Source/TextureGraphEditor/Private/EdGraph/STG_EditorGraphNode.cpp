// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_EditorGraphNode.h"

#include "SGraphPanel.h"
#include "SGraphPin.h"
#include "TG_EdGraphSchema.h"
#include "Expressions/TG_Expression.h"
#include "TG_Style.h"
#include "TG_Texture.h"
#include "2D/Tex.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "EdGraph/TG_EdGraphNode.h"
#include "EdGraph/TG_EdGraph.h"
#include "Job/JobBatch.h"
#include "Widgets/SWidget.h"
#include "SLevelOfDetailBranchNode.h"
#include "IDocumentation.h"
#include "TutorialMetaData.h"
#include "SCommentBubble.h"
#include "Transform/Layer/T_Thumbnail.h"
#include "TG_Graph.h"
#include "TG_HelperFunctions.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "SlateOptMacros.h"
#include "STG_EditorViewport.h"
#include "Widgets/SToolTip.h"
#include "STG_NodeThumbnail.h"
#include "Job/Scheduler.h"
#include "Job/ThumbnailsService.h"
#include "Pins/STG_TextureDescriptor.h"
#include "Widgets/Layout/SSeparator.h"
#define LOCTEXT_NAMESPACE "TextureGraphEditor"

static const FName NAME_Pin_NotConnectable("Graph.Pin.Dummy");
static const FSlateBrush* CacheImg_Pin_NotConnectable = nullptr;

STG_EditorGraphNode::~STG_EditorGraphNode()
{
	if (TGEditorGraphNode)
	{
		TGEditorGraphNode->OnNodeReconstructDelegate.Remove(OnNodeChangedHandle);
		TGEditorGraphNode->OnNodePostEvaluateDelegate.Remove(OnPostEvaluateHandle);
		TGEditorGraphNode->OnPinSelectionChangeDelegate.Remove(OnPinSelectionChangedHandle);
	}

	const ThumbnailsServicePtr SvcThumbnail = TextureGraphEngine::GetScheduler()->GetThumbnailsService().lock();
	if (SvcThumbnail)
	{
		SvcThumbnail->OnUpdateThumbnailDelegate.Remove(OnUpdateThumbHandle);
	}
}

void STG_EditorGraphNode::Construct(const FArguments& InArgs, UTG_EdGraphNode* InNode)
{
	GraphNode = InNode;
	TGEditorGraphNode = InNode;

	if (TGEditorGraphNode)
	{
		OnNodeChangedHandle = TGEditorGraphNode->OnNodeReconstructDelegate.AddRaw(this, &STG_EditorGraphNode::OnNodeReconstruct);
		OnPostEvaluateHandle = TGEditorGraphNode->OnNodePostEvaluateDelegate.AddRaw(this, &STG_EditorGraphNode::OnNodePostEvaluate);
		OnPinSelectionChangedHandle = TGEditorGraphNode->OnPinSelectionChangeDelegate.AddRaw(this, &STG_EditorGraphNode::OnPinSelectionChanged);
	}
	const ThumbnailsServicePtr SvcThumbnail = TextureGraphEngine::GetScheduler()->GetThumbnailsService().lock();
	if (SvcThumbnail)
	{
		// register yourself to the thumbnail service to get events when a thumb batch is done.
		OnUpdateThumbHandle = SvcThumbnail->OnUpdateThumbnailDelegate.AddRaw(this, &STG_EditorGraphNode::OnUpdateThumbnail);
	}
	BodyBrush = *GetNodeBodyBrush();
	BodyBrush.OutlineSettings.Color = UTG_EdGraphSchema::NodeBodyColorOutline;
	
	HeaderBrush = *FTG_Style::Get().GetBrush("TG.Graph.Node.Header");
	HeaderBrush.OutlineSettings.Color = GetNodeTitleColor().GetSpecifiedColor() * UTG_EdGraphSchema::NodeOutlineColorMultiplier;

	UpdateGraphNode();
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STG_EditorGraphNode::UpdateGraphNode()
{
	const FMargin TitleRightWidgetPadding = FMargin(5, 0, 0, 0);
	const FMargin TitleOuterMargin = FMargin(4);

	InputPins.Empty();
	OutputPins.Empty();

	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();
	ParametersBox.Reset();
	LowDetailParametersBox.Reset();
	NodeSettingsBox.Reset();
	NotConnectableParametersBox.Reset();
	NotConnectableInputPinBox.Reset();

	//
	//             ______________________
	//            |      TITLE AREA      |
	//            +-------+------+-------+
	//            | (>) L |      | R (>) |
	//            | (>) E |      | I (>) |
	//            | (>) F |      | G (>) |
	//            | (>) T |      | H (>) |
	//            |       |      | T (>) |
	//            |_______|______|_______|
	//
	TSharedPtr<SVerticalBox> MainVerticalBox;
	SetupErrorReporting();

	TSharedPtr<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode);

	// Get node icon
	IconColor = FLinearColor::White;
	const FSlateBrush* IconBrush = nullptr;
	if (GraphNode != NULL && GraphNode->ShowPaletteIconOnNode())
	{
		IconBrush = GraphNode->GetIconAndTint(IconColor).GetOptionalIcon();
	}

	TitleBorderMargin = FMargin(6);

	TSharedRef<SOverlay> DefaultTitleAreaWidget =
	SNew(SOverlay)
	+ SOverlay::Slot()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Top)
	[
		SNew(SBorder)
		.BorderImage(&HeaderBrush)//FAppStyle::GetBrush("Graph.Node.ColorSpill")
		.BorderBackgroundColor(this, &STG_EditorGraphNode::GetNodeTitleColor)
		.Padding(TitleBorderMargin)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(TitleBorderMargin)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Top)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(IconBrush)
					.ColorAndOpacity(this, &STG_EditorGraphNode::GetNodeTitleIconColor)
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Top)
					.AutoHeight()
					[
						CreateTitleWidget(NodeTitle)
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Top)
					[
						NodeTitle.ToSharedRef()
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						CreateTitleDetailsWidget()
					]
				]
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(TitleRightWidgetPadding)
			.AutoWidth()
			[
				CreateTitleRightWidget()
			]
		]
	];

	SetDefaultTitleAreaWidget(DefaultTitleAreaWidget);

	TSharedRef<SWidget> TitleAreaWidget =
		SNew(SLevelOfDetailBranchNode)
		.UseLowDetailSlot(this, &STG_EditorGraphNode::UseLowDetailNodeTitles)
		.LowDetail()
		[
			SNew(SBorder)
			.BorderImage(&HeaderBrush)
			.Padding(FMargin(75.0f, 22.0f)) // Saving enough space for a 'typical' title so the transition isn't quite so abrupt
			.BorderBackgroundColor(this, &STG_EditorGraphNode::GetNodeTitleColor)
		]
		.HighDetail()
		[
			DefaultTitleAreaWidget
		];


	if (!SWidget::GetToolTip().IsValid())
	{
		TSharedRef<SToolTip> DefaultToolTip = IDocumentation::Get()->CreateToolTip(TAttribute< FText >(this, &SGraphNode::GetNodeTooltip), NULL, GraphNode->GetDocumentationLink(), GraphNode->GetDocumentationExcerptName());
		SetToolTip(DefaultToolTip);
	}

	// Setup a meta tag for this node
	FGraphNodeMetaData TagMeta(TEXT("Graphnode"));
	PopulateMetaTag(&TagMeta);

	TSharedPtr<SVerticalBox> InnerVerticalBox;

	InnerVerticalBox = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		.Padding(TitleOuterMargin)
		[
			TitleAreaWidget
		]

	+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			CreateNodeContentArea()
		];

	FMargin InnerVerticalBoxMargin = Settings->GetNonPinNodeBodyPadding();

	//Add padding for showing border
	int BorderThickness = GetContentAreaBorderThickness();
	InnerVerticalBoxMargin.Left += BorderThickness;
	InnerVerticalBoxMargin.Right += BorderThickness;

	TSharedPtr<SWidget> EnabledStateWidget = GetEnabledStateWidget();
	if (EnabledStateWidget.IsValid())
	{
		InnerVerticalBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			.Padding(FMargin(2 + BorderThickness, 0))
			[
				EnabledStateWidget.ToSharedRef()
			];
	}

	InnerVerticalBox->AddSlot()
		.AutoHeight()
		.Padding(InnerVerticalBoxMargin)
		[
			ErrorReporting->AsWidget()
		];

	InnerVerticalBox->AddSlot()
		.AutoHeight()
		.Padding(InnerVerticalBoxMargin)
		[
			VisualWarningReporting->AsWidget()
		];

	this->GetOrAddSlot(ENodeZone::Center)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SOverlay)
				.AddMetaData<FGraphNodeMetaData>(TagMeta)
				+ SOverlay::Slot()
				.Padding(Settings->GetNonPinNodeBodyPadding())
				[
					SNew(SImage)
					.Image(&BodyBrush)
					.ColorAndOpacity(this, &STG_EditorGraphNode::GetNodeBodyColor)
				]
				+ SOverlay::Slot()
				[
					InnerVerticalBox.ToSharedRef()
				]
			]
		];

	bool SupportsBubble = true;
	if (GraphNode != nullptr)
	{
		SupportsBubble = GraphNode->SupportsCommentBubble();
	}

	if (SupportsBubble)
	{
		// Create comment bubble
		TSharedPtr<SCommentBubble> CommentBubble;
		const FSlateColor CommentColor = GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor;

		SAssignNew(CommentBubble, SCommentBubble)
			.GraphNode(GraphNode)
			.Text(this, &SGraphNode::GetNodeComment)
			.OnTextCommitted(this, &SGraphNode::OnCommentTextCommitted)
			.OnToggled(this, &SGraphNode::OnCommentBubbleToggled)
			.ColorAndOpacity(CommentColor)
			.AllowPinning(true)
			.EnableTitleBarBubble(true)
			.EnableBubbleCtrls(true)
			.GraphLOD(this, &SGraphNode::GetCurrentLOD)
			.IsGraphNodeHovered(this, &SGraphNode::IsHovered);

		GetOrAddSlot(ENodeZone::TopCenter)
			.SlotOffset2f(TAttribute<FVector2f>(CommentBubble.Get(), &SCommentBubble::GetOffset2f))
			.SlotSize2f(TAttribute<FVector2f>(CommentBubble.Get(), &SCommentBubble::GetSize2f))
			.AllowScaling(TAttribute<bool>(CommentBubble.Get(), &SCommentBubble::IsScalingAllowed))
			.VAlign(VAlign_Top)
			[
				CommentBubble.ToSharedRef()
			];
	}

	CreateBelowWidgetControls(MainVerticalBox);
	CreatePinWidgets();
	CreateInputSideAddButton(LeftNodeBox);
	CreateOutputSideAddButton(RightNodeBox);
	CreateAdvancedViewArrow(InnerVerticalBox);
	//CreateNodeSettingsLayout();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

int32 STG_EditorGraphNode::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	{
		ArrangeChildren(AllottedGeometry, ArrangedChildren);
	}

	//THis is a work around for the sorting issue
	//In SGraphPanel all the nodes get the same layer id but the inside overlay increments that id 
	//This cause the side effect that when nodes are overlapping, elements from both nodes are partially visible
	//The SOverlay is using a padding of 20 for such cases so we are making sure next node has the layer id above that padding
	//so in case of overlaping opaque nodes it fully hides the node below it 
	int Index = GraphNode->GetGraph()->Nodes.Find(GraphNode);
	bool bSelected = OwnerGraphPanelPtr.Pin()->GetSelectedGraphNodes().Find(GraphNode) > -1;// GraphNode->IsSelected();
	int32 NodeLayerID = bSelected ? LayerId + (25 * GraphNode->GetGraph()->Nodes.Num()) : LayerId + (25 * Index);
	int32 MaxLayerId = NodeLayerID;

	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		const FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];

		if (!IsChildWidgetCulled(MyCullingRect, CurWidget))
		{
			const int32 CurWidgetsMaxLayerId = CurWidget.Widget->Paint(Args.WithNewParent(this), CurWidget.Geometry, MyCullingRect, OutDrawElements, MaxLayerId, InWidgetStyle, ShouldBeEnabled(bParentEnabled));
			MaxLayerId = FMath::Max(MaxLayerId, CurWidgetsMaxLayerId);
		}
	}

	if (bSelected)
	{
		const FVector2D NodeShadowSize = GetDefault<UGraphEditorSettings>()->GetShadowDeltaSize();
		const FSlateBrush* ShadowBrush = GetShadowBrush(bSelected);
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			NodeLayerID,
			AllottedGeometry.ToInflatedPaintGeometry(NodeShadowSize),
			ShadowBrush,
			ESlateDrawEffect::None,
			FLinearColor(1.0f, 1.0f, 1.0f, 1.0)
		);
	}

	return MaxLayerId;
}

const FSlateBrush* STG_EditorGraphNode::GetShadowBrush(bool bSelected) const
{
	return bSelected ? FAppStyle::GetBrush(TEXT("Graph.Node.ShadowSelected")) : FAppStyle::GetBrush(TEXT("Graph.Node.Shadow"));
}

int STG_EditorGraphNode::GetContentAreaBorderThickness()
{
	int BorderThickness = 2; // This kind of thing is generally defined in UGraphEditorSettings
	return BorderThickness;
}

TSharedRef<SWidget> STG_EditorGraphNode::CreateNodeContentArea()
{
	const int HAdvanceContentPading = GetContentAreaBorderThickness();
	const int LowDetailOverrideSettingWidth = 75;
	const int LowDetailOverrideSettingHeight = 66;
	// NODE CONTENT AREA
	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(FMargin(0, 3))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Top)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.FillWidth(1.0f)
				[
					// LEFT
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(LeftNodeBox, SVerticalBox)
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(NotConnectableInputPinBox, SVerticalBox)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					// RIGHT
					SAssignNew(RightNodeBox, SVerticalBox)
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(HAdvanceContentPading, 0)
			[
				SNew(SLevelOfDetailBranchNode)
				.UseLowDetailSlot(this, &STG_EditorGraphNode::UseLowDetailParameters)
				.LowDetail()
				[
					SAssignNew(LowDetailParametersBox, SVerticalBox)
					.Visibility(this, &STG_EditorGraphNode::ShowParameters) //visible when when node have advance display

					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(SSeparator)
						.Thickness(2)
					]
				]
				.HighDetail()
				[
					SAssignNew(ParametersBox, SVerticalBox)
					.Visibility(this, &STG_EditorGraphNode::ShowParameters) //visible when when node have advance display

					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					[
						SNew(SSeparator)
						.Thickness(2)
					]
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(HAdvanceContentPading, 0)
			[
				SAssignNew(NotConnectableParametersBox, SVerticalBox)
				.Visibility(this, &STG_EditorGraphNode::ShowParameters) //visible when when node have advance display
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(HAdvanceContentPading, 0)
			[
				SNew(SLevelOfDetailBranchNode)
				.UseLowDetailSlot(this, &STG_EditorGraphNode::UseLowDetailParameters)
				.LowDetail()
				[
					//Create empty space when showing low lod
					SNew(SBox)
					.Visibility(this, &STG_EditorGraphNode::ShowOverrideSettings)
					.WidthOverride(LowDetailOverrideSettingWidth)
					.HeightOverride(LowDetailOverrideSettingHeight)
				]
				.HighDetail()
				[
					SAssignNew(NodeSettingsBox, SVerticalBox)
					.Visibility(this, &STG_EditorGraphNode::ShowOverrideSettings)// Visible only when node is expanded
				]
			]
		];
}

EVisibility STG_EditorGraphNode::ShowParameters() const
{
	const bool bShowParameters = GraphNode && (ENodeAdvancedPins::NoPins != GraphNode->AdvancedPinDisplay);
	return bShowParameters ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility STG_EditorGraphNode::ShowOverrideSettings() const
{
	const bool bOverrideSettings = GraphNode && (ENodeAdvancedPins::Shown == GraphNode->AdvancedPinDisplay) && TGEditorGraphNode->GetTextureOutputPins().Num() > 0;
	return bOverrideSettings ? EVisibility::Visible : EVisibility::Collapsed;
}

void STG_EditorGraphNode::CreateNodeSettingsLayout()
{
	NodeSettingsBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SSeparator)
			.Thickness(2)
		];

	NodeSettingsBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(0,3)
		[
			CreateNodeSettings()
		];
}

TSharedRef<SWidget> STG_EditorGraphNode::CreateNodeSettings()
{
	auto Widget = SNew(SVerticalBox);
	const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(GraphNode->GetSchema());

	for (auto Pin : TGEditorGraphNode->GetTextureOutputPins())
	{
		UTG_Pin* TGPin = Schema->GetTGPinFromEdPin(Pin);
		FTG_Texture& Texture = TGPin->GetNodePtr()->GetGraph()->GetVar(TGPin->GetId())->EditAs<FTG_Texture>();

		TSharedPtr<SGraphPin> NewPin = CreatePinWidget(Pin);
		check(NewPin.IsValid());
		SetPinIcon(NewPin.ToSharedRef());

		this->AddPin(NewPin.ToSharedRef());

			//bool bShowLabel = true;
			//// check if there is a display name defined for the property, we use that as the Pin Name
			//if (Property && Property->HasMetaData("PinDisplayName"))
			//{
			//	FString DisplayName = Property->GetMetaData("PinDisplayName");
			//	if (!DisplayName.IsEmpty())
			//	{
			//		CurPin->PinFriendlyName = FText::FromString(DisplayName);
			//	}
			//	else
			//	{
			//		NewPin->SetShowLabel(false);
			//		bShowLabel = false;
			//	}
			//}

			//Hide pin icon for not connectable pins
			//If the pin do not have label then Icon space is not required
			//If the pin has a label then we need to the hidden icon space to allign better with connectable pins
		//	if (!NewPin->GetIsConnectable())
		//	{
		//		NewPin->GetPinImageWidget()->SetVisibility(bShowLabel ? EVisibility::Hidden : EVisibility::Collapsed);
		//	}
		//}

			
		Widget->AddSlot().HAlign(HAlign_Fill)
		.Padding(Settings->GetNonPinNodeBodyPadding())
		[
			SNew(STG_TextureDescriptor, Pin)
		];
	}

	return Widget;
}

TSharedRef<SWidget> STG_EditorGraphNode::CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle)
{
	SAssignNew(InlineEditableText, SInlineEditableTextBlock)
		.Style(FTG_Style::Get(), "TG.Graph.Node.NodeTitleInlineEditableText")
		.Text(NodeTitle.Get(), &SNodeTitle::GetHeadTitle)
		.OnVerifyTextChanged(this, &STG_EditorGraphNode::OnVerifyNameTextChanged)
		.OnTextCommitted(this, &STG_EditorGraphNode::OnNameTextCommited)
		.IsReadOnly(this, &STG_EditorGraphNode::IsNameReadOnly)
		.IsSelected(this, &STG_EditorGraphNode::IsSelectedExclusively)
		.OverflowPolicy(GetNameOverflowPolicy());
	InlineEditableText->SetColorAndOpacity(TAttribute<FLinearColor>::Create(TAttribute<FLinearColor>::FGetter::CreateSP(this, &STG_EditorGraphNode::GetNodeTitleTextColor)));

	return InlineEditableText.ToSharedRef();
}

bool STG_EditorGraphNode::UseLowDetailNodeTitles() const
{
	return SGraphNode::UseLowDetailNodeTitles();
}

bool STG_EditorGraphNode::UseLowDetailParameters() const
{
	if (const SGraphPanel* MyOwnerPanel = GetOwnerPanel().Get())
	{
		return (MyOwnerPanel->GetCurrentLOD() <= EGraphRenderingLOD::LowDetail);
	}

	return false;
}

FSlateColor STG_EditorGraphNode::GetNodeTitleColor() const
{
	FLinearColor ReturnTitleColor = GraphNode->IsDeprecated() ? FLinearColor::Red : GetNodeObj()->GetNodeTitleColor();

	if (!GraphNode->IsNodeEnabled() || GraphNode->IsDisplayAsDisabledForced() || GraphNode->IsNodeUnrelated())
	{
		ReturnTitleColor *= FLinearColor(0.5f, 0.5f, 0.5f, 0.4f);
	}
	return ReturnTitleColor;
}

const FSlateBrush* STG_EditorGraphNode::GetNodeBodyBrush() const
{
	return FTG_Style::Get().GetBrush("TG.Graph.Node.Body");
}

TArray<FString> STG_EditorGraphNode::GetTitleDetailTextLines() const
{
	auto TGGraphNode = Cast<UTG_EdGraphNode>(GraphNode);
	FString DetailText = TGGraphNode->GetTitleDetail();
	TArray<FString> Lines;
	DetailText.ParseIntoArray(Lines, TEXT("\n"), false);
	return Lines;
}

FText STG_EditorGraphNode::GetLeftTitleText() const
{
	TArray<FString> Lines = GetTitleDetailTextLines();
	FString LeftText = Lines.Num() > 0 ? Lines[0] : "";

	return FText::FromString(LeftText);
}

FText STG_EditorGraphNode::GetRightTitleText() const
{
	TArray<FString> Lines = GetTitleDetailTextLines();
	FString RightText = Lines.Num() > 1 ? Lines[1] : "";

	return FText::FromString(RightText);
}

TSharedRef<SWidget> STG_EditorGraphNode::CreateTitleDetailsWidget()
{
	auto DetailTextColor = FLinearColor::FromSRGBColor(FColor::FromHex(TEXT("0F6389")));
	DetailTextColor.A = 1.0;

	FMargin Margin = FMargin(0, 5, 0, 0);

	return SNew(SHorizontalBox)
	.Visibility(this,&STG_EditorGraphNode::IsTitleDetailVisible)
	+ SHorizontalBox::Slot()
	.Padding(Margin)
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Bottom)
	[
		SNew(STextBlock)
		.TextStyle(&FTG_Style::Get(), TEXT("TG.Graph.Node.NodeTitleExtraLines"))
		.Text(this,&STG_EditorGraphNode::GetLeftTitleText)
	]

	//Empty space between the Right and Left Detail Text
	+ SHorizontalBox::Slot()
	.HAlign(HAlign_Left)
	.Padding(Margin)
	.AutoWidth()
	[
		SNew(SBox)
		.MinDesiredWidth(10)
	]

	+ SHorizontalBox::Slot()
	.HAlign(HAlign_Right)
	.Padding(Margin)
	.VAlign(VAlign_Bottom)
	.AutoWidth()
	[
		SNew(STextBlock)
		.TextStyle(&FTG_Style::Get(), TEXT("TG.Graph.Node.NodeTitleExtraLines"))
		.Text(this,&STG_EditorGraphNode::GetRightTitleText)
	];
}

EVisibility STG_EditorGraphNode::IsTitleDetailVisible() const
{
	return TGEditorGraphNode->GetTitleDetail() == "" ? EVisibility::Collapsed : EVisibility::Visible;
}

void STG_EditorGraphNode::CreatePinWidgets()
{
	// Create Pin widgets for each of the pins.
	for( int32 PinIndex=0; PinIndex < GraphNode->Pins.Num(); ++PinIndex )
	{
		UEdGraphPin* CurPin = GraphNode->Pins[PinIndex];

		bool bHideNoConnectionPins = false;

		if (OwnerGraphPanelPtr.IsValid())
		{
			bHideNoConnectionPins = OwnerGraphPanelPtr.Pin()->GetPinVisibility() == SGraphEditor::Pin_HideNoConnection;
		}

		const bool bPinHasConections = CurPin->LinkedTo.Num() > 0;

		bool bPinDesiresToBeHidden = CurPin->bHidden || (bHideNoConnectionPins && !bPinHasConections); 
		
		const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(TGEditorGraphNode->GetSchema());
		UTG_Pin* TSPin = Schema->GetTGPinFromEdPin(CurPin);
		FProperty* Property = TSPin->GetExpressionProperty();
		if (Property && Property->HasMetaData("HideNodeUI"))
		{
			bPinDesiresToBeHidden = true;
		}

		if (!bPinDesiresToBeHidden)
		{
			TSharedPtr<SGraphPin> NewPin = CreatePinWidget(CurPin);
			check(NewPin.IsValid());
			SetPinIcon(NewPin.ToSharedRef());
			
			this->AddPin(NewPin.ToSharedRef());
			
			bool bShowLabel = true;
			// check if there is a display name defined for the property, we use that as the Pin Name
			if (Property && Property->HasMetaData("PinDisplayName"))
			{
				FString DisplayName = Property->GetMetaData("PinDisplayName");
				if (!DisplayName.IsEmpty())
				{
					CurPin->PinFriendlyName = FText::FromString(DisplayName);
				}
				else
				{
					NewPin->SetShowLabel(false);
					bShowLabel = false;
				}
			}

			//Hide pin icon for not connectable pins
			//If the pin do not have label then Icon space is not required
			//If the pin has a label then we need to the hidden icon space to allign better with connectable pins
			if (!NewPin->GetIsConnectable())
			{
				NewPin->GetPinImageWidget()->SetVisibility(bShowLabel ? EVisibility::Hidden : EVisibility::Collapsed);
			}
		}
	}
}

TSharedPtr<STG_NodeThumbnail> STG_EditorGraphNode::FindOrCreateThumbWidget(FTG_Id PinId)
{
	TSharedPtr<STG_NodeThumbnail> ThumbWidget;
	if (PinThumbWidgetMap.Contains(PinId))
	{
		ThumbWidget = PinThumbWidgetMap[PinId];
	}
	else
	{
		ThumbWidget = SNew(STG_NodeThumbnail);
		PinThumbWidgetMap.Add(PinId, ThumbWidget);
	}
	return ThumbWidget;
}

void STG_EditorGraphNode::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	PinToAdd->SetOwner(SharedThis(this));

	const UEdGraphPin* PinObj = PinToAdd->GetPinObj();
	const bool bAdvancedParameter = PinObj && PinObj->bAdvancedView;
	if (bAdvancedParameter)
	{
		PinToAdd->SetVisibility( TAttribute<EVisibility>(PinToAdd, &SGraphPin::IsPinVisibleAsAdvanced) );
	}

	if (PinToAdd->GetDirection() == EEdGraphPinDirection::EGPD_Input)
	{
		if (bAdvancedParameter)
		{
			if (PinToAdd->GetIsConnectable())
			{
				ParametersBox->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(Settings->GetInputPinPadding())
				[
					PinToAdd
				];

				LowDetailParametersBox->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(Settings->GetInputPinPadding())
				[
					PinToAdd
				];
			}
			else
			{
				NotConnectableParametersBox->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(Settings->GetInputPinPadding())
				[
					PinToAdd
				];
			}
			InputPins.Add(PinToAdd);
		}
		else
		{
			if (PinToAdd->GetIsConnectable())
			{
				LeftNodeBox->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(Settings->GetInputPinPadding())
				[
					PinToAdd
				];
			}
			else
			{
				NotConnectableInputPinBox->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(Settings->GetInputPinPadding())
				[
					PinToAdd
				];
			}
			InputPins.Add(PinToAdd);
		}
		//Updating the Input Pin Layout here
		UpdateInputPinLayout(PinToAdd);
	}
	else // Direction == EEdGraphPinDirection::EGPD_Output
	{
		TSharedPtr<SHorizontalBox> OutputPinBox = SNew(SHorizontalBox);

		// Add Pin
		OutputPinBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.HAlign(HAlign_Right)
				.MinDesiredWidth(20.0f)
				[
					PinToAdd
				]
			];

		// Add to RightNodeBox
		RightNodeBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(Settings->GetOutputPinPadding())
		[
			OutputPinBox.ToSharedRef()
		];
		OutputPins.Add(PinToAdd);
	}
}

void STG_EditorGraphNode::UpdateInputPinLayout(const TSharedRef<SGraphPin>& PinToAdd)
{
	//SGraphPin Puts every thing into a WrapBox
	//We are changing it to use Horizontal box
	if (PinToAdd->GetDirection() == EEdGraphPinDirection::EGPD_Input)
	{
		const UEdGraphPin* PinObj = PinToAdd->GetPinObj();
		auto HorizontalBox = SNew(SHorizontalBox);
		auto WrapBox = PinToAdd->GetLabelAndValue();
		auto WrapBoxChildren = WrapBox->GetAllChildren();
		auto FullRow = PinToAdd->GetFullPinHorizontalRowWidget();

		auto FullRowChildren = FullRow.Pin()->GetAllChildren();

		auto NumChild = WrapBoxChildren->NumSlot();

		if (NumChild > 0)
		{
			auto Label = WrapBoxChildren->GetChildAt(0);
			HorizontalBox->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.FillWidth(1.0)
			.Padding(0, 0, 5, 0)
			[
				Label
			];
		}

		for (int i = 1; i < WrapBoxChildren->NumSlot(); i++)
		{
			auto Child = WrapBoxChildren->GetChildAt(i);

			HorizontalBox->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					Child
				];
		}

		auto LabelAndValueWidget = FullRowChildren->GetChildAt(1);
		FullRow.Pin()->RemoveSlot(LabelAndValueWidget);

		FullRow.Pin()->AddSlot()
		[
			HorizontalBox
		];

	}
}

void STG_EditorGraphNode::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SGraphNode::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	RefreshErrorInfo();
}

FLinearColor STG_EditorGraphNode::GetPinThumbSelectionColor()
{
	return FLinearColor(0.828f, 0.364f, 0.003f, 1);
}

FLinearColor STG_EditorGraphNode::GetPinThumbDeselectionColor()
{
	return FLinearColor(0.828f, 0.364f, 0.003f, 0);
}

FReply STG_EditorGraphNode::OnOutputIconClick(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Handled();
}

TSharedRef<SWidget> STG_EditorGraphNode::CreateTitleRightWidget()
{
	auto TGNode = TGEditorGraphNode->GetNode();
	TArray<UTG_Pin*> Pins;
	TGNode->GetOutputPins(Pins);
	TSharedPtr<SWidget> Thumb = SNullWidget::NullWidget;

	// looping here to find first texture output pin and showing thumbnail for that only
	for (auto Pin : Pins)
	{
		// Only interested by the Texture output pin
		FTG_Texture Texture;
		if (Pin->GetValue(Texture))
		{
			auto ThumbnailWidget = FindOrCreateThumbWidget(Pin->GetId());

			// get blob from EdGraphNode's cache if available
			UTG_EdGraph* EdGraph = Cast<UTG_EdGraph>(TGEditorGraphNode->GetGraph());
			if (TiledBlobPtr CachedThumb = EdGraph->GetCachedThumbBlob(Pin->GetId()))
			{
				if (CachedThumb->IsFinalised())
				{
					if (ThumbnailWidget.IsValid())
					{
						ThumbnailWidget->UpdateBlob(CachedThumb);
					}
				}
			}
			
			if (Thumb == SNullWidget::NullWidget)
			{
				constexpr int ThumbSize = 44;

				Thumb = PreviewWidgetBox = SNew(SBox)
				.WidthOverride(ThumbSize)
				.HeightOverride(ThumbSize)
				.Visibility(this, &STG_EditorGraphNode::IsTitleDetailVisible)
				[
					ThumbnailWidget.ToSharedRef()
				];
			}
		}
	}

	return Thumb.ToSharedRef();
}

void STG_EditorGraphNode::SetPinIcon(const TSharedRef<SGraphPin> PinToAdd)
{
	check(TGEditorGraphNode);
	UTG_Node* TSNode = TGEditorGraphNode->GetNode();
	UEdGraphPin* EdPin = PinToAdd->GetPinObj();

	if (TSNode && EdPin)
	{
		// Assign a custom icon to not connectible pins
		if (EdPin->bNotConnectable)
		{
			const FSlateBrush* ConnectedBrush = FTG_Style::Get().GetBrush(TSEditorStyleConstants::Pin_Generic_Image_C);
			PinToAdd->SetCustomPinIcon(ConnectedBrush, ConnectedBrush);
		}
		else
		{
			const bool bIsInPin = EdPin->Direction == EEdGraphPinDirection::EGPD_Input;
			const FName& PinName = EdPin->PinName;

			if (UTG_Pin* Pin = (bIsInPin ? TSNode->GetInputPin(PinName) : TSNode->GetOutputPin(PinName)))
			{
				const FSlateBrush* ConnectedBrush = FTG_Style::Get().GetBrush( TSEditorStyleConstants::Pin_Generic_Image_C );
				const FSlateBrush* DisconnectedBrush = FTG_Style::Get().GetBrush( TSEditorStyleConstants::Pin_Generic_Image_DC);
				PinToAdd->SetCustomPinIcon(ConnectedBrush, DisconnectedBrush);
			}
		}
	}
}

TSharedRef<SWidget> STG_EditorGraphNode::CreatePreviewWidget(UTG_Pin* Pin, const TSharedPtr<SWidget>& ThumbWidget)
{
	constexpr int ThumbSize = 42;

	TSharedPtr<SWidget> PinThumbHolder =
		SNew(SBox)
		.WidthOverride(ThumbSize)
		.HeightOverride(ThumbSize)
		[
			ThumbWidget.ToSharedRef()
		];
	return PinThumbHolder.ToSharedRef();
}
void STG_EditorGraphNode::OnNodeReconstruct()
{
	// reset pin thumb widget map
	PinThumbWidgetMap.Reset();
	UpdateGraphNode();
}

void STG_EditorGraphNode::OnNodePostEvaluate(const FTG_EvaluationContext* InContext)
{
}

void STG_EditorGraphNode::OnPinSelectionChanged(UEdGraphPin* EdPin)
{
	const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(TGEditorGraphNode->GetSchema());
	UTG_Pin* Pin = Schema->GetTGPinFromEdPin(EdPin);

	if (Pin)
	{
		auto PinThumbWidget = PinThumbWidgetMap.Find(Pin->GetId());
		if (PinThumbWidget && PreviewWidgetBox.IsValid())
		{
			PreviewWidgetBox->SetContent(PinThumbWidget->ToSharedRef());
		}
	}
}


void STG_EditorGraphNode::ApplyThumbToWidget()
{
	UTG_EdGraph* EdGraph = Cast<UTG_EdGraph>(TGEditorGraphNode->GetGraph());
	TArray<UTG_Pin*> Pins;
	TGEditorGraphNode->GetNode()->GetOutputPins(Pins);
	for(const UTG_Pin* Pin : Pins)
	{
		auto ThumbWidget = FindOrCreateThumbWidget(Pin->GetId());
		if (ThumbWidget.IsValid())
		{
			ThumbWidget->UpdateBlob(EdGraph->GetCachedThumbBlob(Pin->GetId()));
		}
	}
}

void STG_EditorGraphNode::OnUpdateThumbnail(JobBatchPtr JobBatch)
{
	// check if the JobBatch has the same TextureGraph as ours (As the Thumbnail Service could be catering to multiple TextureGraphs)
	if (JobBatch->GetCycle()->GetMix() == TGEditorGraphNode->GetOutermostObject())
	{
		ApplyThumbToWidget();
	}
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE