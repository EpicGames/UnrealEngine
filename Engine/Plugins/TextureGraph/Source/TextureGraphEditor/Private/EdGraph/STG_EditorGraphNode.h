// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNode.h"
#include "TG_GraphEvaluation.h"
#include "TG_Node.h"
#include "EdGraph/TG_EdGraphNode.h"
#include "Widgets/Images/SImage.h"
#include "AssetThumbnail.h"
#include "STG_NodeThumbnail.h"
#include "Widgets/Layout/SBox.h"
#include "ThumbnailRendering/ThumbnailManager.h"

struct FOverlayBrushInfo;

class UTG_EdGraphNode;
class SWidget;
class SVerticalBox;

class STG_EditorGraphNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(STG_EditorGraphNode){}
	SLATE_END_ARGS()

	virtual ~STG_EditorGraphNode() override;
	
	void Construct(const FArguments& InArgs, UTG_EdGraphNode* InNode);
	virtual void UpdateGraphNode() override;
	// void AddOutputPinThumbnail(UEdGraphPin* CurPin);

	//~ Begin SGraphNode Interface
	//virtual const FSlateBrush* GetNodeBodyBrush() const override;
	//virtual void RequestRenameOnSpawn() override { /* Empty to avoid the default behavior to rename on node spawn */ }
	//virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
	TSharedPtr<STextBlock> RightLabel;

	virtual void CreatePinWidgets() override;
	virtual TSharedRef<SWidget> CreateTitleRightWidget() override;
	//virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const;
	virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
	//~ End SGraphNode Interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	void UpdateInputPinLayout(const TSharedRef<SGraphPin>& PinToAdd);
	/** Thumbnail Widget */
	TSharedRef<SWidget> CreatePreviewWidget(UTG_Pin* Pin, const TSharedPtr<SWidget>& ThumbWidget);

	void SetPinIcon(const TSharedRef<SGraphPin> Shared);

	virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	
	//~ Begin SNodePanel::SNode Interface
	//virtual void GetOverlayBrushes(bool bSelected, const FVector2D WidgetSize, TArray<FOverlayBrushInfo>& Brushes) const override;
	//~ End SNodePanel::SNode Interface

protected:
	/** The area where non connectable input pins reside */
	TSharedPtr<SVerticalBox> NotConnectableInputPinBox;
	/** The area where Param pins and settings pins reside*/
	TSharedPtr<SVerticalBox> ParametersBox;
	/** The area where Param pins and settings pins reside for Low detail*/
	TSharedPtr<SVerticalBox>  LowDetailParametersBox;
	/** The area where non connectable Param pins and settings pins reside*/
	TSharedPtr<SVerticalBox> NotConnectableParametersBox;
	/** The area where node specific settings pins reside alyaws show in advance view */
	TSharedPtr<SVerticalBox> NodeSettingsBox;
	int GetContentAreaBorderThickness();

	virtual void CreateNodeSettingsLayout();
	virtual TSharedRef<SWidget> CreateNodeSettings();
	virtual TSharedRef<SWidget> CreateNodeContentArea() override;
	virtual EVisibility ShowParameters() const;
	virtual EVisibility ShowOverrideSettings() const;
	void OnNodeReconstruct();
	void OnNodePostEvaluate(const FTG_EvaluationContext* InContext);
	void OnPinSelectionChanged(UEdGraphPin* Pin);

	FSlateColor GetNodeTitleColor() const;
	const FSlateBrush* GetNodeBodyBrush() const;
	TSharedRef<SWidget> CreateTitleDetailsWidget();
	virtual EVisibility IsTitleDetailVisible() const;
	virtual bool UseLowDetailNodeTitles() const;
	virtual TSharedRef<SWidget> CreateTitleWidget(TSharedPtr<SNodeTitle> NodeTitle) override;

private:
	UTG_EdGraphNode* TGEditorGraphNode = nullptr;
	FDelegateHandle OnUpdateThumbHandle;
	FDelegateHandle OnPostEvaluateHandle;
	FDelegateHandle OnNodeChangedHandle;
	FDelegateHandle OnPinSelectionChangedHandle;
	FSlateBrush BodyBrush;
	FSlateBrush HeaderBrush;
	TSharedPtr<SBox> PreviewWidgetBox;
	TMap<FTG_Id, TSharedPtr<STG_NodeThumbnail>> PinThumbWidgetMap;
	FLinearColor GetPinThumbSelectionColor();
	FLinearColor GetPinThumbDeselectionColor();
	TSharedPtr<STG_NodeThumbnail> FindOrCreateThumbWidget(FTG_Id PinId);
	void ApplyThumbToWidget();
	void OnUpdateThumbnail(std::shared_ptr<JobBatch> JobBatch);
	UTexture* GetTextureFromBuffer(DeviceBufferPtr Buffer);
	FReply OnOutputIconClick(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent);
	TArray<FString> GetTitleDetailTextLines() const;
	FText GetLeftTitleText() const;
	FText GetRightTitleText() const;
	bool UseLowDetailParameters() const;
};
