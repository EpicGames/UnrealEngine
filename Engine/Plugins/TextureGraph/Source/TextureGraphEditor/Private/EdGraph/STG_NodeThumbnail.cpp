// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_NodeThumbnail.h"

#include "2D/Tex.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include "Engine/Texture2D.h"
#include "Transform/Layer/T_Thumbnail.h"

#define LOCTEXT_NAMESPACE "TextureGraphEditor"

void STG_NodeThumbnail::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(BrushMaterial);
}

FString STG_NodeThumbnail::GetReferencerName() const
{
	return TEXT("STG_NodeThumbnail");
}

/** Constructs this widget with InArgs */
void STG_NodeThumbnail::Construct(const FArguments& InArgs)
{
	BrushName = InArgs._BrushName;
	Brush = MakeShareable(
		new FSlateDynamicImageBrush(
			(UTexture2D*) nullptr,
			FVector2D(RenderMaterial_Thumbnail::GThumbWidth, RenderMaterial_Thumbnail::GThumbHeight),
			//desiredSize,
			BrushName,
			FLinearColor(1.0f, 1.0f, 1.0f, 1.0f),
			ESlateBrushTileType::NoTile,
			ESlateBrushImageType::Linear
		)
	);
	UMaterial* material = GetMaterial();
	BrushMaterial = (UMaterialInstanceDynamic::Create(material, nullptr));

	Position = FVector2D(100.0f, 100.0f);
	UpdateBlob(InArgs._Blob);
}

UMaterial* STG_NodeThumbnail::GetMaterial()
{
	return LoadObject<UMaterial>(nullptr, TEXT("/TextureGraph/Materials/Util/RoundedThumb"));
}

UTexture* STG_NodeThumbnail::GetTextureFromBlob(TiledBlobPtr InBlob)
{
	UTexture* BlobTexture = nullptr;
	//By now the Result should have valid DeviceBuffer
	if (InBlob)
	{
		check(InBlob->IsFinalised());
		DeviceBufferPtr Buffer = InBlob->GetBufferRef().GetPtr();

		checkSlow(Buffer);
		
		auto FXBuffer = std::static_pointer_cast<DeviceBuffer_FX>(Buffer);
		// Got the description, now need to retreive the true live buffer and the texture
		{
			// If the Buffer is an FX buffer we can extract the Texture from it, Can be a texture2D or RT
			if (FXBuffer)
			{
				auto Tex = FXBuffer->GetTexture();
				check(Tex);
				Tex->SetFilter(TextureFilter::TF_Nearest);
				BlobTexture = Tex->GetTexture();
				//PinAssetMap[Pin->GetId()].ThumbAsset->SetAsset(BlobTexture);
			}
		}
		// auto Texture = Cast<UTexture>(AssetThumbnail->GetAsset());
	}

	return BlobTexture;
}

void STG_NodeThumbnail::UpdateParams(TiledBlobPtr InBlob)
{
	if (!InBlob)
		return;
	
	/// If the blob hasn't been finalised then we don't do anything. This sort of scenario can happen if you end 
	/// up with your thumbnail generation split between multiple batches on the ThumbnailsService
	if (InBlob->IsFinalised())
	{
		UTexture* BlobTexture = GetTextureFromBlob(InBlob);
		check(BlobTexture);

		//FRHITexture* SlateTexture = (FRHITexture*)(AssetThumbnail->GetViewportRenderTargetTexture());
		UTexture2D* Texture2D = Cast<UTexture2D>(BlobTexture);
		UTextureRenderTarget2D* TextureRT2D = Cast<UTextureRenderTarget2D>(BlobTexture);
	
		float ShowChecker = 1.0;
		
		FLinearColor NormalizedDimensions(0.9f, 0.9f, 0.0f);
		
		if (BlobTexture)
		{
			ShowChecker = 0.0;
			BrushMaterial->SetTextureParameterValue("ThumbTex", BlobTexture);

			int32 Width = BlobTexture->GetSurfaceWidth();
			int32 Height = BlobTexture->GetSurfaceHeight();

			float MaxDimension = FMath::Max(Width, Height);

			NormalizedDimensions.R *= Width / MaxDimension;
			NormalizedDimensions.G *= Height / MaxDimension;
		}

		float SingleChannel = 0.0;

		if (Texture2D)
		{
			SingleChannel = GPixelFormats[Texture2D->GetPixelFormat()].NumComponents == 1 ? 1.0 : 0.0;
		}

		if (TextureRT2D)
		{
			SingleChannel = GPixelFormats[TextureRT2D->GetFormat()].NumComponents == 1 ? 1.0 : 0.0;
		}

		BrushMaterial->SetScalarParameterValue("ShowChecker", ShowChecker);
		BrushMaterial->SetScalarParameterValue("SingleChannel", SingleChannel);
		
		BrushMaterial->SetVectorParameterValue("NormalizedDimensions", NormalizedDimensions);
		Brush->SetResourceObject(BrushMaterial);
	}
}

void STG_NodeThumbnail::UpdateBlob(TiledBlobPtr InBlob)
{
	UpdateParams(InBlob);
	
	TSharedPtr<SOverlay> overlay;
	ChildSlot
	[
		SAssignNew(overlay, SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SImage)
			.Image(Brush.Get())
		]
	];
}

//////////////////////////////////////////////////////////////////////////
#undef LOCTEXT_NAMESPACE