// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/Material.h"
#include "AssetThumbnail.h"
#include "Data/TiledBlob.h"

class STG_NodeThumbnail : public SCompoundWidget, public FGCObject
{
public:
	SLATE_BEGIN_ARGS(STG_NodeThumbnail)
		:_Blob(nullptr)
		,_BrushName("NodeThumb")
	{}

	SLATE_ARGUMENT(TiledBlobPtr , Blob)
	SLATE_ARGUMENT(FName , BrushName)
	SLATE_END_ARGS()

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	void UpdateBlob(TiledBlobPtr InBlob);
	virtual void UpdateParams(TiledBlobPtr InBlob);
	virtual UMaterial* GetMaterial();
	UTexture* GetTextureFromBlob(TiledBlobPtr InBlob);

private:
	FVector2D Position;
	FName BrushName;
	TSharedPtr<FSlateBrush> Brush;
	TObjectPtr<UMaterialInstanceDynamic> BrushMaterial = nullptr;
};
