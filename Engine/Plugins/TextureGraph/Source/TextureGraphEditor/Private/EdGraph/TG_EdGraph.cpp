// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/TG_EdGraph.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/TG_EdGraphNode.h"
#include "Expressions/TG_Expression.h"
#include "TextureGraph.h"
#include "TG_Editor.h"
#include "TG_Graph.h"
#include "Transform/Layer/T_Thumbnail.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_EdGraph)

void UTG_EdGraph::InitializeFromTextureGraph(UTextureGraph* InTextureGraph, TWeakPtr<FTG_Editor> InTGEditor)
{
	TextureGraph = InTextureGraph;
	TGEditor = InTGEditor;

	// And now build the EdGraph viewmodel matching the script
	BuildEdGraphViewmodel();
}

bool UTG_EdGraph::CreateViewModelLinkFromModelPins(const UTG_Pin* pinFrom, const UTG_Pin* pinTo)
{
	if (!(pinFrom && pinTo))
		return false; // skip this edge if invalid pin

	FTG_Id pinFromId = pinFrom->GetNodeId();
	FTG_Id pinToId = pinTo->GetNodeId();

	UEdGraphNode* edNodeFrom = this->GetViewModelNode(pinFromId);
	UEdGraphNode* edNodeTo = this->GetViewModelNode(pinToId);

	auto edPinFrom = edNodeFrom->FindPin(pinFrom->GetArgumentName(), EGPD_Output);
	auto edPinTo = edNodeTo->FindPin(pinTo->GetArgumentName(), EGPD_Input);

	if (edPinFrom && edPinTo)
	{
		edPinFrom->Modify();
		edPinTo->Modify();
		edPinFrom->MakeLinkTo(edPinTo);
		return true;
	}

	UE_LOGF(LogTemp, Warning, "UTG_EdGraph::CreateViewModelLinkFromModelPins Invalid Link attempted %ls => %ls", *pinFrom->GetArgumentName().ToString(), *pinTo->GetArgumentName().ToString())
	return false;
}

void UTG_EdGraph::CacheThumbBlob(FTG_Id PinId, TiledBlobPtr InBlob)
{
	PinThumbBlobMap.FindOrAdd(PinId) = InBlob;
}

TiledBlobPtr UTG_EdGraph::GetCachedThumbBlob(FTG_Id PinId)
{
	return (PinThumbBlobMap.IsEmpty() || !PinThumbBlobMap.Contains(PinId)) ? nullptr : *PinThumbBlobMap.Find(PinId);
}

void UTG_EdGraph::BuildEdGraphViewmodel()
{
	// For every model nodes, let's create an equivalent viewmodel node
	TextureGraph->Graph()->ForEachNodes(
		[&](const UTG_Node* node, uint32 index)
		{
			if (node)
				AddModelNode(const_cast<UTG_Node*> (node), false, FVector2D(node->EditorData.PosX, node->EditorData.PosY));
		}
	);

	// Then for every edges, let's create an equivalent edge in viewmodel
	TextureGraph->Graph()->ForEachEdges(
		[&](const UTG_Pin* pinFrom, const UTG_Pin* pinTo) {
			CreateViewModelLinkFromModelPins(pinFrom, pinTo);
		}
	);

	for (const UObject* ExtraNode : TextureGraph->Graph()->GetExtraEditorNodes())
	{
		if (const UEdGraphNode* ExtraGraphNode = Cast<UEdGraphNode>(ExtraNode))
		{
			UEdGraphNode* NewNode = DuplicateObject(ExtraGraphNode, /*Outer=*/this);
			AddNode(NewNode);
		}
	}

	// register for changes
	TextureGraph->Graph()->OnNodeSignatureChangedDelegate.AddUObject(this, &UTG_EdGraph::OnNodeSignatureChanged);
	TextureGraph->Graph()->OnNodePostEvaluateDelegate.AddUObject(this, &UTG_EdGraph::OnNodePostEvaluation);
	TextureGraph->Graph()->OnGraphChangedDelegate.AddUObject(this, &UTG_EdGraph::GraphChanged);
}


UTG_EdGraphNode* UTG_EdGraph::AddModelNode(UTG_Node* ModelNode, bool bUserInvoked, const FVector2D& Location)
{
	UTG_EdGraphNode* newEdNode = nullptr;

	FGraphNodeCreator<UTG_EdGraphNode> NodeCreator(*this);
	if (bUserInvoked)
	{
		newEdNode = NodeCreator.CreateUserInvokedNode();
	} else
	{
		newEdNode = NodeCreator.CreateNode(false);
	}

	newEdNode->Construct(ModelNode);
	newEdNode->NodePosX = Location.X;
	newEdNode->NodePosY = Location.Y;
	NodeCreator.Finalize();

	return newEdNode;
}

UTG_EdGraphNode* UTG_EdGraph::GetViewModelNode(FTG_Id NodeId)
{
	auto Predicate = [NodeId](const UEdGraphNode* Element)
	{
		auto TSEdNode = Cast<UTG_EdGraphNode>(Element);
		if (TSEdNode && TSEdNode->GetNode())
			return (TSEdNode->GetNode()->GetId() == NodeId);
		return false;
	};
	auto Result = Nodes.FindByPredicate(Predicate);
	return (Result ? Cast<UTG_EdGraphNode>(Result->Get()) : nullptr);
}

void UTG_EdGraph::RefreshEditorDetails() const
{
	//Tell Editor to update details
	if (TGEditor.IsValid())
	{
		TGEditor.Pin()->RefreshDetailsView();
	}
}

void UTG_EdGraph::OnNodeCreateThumbnail(UTG_Node* InNode, const FTG_EvaluationContext* InContext)
{
	if(!InContext->Cycle->GetDetails().bExporting)
	{
		UTG_EdGraphNode* EdGraphNode = GetViewModelNode(InNode->GetId());
		if (EdGraphNode)
		{
			TArray<UTG_Pin*> OutputPins;
			InNode->GetOutputPins(OutputPins);
			// Loop over all Output pins
			for (UTG_Pin* Pin : OutputPins)
			{
				check (Pin);
				FTG_Texture OutTexture;

				if (Pin->IsArgArray())
				{
					FTG_VariantArray OutVarArray;
					Pin->GetValue(OutVarArray);
					OutTexture = OutVarArray.GetViewTexture();
				}
				else if (Pin->IsArgTexture())
				{
					Pin->GetValue(OutTexture);
				}

				if (OutTexture)
				{
					UMixInterface* Mix = InContext->Cycle->GetMix();
					int32 TargetId = InContext->TargetId;

					if (!OutTexture.RasterBlob)
					{
						OutTexture = FTG_Texture::GetBlack();
					}

					TiledBlobPtr ThumbBlob = T_Thumbnail::Bind(Mix, Pin, OutTexture.RasterBlob, TargetId);

					CacheThumbBlob(Pin->GetId(), ThumbBlob);
				}
			}
		}
	}
}

void UTG_EdGraph::GraphChanged(UTG_Graph* InGraph, UTG_Node* InNode, bool Tweaking)
{
	if (InNode)
	{
		UTG_EdGraphNode* EdGraphNode = GetViewModelNode(InNode->GetId());
		if (EdGraphNode)
		{
			EdGraphNode->OnNodeChanged(InNode);
		}	
	}
}

void UTG_EdGraph::FixDuplicatedNodesPinConnections(TSet<UEdGraphNode*>& PastedNodes)
{
	// we recreate the EdPin connections (coming from the copy/paste) to trigger the creation of TG_Node/TG_Pin connections correctly
	for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
	{
		UEdGraphNode* EdNode = *It;
	
		for(UEdGraphPin* SourceEdGraphPin : EdNode->Pins)
		{
			if (SourceEdGraphPin->Direction == EGPD_Output)
			{
				for (UEdGraphPin* DestEdGraphPin : SourceEdGraphPin->LinkedTo)
				{
					GetSchema()->TryCreateConnection(SourceEdGraphPin, DestEdGraphPin);	
				}
			}
		}
	}
}

void UTG_EdGraph::OnNodeSignatureChanged(UTG_Node* InNode)
{
	UTG_EdGraphNode* EdGraphNode = GetViewModelNode(InNode->GetId());
	if (EdGraphNode)
	{
		EdGraphNode->ReconstructNode();
	}
	else
	{
		UE_LOGF(LogTemp, Warning, "UTG_EdGraph::OnNodeSignatureChanged TG_Node not found in the EdNodes? this should not happen.")
	}
}

void UTG_EdGraph::OnNodePostEvaluation(UTG_Node* InNode, const FTG_EvaluationContext* Context)
{
	UTG_EdGraphNode* EdGraphNode = GetViewModelNode(InNode->GetId());
	if (EdGraphNode)
	{
		EdGraphNode->OnNodePostEvaluate(Context);
	}
	OnNodeCreateThumbnail(InNode, Context);
}
	
