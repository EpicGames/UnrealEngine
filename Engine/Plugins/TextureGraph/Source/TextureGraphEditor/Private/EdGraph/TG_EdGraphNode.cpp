// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/TG_EdGraphNode.h"
#include "Misc/TransactionObjectEvent.h"

#include "TG_Node.h"
#include "TG_Pin.h"
#include "EdGraph/TG_EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "TextureGraph.h"
#include "TG_Graph.h"

#include "Framework/Commands/GenericCommands.h"
#include "TG_EditorCommands.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "TG_EdGraphSchema.h"
#include "Expressions/Input/TG_Expression_InputParam.h"
#include "Materials/Material.h"
#include "Transform/Expressions/T_ExtractMaterialIds.h"
#include "TG_HelperFunctions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_EdGraphNode)

#define LOCTEXT_NAMESPACE "UTG_EdGraphNode"

static TAutoConsoleVariable<int32> CVarTGNodeOpacity(
	TEXT("TG.Node.Opacity"),
	1,
	TEXT("Defines the Opacity of texture graph node.\n")
	TEXT("<=0: Transparent\n")
	TEXT("  1: Opaque\n"));

void UTG_EdGraphNode::Construct(UTG_Node* InNode)
{
	check(InNode);
	Node = InNode;
	NodePosX = InNode->EditorData.PosX;
	NodePosY = InNode->EditorData.PosY;
	NodeComment = InNode->EditorData.NodeComment;
	bCommentBubblePinned = InNode->EditorData.bCommentBubblePinned;
	bCommentBubbleVisible = InNode->EditorData.bCommentBubbleVisible;
	bCanRenameNode = InNode->GetExpression()->CanRenameTitle();
}

void UTG_EdGraphNode::BeginDestroy()
{
	Super::BeginDestroy();
}

UObject* UTG_EdGraphNode::GetDetailsObject()
{
	return GetNode()->GetExpression();
}

FText UTG_EdGraphNode::GetTooltipText() const
{
	auto Expr = GetNode()->GetExpression();
	if (Expr)
		return Expr->GetTooltipText();
	return Super::GetTooltipText();
}

void UTG_EdGraphNode::SelectPin(UEdGraphPin* Pin, bool IsSelected)
{
	SelectedPin = nullptr;

	if (IsSelected)
	{
		SelectedPin = Pin;
	}

	if(OnPinSelectionChangeDelegate.IsBound())
		OnPinSelectionChangeDelegate.Broadcast(SelectedPin);
}

const UEdGraphPin* UTG_EdGraphNode::GetSelectedPin() const
{
	if (SelectedPin && Pins.Contains(SelectedPin))
		return SelectedPin;
	else
		return nullptr;
}

void UTG_EdGraphNode::GetNodeContextMenuActions(UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{
	if (!Context->Node)
	{
		return;
	}

	{
		FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaNodeActions", LOCTEXT("NodeActionsHeader", "Node Actions"));

		UTG_Expression_InputParam* InputParamExpression = Cast< UTG_Expression_InputParam>(Node->GetExpression());
		if (InputParamExpression)
		{
			if (InputParamExpression->bIsConstant)
				Section.AddMenuEntry(FTG_EditorCommands::Get().ConvertInputParameterFromConstant);
			else
				Section.AddMenuEntry(FTG_EditorCommands::Get().ConvertInputParameterToConstant);

		}
	}

	{
		if (Context->Pin && Context->Pin->Direction == EEdGraphPinDirection::EGPD_Output)
		{
			FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaPinActions", LOCTEXT("PinActionsMenuHeader", "Pin Actions"));
			//Section.AddMenuEntry(FPCGEditorCommands::Get().AddSourcePin);
			Section.AddMenuEntry("SetPreview",
			LOCTEXT("SelectPin", "Set as preview"),
			LOCTEXT("SelectPinTooltip", "Set this pin as thumb and preview"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([Pin = Context->Pin, InNode = Context->Node, this]
				{
					const UTG_EdGraphNode* TGEdNode = Cast<UTG_EdGraphNode>(InNode);
					if (TGEdNode)
					{
						const_cast<UTG_EdGraphNode*>(TGEdNode)->SelectPin(const_cast<UEdGraphPin*>(Pin), true); // assign the current selected pin ion the node
						UTG_EdGraph* TSEdGraph = Cast<UTG_EdGraph>(InNode->GetGraph());
						TSEdGraph->PinSelectionManager.UpdateSelection(const_cast<UEdGraphPin*>(Pin));
					}
				}),
				FCanExecuteAction::CreateLambda([Pin = Context->Pin, this]
				{
					return Pin != GetSelectedPin();
				})));
		}
	}

	{
		FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaGeneral", LOCTEXT("GeneralHeader", "General"));
		if (this->GetCanRenameNode())
		{
			Section.AddMenuEntry(FGenericCommands::Get().Rename);
		}
		Section.AddMenuEntry(FGenericCommands::Get().Delete);
		// TODO: Add also Copy, Paste, Duplicate...
		Section.AddMenuEntry(FGenericCommands::Get().Cut);
		Section.AddMenuEntry(FGenericCommands::Get().Copy);
		Section.AddMenuEntry(FGenericCommands::Get().Duplicate);
	}

	Super::GetNodeContextMenuActions(Menu, Context);
}

void UTG_EdGraphNode::AllocateDefaultPins()
{
	bool showAdvancedPinDisplay = false;
	for (auto Pin : GetNode()->Pins)
	{

		if (Pin  // Need a valid Pin obviously
			&& !(Pin->IsPrivate())  // Not PRIVATE
			&& !(Pin->IsParam() && Pin->IsOutput()) // Not PARAM OUTPUT
			&& !(Pin->IsParam() && Pin->IsInput() && (Pin->GetArgumentCPPTypeName() == TEXT("FTG_Texture"))) // Not Input Param of type FTG_Texture (special case of the InputTextureParam)
			)
		{
			// Pin name is the Argument name since we use it to retrieve the actual TG_Pin in the node
			// The Schema can overwrite the pin name displayed
			TWeakObjectPtr<UObject> PinSubCategoryObject = nullptr;
			FName PinCategory = GetPinCategory(Pin, PinSubCategoryObject);
			FName PinSubCategory(Pin->GetArgumentCPPTypeName());
			FName PinName(Pin->GetArgumentName());

			// early out if PinCategory isn't recognized
			if (PinCategory.IsNone())
				continue;
			
			UEdGraphPin* NewPin = nullptr;
			if (Pin->IsInput())
			{
				NewPin = CreatePin(EGPD_Input, PinCategory, PinSubCategory, PinSubCategoryObject.Get(), PinName);
			}
			else if (Pin->IsOutput())
			{
				NewPin = CreatePin(EGPD_Output, PinCategory, PinSubCategory, PinSubCategoryObject.Get(), PinName);
				FProperty* Property = Pin->GetExpressionProperty();
				// Hide buffer descriptor from the nodes where we do not want to show
				if (Property && !Property->HasMetaData("HideInnerPropertiesInNode"))
				{
					showAdvancedPinDisplay |= Pin->IsOutput() && Pin->GetArgument().IsTexture();
				}
			}
			else
			{
				UE_LOGF(LogTemp, Warning, "UTG_EdGraphNode::AllocateDefaultPins Unimplemented type: %ls", *Pin->GetArgumentCPPTypeName().ToString())
				continue;
			}

			NewPin->bAdvancedView = Pin->IsSetting();
			showAdvancedPinDisplay |= Pin->IsSetting();

			NewPin->bNotConnectable = Pin->IsNotConnectable() || Pin->IsParam();

			// Fetch the current pin value as a string and assign as the current value
			FString DefaultValue = Pin->GetEvaluatedVarValue();
			NewPin->DefaultValue = DefaultValue;

			// Also grab the Property associated if exist
			// BEWARE Pin do NOT all have a Property, it is potentially NULL
			FProperty* Property = Pin->GetExpressionProperty();

			// This updates the UObject (Texture/Material) picker UI in the Node to get updated
			if (Property && Property->GetClass()->IsChildOf(FObjectProperty::StaticClass()))
			{
				NewPin->DefaultObject = Pin->EditSelfVar()->GetAs<TObjectPtr<UObject>>();
			}

			NewPin->PinFriendlyName = FText::FromName(Pin->GetAliasName());
			NewPin->PinToolTip = Pin->LogTooltip();

#if WITH_EDITOR
			UpdatePinVisibility(NewPin, Pin);
#endif		
		}
	}
	AdvancedPinDisplay = showAdvancedPinDisplay ? ENodeAdvancedPins::Hidden : ENodeAdvancedPins::NoPins;
}

FName UTG_EdGraphNode::GetPinCategory(UTG_Pin* Pin, TWeakObjectPtr<UObject>& SubCategoryObj)
{
	FName PinCategory("PinCategory");

	// Find the UPROPERTY associated with the pin
	FProperty* Property = Pin->GetExpressionProperty();
	if (Property)
	{
		FFieldClass* PropertyClass = Property->GetClass();
		if (PropertyClass == FFloatProperty::StaticClass())
		{
			PinCategory = UTG_EdGraphSchema::PC_Float;
		}
		else if (PropertyClass == FDoubleProperty::StaticClass())
		{
			PinCategory = UTG_EdGraphSchema::PC_Double;
		}
		else if (PropertyClass == FIntProperty::StaticClass())
		{
			PinCategory = UTG_EdGraphSchema::PC_Int;
		}
		else if (PropertyClass == FUInt32Property::StaticClass())
		{
			PinCategory = UTG_EdGraphSchema::PC_Int;
		}
		else if (PropertyClass == FEnumProperty::StaticClass())
		{
			PinCategory = UTG_EdGraphSchema::PC_Enum;
			SubCategoryObj = CastField<FEnumProperty>(Property)->GetEnum();
		}
		else if (PropertyClass == FByteProperty::StaticClass())
		{
			PinCategory = UTG_EdGraphSchema::PC_Byte;
			SubCategoryObj = CastField<FByteProperty>(Property)->GetIntPropertyEnum();
		}
		else if (PropertyClass == FBoolProperty::StaticClass())
		{
			PinCategory = UTG_EdGraphSchema::PC_Boolean;
		}
		else if (PropertyClass == FStructProperty::StaticClass())
		{
			PinCategory = UTG_EdGraphSchema::PC_Struct;
			UScriptStruct* Struct = CastField<FStructProperty>(Property)->Struct;

			if (Struct)
			{
				SubCategoryObj = Struct;
			}
		}
		else if (PropertyClass == FArrayProperty::StaticClass())
		{
			PinCategory = UTG_EdGraphSchema::PC_Array;

			FArrayProperty* ArrayProperty  = static_cast<FArrayProperty*>( Property );
			FProperty* ArrayTypeProperty = ArrayProperty->Inner;
			
			if (ArrayTypeProperty->IsA(FStructProperty::StaticClass()))
			{
				UScriptStruct* Struct = CastField<FStructProperty>(ArrayTypeProperty)->Struct;

				if (Struct)
				{
					SubCategoryObj = Struct;
				}
			}
		}
		else if (PropertyClass->IsChildOf(FObjectProperty::StaticClass()))
		{
			UClass* ObjPropertyClass = CastField<FObjectProperty>(Property)->PropertyClass;

			// disable material pin for now
			if (ObjPropertyClass == UMaterial::StaticClass())
			{
				return NAME_None;
			}
			
			PinCategory = UTG_EdGraphSchema::PC_Object;
			SubCategoryObj = TWeakObjectPtr<UClass>(ObjPropertyClass);//Pin->EditSelfVar()->GetAs<TObjectPtr<PropertyClass>>());
			
		}
	}
	else
		// if there is no property, e.g. when pins are made from Material
	{
		// Determine category based on ArgumentCPPType
		auto ArgCPPType = Pin->GetArgumentCPPTypeName();
		if (ArgCPPType == TG_TypeUtils::FloatTypeName)
		{
			PinCategory = UTG_EdGraphSchema::PC_Float;
		}
		else if(ArgCPPType == TG_TypeUtils::Int32TypeName)
		{
			PinCategory = UTG_EdGraphSchema::PC_Int;
		}
		else if( ArgCPPType == TG_TypeUtils::Int64TypeName)
		{
			PinCategory = UTG_EdGraphSchema::PC_Int64;
		}
		else if( ArgCPPType == TG_TypeUtils::BoolTypeName)
		{
			PinCategory = UTG_EdGraphSchema::PC_Boolean;
		}
		else if (ArgCPPType == TG_TypeUtils::DoubleTypeName)
		{
			PinCategory = UTG_EdGraphSchema::PC_Double;
		}
		// else if (ArgCPPType.ToString().Contains( ""))
	}
	
	return PinCategory;
}

void UTG_EdGraphNode::ReconstructNode()
{
	Modify();
	const UEdGraphPin* SelectedPinPtr = GetSelectedPin();
	const FName SelectedPinName = (SelectedPinPtr) ? SelectedPinPtr->PinName : FName();
	// forget SelectedPin, eventually reassign below via the PinSelectionManager
	SelectedPin = nullptr;

	UEdGraphPin* NewSelectedPin = nullptr;
	// Store copy of old pins
	TArray<UEdGraphPin*> OldPins = MoveTemp(Pins);
	Pins.Reset();
	
	// Generate new pins
	AllocateDefaultPins();

	// Transfer persistent data from old to new pins
	for (UEdGraphPin* OldPin : OldPins)
	{
		// Only same name pin could be candidates
		const FName& OldPinName = OldPin->PinName;
		UEdGraphPin** NewPin = Pins.FindByPredicate([&OldPinName](UEdGraphPin* InPin) { return InPin->PinName == OldPinName; });	
		if (NewPin)
		{
			// And also check that the new pin at that name is connected to anything, if so grab the UI connections
			UTG_Pin* TGPin = Node->GetPin(Node->GetPinId(OldPinName));
			if (TGPin->IsConnected() || OldPin->HasAnyConnections())
			{
				(*NewPin)->MovePersistentDataFromOldPin(*OldPin);
			}

			if ((*NewPin)->PinName == SelectedPinName)
			{
				NewSelectedPin = (*NewPin);
			}
		}
	}
	
	// Remove old pins
	for (UEdGraphPin* OldPin : OldPins)
	{
		RemovePin(OldPin);
	}

	// Notify editor
	OnNodeReconstructDelegate.Broadcast();

	// if no selected pin, we set it to the first output pin
	if (!NewSelectedPin && GetOutputPins().Num() > 0)
	{
		NewSelectedPin = GetOutputPins()[0];
	}
	
	SelectPin(NewSelectedPin, true); // assign the current selected pin ion the node
	UTG_EdGraph* TSEdGraph = Cast<UTG_EdGraph>(GetGraph());
	TSEdGraph->PinSelectionManager.UpdateSelection(NewSelectedPin);
}

FString UTG_EdGraphNode::GetTitleDetail()
{
	FString Details = "";
	const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(GetSchema());
	TArray<const UTG_Pin*> OutPins;
	GetNode()->GetOutputPins(OutPins);
	UTG_Expression* Expression = GetNode()->GetExpression();

	const UEdGraphPin* SelectedPinPtr = GetSelectedPin();

	for (auto Pin : OutPins)
	{
		const UTG_Pin* TGPin = Pin;
		
		if (SelectedPinPtr)
		{
			TGPin = Schema->GetTGPinFromEdPin(SelectedPinPtr);
		}

		// Only interested by the texture type pin
		FTG_Texture Texture;
		if ((Pin == TGPin) && Pin->GetValue(Texture))
		{
			if (Texture.RasterBlob)
			{
				FString ArrayInfo;

				if (Pin->IsArgTextureArray())
				{
					FTG_VariantArray VarArray;
					Pin->GetValue(VarArray);
					ArrayInfo = FString::Printf(TEXT("[%d]"), VarArray.Num());
				}

				const auto Desc = Texture.RasterBlob->GetDescriptor();
				FStringFormatNamedArguments Args;

				bool bIsSRGB = Desc.bIsSRGB;

				FTG_TextureDescriptor OutputSettings;

				if (Expression)
				{
					bIsSRGB = Expression->BaseOutputSettings.bIsSRGB;
				}

				Args.Add(TEXT("Channels"), TextureHelper::GetChannelsTextFromItemsPerPoint(Desc.ItemsPerPoint));
				Args.Add(TEXT("Format"), Desc.FormatToString(Desc.Format));
				Args.Add(TEXT("IsSRGB"), bIsSRGB ? "sRGB" : "Linear");
				Args.Add(TEXT("ArrayInfo"), ArrayInfo);
				Args.Add(TEXT("Width"), FString::FromInt(Desc.Width));
				Args.Add(TEXT("Height"), FString::FromInt(Desc.Height));
				Details = FString::Format(TEXT("{Channels}_{Format}, {IsSRGB} {ArrayInfo}\n{Width}x{Height}"), Args);
				break;
			}
		}
	}
	return Details;
}

void UTG_EdGraphNode::PrepareForCopying()
{
	if (Node)
	{
		// Temporarily take ownership of the TG_Node, so that it is not deleted when cutting
		Node->Rename(NULL, this, REN_DontCreateRedirectors);
	}
}

bool UTG_EdGraphNode::CanUserDeleteNode() const
{
	return true;
}

void UTG_EdGraphNode::PostCopyNode()
{
	if (Node)
	{
		UTG_EdGraph* EdGraph = CastChecked<UTG_EdGraph>(GetGraph());
		UTG_Graph* Graph = EdGraph->TextureGraph->Graph();
		check(Graph);
		Node->Rename(nullptr, Graph, REN_DontCreateRedirectors | REN_DoNotDirty);
	}
}

void UTG_EdGraphNode::PostPasteNode()
{	
	UTG_EdGraph* EdGraph = CastChecked<UTG_EdGraph>(GetGraph());
	UTG_Graph* Graph = EdGraph->TextureGraph->Graph();
	check(Graph);
	
	Node->Pins.Empty();
	Node->Rename(nullptr, Graph, REN_DontCreateRedirectors | REN_DoNotDirty);
	
	// Our TG node is a new node that need to be taken care of and added to the graph
	Graph->AddPostPasteNode(Node);

}

#if WITH_EDITOR
void UTG_EdGraphNode::UpdatePinVisibility(UEdGraphPin* Pin, UTG_Pin* TGPin) const
{
	UTG_Expression* Expression = GetNode()->GetExpression();
	bool bCanEditChange = !TGPin->IsConnected();
	bool bEditConditionHides = false;

	// Rely on the FProperty meta information for potential conditional editability / visibility
	FProperty* Property = TGPin->GetExpressionProperty();
	if (Property)
	{
		bCanEditChange = Expression->CanEditChange(Property);
		bEditConditionHides = Property->HasMetaData("EditConditionHides");

		// update Metadata with EditCondition to also update details view
		FName NAME_EditCondition(TEXT("EditCondition"));
		Property->SetMetaData(NAME_EditCondition, bCanEditChange ? TEXT("true") : TEXT("false"));
	}

	Pin->bHidden = !bCanEditChange && bEditConditionHides;
	Pin->bDefaultValueIsReadOnly = !bCanEditChange;
	
	//Adding GIsTransacting check here as functions that are creating the Transaction
	//should not be called from here 
	if(Pin->bHidden && !GIsTransacting)
	{
		Pin->GetSchema()->BreakPinLinks(*Pin, false);
	}
}

void UTG_EdGraphNode::UpdateInputPinsVisibility() const
{
	auto Expression = Node->GetExpression();
	// go through all pins and match them with their property hide states
	for (FTG_Id OtherPinId : Node->GetInputPinIds())
	{
		TObjectPtr<UTG_Pin> TGPin = Node->GetGraph()->GetPin(OtherPinId);
		check(TGPin);

		const FName& OtherPinName = TGPin->GetArgumentName();
		UEdGraphPin* EdPin = FindPin(OtherPinName, EEdGraphPinDirection::EGPD_Input);

		// check hide state
		if (EdPin)
		{
			UpdatePinVisibility(EdPin, TGPin);
		}
	}
}

void UTG_EdGraphNode::UpdatePinsEditCondition() const
{
	auto Expression = Node->GetExpression();
	// go through all pins and match them with their property hide states
	for (FTG_Id OtherPinId : Node->GetInputPinIds())
	{
		TObjectPtr<UTG_Pin> TGPin = Node->GetGraph()->GetPin(OtherPinId);
		
		// check hide state
		if (TGPin)
		{
			// Rely on the FProperty meta information for potential conditional editability / visibility
			FProperty* Property = TGPin->GetExpressionProperty();
			if (Property)
			{
				bool bCanEditChange = Expression->CanEditChange(Property);
				
				// update Metadata with EditCondition to also update details view
				FName NAME_EditCondition(TEXT("EditCondition"));
				Property->SetMetaData(NAME_EditCondition, bCanEditChange ? TEXT("true") : TEXT("false"));
			}
		}
	}
}
#endif

void UTG_EdGraphNode::PinDefaultValueChangedWithTweaking(UEdGraphPin* Pin, bool bIsTweaking)
{
	const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(GetSchema());
	UTG_Pin* TGPin = Schema->GetTGPinFromEdPin(Pin);

	TGPin->FromString(Pin->DefaultValue, bIsTweaking);

	// This updates the UObject (Texture/Material) picker UI in the Node to get updated
	FProperty* Property = TGPin->GetExpressionProperty();
	if (Property && Property->GetClass()->IsChildOf(FObjectProperty::StaticClass()))
	{
		Pin->DefaultObject = TGPin->EditSelfVar()->GetAs<TObjectPtr<UObject>>();
	}

#if WITH_EDITOR
	// Update Node UI if the property has meta RegenPinsOnChange
	if(Property != nullptr && Property->HasMetaData("RegenPinsOnChange"))
	{
		UpdatePinsAndReconstructNode();
	}
#endif
}

void UTG_EdGraphNode::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);
	Pin->bDefaultValueIsReadOnly = Pin->HasAnyConnections();
}

void UTG_EdGraphNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	if (Node == nullptr || FromPin == nullptr)
	{
		return;
	}

	const bool bFromPinIsInput = FromPin->Direction == EEdGraphPinDirection::EGPD_Input;
	TArray<FTG_Id> OtherPinsList;
	if(bFromPinIsInput)
	{
		OtherPinsList.Append(Node->GetOutputPinIds());
	}
	else
	{
		OtherPinsList.Append(Node->GetInputPinIds());
	}

	// Try to connect to the first compatible pin
	for (FTG_Id OtherPinId : OtherPinsList)
	{
		TObjectPtr<UTG_Pin> OtherPin = Node->GetGraph()->GetPin(OtherPinId);
		check(OtherPin);

		if(!OtherPin->IsNotConnectable() && !OtherPin->IsParam())
		{
			const FName& OtherPinName = OtherPin->GetArgumentName();
			UEdGraphPin* ToPin = FindPinChecked(OtherPinName, bFromPinIsInput ? EEdGraphPinDirection::EGPD_Output : EEdGraphPinDirection::EGPD_Input);
			if (ToPin && GetSchema()->TryCreateConnection(FromPin, ToPin))
			{
				// Connection succeeded
				break;
			}	
		}
	}

	NodeConnectionListChanged();
}

FText UTG_EdGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::FromString(GetNode()->GetNodeName().ToString());
}

void UTG_EdGraphNode::OnRenameNode(const FString& NewName)
{
	Node->GetExpression()->Modify();
	Node->GetExpression()->SetTitleName(FName(NewName));
}

float UTG_EdGraphNode::GetNodeAlpha() const
{
	return CVarTGNodeOpacity.GetValueOnGameThread() == 1 ? 1.0f : 0.7f;
}

FLinearColor UTG_EdGraphNode::GetTitleColor() const
{
	FName ExpressionCategory = GetNode()->GetExpression()->GetCategory();
	return UTG_EdGraphSchema::GetCategoryColor(ExpressionCategory);
}

FLinearColor UTG_EdGraphNode::GetNodeTitleColor() const
{
	auto TitleColor = GetTitleColor();
	TitleColor.A = GetNodeAlpha();
	
	return TitleColor;
}

FLinearColor UTG_EdGraphNode::GetNodeBodyTintColor() const
{
	FLinearColor BodyColor = UTG_EdGraphSchema::NodeBodyColor;
	BodyColor.A = GetNodeAlpha();
	return BodyColor;
}

void UTG_EdGraphNode::UpdatePosition()
{
	if (Node)
	{
		Node->Modify();
		Node->EditorData.PosX = NodePosX;
		Node->EditorData.PosY = NodePosY;
	}
}

void UTG_EdGraphNode::OnNodePostEvaluate(const FTG_EvaluationContext* EvaluationContext)
{
	if (OnNodePostEvaluateDelegate.IsBound())
	{
		OnNodePostEvaluateDelegate.Broadcast(EvaluationContext);
	};
	OnNodeChanged(GetNode());
}

bool UTG_EdGraphNode::UpdateEdPinDefaultValue(UEdGraphPin* EdPin, const UTG_EdGraphSchema* Schema)
{
	bool bShouldUpdatePinsVisibility = false;
	UTG_Pin* TSPin = Schema->GetTGPinFromEdPin(EdPin);
	if (TSPin)
	{
		UTG_EdGraph* EdGraph = CastChecked<UTG_EdGraph>(GetGraph());
		
		// Access the current pin evaluated value as a string
		FString DefaultValue = TSPin->GetEvaluatedVarValue();

		// Also grab the Property associated if exist
		FProperty* Property = TSPin->GetExpressionProperty();

#if WITH_EDITOR
		// check here if the property has a tag for regen pins and value has changed
		bShouldUpdatePinsVisibility = (Property != nullptr && Property->HasMetaData("RegenPinsOnChange"))
										&& EdPin->DefaultValue != DefaultValue;
#endif
		
		EdPin->bDefaultValueIsReadOnly = false;
		EdPin->DefaultValue = DefaultValue;

		// This updates the UObject (Texture/Material) picker UI in the Node to get updated
		if (Property && Property->GetClass()->IsChildOf(FObjectProperty::StaticClass()))
		{
			EdPin->DefaultObject = TSPin->EditSelfVar()->GetAs<TObjectPtr<UObject>>();
		}
		EdPin->bDefaultValueIsReadOnly = EdPin->HasAnyConnections();
	}
	return bShouldUpdatePinsVisibility;
}

void UTG_EdGraphNode::UpdatePinsAndReconstructNode()
{
	UpdateInputPinsVisibility();
		
	// cache advanced visibility
	TEnumAsByte<ENodeAdvancedPins::Type> CachedAdvancedPinDisplay = this->AdvancedPinDisplay;
	ReconstructNode();
	this->AdvancedPinDisplay = CachedAdvancedPinDisplay;

	//Tell Editor to update details
	Cast<UTG_EdGraph>(GetGraph())->RefreshEditorDetails();
}

void UTG_EdGraphNode::OnNodeChanged(UTG_Node* InNode)
{
	bool bShouldUpdatePinsVisibility = false;
	// Update Default Values to their respective Var values;
	for(UEdGraphPin* EdPin : Pins)
	{
		const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(GetSchema());
		bShouldUpdatePinsVisibility |= UpdateEdPinDefaultValue(EdPin, Schema);
	}
#if WITH_EDITOR
	// if there was a pin which triggered regeneration of all other input pins (used meta "RegenPinsOnChange"), we update them
	if (bShouldUpdatePinsVisibility)
	{
		UpdatePinsAndReconstructNode();
	}
#endif
	
}

void UTG_EdGraphNode::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	TArray<FName> PropertiesChanged = TransactionEvent.GetChangedProperties();

	if (PropertiesChanged.Contains(TEXT("bCommentBubblePinned")))
	{
		UpdateCommentBubblePinned();
	}

	if (PropertiesChanged.Contains(TEXT("NodePosX")) || PropertiesChanged.Contains(TEXT("NodePosY")))
	{
		UpdatePosition();
	}
}

void UTG_EdGraphNode::OnUpdateCommentText(const FString& NewComment)
{
	Super::OnUpdateCommentText(NewComment);
	if (Node && Node->EditorData.NodeComment != NewComment)
	{
		Node->Modify();
		Node->EditorData.NodeComment = NewComment;
	}
}

void UTG_EdGraphNode::OnCommentBubbleToggled(bool bInCommentBubbleVisible)
{
	Super::OnCommentBubbleToggled(bInCommentBubbleVisible);

	if (Node && Node->EditorData.bCommentBubbleVisible != bInCommentBubbleVisible)
	{
		Node->Modify();
		Node->EditorData.bCommentBubbleVisible = bInCommentBubbleVisible;
	}
}

void UTG_EdGraphNode::UpdateCommentBubblePinned()
{
	if (Node)
	{
		Node->Modify();
		Node->EditorData.bCommentBubblePinned = bCommentBubblePinned;
	}
}

TArray<UEdGraphPin*> UTG_EdGraphNode::GetOutputPins() const
{
	TArray<UEdGraphPin*> OutputPins;
	for (auto Pin : Pins)
	{
		if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
		{
			OutputPins.Add(Pin);
		}
	}
	return MoveTemp(OutputPins);
}

TArray<UEdGraphPin*> UTG_EdGraphNode::GetTextureOutputPins() const
{
	TArray<UEdGraphPin*> TexturedOutputPins;
	auto OutputPins = GetOutputPins();
	const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(GetSchema());
	
	for (auto Pin : OutputPins)
	{
		UTG_Pin* TGPin = Schema->GetTGPinFromEdPin(Pin);

		if (TGPin->GetArgument().IsTexture())
		{
			//FTG_Texture& Texture = TGPin->GetNodePtr()->GetGraph()->GetVar(TGPin->GetId())->EditAs<FTG_Texture>();
			TexturedOutputPins.Add(Pin);
		}
	}
	return MoveTemp(TexturedOutputPins);
}
#undef LOCTEXT_NAMESPACE
