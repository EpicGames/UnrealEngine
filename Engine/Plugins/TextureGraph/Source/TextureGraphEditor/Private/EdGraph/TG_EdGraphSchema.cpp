// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/TG_EdGraphSchema.h"

#include "EdGraph/EdGraphSchema.h"

#include "EdGraph/TG_EdGraphSchemaActions.h"
#include "EdGraph/TG_EdGraphNode.h"

#include "ScopedTransaction.h"
#include "2D/TextureHelper.h"
#include "Editor/AssetReferenceFilter.h"
#include "TG_Editor.h"
#include "TG_Graph.h"
#include "UObject/UObjectIterator.h"
#include "MaterialValueType.h"
#include "Selection.h"
#include "TextureGraph.h"
#include "EdGraph/TG_EdGraph.h"
#include "Expressions/Input/TG_Expression_Graph.h"
#include "Expressions/Input/TG_Expression_Material.h"
#include "Expressions/Input/TG_Expression_MaterialFunction.h"
#include "Expressions/Input/TG_Expression_Texture.h"
#include "Expressions/CommandChange/TG_PinConnectionChange.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_EdGraphSchema)

#define LOCTEXT_NAMESPACE "TG_GraphSchema"

const FLinearColor UTG_EdGraphSchema::InactivePinColor = FLinearColor(0.05f, 0.05f, 0.05f);
const FLinearColor UTG_EdGraphSchema::ActivePinColor = FLinearColor::White;
const FLinearColor UTG_EdGraphSchema::ScalarPinColor = FLinearColor(0.0f, 0.4f, 0.910000f, 1.0f);			// light blue
const FLinearColor UTG_EdGraphSchema::VectorPinColor = FLinearColor::Yellow;
const FLinearColor UTG_EdGraphSchema::ImagePinColor = FLinearColor(0.8f, 0.2f, 0.4f, 1.0f);					// salmon (light pink)

// Node title colors
const FLinearColor UTG_EdGraphSchema::InputNodesColor = FLinearColor::FromSRGBColor(FColor::FromHex(TEXT("3E1950")));
const FLinearColor UTG_EdGraphSchema::FunctionNodesColor = FLinearColor::FromSRGBColor(FColor::FromHex(TEXT("2A97704D")));
const FLinearColor UTG_EdGraphSchema::MathsNodesColor = FLinearColor::FromSRGBColor(FColor::FromHex(TEXT("2C3B29"))); 
const FLinearColor UTG_EdGraphSchema::ProceduralNodesColor = FLinearColor::FromSRGBColor(FColor::FromHex(TEXT("293E16")));
const FLinearColor UTG_EdGraphSchema::OperatorNodesColor = FLinearColor::FromSRGBColor(FColor::FromHex(TEXT("383316")));
const FLinearColor UTG_EdGraphSchema::FilterNodesColor = FLinearColor::FromSRGBColor(FColor::FromHex(TEXT("253016")));
const FLinearColor UTG_EdGraphSchema::ChannelNodesColor = FLinearColor::FromSRGBColor(FColor::FromHex(TEXT("203328")));
const FLinearColor UTG_EdGraphSchema::OutputNodesColor = FLinearColor::FromSRGBColor(FColor::FromHex(TEXT("4B453B")));
const FLinearColor UTG_EdGraphSchema::CustomNodesColor = FLinearColor::Black;
const FLinearColor UTG_EdGraphSchema::DevOnlyNodesColor = FLinearColor::FromSRGBColor(FColor::Cyan);
const FLinearColor UTG_EdGraphSchema::UtilitiesNodesColor = FLinearColor::FromSRGBColor(FColor::FromHex(TEXT("133C33")));

const FLinearColor UTG_EdGraphSchema::TGNodeColor = FLinearColor::FromSRGBColor(FColor::FromHex(TEXT("0F6389")));
const FLinearColor UTG_EdGraphSchema::NodeBodyColor = FLinearColor::FromSRGBColor(FColor::FromHex(TEXT("0F0F0F")));
const FLinearColor UTG_EdGraphSchema::NodeBodyColorOutline = FLinearColor::FromSRGBColor(FColor::FromHex(TEXT("353535")));

const float UTG_EdGraphSchema::NodeOutlineColorMultiplier = 2.2f;

const FName UTG_EdGraphSchema::PC_Boolean(TEXT("bool"));
const FName UTG_EdGraphSchema::PC_Byte(TEXT("byte"));
const FName UTG_EdGraphSchema::PC_Class(TEXT("class"));
const FName UTG_EdGraphSchema::PC_Int(TEXT("int"));
const FName UTG_EdGraphSchema::PC_Int64(TEXT("int64"));
const FName UTG_EdGraphSchema::PC_Float(TEXT("float"));
const FName UTG_EdGraphSchema::PC_Double(TEXT("double"));
const FName UTG_EdGraphSchema::PC_Object(TEXT("object"));
const FName UTG_EdGraphSchema::PC_Interface(TEXT("interface"));
const FName UTG_EdGraphSchema::PC_String(TEXT("string"));
const FName UTG_EdGraphSchema::PC_Text(TEXT("text"));
const FName UTG_EdGraphSchema::PC_Struct(TEXT("struct"));
const FName UTG_EdGraphSchema::PC_Array(TEXT("array"));
const FName UTG_EdGraphSchema::PC_Enum(TEXT("enum"));

void UTG_EdGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	Super::GetGraphContextActions(ContextMenuBuilder);

	GetTG_ExpressionsActions(ContextMenuBuilder);
	
	GetCommentAction(ContextMenuBuilder, ContextMenuBuilder.CurrentGraph);
}

void UTG_EdGraphSchema::GetTG_ExpressionsActions(FGraphActionMenuBuilder& ActionMenuBuilder) const
{
	TArray<UClass*> TG_ExpressionClasses;
	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* Class = *It;

		if (Class->IsChildOf(UTG_Expression::StaticClass()) &&
			!Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_Hidden))
		{
			TG_ExpressionClasses.Add(Class);
		}
	}

	for (UClass* TG_ExpressionClass : TG_ExpressionClasses)
	{
		if (const UTG_Expression* TG_Expression = TG_ExpressionClass->GetDefaultObject<UTG_Expression>())
		{
			static const FName DefaultCategory = TG_Category::Default;

			const FText MenuDesc = FText::FromString(TG_Expression->GetDefaultName().ToString());
			FName Category = TG_Expression->GetCategory();

			if (Category.IsNone())
				Category = DefaultCategory;

			if (!MenuDesc.IsEmpty())
			{
				const FText Description = TG_Expression->GetTooltipText();

				TSharedPtr<FTG_EdGraphSchemaAction_NewNode> NewAction(new FTG_EdGraphSchemaAction_NewNode(FText::FromString(Category.ToString()), MenuDesc, Description,0));
				NewAction->TG_ExpressionClass = TG_ExpressionClass;
				ActionMenuBuilder.AddAction(NewAction);
			}
		}
	}
}
void UTG_EdGraphSchema::GetCommentAction(FGraphActionMenuBuilder& ActionMenuBuilder, const UEdGraph* CurrentGraph) const
{
	if (!ActionMenuBuilder.FromPin)
	{
		// Comment action
		const FText CommentMenuDesc = LOCTEXT("CommentDesc", "New Comment");
		const FText CommentCategory;
		const FText CommentDescription = LOCTEXT("CommentToolTip", "Creates a comment.");

		const TSharedPtr<FTG_EdGraphSchemaAction_NewComment> NewCommentAction(new FTG_EdGraphSchemaAction_NewComment(CommentCategory, CommentMenuDesc, CommentDescription, 0));
		ActionMenuBuilder.AddAction(NewCommentAction);
	}
}
UTG_Node* UTG_EdGraphSchema::GetTGNodeFromEdPin(const UEdGraphPin* InPin) const
{
	if (InPin)
	{
		UTG_EdGraphNode* EdNode = CastChecked<UTG_EdGraphNode>(InPin->GetOwningNode());
		if (EdNode)
		{
			return EdNode->GetNode();
		}
	}
	return nullptr;
}

UTG_Pin* UTG_EdGraphSchema::GetTGPinFromEdPin(const UEdGraphPin* InPin) const
{
	UTG_Node* Node = GetTGNodeFromEdPin(InPin);
	if (Node)
	{
		// The TG_Pin is found in the node from the EdPIn name which should be the ArgumentName
		return Node->GetPin(Node->GetPinId(InPin->GetFName()));
	}
	return nullptr;
}
void UTG_EdGraphSchema::TrySetDefaultValue(UEdGraphPin& Pin, const FString& NewDefaultValue, bool bMarkAsModified /*=true*/) const
{
	Pin.DefaultValue = NewDefaultValue;

#if WITH_EDITOR
	UTG_EdGraphNode* Node = Cast<UTG_EdGraphNode>(Pin.GetOwningNode());
	check(Node);
	Node->PinDefaultValueChangedWithTweaking(&Pin, !bMarkAsModified);

#endif	//#if WITH_EDITOR
}

void UTG_EdGraphSchema::TrySetDefaultObject(UEdGraphPin& Pin, UObject* NewDefaultObject, bool bMarkAsModified) const
{
	Pin.DefaultObject = NewDefaultObject;
#if WITH_EDITOR
	UTG_EdGraphNode* Node = Cast<UTG_EdGraphNode>(Pin.GetOwningNode());
	check(Node);

	// get file path from Object changed and set as default value
	TrySetDefaultValue(Pin, NewDefaultObject ? NewDefaultObject->GetPathName() : FString(), bMarkAsModified);

#endif	//#if WITH_EDITOR
}

const FPinConnectionResponse UTG_EdGraphSchema::CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const
{
//	bool bPreventInvalidConnections = CVarPreventInvalidMaterialConnections.GetValueOnGameThread() != 0;

	// Make sure the pins are not on the same node
	if (A->GetOwningNode() == B->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionSameNode", "Both are on the same node"));
	}

	// Compare the directions
	const UEdGraphPin* InputPin = nullptr;
	const UEdGraphPin* OutputPin = nullptr;

	if (!CategorizePinsByDirection(A, B, /*out*/ InputPin, /*out*/ OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionDirectionsNotCompatible", "Directions are not compatible"));
	}

	// Check for new and existing loops
	FText ResponseMessage;
	if (ConnectionCausesLoop(OutputPin, InputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionLoop", "Connection would cause loop").ToString());
	}

	// Check for incompatible pins and get description if they cannot connect
	if (!ArePinsCompatible_Internal(OutputPin, InputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionPinsNotCompatible", "Pins are not compatible").ToString());
	}

	// For non-exec pins, break existing connections on inputs only - multiple output connections are acceptable
	if (InputPin->LinkedTo.Num() > 0)
	{
		const EMaterialValueType InputType = MCT_Float1; //GetMaterialValueType(InputPin);
		if (!(InputType & MCT_Execution))
		{
			ECanCreateConnectionResponse ReplyBreakOutputs;
			if (InputPin == A)
			{
				ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_A;
			} else
			{
				ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_B;
			}
			if (ResponseMessage.IsEmpty())
			{
				ResponseMessage = LOCTEXT("ConnectionReplace", "Replace existing connections");
			}
			return FPinConnectionResponse(ReplyBreakOutputs, ResponseMessage);
		}
	}

	// For exec pins, reverse is true - multiple input connections are acceptable
	if (OutputPin->LinkedTo.Num() > 0)
	{
		const EMaterialValueType OutputType = MCT_Float1; //GetMaterialValueType(InputPin);
		if (OutputType & MCT_Execution)
		{
			ECanCreateConnectionResponse ReplyBreakInputs;
			if (OutputPin == A)
			{
				ReplyBreakInputs = CONNECT_RESPONSE_BREAK_OTHERS_A;
			} else
			{
				ReplyBreakInputs = CONNECT_RESPONSE_BREAK_OTHERS_B;
			}
			if (ResponseMessage.IsEmpty())
			{
				ResponseMessage = LOCTEXT("ConnectionReplace", "Replace existing connections");
			}
			return FPinConnectionResponse(ReplyBreakInputs, ResponseMessage);
		}
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, ResponseMessage);
}

bool UTG_EdGraphSchema::ConnectionCausesLoop(const UEdGraphPin* FromPin, const UEdGraphPin* ToPin) const
{
	UTG_Pin* TSPinA = GetTGPinFromEdPin(FromPin);
	UTG_Pin* TSPinB = GetTGPinFromEdPin(ToPin);
	if (TSPinA && TSPinB)
	{
		if (UTG_Graph::ConnectionCausesLoop(TSPinA, TSPinB))
		{
			return true;
		}
	}
	return false;
}

bool UTG_EdGraphSchema::ArePinsCompatible_Internal(const UEdGraphPin* InA, const UEdGraphPin* InB) const
{
	UTG_Pin* TSPinA = GetTGPinFromEdPin(InA);
	UTG_Pin* TSPinB = GetTGPinFromEdPin(InB);
	if (TSPinA && TSPinB)
	{
		FName ConverterKey;
		if (UTG_Graph::ArePinsCompatible(TSPinA, TSPinB, ConverterKey))
		{
			return true;
		}
	}
	return false;
}

bool UTG_EdGraphSchema::TryCreateConnection(UEdGraphPin* InA, UEdGraphPin* InB) const
{
	const bool bModified = Super::TryCreateConnection(InA, InB);

	if (bModified)
	{
		check(InA && InB);
		const UEdGraphPin* A = (InA->Direction == EGPD_Output) ? InA : InB;
		const UEdGraphPin* B = (InA->Direction == EGPD_Input) ? InA : InB;
		check(A->Direction == EGPD_Output && B->Direction == EGPD_Input);

		UTG_Node* TG_NodeA = GetTGNodeFromEdPin(A);
		UTG_Node* TG_NodeB = GetTGNodeFromEdPin(B);
		check(TG_NodeA && TG_NodeB);

		UTG_Graph* TG_Graph = TG_NodeA->GetGraph();
		check(TG_Graph);

		FTG_PinConnectionChange::StoreChange(TG_Graph, *TG_NodeB, B->PinName);

		auto edge = TG_Graph->Connect(*TG_NodeA, A->PinName, *TG_NodeB, B->PinName);
	}

    return bModified;
}

FLinearColor UTG_EdGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	// if (PinType.PinCategory == TEXT("Setting"))
	// {
	// 	return InactivePinColor;
	// }
	if (PinType.PinSubCategory == TEXT("FTG_Texture"))
	{
		return ImagePinColor;
	}
	else if (PinType.PinSubCategory == TEXT("float") || PinType.PinSubCategory  == TEXT("int32"))
	{
		return ScalarPinColor;
	}
	else if (PinType.PinSubCategory == TEXT("FLinearColor"))
	{
		return VectorPinColor;
	}
	return ActivePinColor;
}
FLinearColor UTG_EdGraphSchema::GetSecondaryPinTypeColor(const FEdGraphPinType& PinType) const
{
	return FLinearColor::Black;
}

void UTG_EdGraphSchema::BreakNodeLinks(UEdGraphNode& TargetNode) const
{
	Super::BreakNodeLinks(TargetNode);
}

void UTG_EdGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotification) const
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "TG_EdGraphSchema_BreakPinLinks", "Break Pin Links"));
	
	Super::BreakPinLinks(TargetPin, bSendsNodeNotification);

	UEdGraphNode* GraphNode = TargetPin.GetOwningNode();

	UTG_EdGraphNode* TextureGraphNode = CastChecked<UTG_EdGraphNode>(GraphNode);

	UTG_Node* TSNode = TextureGraphNode->GetNode();
	check(TSNode);

	UTG_Graph* TextureGraph = TSNode->GetGraph();
	check(TextureGraph);

	FTG_PinConnectionBreakChange::StoreChange(TextureGraph, *TSNode, TargetPin.PinName);

	TextureGraph->RemovePinEdges(*TSNode, TargetPin.PinName);
}

void UTG_EdGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "TG_EdGraphSchema_BreakSinglePinLink", "Break Single Pin Links"));

	Super::BreakSinglePinLink(SourcePin, TargetPin);

	UEdGraphNode* SourceGraphNode = SourcePin->GetOwningNode();
	UEdGraphNode* TargetGraphNode = TargetPin->GetOwningNode();

	UTG_EdGraphNode* SourceTextureGraphNode = CastChecked<UTG_EdGraphNode>(SourceGraphNode);
	UTG_EdGraphNode* TargetTextureGraphNode = CastChecked<UTG_EdGraphNode>(TargetGraphNode);

	UTG_Node* SourceTSNode = SourceTextureGraphNode->GetNode();
	UTG_Node* TargetTSNode = TargetTextureGraphNode->GetNode();
	check(SourceTSNode && TargetTSNode);

	UTG_Graph* TextureGraph = SourceTSNode->GetGraph();
	check(TextureGraph);

	FTG_PinConnectionBreakChange::StoreChange(TextureGraph, *TargetTSNode, TargetPin->PinName);
	TextureGraph->RemoveEdge(*SourceTSNode, SourcePin->PinName, *TargetTSNode, TargetPin->PinName);
}

void UTG_EdGraphSchema::DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	UTG_EdGraph* TG_Graph = CastChecked<UTG_EdGraph>(Graph);

	const int32 LocOffsetBetweenNodes = 32;

	FVector2f ExpressionPosition = GraphPosition;

	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(Graph);
	for (const FAssetData& AssetData : Assets)
	{
		if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(AssetData))
		{
			continue;
		}

		UObject* Asset = AssetData.GetAsset();
		UEdGraphNode* ExpressionNode = nullptr;
		
		auto Tex = Cast<UTexture>(Asset);
		if(TextureHelper::CanSupportTexture(Tex))
		{
			// Create a new expression in the script
			UTG_Expression_Texture* TextureExpression = NewObject<UTG_Expression_Texture>();
			TextureExpression->Source = Tex;

			ExpressionNode = FTG_EdGraphSchemaAction_NewNode::CreateExpressionNode(TG_Graph, TextureExpression, nullptr, ExpressionPosition, true);
		}
		else if(UMaterialInterface* Material = Cast<UMaterialInterface>(Asset); Material != nullptr)
		{
			UTG_Expression_Material* MaterialExpression = NewObject<UTG_Expression_Material>();
			ExpressionNode = FTG_EdGraphSchemaAction_NewNode::CreateExpressionNode(TG_Graph, MaterialExpression, nullptr, ExpressionPosition, true);

			// We need to call this here
			// This call setups the material
			// Mainly used to initialize dynamic instance and setup node pins
			MaterialExpression->SetAsset(Material);
		}
		else if (UMaterialFunctionInterface* MaterialFunction = Cast<UMaterialFunctionInterface>(Asset); MaterialFunction != nullptr)
		{
			UTG_Expression_MaterialFunction* MaterialFunctionExpression = NewObject<UTG_Expression_MaterialFunction>();
			ExpressionNode = FTG_EdGraphSchemaAction_NewNode::CreateExpressionNode(TG_Graph, MaterialFunctionExpression, nullptr, ExpressionPosition, true);

			// Assign the material function to the expression
			MaterialFunctionExpression->SetAsset(MaterialFunction);
		}
		else if(UTextureGraph* TextureGraph = Cast<UTextureGraph>(Asset); TextureGraph != nullptr)
		{
			UTG_Expression_TextureGraph* TextureScriptExpression = NewObject<UTG_Expression_TextureGraph>();
			ExpressionNode = FTG_EdGraphSchemaAction_NewNode::CreateExpressionNode(TG_Graph, TextureScriptExpression, nullptr, ExpressionPosition, true);

			// Assign the asset TextureGraph
			TextureScriptExpression->SetAsset(TextureGraph);
		}
		
		
		if (ExpressionNode)
		{	
			ExpressionPosition.X += LocOffsetBetweenNodes;
			ExpressionPosition.Y += LocOffsetBetweenNodes;
		}
	}
}

void UTG_EdGraphSchema::DroppedAssetsOnNode(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const
{
	const UTG_EdGraphNode* EdNode = Cast<UTG_EdGraphNode>(Node);

	if(EdNode)
	{
		const FAssetData& AssetData = Assets.Last();
		UObject* Asset = AssetData.GetAsset();
		UTG_Expression* Expression = EdNode->GetNode()->GetExpression();

		if(Expression->CanHandleAsset(Asset))
		{
			const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(EdNode->GetGraph());
			if (!AssetReferenceFilter.IsValid() || AssetReferenceFilter->PassesFilter(AssetData))
			{
				const FScopedTransaction Transaction(NSLOCTEXT("UTG_EdGraphSchema", "DroppedAssetsOnNode", "Dropped Assets OnNode"));
				Expression->Modify();

				Expression->SetAsset(Asset);
			}
		}
	}
}

void UTG_EdGraphSchema::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph,
	FString& OutTooltipText, bool& OutOkIcon) const
{
	OutTooltipText.Reset();
	OutOkIcon = false;

	const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(HoverGraph);
	for (const FAssetData& AssetData : Assets)
	{
		UObject* Asset = AssetData.GetAsset();
		
		if (TextureHelper::CanSupportTexture(Cast<UTexture>(Asset)) ||
			Cast<UMaterialInterface>(Asset) != nullptr ||
			Cast<UMaterialFunctionInterface>(Asset) != nullptr ||
			Cast<UTextureGraph>(Asset) != nullptr)
		{
			if (AssetReferenceFilter)
			{
				FText FailureReason;
				if (!AssetReferenceFilter->PassesFilter(AssetData, &FailureReason))
				{
					if (OutTooltipText.IsEmpty())
					{
						OutTooltipText = FailureReason.ToString();
					}
					continue;
				}
			}

			OutTooltipText.Reset();
			OutOkIcon = true;
			return;
		}
	}
}

void UTG_EdGraphSchema::GetAssetsNodeHoverMessage(const TArray<FAssetData>& Assets, const UEdGraphNode* HoverNode, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutTooltipText.Reset();
	OutOkIcon = false;

	const UTG_EdGraphNode* EdNode = Cast<UTG_EdGraphNode>(HoverNode);
	
	if(EdNode)
	{
		const FAssetData& AssetData = Assets.Last();
		UObject* Asset = AssetData.GetAsset();
		UTG_Expression* Expression = EdNode->GetNode()->GetExpression();
		
		if (Expression->CanHandleAsset(Asset))
		{
			const TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(EdNode->GetGraph());
			FText FailureReason;
			if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(AssetData, &FailureReason))
			{
				OutTooltipText = FailureReason.ToString();
			}
			else
			{
				OutTooltipText.Reset();
				OutOkIcon = true;
			}
		}
	}
}

void UTG_EdGraphSchema::GetPaletteActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FString& CategoryName) const
{
	GetTG_ExpressionsActions(ActionMenuBuilder);
}

FLinearColor UTG_EdGraphSchema::GetCategoryColor(FName Category)
{
	if (Category == TG_Category::Output)
	{
		return UTG_EdGraphSchema::OutputNodesColor;
	}
	if (Category == TG_Category::Input)
	{
		return UTG_EdGraphSchema::InputNodesColor;
	}
	if (Category == TG_Category::Maths)
	{
		return UTG_EdGraphSchema::MathsNodesColor;
	}
	if (Category == TG_Category::Procedural)
	{
		return UTG_EdGraphSchema::ProceduralNodesColor;
	}
	if (Category == TG_Category::Adjustment)
	{
		return UTG_EdGraphSchema::OperatorNodesColor;
	}
	if (Category == TG_Category::Filter)
	{
		return UTG_EdGraphSchema::FilterNodesColor;
	}
	if (Category == TG_Category::Channel)
	{
		return UTG_EdGraphSchema::ChannelNodesColor;
	}
	if (Category == TG_Category::Utilities)
	{
		return UTG_EdGraphSchema::UtilitiesNodesColor;
	}

	if (Category == TG_Category::DevOnly)
	{
		return UTG_EdGraphSchema::DevOnlyNodesColor;
	}
	if (Category == TG_Category::Custom ||
		Category == TG_Category::Default)
	{
		return UTG_EdGraphSchema::CustomNodesColor;
	}

	return UTG_EdGraphSchema::TGNodeColor;
}

TSharedPtr<IAssetReferenceFilter> UTG_EdGraphSchema::MakeAssetReferenceFilter(const UEdGraph* Graph)
{
	if (const UTG_EdGraph* EdNode = Cast<const UTG_EdGraph>(Graph))
	{
		if (TSharedPtr<FTG_Editor> TGEditor = EdNode->TGEditor.Pin())
		{
			return TGEditor->MakeAssetReferenceFilter();
		}
	}

	return {};
}

#undef LOCTEXT_NAMESPACE
