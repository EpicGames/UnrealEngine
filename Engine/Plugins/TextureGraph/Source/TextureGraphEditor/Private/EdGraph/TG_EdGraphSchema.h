// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphSchema.h"
#include "TG_EdGraphSchema.generated.h"

class IAssetReferenceFilter;
class UEdGraph;
class UTG_Node;
class UTG_Pin;

UCLASS()
class UTG_EdGraphSchema : public UEdGraphSchema
{
	GENERATED_BODY()

public:
	static const FLinearColor ImagePinColor;
	static const FLinearColor ScalarPinColor;
	static const FLinearColor VectorPinColor;
	static const FLinearColor ActivePinColor;
	static const FLinearColor InactivePinColor;

	// Node title colors
	static const FLinearColor InputNodesColor;
	static const FLinearColor FunctionNodesColor;
	static const FLinearColor MathsNodesColor;
	static const FLinearColor ProceduralNodesColor;
	static const FLinearColor OperatorNodesColor;
	static const FLinearColor OutputNodesColor;
	static const FLinearColor FilterNodesColor;
	static const FLinearColor ChannelNodesColor;
	static const FLinearColor CustomNodesColor;
	static const FLinearColor DevOnlyNodesColor;
	static const FLinearColor UtilitiesNodesColor;
	static const FLinearColor TGNodeColor;
	static const FLinearColor NodeBodyColor;
	static const FLinearColor NodeBodyColorOutline;

	static const FName PC_Boolean;
	static const FName PC_Byte;
	static const FName PC_Class;    // SubCategoryObject is the MetaClass of the Class passed thru this pin, or SubCategory can be 'self'. The DefaultValue string should always be empty, use DefaultObject.
	static const FName PC_Int;
	static const FName PC_Int64;
	static const FName PC_Float;
	static const FName PC_Double;
	static const FName PC_Object;    // SubCategoryObject is the Class of the object passed thru this pin, or SubCategory can be 'self'. The DefaultValue string should always be empty, use DefaultObject.
	static const FName PC_Interface;	// SubCategoryObject is the Class of the object passed thru this pin.
	static const FName PC_String;
	static const FName PC_Text;
	static const FName PC_Struct;    // SubCategoryObject is the ScriptStruct of the struct passed thru this pin, 'self' is not a valid SubCategory. DefaultObject should always be empty, the DefaultValue string may be used for supported structs.
	static const FName PC_Array;
	static const FName PC_Enum;    // SubCategoryObject is the UEnum object passed thru this pin

	static const float NodeOutlineColorMultiplier;
	
	//~ Begin EdGraphSchema Interface
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	virtual bool TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const override;

	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	virtual FLinearColor GetSecondaryPinTypeColor(const FEdGraphPinType& PinType) const override;

	virtual void BreakNodeLinks(UEdGraphNode& TargetNode) const override;
	virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const override;
	virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;

	virtual void DroppedAssetsOnGraph(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	virtual void DroppedAssetsOnNode(const TArray<struct FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const override;
	virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;
	virtual void GetAssetsNodeHoverMessage(const TArray<struct FAssetData>& Assets, const UEdGraphNode* HoverNode, FString& OutTooltipText, bool& OutOkIcon) const override;
	virtual void TrySetDefaultValue(UEdGraphPin& Pin, const FString& NewDefaultValue, bool bMarkAsModified = true) const override;
	virtual void TrySetDefaultObject(UEdGraphPin& Pin, UObject* NewDefaultObject, bool bMarkAsModified = true) const;
	//~ End EdGraphSchema Interface

	UTG_Pin* GetTGPinFromEdPin(const UEdGraphPin* InPin) const;

	void GetPaletteActions(FGraphActionMenuBuilder& ActionMenuBuilder, const FString& CategoryName) const;
	static FLinearColor GetCategoryColor(FName Category);

private:
	void GetTG_ExpressionsActions(FGraphActionMenuBuilder& ActionMenuBuilder) const;
	void GetCommentAction(FGraphActionMenuBuilder& ActionMenuBuilder, const UEdGraph* CurrentGraph) const;

	// From a Pin to its corresponding UTG_Node 
	UTG_Node* GetTGNodeFromEdPin(const UEdGraphPin* InPin) const;
	// From a Pin to its corresponding UTG_Pin 

	// Check whether connecting these pins would cause a loop
	bool ConnectionCausesLoop(const UEdGraphPin* FromPin, const UEdGraphPin* ToPin) const;

	// Internal check that 2 pins are compatible for connection 
	bool ArePinsCompatible_Internal(const UEdGraphPin* InA, const UEdGraphPin* InB) const;

	static TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter(const UEdGraph* Graph);
};

