// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/TG_EdGraphSchemaActions.h"

#include "EdGraphNode_Comment.h"
#include "EdGraph/TG_EdGraph.h"
#include "EdGraph/TG_EdGraphNode.h"

#include "Expressions/TG_Expression.h"
#include "TextureGraph.h"
#include "TG_Graph.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_EdGraphSchemaActions)

#define LOCTEXT_NAMESPACE "TG_EdGraphSchemaActions"

UEdGraphNode* FTG_EdGraphSchemaAction_NewNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	return CreateExpressionNode(ParentGraph, TG_ExpressionClass.Get(), FromPin, Location, bSelectNewNode);
}

UEdGraphNode* FTG_EdGraphSchemaAction_NewNode::CreateExpressionNode(UEdGraph* ParentGraph, const UClass* ExpressionClass, UEdGraphPin* FromPin, const UE::Slate::FDeprecateVector2DParameter& Location, bool bSelectNewNode)
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "TSEditorNewNode", "TS Editor: New Node"));

	UTG_EdGraph* TG_EdGraph = Cast<UTG_EdGraph>(ParentGraph);
	TG_EdGraph->Modify();
	TG_EdGraph->TextureGraph->Modify();
	TG_EdGraph->TextureGraph->Graph()->Modify();

	// Create a new expression in the script
	UTG_Node* NewExpressionNode = TG_EdGraph->TextureGraph->Graph()->CreateExpressionNode(ExpressionClass);
	
	if (NewExpressionNode)
	{	
		UTG_EdGraphNode* NewNode = TG_EdGraph->AddModelNode(NewExpressionNode, true, FDeprecateSlateVector2D(Location));
		// if (MaterialExpressionClass == UMaterialExpressionFunctionInput::StaticClass() && FromPin)
		// {
		// 	// Set this to be an input of the type we dragged from
		// 	SetFunctionInputType(CastChecked<UMaterialExpressionFunctionInput>(NewExpression), UMaterialGraphSchema::GetMaterialValueType(FromPin));
		// }
		//
		// NewExpression->GraphNode->AutowireNewNode(FromPin);
		NewNode->AutowireNewNode(FromPin);
		
		return NewNode;
	}
	return nullptr;
}

UEdGraphNode* FTG_EdGraphSchemaAction_NewNode::CreateExpressionNode(UEdGraph* ParentGraph, UTG_Expression* Expression, UEdGraphPin* FromPin,  const UE::Slate::FDeprecateVector2DParameter& Location, bool bSelectNewNode)
{
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "TSEditorNewNode", "TS Editor: New Node"));

	UTG_EdGraph* TG_EdGraph = Cast<UTG_EdGraph>(ParentGraph);
	TG_EdGraph->Modify();
	TG_EdGraph->TextureGraph->Modify();
	TG_EdGraph->TextureGraph->Graph()->Modify();

	// Create a new expression in the script
	UTG_Node* NewExpressionNode = TG_EdGraph->TextureGraph->Graph()->CreateExpressionNode(Expression);
	
	if (NewExpressionNode)
	{	
		UTG_EdGraphNode* NewNode = TG_EdGraph->AddModelNode(NewExpressionNode, true, FDeprecateSlateVector2D(Location));
		NewNode->AutowireNewNode(FromPin);
		return NewNode;
	}

	return nullptr;
}

UEdGraphNode* FTG_EdGraphSchemaAction_NewComment::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin,
	const FVector2f& Location, bool bSelectNewNode)
{
	UEdGraphNode_Comment* CommentTemplate = NewObject<UEdGraphNode_Comment>();

	TSharedPtr<SGraphEditor> GraphEditorPtr = SGraphEditor::FindGraphEditorForGraph(ParentGraph);

	FVector2f SpawnLocation = Location;
	FSlateRect Bounds;
	if (GraphEditorPtr && GraphEditorPtr->GetBoundsForSelectedNodes(Bounds, 50.0f))
	{
		CommentTemplate->SetBounds(Bounds);
		SpawnLocation.X = CommentTemplate->NodePosX;
		SpawnLocation.Y = CommentTemplate->NodePosY;
	}
	
	UTG_EdGraph* TG_EdGraph = Cast<UTG_EdGraph>(ParentGraph);
	TG_EdGraph->Modify();
	TG_EdGraph->TextureGraph->Modify();
	TG_EdGraph->TextureGraph->Graph()->Modify();

	
	UEdGraphNode_Comment* NewNode = FEdGraphSchemaAction_NewNode::SpawnNodeFromTemplate<UEdGraphNode_Comment>(ParentGraph, CommentTemplate, SpawnLocation, bSelectNewNode);

	return NewNode;
}


#undef LOCTEXT_NAMESPACE
