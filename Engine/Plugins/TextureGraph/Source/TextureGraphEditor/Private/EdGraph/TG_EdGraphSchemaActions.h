// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"
#include "Templates/SubclassOf.h"
#include "UObject/SoftObjectPath.h"

#include "TG_EdGraphSchemaActions.generated.h"

class UEdGraph;
class UEdGraphPin;
class UTG_Expression;

USTRUCT()
struct FTG_EdGraphSchemaAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FEdGraphSchemaAction::FEdGraphSchemaAction;

public:
	UPROPERTY()
	TSubclassOf<UTG_Expression> TG_ExpressionClass;

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of FVector2D deprecated function in base class
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface

	static UEdGraphNode* CreateExpressionNode(UEdGraph* ParentGraph, const UClass* ExpressionClass, UEdGraphPin* FromPin,  const UE::Slate::FDeprecateVector2DParameter& Location, bool bSelectNewNode = true);
	static UEdGraphNode* CreateExpressionNode(UEdGraph* ParentGraph, UTG_Expression* Expression, UEdGraphPin* FromPin, const UE::Slate::FDeprecateVector2DParameter& Location, bool bSelectNewNode = true);
};

/** Action to add a 'comment' node to the graph */
USTRUCT()
struct FTG_EdGraphSchemaAction_NewComment : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FEdGraphSchemaAction::FEdGraphSchemaAction;

	// Simple type info
	static FName StaticGetTypeId()
	{
		static FName Type("FTG_EdGraphSchemaAction_NewComment");
		return Type;
	}

	// FEdGraphSchemaAction interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of FVector2D deprecated function in base class
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface
};

