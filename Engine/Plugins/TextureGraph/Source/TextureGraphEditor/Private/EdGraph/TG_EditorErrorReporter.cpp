// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_EditorErrorReporter.h"

#include "TG_Node.h"
#include "EdGraph/TG_EdGraph.h"
#include "EdGraph/TG_EdGraphNode.h"
#include "Misc/MessageDialog.h"

FTextureGraphErrorReport FTG_EditorErrorReporter::ReportLog(int32 ErrorId, const FString& ErrorMsg, UObject* ReferenceObj /*= nullptr*/)
{
	return {ErrorId, ErrorMsg, ReferenceObj};
}

FTextureGraphErrorReport FTG_EditorErrorReporter::Report(int32 ErrorId, const FString& ErrorMsg, UObject* ReferenceObj, EMessageSeverity::Type ErrorType)
{
	FTextureGraphErrorReport Report {ErrorId, ErrorMsg, ReferenceObj};
	
	// If the reference object is a TG_Node
	if (UTG_Node* NodeObj = Cast<UTG_Node>(ReferenceObj))
	{
		// we update the error message on the EdNode 
		auto FormattedMsg = Report.GetFormattedMessage();
		if (Editor && IsValid(Editor->TG_EdGraph))
		{
			auto EdNode = Editor->TG_EdGraph->GetViewModelNode(NodeObj->GetId());

			if(EdNode)
			{
				EdNode->ErrorMsg = FormattedMsg;
				EdNode->bHasCompilerMessage = true;
				EdNode->ErrorType = ErrorType;	
			}
		}
	}
	else if (ReferenceObj == nullptr)
	{
		// if no node provided we do a popup on the Editor
		// Editor->
		const FText Message = FText::FromString(ErrorMsg);
		EAppMsgCategory MsgCategory = GetMsgAppCategoryFromEMessageSeverity(ErrorType);
		FMessageDialog::Open(MsgCategory, EAppMsgType::Ok, Message);
	}
	TArray<FTextureGraphErrorReport>& ErrorEntry = CompilationErrors.FindOrAdd(ErrorId);

	// check to see if this error on this object is already reported
	const bool ReportAlreadyExists = ErrorEntry.ContainsByPredicate([&](const FTextureGraphErrorReport& InReport) -> bool
	{
		return InReport.ReferenceObj == ReferenceObj;
	});
	if (!ReportAlreadyExists)
	{
		// if not, we add it to our list
		ErrorEntry.Add(Report);
	}
	return Report;
}

FTextureGraphErrorReport FTG_EditorErrorReporter::ReportWarning(int32 ErrorId, const FString& ErrorMsg, UObject* ReferenceObj /*= nullptr*/)
{
	return Report(ErrorId, ErrorMsg, ReferenceObj, EMessageSeverity::Warning);
}

FTextureGraphErrorReport FTG_EditorErrorReporter::ReportError(int32 ErrorId, const FString& ErrorMsg, UObject* ReferenceObj /*= nullptr*/)
{
	return Report(ErrorId, ErrorMsg, ReferenceObj, EMessageSeverity::Error);
}

void FTG_EditorErrorReporter::Clear()
{
	for(auto ErrorEntries : CompilationErrors)
	{
		for(const FTextureGraphErrorReport& Error : ErrorEntries.Value)
		{
			// Expect the reference object to be a TG_Node
			const UTG_Node* NodeObj = Cast<UTG_Node>(Error.ReferenceObj);

			if (NodeObj && Editor && IsValid(Editor->TG_EdGraph))
			{
				auto EdNode = Editor->TG_EdGraph->GetViewModelNode(NodeObj->GetId());

				if(EdNode)
				{
					EdNode->bHasCompilerMessage = false;
					EdNode->ErrorMsg = "";
					EdNode->ErrorType = -1;	
				}
			}
		}
	}
	
	FTextureGraphErrorReporter::Clear();
}

EAppMsgCategory FTG_EditorErrorReporter::GetMsgAppCategoryFromEMessageSeverity(EMessageSeverity::Type ErrorType)
{
	switch (ErrorType)
	{
	case EMessageSeverity::Info:
	default:
		return EAppMsgCategory::Info;

	case EMessageSeverity::Warning:
		return EAppMsgCategory::Warning;
		
	case EMessageSeverity::Error:
   		return EAppMsgCategory::Error;
	}
}