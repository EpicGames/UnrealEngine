// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "TextureGraphErrorReporter.h"
#include "TG_Editor.h"
#include "Logging/TokenizedMessage.h"


class FTG_EditorErrorReporter: public FTextureGraphErrorReporter
{
public:
	FTG_EditorErrorReporter(FTG_Editor* InEditor)
	: FTextureGraphErrorReporter()
	, Editor(InEditor)
	{
	}
	virtual ~FTG_EditorErrorReporter() = default;

	virtual FTextureGraphErrorReport ReportLog(int32 ErrorId, const FString& ErrorMsg, UObject* ReferenceObj = nullptr) override;
	
	virtual FTextureGraphErrorReport ReportWarning(int32 ErrorId, const FString& ErrorMsg, UObject* ReferenceObj = nullptr) override;

	virtual FTextureGraphErrorReport ReportError(int32 ErrorId, const FString& ErrorMsg, UObject* ReferenceObj = nullptr) override;

	virtual void Clear() override;

private:
	static EAppMsgCategory GetMsgAppCategoryFromEMessageSeverity(EMessageSeverity::Type ErrorType);
	
	FTextureGraphErrorReport Report(int32 ErrorId, const FString& ErrorMsg, UObject* ReferenceObj, EMessageSeverity::Type ErrorType);

	FTG_Editor* Editor;
};
