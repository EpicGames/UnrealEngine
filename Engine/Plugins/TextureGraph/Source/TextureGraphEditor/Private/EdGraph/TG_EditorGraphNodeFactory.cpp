// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/TG_EditorGraphNodeFactory.h"

#include "EdGraph/TG_EdGraphNode.h"
#include "EdGraph/STG_EditorGraphNode.h"

TSharedPtr<SGraphNode> FTG_EditorGraphNodeFactory::CreateNode(UEdGraphNode* InNode) const
{
	if (UTG_EdGraphNode* GraphNode = Cast<UTG_EdGraphNode>(InNode))
	{
		TSharedRef<SGraphNode> VisualNode =
			SNew(STG_EditorGraphNode, GraphNode);

		VisualNode->SlatePrepass();

		return VisualNode;
	}

	return nullptr;
}
