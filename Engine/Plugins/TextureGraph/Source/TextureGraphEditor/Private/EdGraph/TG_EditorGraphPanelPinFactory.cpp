// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/TG_EditorGraphPanelPinFactory.h"

#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "KismetPins/SGraphPinEnum.h"
#include "HAL/PlatformCrt.h"
#include "KismetPins/SGraphPinBool.h"
#include "KismetPins/SGraphPinColor.h"
#include "KismetPins/SGraphPinExec.h"
#include "KismetPins/SGraphPinInteger.h"
#include "KismetPins/SGraphPinNum.h"
#include "KismetPins/SGraphPinNumSlider.h"
#include "KismetPins/SGraphPinVector.h"
#include "KismetPins/SGraphPinVectorSlider.h"
#include "KismetPins/SGraphPinVector2D.h"
#include "KismetPins/SGraphPinVector2DSlider.h"
#include "KismetPins/SGraphPinVector4.h"
#include "KismetPins/SGraphPinVector4Slider.h"
#include "KismetPins/SGraphPinObject.h"
#include "MaterialGraph/MaterialGraphSchema.h"
#include "Misc/Optional.h"
#include "SGraphPin.h"
#include "TG_EdGraphSchema.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "TG_SystemTypes.h"
#include "Pins/STG_GraphMaterialIdMaskInfos.h"
#include "Transform/Expressions/T_ExtractMaterialIds.h"
#include "Pins/STG_GraphPinOutputSettings.h"
#include "Pins/STG_GraphPinTextureDescriptor.h"
#include "Pins/STG_GraphPinString.h"
#include "Pins/STG_GraphPinMaterial.h"


#include "NumericPropertyParams.h"

TSharedPtr<class SGraphPin> FTG_EditorGraphPanelPinFactory::CreatePin(class UEdGraphPin* InPin) const
{
	if (InPin->GetSchema()->IsA<UTG_EdGraphSchema>())
	{
		// Get TG_Pin and it's corresponding FProperty to extract ClampMin/ClampMax values
		const UTG_EdGraphNode* TSEdGraphNode = Cast<UTG_EdGraphNode>(InPin->GetOwningNode());
		const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(TSEdGraphNode->GetSchema());
		UTG_Pin* TSPin = Schema->GetTGPinFromEdPin(InPin);

		FProperty* Property = TSPin->GetExpressionProperty();
		
		float MinDesiredWidth = 60.0f;
		
		if (InPin->PinType.PinCategory == UTG_EdGraphSchema::PC_Int)
		{
			return SNew(SGraphPinNumSlider<int32>, InPin, Property)
					.MinDesiredBoxWidth(MinDesiredWidth)
					.ShouldShowDisabledWhenConnected(true);
		}
		else if (InPin->PinType.PinCategory == UTG_EdGraphSchema::PC_Int64)
		{
			return SNew(SGraphPinNumSlider<int64>, InPin, Property)
					.MinDesiredBoxWidth(MinDesiredWidth)
					.ShouldShowDisabledWhenConnected(true);
		}
		else if (InPin->PinType.PinCategory == UTG_EdGraphSchema::PC_Boolean)
		{
			return SNew(SGraphPinBool, InPin);
		}
		else if (InPin->PinType.PinCategory == UTG_EdGraphSchema::PC_Double)
		{
			return SNew(SGraphPinNumSlider<double>, InPin, Property)
					.MinDesiredBoxWidth(MinDesiredWidth)
					.ShouldShowDisabledWhenConnected(true);
		}
		else if (InPin->PinType.PinCategory == UTG_EdGraphSchema::PC_Float)
		{
			return SNew(SGraphPinNumSlider<float>, InPin, Property)
					.MinDesiredBoxWidth(MinDesiredWidth)
					.ShouldShowDisabledWhenConnected(true);
		}
		else if (	InPin->PinType.PinCategory == UTG_EdGraphSchema::PC_Byte
				||	InPin->PinType.PinCategory == UTG_EdGraphSchema::PC_Enum)
		{
			// Check for valid enum object reference
			if ((InPin->PinType.PinSubCategoryObject != NULL) && (InPin->PinType.PinSubCategoryObject->IsA(UEnum::StaticClass())))
			{
				return CreateEnumPin(InPin);
			}
			else
			{
				return SNew(SGraphPinNumSlider<int32>, InPin, Property)
						.MinDesiredBoxWidth(MinDesiredWidth)
						.ShouldShowDisabledWhenConnected(true);
			}
		}
		else if (InPin->PinType.PinCategory == UTG_EdGraphSchema::PC_Struct)
		{
			// If you update this logic you'll probably need to update UEdGraphSchema_K2::ShouldHidePinDefaultValue!
			UScriptStruct* ColorStruct = TBaseStructure<FLinearColor>::Get();
			UScriptStruct* Vector4fStruct = TVariantStructure<FVector4f>::Get();
			UScriptStruct* VectorStruct = TBaseStructure<FVector>::Get();
			UScriptStruct* Vector3fStruct = TVariantStructure<FVector3f>::Get();
			UScriptStruct* Vector2fStruct = TVariantStructure<FVector2f>::Get();
			UScriptStruct* Vector2DStruct = TBaseStructure<FVector2D>::Get();
			UScriptStruct* RotatorStruct = TBaseStructure<FRotator>::Get();
			UScriptStruct* OutputSettingsStruct = TBaseStructure<FTG_OutputSettings>::Get();
			UScriptStruct* TextureDescSettingsStruct = TBaseStructure<FTG_TextureDescriptor>::Get();
			UScriptStruct* TG_Material = TBaseStructure<FTG_Material>::Get();

			if (InPin->PinType.PinSubCategoryObject == ColorStruct)
			{
				return SNew(SGraphPinColor, InPin);
			}
			else if (InPin->PinType.PinSubCategoryObject == Vector4fStruct)
			{
				return SNew(SGraphPinVector4Slider<float>, InPin, Property);
			}
			else if ((InPin->PinType.PinSubCategoryObject == VectorStruct) || (InPin->PinType.PinSubCategoryObject == Vector3fStruct) || (InPin->PinType.PinSubCategoryObject == RotatorStruct))
			{
				return SNew(SGraphPinVectorSlider<double>, InPin, Property);
			}
			else if (InPin->PinType.PinSubCategoryObject == Vector2fStruct)
			{
				return SNew(SGraphPinVector2DSlider<float>, InPin, Property);
			}
			else if (InPin->PinType.PinSubCategoryObject == Vector2DStruct)
			{
				return SNew(SGraphPinVector2DSlider<double>, InPin, Property);
			}
			else if (InPin->PinType.PinSubCategoryObject == OutputSettingsStruct)
			{
				return SNew(STG_GraphPinOutputSettings, InPin);
			}
			else if (InPin->PinType.PinSubCategoryObject == TextureDescSettingsStruct)
			{
				return SNew(STG_GraphPinTextureDescriptor, InPin);
			}
			else if (InPin->PinType.PinSubCategoryObject == TG_Material)
			{
				return SNew(STG_GraphPinMaterial, InPin);
			}
		}
		else if(InPin->PinType.PinCategory == UTG_EdGraphSchema::PC_Array)
		{
			if (InPin->PinType.PinSubCategoryObject == FMaterialIDMaskInfo::StaticStruct())
			{
				return SNew(STextureGraphMaterialIdMaskInfos, InPin);
			}
		}
		else if (InPin->PinType.PinCategory == UTG_EdGraphSchema::PC_Object)
		{
			//const UClass* ObjectMetaClass = Cast<UClass>(InPin->PinType.PinSubCategoryObject.Get());
			return SNew(SGraphPinObject, InPin);
		}
		else if (InPin->PinType.PinCategory == "PinCategory")
		{
			if (InPin->PinType.PinSubCategory == "FName")
			{
				return SNew(STG_GraphPinString, InPin);
			}
		}
		return SNew(SGraphPin, InPin);
	}
	return nullptr;
}

class TG_GraphPinEnum : public SGraphPinEnum
{
public:
	virtual EVisibility GetDefaultValueVisibility() const override
	{
		return EVisibility::Visible;
	}
};

TSharedPtr<class SGraphPin> FTG_EditorGraphPanelPinFactory::CreateEnumPin(class UEdGraphPin* InPin) const
{
	TSharedPtr<SGraphPin> PinEnum = SNew(TG_GraphPinEnum, InPin);

	//The default desired minimum width of SGraphPinEnum is 150 and there is no control to set the width
	//We will try to find Widgets by type in hierarchy of SGraphPinEnum and update the min desired width
	TSharedPtr<SWidget> FoundComboButton = FindWidgetInChildren(PinEnum, "SComboButton");
	if (FoundComboButton.IsValid())
	{
		TSharedPtr<SWidget> FoundBox = FindWidgetInChildren(FoundComboButton, "SBox");
		if (FoundBox.IsValid())
		{
			TSharedPtr<SBox> Box = StaticCastSharedPtr<SBox>(FoundBox);
			Box->SetMinDesiredWidth(20);
		}
	}

	return PinEnum;
}

TSharedPtr<SWidget> FTG_EditorGraphPanelPinFactory::FindWidgetInChildren(TSharedPtr<SWidget> Parent, FName ChildType) const
{
	TSharedPtr<SWidget> FoundWidget = TSharedPtr<SWidget>();
	for (int32 ChildIndex = 0; ChildIndex < Parent->GetAllChildren()->Num(); ++ChildIndex)
	{
		TSharedRef<SWidget> ChildWidget = Parent->GetAllChildren()->GetChildAt(ChildIndex);
		if (ChildWidget->GetType() == ChildType)
		{
			return ChildWidget.ToSharedPtr();
		}
		else if (ChildWidget->GetAllChildren()->Num() > 0)
		{
			FoundWidget = FindWidgetInChildren(ChildWidget, ChildType);
			if (FoundWidget.IsValid() && FoundWidget->GetType() == ChildType)
			{
				return FoundWidget;
			}
		}
	}
	return FoundWidget;
}