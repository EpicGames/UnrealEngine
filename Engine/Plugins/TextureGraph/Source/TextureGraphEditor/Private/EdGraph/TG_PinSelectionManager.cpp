// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/TG_PinSelectionManager.h"
#include "EdGraph/TG_EdGraphNode.h"

void FTG_PinSelectionManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	//Collector.AddReferencedObjects(SelectedItems);
}

void FTG_PinSelectionManager::UpdateSelection(UEdGraphPin* Pin)
{
	OnPinSelectionUpdated.Broadcast(Pin);
}
