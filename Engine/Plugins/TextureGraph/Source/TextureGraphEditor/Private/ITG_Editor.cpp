// Copyright Epic Games, Inc. All Rights Reserved.

#include "ITG_Editor.h"