// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBox.h"
#include "ScopedTransaction.h"
#include "SGraphPin.h"
#include "EdGraph/TG_EdGraphNode.h"
#include "Editor.h"
#include "TG_Node.h"
#include "Expressions/Utilities/TG_Expression_MaterialID.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Transform/Expressions/T_ExtractMaterialIds.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Colors/SColorBlock.h"

class STextureGraphMaterialIdMaskInfos : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(STextureGraphMaterialIdMaskInfos) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
	{
		SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);

		HandleSelectionUpdate();
	}

protected:

	FString SelectedIdsText;
	TArray<int32> SelectedIds;
	
	FText HandleOutputText() const
	{
		return FText::Format(NSLOCTEXT("STextureGraphMaterialIdMaskInfos", "SelectedIds", "{0}"), FText::FromString(SelectedIdsText));
	}
	// SGraphPin interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override
	{
		return SNew(SBox)
			.MinDesiredWidth(18)
			.MaxDesiredWidth(400)
			[
				SNew(SComboButton)
				.OnGetMenuContent(this, &STextureGraphMaterialIdMaskInfos::GenerateMaterialIdMaskInfos)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.Text(this, &STextureGraphMaterialIdMaskInfos::HandleOutputText)
				]
				
			];
	}

	TSharedRef<SWidget> GenerateMaterialIdMaskInfos()
    {
    	FMenuBuilder MenuBuilder(true, nullptr);
    
    	TArray<FMaterialIDMaskInfo> Infos = GetMaterialIdMaskInfos();

		for (const FMaterialIDMaskInfo& Info : Infos)
		{
			if(Info.bIsEnabled)
			{
				if(!SelectedIds.Contains(Info.MaterialIdReferenceId))
				{
					SelectedIds.Add(Info.MaterialIdReferenceId);
				}
			}
			else
			{
				if(SelectedIds.Contains(Info.MaterialIdReferenceId))
				{
					SelectedIds.Remove(Info.MaterialIdReferenceId);					
				}
			}

			TSharedRef<SHorizontalBox> MenuOverlayBox = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(2, 0)
			.AutoWidth()
			[
				SNew( SColorBlock )
				.Color(Info.Color)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock).Text(FText::FromString(GetColorName(Info)))
			];
			
			MenuBuilder.AddMenuEntry(
			FUIAction(
			FExecuteAction::CreateSP(this, &STextureGraphMaterialIdMaskInfos::HandleOnMaterialIdClicked, Info.MaterialIdReferenceId),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this, Info]() {return SelectedIds.Contains(Info.MaterialIdReferenceId); })
			),
			MenuOverlayBox,
			NAME_None,
			FText::Format(NSLOCTEXT("STG_MaterialIdMaskInfo", "MaterialMaskId", "{0}"), FText::FromString(GetColorName(Info))),
			EUserInterfaceActionType::Check
			);
		}
    
    	return MenuBuilder.MakeWidget();
    }

	void HandleSelectionUpdate()
	{
		const TArray<FMaterialIDMaskInfo> Infos = GetMaterialIdMaskInfos();
		
		for (const FMaterialIDMaskInfo& Info : Infos)
		{
			if(Info.bIsEnabled)
			{
				if(!SelectedIds.Contains(Info.MaterialIdReferenceId))
				{
					SelectedIds.Add(Info.MaterialIdReferenceId);
				}
			}
			else
			{
				if(SelectedIds.Contains(Info.MaterialIdReferenceId))
				{
					SelectedIds.Remove(Info.MaterialIdReferenceId);					
				}
			}
		}

		if(SelectedIds.Num() > 1)
		{
			SelectedIdsText = "Multiple Values";
		}
		else if(SelectedIds.Num() == 0)
		{
			SelectedIdsText = "None";
		}
		else
		{
			UTG_Expression_MaterialID* MaterialIdExpression = GetMaterialIDExpression();
			
			SelectedIdsText = MaterialIdExpression->MaterialIDInfoCollection.GetColorName(SelectedIds[0]);
		}
	}
	
	void HandleOnMaterialIdClicked(const int32 Id)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("STG_MaterialIdMaskInfo", "Enabled", "Edit Is Enabled"));
		
		UTG_Expression_MaterialID* MaterialIdExpression = GetMaterialIDExpression();

		const bool NewValue = !MaterialIdExpression->MaterialIDMaskInfos[Id].bIsEnabled;
		
		MaterialIdExpression->SetMaterialIdMask(Id, NewValue);
		HandleSelectionUpdate();
	}

	TArray<FMaterialIDMaskInfo> GetMaterialIdMaskInfos() const
	{
		UTG_Expression_MaterialID* MaterialIdExpression = GetMaterialIDExpression();

		return MaterialIdExpression->MaterialIDMaskInfos;	
	}

	UTG_Expression_MaterialID* GetMaterialIDExpression() const
	{
		const UTG_EdGraphNode* Node = Cast<UTG_EdGraphNode>(GraphPinObj->GetOwningNode());
		return Cast<UTG_Expression_MaterialID>(Node->GetNode()->GetExpression());
	}

	FString GetColorName(const FMaterialIDMaskInfo& MaskInfo) const
	{
		UTG_Expression_MaterialID* MaterialIdExpression = GetMaterialIDExpression();

		return MaterialIdExpression->MaterialIDInfoCollection.GetColorName(MaskInfo.MaterialIdReferenceId);
	}
};
