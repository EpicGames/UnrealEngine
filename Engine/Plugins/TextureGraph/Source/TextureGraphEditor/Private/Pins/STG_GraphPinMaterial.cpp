// Copyright Epic Games, Inc. All Rights Reserved.


#include "STG_GraphPinMaterial.h"

#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "GenericPlatform/GenericApplication.h"
#include "K2Node_CallFunction.h"
#include "PropertyEditorUtils.h"
#include "Misc/Attribute.h"
#include "ScopedTransaction.h"
#include "UObject/NameTypes.h"
#include "EdGraph/TG_EdGraphSchema.h"
#include "Expressions/TG_Expression.h"

#include "PropertyCustomizationHelpers.h"


void STG_GraphPinMaterial::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

FString	STG_GraphPinMaterial::GetCurrentAssetPath() const
{
	const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(GraphPinObj->GetOwningNode()->GetSchema());
	UTG_Pin* TSPin = Schema->GetTGPinFromEdPin(GraphPinObj);
	return TSPin->GetSelfVar()->GetAs<FTG_Material>().AssetPath.GetAssetPathString();
}

void STG_GraphPinMaterial::OnAssetSelected(const FAssetData& AssetData)
{
	const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(GraphPinObj->GetOwningNode()->GetSchema());
	UTG_Pin* TSPin = Schema->GetTGPinFromEdPin(GraphPinObj);

	FTG_Material NewMaterial;
	NewMaterial.AssetPath = AssetData.GetObjectPathString();
	TSPin->SetValue(NewMaterial);
}

TSharedRef<SWidget>	STG_GraphPinMaterial::GetDefaultValueWidget()
{
	return  SNew(SObjectPropertyEntryBox)
		.ObjectPath(this, &STG_GraphPinMaterial::GetCurrentAssetPath)
		.AllowedClass(UMaterialInterface::StaticClass())
		.OnObjectChanged(this, &STG_GraphPinMaterial::OnAssetSelected)
		.AllowCreate(true)
		.AllowClear(true)
		.DisplayUseSelected(true)
		.DisplayBrowse(true)
		;
}
