// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "TG_Material.h"

class SWidget;
class UEdGraphPin;

class STG_GraphPinMaterial : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(STG_GraphPinMaterial) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	FString GetCurrentAssetPath() const;
	void OnAssetSelected(const FAssetData& AssetData);

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface
};
