// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_GraphPinOutputSettings.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraph/TG_EdGraphSchema.h"
#include "ScopedTransaction.h"
#include "TG_Pin.h"
#include "STG_GraphPinOutputSettingsWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "OutputSettingsGraphPin"


void STG_GraphPinOutputSettings::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct( SGraphPin::FArguments(), InGraphPinObj );

	bIsUIHidden = !ShowChildProperties();
	
	if (GetDirection() == EEdGraphPinDirection::EGPD_Input)
	{
		if (!CollapsibleChildProperties() && ShowChildProperties())
		{
			PinImage->SetVisibility(EVisibility::Collapsed);
		}

		GetPinObj()->bAdvancedView = ShowChildProperties();
		if (GetPinObj()->GetOwningNode()->AdvancedPinDisplay != ENodeAdvancedPins::Shown)
		{
			GetPinObj()->GetOwningNode()->AdvancedPinDisplay = GetPinObj()->bAdvancedView ? ENodeAdvancedPins::Hidden : ENodeAdvancedPins::NoPins;
		}
	}

	CachedImg_Pin_BackgroundHovered = CachedImg_Pin_Background;
}

void STG_GraphPinOutputSettings::OnOutputSettingsChanged(const FTG_OutputSettings& NewOutputSettings)
{
	OutputSettings = NewOutputSettings;
	const FString OutputSettingsExportText = OutputSettings.ToString();

	if (OutputSettingsExportText != GraphPinObj->GetDefaultAsString())
	{
		// Set Pin Data
		const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangePinValue", "Change Pin Value"));
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, OutputSettingsExportText);
	}
}

FProperty* STG_GraphPinOutputSettings::GetPinProperty() const
{
	const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(GraphPinObj->GetOwningNode()->GetSchema());
	UTG_Pin* TSPin = Schema->GetTGPinFromEdPin(GraphPinObj);
	FProperty* Property = TSPin->GetExpressionProperty();
	return Property;
}

bool STG_GraphPinOutputSettings::ShowChildProperties() const
{
	FProperty* Property = GetPinProperty();
	bool ShowChildProperties = true;
	// check if there is a display name defined for the property, we use that as the Pin Name
	if (Property && Property->HasMetaData("HideChildProperties"))
	{
		ShowChildProperties = false;
	}
	return ShowChildProperties;
}

bool STG_GraphPinOutputSettings::CollapsibleChildProperties() const
{
	FProperty* Property = GetPinProperty();
	bool Collapsible = false;
	// check if there is a display name defined for the property, we use that as the Pin Name
	if (Property && Property->HasMetaData("CollapsableChildProperties"))
	{
		Collapsible = true;
	}
	return Collapsible;
}

EVisibility STG_GraphPinOutputSettings::ShowLabel() const
{
	bool bOutput = GetDirection() == EEdGraphPinDirection::EGPD_Output;
	bool bHide = ShowChildProperties() && !bOutput;

	if ((GraphPinObj->GetOwningNode()->AdvancedPinDisplay == ENodeAdvancedPins::Type::Hidden && GraphPinObj->LinkedTo.Num() > 0))
	{
		bHide = false;
	}

	return bHide ? EVisibility::Collapsed : EVisibility::Visible;
}

TSharedRef<SWidget>	STG_GraphPinOutputSettings::GetDefaultValueWidget()
{
	if (ShowChildProperties())
	{
		ParseDefaultValueData();

		return SNew(STG_GraphPinOutputSettingsWidget, GraphPinObj)
			.Visibility(this, &STG_GraphPinOutputSettings::IsUIEnabled)
			.DescriptionMaxWidth(250.0f)
			.OutputSettings(this, &STG_GraphPinOutputSettings::GetOutputSettings)
			.OnOutputSettingsChanged(this, &STG_GraphPinOutputSettings::OnOutputSettingsChanged)
			.IsEnabled(this, &STG_GraphPinOutputSettings::GetDefaultValueIsEnabled);
	}
	else
	{
		return SGraphPin::GetDefaultValueWidget();
	}
}

TSharedRef<SWidget> STG_GraphPinOutputSettings::GetLabelWidget(const FName& InLabelStyle)
{
	return SNew(STextBlock)
			.Text(this, &STG_GraphPinOutputSettings::GetPinLabel)
			.TextStyle(FAppStyle::Get(), InLabelStyle)
			.Visibility(this, &STG_GraphPinOutputSettings::ShowLabel)
			.ColorAndOpacity(this, &STG_GraphPinOutputSettings::GetPinTextColor);
}

void STG_GraphPinOutputSettings::OnAdvancedViewChanged(const ECheckBoxState NewCheckedState)
{
	bIsUIHidden = NewCheckedState != ECheckBoxState::Checked;
}

ECheckBoxState STG_GraphPinOutputSettings::IsAdvancedViewChecked() const
{
	return bIsUIHidden ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}

EVisibility STG_GraphPinOutputSettings::IsUIEnabled() const
{
	return bIsUIHidden ? EVisibility::Collapsed : EVisibility::Visible;
}

const FSlateBrush* STG_GraphPinOutputSettings::GetAdvancedViewArrow() const
{
	return FAppStyle::GetBrush(bIsUIHidden ? TEXT("Icons.ChevronDown") : TEXT("Icons.ChevronUp"));
}

FTG_OutputSettings STG_GraphPinOutputSettings::GetOutputSettings() const
{
	return OutputSettings;
}

void STG_GraphPinOutputSettings::ParseDefaultValueData()
{
	FString const OutputSettingsString = GraphPinObj->GetDefaultAsString();
	OutputSettings.InitFromString(OutputSettingsString);
}

#undef LOCTEXT_NAMESPACE
