// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"
#include "TG_OutputSettings.h"

class SComboButton;

class STG_GraphPinOutputSettings : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(STG_GraphPinOutputSettings) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);
	FProperty* GetPinProperty() const;
	bool ShowChildProperties() const;
	bool CollapsibleChildProperties() const;
	EVisibility ShowLabel() const;
	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	virtual TSharedRef<SWidget> GetLabelWidget(const FName& InLabelStyle) override;
	//~ End SGraphPin Interface

private:
	/** Parses the Data from the pin to fill in the names of the array. */
	void ParseDefaultValueData();

	bool GetDefaultValueIsEnabled() const
	{
		return !GraphPinObj->bDefaultValueIsReadOnly;
	}

	const FSlateBrush* GetAdvancedViewArrow() const;
	void OnAdvancedViewChanged(const ECheckBoxState NewCheckedState);
	ECheckBoxState IsAdvancedViewChecked() const;
	EVisibility IsUIEnabled() const;

	FTG_OutputSettings GetOutputSettings() const;
	void OnOutputSettingsChanged(const FTG_OutputSettings& NewOutputSettings);

	/** Parse OutputSettings used for editing. */
	FTG_OutputSettings OutputSettings;
	bool bIsUIHidden = true;
};
