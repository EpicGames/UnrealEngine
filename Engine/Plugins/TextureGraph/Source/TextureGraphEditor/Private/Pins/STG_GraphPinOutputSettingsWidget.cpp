// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_GraphPinOutputSettingsWidget.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "Widgets/Layout/SWrapBox.h"
#include "SGraphPinComboBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Customizations/TG_OutputSettingsCustomization.h"

#define LOCTEXT_NAMESPACE "STG_GraphPinOutputSettingsWidget"

//------------------------------------------------------------------------------
// STG_GraphPinOutputSettingsWidget
//------------------------------------------------------------------------------

SLATE_IMPLEMENT_WIDGET(STG_GraphPinOutputSettingsWidget)
void STG_GraphPinOutputSettingsWidget::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "OutputSettings", OutputSettingsAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				//static_cast<STG_GraphPinOutputSettingsWidget&>(Widget).CacheQueryList();
			}));
}

STG_GraphPinOutputSettingsWidget::STG_GraphPinOutputSettingsWidget()
	: OutputSettingsAttribute(*this)
{

}

STG_GraphPinOutputSettingsWidget::~STG_GraphPinOutputSettingsWidget()
{
	/*if (bRegisteredForUndo)
	{
		GEditor->UnregisterForUndo(this);
	}*/
}

void STG_GraphPinOutputSettingsWidget::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	GetPathDelegate.BindRaw(this, &STG_GraphPinOutputSettingsWidget::GetPathAsText);
	PathCommitted.BindRaw(this, &STG_GraphPinOutputSettingsWidget::OnPathCommitted);
	GetNameDelegate.BindRaw(this, &STG_GraphPinOutputSettingsWidget::GetNameAsText);
	NameCommitted.BindRaw(this, &STG_GraphPinOutputSettingsWidget::OnNameCommitted);
	OnGenerateWidthMenu.BindRaw(this, &STG_GraphPinOutputSettingsWidget::OnGenerateWidthEnumMenu);
	GetWidthDelegate.BindRaw(this, &STG_GraphPinOutputSettingsWidget::HandleWidthText);
	OnGenerateHeightMenu.BindRaw(this, &STG_GraphPinOutputSettingsWidget::OnGenerateHeightEnumMenu);
	GetHeightDelegate.BindRaw(this, &STG_GraphPinOutputSettingsWidget::HandleHeightText);
	OnGenerateFormatMenu.BindRaw(this, &STG_GraphPinOutputSettingsWidget::OnGenerateFormatEnumMenu);
	GetFormatDelegate.BindRaw(this, &STG_GraphPinOutputSettingsWidget::HandleFormatText);
	OnGenerateTexturePresetTypeMenu.BindRaw(this, &STG_GraphPinOutputSettingsWidget::OnGenerateTexturePresetTypeEnumMenu);
	GetTexturePresetTypeDelegate.BindRaw(this, &STG_GraphPinOutputSettingsWidget::HandleTexturePresetTypeText);
	OnGenerateLodGroupMenu.BindRaw(this, &STG_GraphPinOutputSettingsWidget::OnGenerateLodGroupEnumMenu);
	GetLodGroupDelegate.BindRaw(this, &STG_GraphPinOutputSettingsWidget::HandleLodGroupText);
	OnGenerateCompressionMenu.BindRaw(this, &STG_GraphPinOutputSettingsWidget::OnGenerateCompressionEnumMenu);
	GetCompressionDelegate.BindRaw(this, &STG_GraphPinOutputSettingsWidget::HandleCompressionText);

	GraphPinObj = InGraphPinObj;

	OnOutputSettingsChanged = InArgs._OnOutputSettingsChanged;

	OutputSettingsAttribute.Assign(*this, InArgs._OutputSettings);

	const int UniformPadding = 2;

	ChildSlot
	[
		//Param Box hidden when pin is connected and Advanced view is collapsed
		SNew(SBox)
		.MinDesiredWidth(250)
		.MaxDesiredWidth(450)
		.Visibility(this, &STG_GraphPinOutputSettingsWidget::ShowParameters)
		[
			SNew(SVerticalBox)

			//Path Slot
			+ SVerticalBox::Slot()
			.Padding(UniformPadding)
			.AutoHeight()
			[
				AddEditBoxWithBrowseButton(LOCTEXT("OutputPath", "Path"), GetPathDelegate, PathCommitted)
			]

			//Name Slot
			+ SVerticalBox::Slot()
			//.HAlign(HAlign_Fill)
			.Padding(UniformPadding)
			.AutoHeight()
			[
				AddEditBox(LOCTEXT("OutputName", "File Name"), GetNameDelegate, NameCommitted)
			]

			//Path Width
			+ SVerticalBox::Slot()
			.Padding(UniformPadding)
			[
				AddEnumComobox(LOCTEXT("OutputWidth", "Width"), GetWidthDelegate, OnGenerateWidthMenu)
			]

			//Path Height
			+ SVerticalBox::Slot()
			.Padding(UniformPadding)
			[
				AddEnumComobox(LOCTEXT("OutputHeight", "Height"), GetHeightDelegate, OnGenerateHeightMenu)
			]

			//Path Format
			+ SVerticalBox::Slot()
			.Padding(UniformPadding)
			[
				AddEnumComobox(LOCTEXT("OutputFormat", "Format"), GetFormatDelegate, OnGenerateFormatMenu)
			]

			//Texture Preset Type
			+ SVerticalBox::Slot()
			.Padding(UniformPadding)
			[
				AddEnumComobox(LOCTEXT("TextureType", "Texture Type"), GetTexturePresetTypeDelegate, OnGenerateTexturePresetTypeMenu)
			]

			//Lod Group
			+ SVerticalBox::Slot()
			.Padding(UniformPadding)
			[
				SNew(SHorizontalBox)
				.IsEnabled(this, &STG_GraphPinOutputSettingsWidget::IsDefaultPreset)

				+ SHorizontalBox::Slot()
				[
					AddEnumComobox(LOCTEXT("LodGroup", "LOD Group"), GetLodGroupDelegate, OnGenerateLodGroupMenu)
		
				]
			]

			//Compression Settings
			+ SVerticalBox::Slot()
			.Padding(UniformPadding)
			[
				SNew(SHorizontalBox)
				.IsEnabled(this, &STG_GraphPinOutputSettingsWidget::IsDefaultPreset)

				+ SHorizontalBox::Slot()
				[
					AddEnumComobox(LOCTEXT("CompressionSettings", "Compression Settings"), GetCompressionDelegate, OnGenerateCompressionMenu)
				]
			]

			//SRGB
			+ SVerticalBox::Slot()
			.Padding(UniformPadding)
			[
				SNew(SHorizontalBox)
				.IsEnabled(this, &STG_GraphPinOutputSettingsWidget::IsDefaultPreset)

				+ SHorizontalBox::Slot()
				[
					AddSRGBWidget().ToSharedRef()
				]
			]			
		]
	];
}

EVisibility STG_GraphPinOutputSettingsWidget::ShowPinLabel() const
{
	return ShowParameters() == EVisibility::Collapsed ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility STG_GraphPinOutputSettingsWidget::ShowParameters() const
{
	return (GraphPinObj->GetOwningNode()->AdvancedPinDisplay == ENodeAdvancedPins::Type::Hidden && GraphPinObj->LinkedTo.Num() > 0) ? EVisibility::Collapsed : EVisibility::Visible;
}

TSharedRef<SWidget> STG_GraphPinOutputSettingsWidget::AddEditBoxWithBrowseButton(FText Label, FGetTextDelegate GetText, FTextCommitted OnTextCommitted)
{
	return SNew(SHorizontalBox)

	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.AutoWidth()
	[
		SNew(SBox)
		.MinDesiredWidth(LabelSize)
		.MaxDesiredWidth(LabelSize)
		[
			SNew(STextBlock)
			.Text(Label)
			.TextStyle(FAppStyle::Get(), TEXT("Graph.Node.PinName"))
		]
	]

	+ SHorizontalBox::Slot()
	.FillWidth(1.0)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		[
			SNew(SEditableTextBox)
			.Text_Lambda([GetText]() { return GetText.Execute(); })
			.SelectAllTextWhenFocused(true)
			.SelectAllTextOnCommit(true)
			.OnTextCommitted_Lambda([OnTextCommitted](const FText& InText, ETextCommit::Type InCommitType) {
				OnTextCommitted.ExecuteIfBound(InText, InCommitType);
			})
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &STG_GraphPinOutputSettingsWidget::OnBrowseClick)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.ToolTipText(LOCTEXT("Browsepath_ToolTip", "Select the path for the output"))
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("Icons.BrowseContent"))
			]
		]
	];
}

TSharedRef<SWidget> STG_GraphPinOutputSettingsWidget::AddEditBox(FText Label, FGetTextDelegate GetText, FTextCommitted OnTextCommitted)
{
	return SNew(SHorizontalBox)

	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.AutoWidth()
	[
		SNew(SBox)
		.MinDesiredWidth(LabelSize)
		.MaxDesiredWidth(LabelSize)
		[
			SNew(STextBlock)
			.Text(Label)
			.TextStyle(FAppStyle::Get(), TEXT("Graph.Node.PinName"))
		]
	]

	+ SHorizontalBox::Slot()
	[
		SNew(SEditableTextBox)
		.Text_Lambda([GetText]() { return GetText.Execute(); })
		.SelectAllTextWhenFocused(true)
		.SelectAllTextOnCommit(true)
		.OnTextCommitted_Lambda([OnTextCommitted](const FText& InText, ETextCommit::Type InCommitType) {
				OnTextCommitted.ExecuteIfBound(InText, InCommitType);
		})
	];
}

TSharedRef<SWidget> STG_GraphPinOutputSettingsWidget::AddEnumComobox(FText Label, FGetTextDelegate GetText, FGenerateEnumMenu OnGenerateEnumMenu)
{
	return SNew(SHorizontalBox)

	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Left)
	.AutoWidth()
	[
		SNew(SBox)
		.MinDesiredWidth(LabelSize)
		.MaxDesiredWidth(LabelSize)
		[
			SNew(STextBlock)
			.Text(Label)
			.TextStyle(FAppStyle::Get(), TEXT("Graph.Node.PinName"))
		]
	]

	+ SHorizontalBox::Slot()
	[
		SNew(SComboButton)
		.OnGetMenuContent(OnGenerateEnumMenu)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text_Lambda([GetText]() { return GetText.Execute(); })
		]
	];
}

FReply STG_GraphPinOutputSettingsWidget::OnBrowseClick()
{
	FString PackagePath;
	FTG_OutputSettingsCustomization::BrowseFolderPath("", "/Output", PackagePath);
	auto Settings = GetSettings();
	Settings.FolderPath = *PackagePath;
	OnOutputSettingsChanged.ExecuteIfBound(Settings);
	return FReply::Handled();
}

TSharedPtr<SWidget> STG_GraphPinOutputSettingsWidget::AddSRGBWidget()
{
	TSharedPtr<SWidget> SRGB = SNew(SHorizontalBox)

	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Left)
	.AutoWidth()
	[
		SNew(SBox)
		.MinDesiredWidth(LabelSize)
		.MaxDesiredWidth(LabelSize)
		[
			SNew(STextBlock)
			.Text(FText::FromString("sRGB"))
		]
	]

	+ SHorizontalBox::Slot()
	[
		SNew(SCheckBox)
		.IsChecked(this, &STG_GraphPinOutputSettingsWidget::HandleSRGBIsChecked)
		.OnCheckStateChanged(this, &STG_GraphPinOutputSettingsWidget::HandleSRGBExecute)
	];

	return SRGB;
}

ECheckBoxState STG_GraphPinOutputSettingsWidget::HandleSRGBIsChecked() const
{
	auto Settings = GetSettings();
	return Settings.bSRGB ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void STG_GraphPinOutputSettingsWidget::HandleSRGBExecute(ECheckBoxState InNewState)
{
	bSRGB = InNewState == ECheckBoxState::Checked;
	auto Settings = GetSettings();
	Settings.bSRGB = bSRGB;
	OnOutputSettingsChanged.ExecuteIfBound(Settings);
}

FTG_OutputSettings STG_GraphPinOutputSettingsWidget::GetSettings() const
{
	FString OutputSettingString = GraphPinObj->GetDefaultAsString();
	FTG_OutputSettings Settings;
	Settings.InitFromString(OutputSettingString);

	return Settings;
}

void STG_GraphPinOutputSettingsWidget::GenerateStringsFromEnum(TArray<FString>& OutEnumNames,const FString& EnumPathName)
{
	UEnum* EnumPtr = FindObject<UEnum>(nullptr, *EnumPathName);
	if (EnumPtr)
	{
		for (int32 i = 0; i < EnumPtr->NumEnums() - 1; ++i)
		{
			if (!EnumPtr->HasMetaData(TEXT("Hidden"), i))
			{
				FString DisplayName = EnumPtr->GetDisplayNameTextByIndex(i).ToString();
				uint8 EnumValue = EnumPtr->GetValueByIndex(i);
				UE_LOGF(LogTemp, Warning, "Enum Value: %d, Display Name: %ls", EnumValue, *DisplayName);
				OutEnumNames.Add(DisplayName);
			}
		}
	}
}

template<typename T>
void STG_GraphPinOutputSettingsWidget::GenerateValuesFromEnum(TArray<T>& OutEnumValues, const FString& EnumPathName) const
{
	UEnum* EnumPtr = FindObject<UEnum>(nullptr, *EnumPathName);
	if (EnumPtr)
	{
		for (int32 i = 0; i < EnumPtr->NumEnums() - 1; ++i)
		{
			if (!EnumPtr->HasMetaData(TEXT("Hidden"), i))
			{
				FString DisplayName = EnumPtr->GetDisplayNameTextByIndex(i).ToString();
				T EnumValue = EnumPtr->GetValueByIndex(i);
				UE_LOGF(LogTemp, Warning, "Enum Value: %d, Display Name: %ls", EnumValue, *DisplayName);
				OutEnumValues.Add(EnumValue);
			}
		}
	}
}

template<typename T>
int STG_GraphPinOutputSettingsWidget::GetValueFromIndex(const FString& EnumPathName, int Index) const
{
	int Value = 0;
	TArray<T> EnumValues;
	UEnum* EnumPtr = FindObject<UEnum>(nullptr, *EnumPathName);
	if (EnumPtr)
	{
		GenerateValuesFromEnum<T>(EnumValues, EnumPathName);
		if (EnumValues.Num() > Index)
		{
			Value = EnumValues[Index];
		}
	}
	return Value;
}

FString STG_GraphPinOutputSettingsWidget::GetEnumValueDisplayName(const FString& EnumPathName, int EnumValue) const
{
	UEnum* EnumPtr = FindObject<UEnum>(nullptr, *EnumPathName, EFindObjectFlags::ExactClass);
	if (EnumPtr)
	{
		FString DisplayName = EnumPtr->GetDisplayNameTextByValue(EnumValue).ToString();
		return DisplayName;
	}
	return FString();
}

TSharedRef<SWidget> STG_GraphPinOutputSettingsWidget::OnGenerateWidthEnumMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	//Get list for Enum
	TArray<FString> ResolutionEnumItems;
	UEnum* Resolution = StaticEnum<EResolution>();
	GenerateStringsFromEnum(ResolutionEnumItems, Resolution->GetPathName());

	for (int i =0;i< ResolutionEnumItems.Num();i++)
	{
		auto Item = ResolutionEnumItems[i];
		MenuBuilder.AddMenuEntry(
			FText::FromString(Item),
			FText::FromString(Item),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STG_GraphPinOutputSettingsWidget::HandleWidthChanged, Item , i),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, i]() {return SelectedWidthIndex == i; })
			));
	}

	return MenuBuilder.MakeWidget();
}

void STG_GraphPinOutputSettingsWidget::HandleWidthChanged(FString Name,int Index)
{
	auto Settings = GetSettings();
	Settings.Width = (EResolution)GetValueFromIndex<int>(StaticEnum<EResolution>()->GetPathName(), Index);
	
	OnOutputSettingsChanged.ExecuteIfBound(Settings);

	SelectedWidthIndex = Index;
	SelectedWidthName = Name;
}

FText STG_GraphPinOutputSettingsWidget::HandleWidthText() const
{
	return FText::FromString(GetEnumValueDisplayName(StaticEnum<EResolution>()->GetPathName(), (int)GetSettings().Width));
}

TSharedRef<SWidget> STG_GraphPinOutputSettingsWidget::OnGenerateHeightEnumMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	//Get list for Enum
	TArray<FString> ResolutionEnumItems;
	GenerateStringsFromEnum(ResolutionEnumItems, StaticEnum<EResolution>()->GetPathName());

	for (int i = 0; i < ResolutionEnumItems.Num(); i++)
	{
		auto Item = ResolutionEnumItems[i];
		MenuBuilder.AddMenuEntry(
			FText::FromString(Item),
			FText::FromString(Item),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STG_GraphPinOutputSettingsWidget::HandleHeightChanged, Item, i),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, i]() {return SelectedHeightIndex == i; })
			));
	}

	return MenuBuilder.MakeWidget();
}

void STG_GraphPinOutputSettingsWidget::HandleHeightChanged(FString Name, int Index)
{
	auto Settings = GetSettings();
	Settings.Height = (EResolution)GetValueFromIndex<int>(StaticEnum<EResolution>()->GetPathName(), Index);
	OnOutputSettingsChanged.ExecuteIfBound(Settings);

	SelectedHeightIndex = Index;
}

FText STG_GraphPinOutputSettingsWidget::HandleHeightText() const
{
	return FText::FromString(GetEnumValueDisplayName(StaticEnum<EResolution>()->GetPathName(), (int)GetSettings().Height));
}

TSharedRef<SWidget> STG_GraphPinOutputSettingsWidget::OnGenerateFormatEnumMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	//Get list for Enum
	TArray<FString> ResolutionEnumItems;
	GenerateStringsFromEnum(ResolutionEnumItems, StaticEnum<ETG_TextureFormat>()->GetPathName());

	for (int i = 0; i < ResolutionEnumItems.Num(); i++)
	{
		auto Item = ResolutionEnumItems[i];
		MenuBuilder.AddMenuEntry(
			FText::FromString(Item),
			FText::FromString(Item),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STG_GraphPinOutputSettingsWidget::HandleFormatChanged, Item, i),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, i]() {return SelectedFormatIndex == i; })
			));
	}

	return MenuBuilder.MakeWidget();
}

void STG_GraphPinOutputSettingsWidget::HandleFormatChanged(FString Name, int Index)
{
	auto Settings = GetSettings();
	Settings.TextureFormat = (ETG_TextureFormat)GetValueFromIndex<uint8>(StaticEnum<ETG_TextureFormat>()->GetPathName(), Index);

	OnOutputSettingsChanged.ExecuteIfBound(Settings);

	SelectedFormatIndex = Index;
}

FText STG_GraphPinOutputSettingsWidget::HandleFormatText() const
{
	return FText::FromString(GetEnumValueDisplayName(StaticEnum<ETG_TextureFormat>()->GetPathName(), (int)GetSettings().TextureFormat));
}

TSharedRef<SWidget> STG_GraphPinOutputSettingsWidget::OnGenerateTexturePresetTypeEnumMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	//Get list for Enum
	TArray<FString> EnumItems;
	GenerateStringsFromEnum(EnumItems, StaticEnum<ETG_TexturePresetType>()->GetPathName());

	for (int i = 0; i < EnumItems.Num(); i++)
	{
		auto Item = EnumItems[i];
		MenuBuilder.AddMenuEntry(
			FText::FromString(Item),
			FText::FromString(Item),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STG_GraphPinOutputSettingsWidget::HandleTexturePresetTypeChanged, Item, i),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, i]() {return SelectedTextureTypeIndex == i; })
			));
	}

	return MenuBuilder.MakeWidget();
}

void STG_GraphPinOutputSettingsWidget::HandleTexturePresetTypeChanged(FString Name, int Index)
{
	auto Settings = GetSettings();
	Settings.TexturePresetType = (ETG_TexturePresetType)GetValueFromIndex<uint8>(StaticEnum<ETG_TexturePresetType>()->GetPathName(), Index);

	Settings.OnSetTexturePresetType(Settings.TexturePresetType);

	OnOutputSettingsChanged.ExecuteIfBound(Settings);

	SelectedFormatIndex = Index;
}

FText STG_GraphPinOutputSettingsWidget::HandleTexturePresetTypeText() const
{
	return FText::FromString(GetEnumValueDisplayName(StaticEnum<ETG_TexturePresetType>()->GetPathName(), (int)GetSettings().TexturePresetType));
}

TSharedRef<SWidget> STG_GraphPinOutputSettingsWidget::OnGenerateLodGroupEnumMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	//Get list for Enum
	TArray<FString> EnumItems;
	GenerateStringsFromEnum(EnumItems, StaticEnum<TextureGroup>()->GetPathName());

	for (int i = 0; i < EnumItems.Num(); i++)
	{
		auto Item = EnumItems[i];
		MenuBuilder.AddMenuEntry(
			FText::FromString(Item),
			FText::FromString(Item),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STG_GraphPinOutputSettingsWidget::HandleLodGroupChanged, Item, i),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, i]() {return SelectedLodGroupIndex == i; })
			));
	}

	return MenuBuilder.MakeWidget();
}

void STG_GraphPinOutputSettingsWidget::HandleLodGroupChanged(FString Name, int Index)
{
	auto Settings = GetSettings();
	Settings.LODGroup = (TextureGroup)GetValueFromIndex<int>(StaticEnum<TextureGroup>()->GetPathName(), Index);

	OnOutputSettingsChanged.ExecuteIfBound(Settings);

	SelectedFormatIndex = Index;
}

FText STG_GraphPinOutputSettingsWidget::HandleLodGroupText() const
{
	return FText::FromString(GetEnumValueDisplayName(StaticEnum<TextureGroup>()->GetPathName(), (int)GetSettings().LODGroup));
}

FText STG_GraphPinOutputSettingsWidget::GetNameAsText() const
{
	return FText::FromName(GetSettings().BaseName);
}

TSharedRef<SWidget> STG_GraphPinOutputSettingsWidget::OnGenerateCompressionEnumMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	//Get list for Enum
	TArray<FString> EnumItems;
	GenerateStringsFromEnum(EnumItems, StaticEnum<TextureCompressionSettings>()->GetPathName());

	for (int i = 0; i < EnumItems.Num(); i++)
	{
		auto Item = EnumItems[i];
		MenuBuilder.AddMenuEntry(
			FText::FromString(Item),
			FText::FromString(Item),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STG_GraphPinOutputSettingsWidget::HandleCompressionChanged, Item, i),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, i]() {return SelectedCompressionIndex == i; })
			));
	}

	return MenuBuilder.MakeWidget();
}

void STG_GraphPinOutputSettingsWidget::HandleCompressionChanged(FString Name, int Index)
{
	auto Settings = GetSettings();
	Settings.Compression = (TextureCompressionSettings)GetValueFromIndex<int>(StaticEnum<TextureCompressionSettings>()->GetPathName(), Index);

	OnOutputSettingsChanged.ExecuteIfBound(Settings);

	SelectedFormatIndex = Index;
}

FText STG_GraphPinOutputSettingsWidget::HandleCompressionText() const
{
	return FText::FromString(GetEnumValueDisplayName(StaticEnum<TextureCompressionSettings>()->GetPathName(), (int)GetSettings().Compression));
}

void STG_GraphPinOutputSettingsWidget::OnNameCommitted(const FText& NewText, ETextCommit::Type /*CommitInfo*/)
{
	auto Settings = GetSettings(); 
	FName BaseName(*NewText.ToString());
	Settings.BaseName = BaseName;

	OnOutputSettingsChanged.ExecuteIfBound(Settings);
}

FText STG_GraphPinOutputSettingsWidget::GetPathAsText() const
{
	return FText::FromName(GetSettings().FolderPath);
}

void STG_GraphPinOutputSettingsWidget::OnPathCommitted(const FText& NewText, ETextCommit::Type /*CommitInfo*/)
{
	FName PathName(*NewText.ToString());
	auto Settings = GetSettings(); 
	Settings.FolderPath = PathName;
	OnOutputSettingsChanged.ExecuteIfBound(Settings);
}

bool STG_GraphPinOutputSettingsWidget::IsDefaultPreset() const
{
	auto Settings = GetSettings();
	return Settings.TexturePresetType == ETG_TexturePresetType::None;
}

void STG_GraphPinOutputSettingsWidget::PostUndo(bool bSuccess)
{
	
}

void STG_GraphPinOutputSettingsWidget::PostRedo(bool bSuccess)
{

}

#undef LOCTEXT_NAMESPACE
