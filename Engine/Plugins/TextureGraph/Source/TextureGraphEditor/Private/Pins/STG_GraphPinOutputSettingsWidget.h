// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "TG_OutputSettings.h"
#include "EditorUndoClient.h"
#include "EdGraph/EdGraphPin.h"

class IPropertyHandle;
class SHorizontalBox;

/**
 * Widget for editing OutputSettings.
 */
class STG_GraphPinOutputSettingsWidget : public SCompoundWidget, public FEditorUndoClient
{
	SLATE_DECLARE_WIDGET(STG_GraphPinOutputSettingsWidget, SCompoundWidget)
	
private:
	DECLARE_DELEGATE_RetVal(FText, FGetTextDelegate);
	DECLARE_DELEGATE_TwoParams(FTextCommitted, const FText&, ETextCommit::Type);
	DECLARE_DELEGATE_RetVal(TSharedRef<SWidget>, FGenerateEnumMenu);

public:

	DECLARE_DELEGATE_OneParam(FOnOutputSettingsChanged, const FTG_OutputSettings& /*OutputSettings*/)

	SLATE_BEGIN_ARGS(STG_GraphPinOutputSettingsWidget)
		:
		_DescriptionMaxWidth(250.0f)
		, _PropertyHandle(nullptr)
	{}

		/** Maximum with of the query description field. */
		SLATE_ARGUMENT(float, DescriptionMaxWidth)

		/** OutputSettings to edit */
		SLATE_ATTRIBUTE(FTG_OutputSettings, OutputSettings)

		/** If set, the OutputSettings is read from the property, and the property is update when OutputSettings is edited. */ 
		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PropertyHandle)

		SLATE_EVENT(FOnOutputSettingsChanged, OnOutputSettingsChanged)
	SLATE_END_ARGS();

	STG_GraphPinOutputSettingsWidget();
	virtual ~STG_GraphPinOutputSettingsWidget() override;

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);
	TSharedRef<SWidget> AddEditBoxWithBrowseButton(FText Label, FGetTextDelegate GetText, FTextCommitted OnTextCommitted);
	TSharedRef<SWidget> AddEditBox(FText Label, FGetTextDelegate GetText, FTextCommitted OnTextCommitted);
	TSharedRef<SWidget> AddEnumComobox(FText Label, FGetTextDelegate GetText, FGenerateEnumMenu OnGenerateEnumMenu);
	TSharedPtr<SWidget> AddSRGBWidget();
	FTG_OutputSettings GetSettings() const;
	void GenerateStringsFromEnum(TArray<FString>& OutEnumNames, const FString& EnumPathName);

	template<typename T>
	void GenerateValuesFromEnum(TArray<T>& OutEnumValues, const FString& EnumPathName) const;

	template<typename T>
	int GetValueFromIndex(const FString& EnumPathName, int Index) const;
	FString GetEnumValueDisplayName(const FString& EnumPathName, int EnumValue) const;
	EVisibility ShowParameters() const;
	EVisibility ShowPinLabel() const;

protected:
	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FEditorUndoClient Interface

private:
	UEdGraphPin* GraphPinObj;
	
	TSlateAttribute<FTG_OutputSettings> OutputSettingsAttribute;
	FOnOutputSettingsChanged OnOutputSettingsChanged;
	FTG_OutputSettings CachedOutputSettings;
	FString SelectedWidthName;

	int SelectedWidthIndex;
	int SelectedHeightIndex;
	int SelectedFormatIndex;
	int SelectedTextureTypeIndex;
	int SelectedLodGroupIndex;
	int SelectedCompressionIndex;
	bool bSRGB = false;

	FGetTextDelegate GetPathDelegate;
	FTextCommitted PathCommitted;
	FGetTextDelegate GetNameDelegate;
	FTextCommitted NameCommitted;
	FGenerateEnumMenu OnGenerateWidthMenu;
	FGetTextDelegate GetWidthDelegate;
	FGenerateEnumMenu OnGenerateHeightMenu;
	FGetTextDelegate GetHeightDelegate;
	FGenerateEnumMenu OnGenerateFormatMenu;
	FGetTextDelegate GetFormatDelegate;
	FGenerateEnumMenu OnGenerateTexturePresetTypeMenu;
	FGetTextDelegate GetTexturePresetTypeDelegate;
	FGenerateEnumMenu OnGenerateLodGroupMenu;
	FGetTextDelegate GetLodGroupDelegate;
	FGenerateEnumMenu OnGenerateCompressionMenu;
	FGetTextDelegate GetCompressionDelegate;

	FText GetNameAsText() const;
	void OnNameCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	FText GetPathAsText() const;
	void OnPathCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	TSharedRef<SWidget> OnGenerateWidthEnumMenu();
	void HandleWidthChanged(FString OuputName, int Index);
	FText HandleWidthText() const;

	TSharedRef<SWidget> OnGenerateHeightEnumMenu();
	void HandleHeightChanged(FString OuputName, int Index);
	FText HandleHeightText() const;

	TSharedRef<SWidget> OnGenerateFormatEnumMenu();
	void HandleFormatChanged(FString OuputName, int Index);
	FText HandleFormatText() const;

	TSharedRef<SWidget> OnGenerateTexturePresetTypeEnumMenu();
	void HandleTexturePresetTypeChanged(FString OuputName, int Index);
	FText HandleTexturePresetTypeText() const;

	TSharedRef<SWidget> OnGenerateLodGroupEnumMenu();
	void HandleLodGroupChanged(FString OuputName, int Index);
	FText HandleLodGroupText() const;

	TSharedRef<SWidget> OnGenerateCompressionEnumMenu();
	void HandleCompressionChanged(FString OuputName, int Index);
	FText HandleCompressionText() const;

	ECheckBoxState HandleSRGBIsChecked() const;
	void HandleSRGBExecute(ECheckBoxState InNewState);

	FReply OnBrowseClick();

	bool IsDefaultPreset() const;

	const int LabelSize = 150;
};
