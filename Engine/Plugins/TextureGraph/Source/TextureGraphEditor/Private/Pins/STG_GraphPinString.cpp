// Copyright Epic Games, Inc. All Rights Reserved.


#include "STG_GraphPinString.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "GenericPlatform/GenericApplication.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "PropertyEditorUtils.h"
#include "Misc/Attribute.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "EdGraph/TG_EdGraphSchema.h"
#include "Expressions/TG_Expression.h"
class SWidget;

void STG_GraphPinString::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	STG_GraphPinString::GetDefaultValueWidget()
{
	const TSharedPtr<SWidget> DropdownWidget = TryBuildComboBoxWidget();
	if (DropdownWidget.Get())
	{
		return DropdownWidget.ToSharedRef();
	}

	return SNew(SBox)
		.MinDesiredWidth(18)
		.MaxDesiredWidth(400)
		[
			SNew(SEditableTextBox)
				.Style(FAppStyle::Get(), "Graph.EditableTextBox")
				.Text(this, &STG_GraphPinString::GetTypeInValue)
				.SelectAllTextWhenFocused(true)
				.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
				.IsReadOnly(this, &STG_GraphPinString::GetDefaultValueIsReadOnly)
				.OnTextCommitted(this, &STG_GraphPinString::SetTypeInValue)
				.ForegroundColor(FSlateColor::UseForeground())
		];
}

FText STG_GraphPinString::GetTypeInValue() const
{
	return FText::FromString( GraphPinObj->GetDefaultAsString() );
}

void STG_GraphPinString::SetTypeInValue(const FText& NewTypeInValue, ETextCommit::Type /*CommitInfo*/)
{
	if(GraphPinObj->IsPendingKill())
	{
		return;
	}

	if(!GraphPinObj->GetDefaultAsString().Equals(NewTypeInValue.ToString()))
	{
		const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ChangeStringPinValue", "Change String Pin Value" ) );
		GraphPinObj->Modify();

		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, NewTypeInValue.ToString());
	}
}

bool STG_GraphPinString::GetDefaultValueIsReadOnly() const
{
	return GraphPinObj->bDefaultValueIsReadOnly;
}

TSharedRef<SWidget> STG_GraphPinString::GenerateComboBoxEntry(const TSharedPtr<FString> Value)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*Value))
		.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));
}

void STG_GraphPinString::HandleComboBoxSelectionChanged(const TSharedPtr<FString> Value,
	const ESelectInfo::Type InSelectInfo)
{
	if (GraphPinObj->IsPendingKill())
	{
		return;
	}

	if (!GraphPinObj->GetDefaultAsString().Equals(*Value))
	{
		const FScopedTransaction Transaction(
			NSLOCTEXT("GraphEditor", "ChangeStringPinValue", "Change String Pin Value")
		);

		GraphPinObj->Modify();
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, *Value);
	}
}

TSharedPtr<SWidget> STG_GraphPinString::TryBuildComboBoxWidget()
{
	const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(GraphPinObj->GetOwningNode()->GetSchema());
	UTG_Pin* TSPin = Schema->GetTGPinFromEdPin(GraphPinObj);
	UTG_Expression* PropObject = TSPin->GetNodePtr()->GetExpression();
	FProperty* PinProperty = TSPin->GetExpressionProperty();

	FString GetOptionsFunctionName = PinProperty->GetMetaData("GetOptions");

	if (GetOptionsFunctionName.IsEmpty())
	{
		return nullptr;
	}


	UFunction* GetOpetionsFunction = PropObject->GetClass()->FindFunctionByName(FName(GetOptionsFunctionName));
	if (!GetOpetionsFunction)
	{
		return nullptr;
	}

	ComboBoxOptions.Empty();

	TArray<UObject*> PropertyContainers = { PropObject };
	PropertyEditorUtils::GetPropertyOptions(PropertyContainers, GetOptionsFunctionName, ComboBoxOptions);

	const FString DefaultString = GraphPinObj->GetDefaultAsString();
	const TSharedPtr<FString>* DefaultValue = ComboBoxOptions.FindByPredicate([&DefaultString](
		const TSharedPtr<FString>& Value)
	{
		return Value.IsValid() ? *Value == DefaultString : false;
	});

	return SNew(SBox)
		.MinDesiredWidth(18)
		.MaxDesiredWidth(400)
		[
			SNew(SSearchableComboBox)
				.OptionsSource(&ComboBoxOptions)
				.OnGenerateWidget(this, &STG_GraphPinString::GenerateComboBoxEntry)
				.OnSelectionChanged(this, &STG_GraphPinString::HandleComboBoxSelectionChanged)
				.ContentPadding(3.0f)
				.InitiallySelectedItem(DefaultValue ? *DefaultValue : nullptr)
				.Content()
				[
					SNew(STextBlock)
						.Text(this, &STG_GraphPinString::GetTypeInValue)
						.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				]
		];
}
