// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "SGraphPin.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

class SWidget;
class UEdGraphPin;

class STG_GraphPinString : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(STG_GraphPinString) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	FText GetTypeInValue() const;
	virtual void SetTypeInValue(const FText& NewTypeInValue, ETextCommit::Type CommitInfo);

	/** @return True if the pin default value field is read-only */
	bool GetDefaultValueIsReadOnly() const;

private:
	TSharedRef<SWidget> GenerateComboBoxEntry(TSharedPtr<FString> Value);
	void HandleComboBoxSelectionChanged(TSharedPtr<FString> Value, ESelectInfo::Type InSelectInfo);
	TSharedPtr<SWidget> TryBuildComboBoxWidget();

	TArray<TSharedPtr<FString>> ComboBoxOptions;
};
