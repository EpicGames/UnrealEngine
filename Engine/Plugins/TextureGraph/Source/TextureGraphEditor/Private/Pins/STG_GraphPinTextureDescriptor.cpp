// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_GraphPinTextureDescriptor.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraph/TG_EdGraphSchema.h"
#include "ScopedTransaction.h"
#include "TG_Pin.h"
#include "STG_GraphPinTextureDescriptorWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "TextureDescriptorGraphPin"

void STG_GraphPinTextureDescriptor::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct( SGraphPin::FArguments(), InGraphPinObj );

	bIsUIHidden = !ShowChildProperties();
	
	if (GetDirection() == EEdGraphPinDirection::EGPD_Input)
	{
		if (!CollapsibleChildProperties() && ShowChildProperties())
		{
			PinImage->SetVisibility(EVisibility::Collapsed);
		}

		GetPinObj()->bAdvancedView = ShowChildProperties();
		if (GetPinObj()->GetOwningNode()->AdvancedPinDisplay != ENodeAdvancedPins::Shown)
		{
			GetPinObj()->GetOwningNode()->AdvancedPinDisplay = GetPinObj()->bAdvancedView ? ENodeAdvancedPins::Hidden : ENodeAdvancedPins::NoPins;
		}
	}

	CachedImg_Pin_BackgroundHovered = CachedImg_Pin_Background;
}

void STG_GraphPinTextureDescriptor::OnTextureDescriptorChanged(const FTG_TextureDescriptor& NewTextureDescriptor)
{
	TextureDescriptor = NewTextureDescriptor;
	const FString TextureDescriptorExportText = TextureDescriptor.ToString();

	if (TextureDescriptorExportText != GraphPinObj->GetDefaultAsString())
	{
		// Set Pin Data
		const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangePinValue", "Change Pin Value"));
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, TextureDescriptorExportText);
	}
}

FProperty* STG_GraphPinTextureDescriptor::GetPinProperty() const
{
	const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(GraphPinObj->GetOwningNode()->GetSchema());
	UTG_Pin* TSPin = Schema->GetTGPinFromEdPin(GraphPinObj);
	FProperty* Property = TSPin->GetExpressionProperty();
	return Property;
}

bool STG_GraphPinTextureDescriptor::ShowChildProperties() const
{
	FProperty* Property = GetPinProperty();
	bool ShowChildProperties = true;
	// check if there is a display name defined for the property, we use that as the Pin Name
	if (Property && Property->HasMetaData("HideChildProperties"))
	{
		ShowChildProperties = false;
	}
	return ShowChildProperties;
}

bool STG_GraphPinTextureDescriptor::CollapsibleChildProperties() const
{
	FProperty* Property = GetPinProperty();
	bool Collapsible = false;
	// check if there is a display name defined for the property, we use that as the Pin Name
	if (Property && Property->HasMetaData("CollapsableChildProperties"))
	{
		Collapsible = true;
	}
	return Collapsible;
}

EVisibility STG_GraphPinTextureDescriptor::ShowLabel() const
{
	bool bOutput = GetDirection() == EEdGraphPinDirection::EGPD_Output;
	bool bHide = ShowChildProperties() && !bOutput;

	if ((GraphPinObj->GetOwningNode()->AdvancedPinDisplay == ENodeAdvancedPins::Type::Hidden && GraphPinObj->LinkedTo.Num() > 0))
	{
		bHide = false;
	}

	return bHide ? EVisibility::Collapsed : EVisibility::Visible;
}

TSharedRef<SWidget>	STG_GraphPinTextureDescriptor::GetDefaultValueWidget()
{
	if (ShowChildProperties())
	{
		ParseDefaultValueData();

		return SNew(STG_GraphPinTextureDescriptorWidget, GraphPinObj)
			.Visibility(this, &STG_GraphPinTextureDescriptor::IsUIEnabled)
			.DescriptionMaxWidth(250.0f)
			.TextureDescriptor(this, &STG_GraphPinTextureDescriptor::GetTextureDescriptor)
			.OnTextureDescriptorChanged(this, &STG_GraphPinTextureDescriptor::OnTextureDescriptorChanged)
			.IsEnabled(this, &STG_GraphPinTextureDescriptor::GetDefaultValueIsEnabled);
	}
	else
	{
		return SGraphPin::GetDefaultValueWidget();
	}
}

TSharedRef<SWidget> STG_GraphPinTextureDescriptor::GetLabelWidget(const FName& InLabelStyle)
{
	return SNew(STextBlock)
			.Text(this, &STG_GraphPinTextureDescriptor::GetPinLabel)
			.TextStyle(FAppStyle::Get(), InLabelStyle)
			.Visibility(this, &STG_GraphPinTextureDescriptor::ShowLabel)
			.ColorAndOpacity(this, &STG_GraphPinTextureDescriptor::GetPinTextColor);
}

void STG_GraphPinTextureDescriptor::OnAdvancedViewChanged(const ECheckBoxState NewCheckedState)
{
	bIsUIHidden = NewCheckedState != ECheckBoxState::Checked;
}

ECheckBoxState STG_GraphPinTextureDescriptor::IsAdvancedViewChecked() const
{
	return bIsUIHidden ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}

EVisibility STG_GraphPinTextureDescriptor::IsUIEnabled() const
{
	return bIsUIHidden ? EVisibility::Collapsed : EVisibility::Visible;
}

const FSlateBrush* STG_GraphPinTextureDescriptor::GetAdvancedViewArrow() const
{
	return FAppStyle::GetBrush(bIsUIHidden ? TEXT("Icons.ChevronDown") : TEXT("Icons.ChevronUp"));
}

FTG_TextureDescriptor STG_GraphPinTextureDescriptor::GetTextureDescriptor() const
{
	return TextureDescriptor;
}

void STG_GraphPinTextureDescriptor::ParseDefaultValueData()
{
	FString const TextureDescriptorString = GraphPinObj->GetDefaultAsString();
	TextureDescriptor.InitFromString(TextureDescriptorString);
}

#undef LOCTEXT_NAMESPACE
