// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"
#include "TG_Texture.h"

class SComboButton;

class STG_GraphPinTextureDescriptor : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(STG_GraphPinTextureDescriptor) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);
	FProperty* GetPinProperty() const;
	bool ShowChildProperties() const;
	bool CollapsibleChildProperties() const;
	EVisibility ShowLabel() const;
	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	virtual TSharedRef<SWidget> GetLabelWidget(const FName& InLabelStyle) override;
	//~ End SGraphPin Interface

private:
	/** Parses the Data from the pin to fill in the names of the array. */
	void ParseDefaultValueData();

	bool GetDefaultValueIsEnabled() const
	{
		return !GraphPinObj->bDefaultValueIsReadOnly;
	}

	const FSlateBrush* GetAdvancedViewArrow() const;
	void OnAdvancedViewChanged(const ECheckBoxState NewCheckedState);
	ECheckBoxState IsAdvancedViewChecked() const;
	EVisibility IsUIEnabled() const;

	FTG_TextureDescriptor GetTextureDescriptor() const;
	void OnTextureDescriptorChanged(const FTG_TextureDescriptor& NewTextureDescriptor);

	/** Parse TextureDescriptor used for editing. */
	FTG_TextureDescriptor TextureDescriptor;
	bool bIsUIHidden = true;
};
