// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_GraphPinTextureDescriptorWidget.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "Widgets/Layout/SWrapBox.h"
#include "SGraphPinComboBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "STG_GraphPinTextureDescriptorWidget"

//------------------------------------------------------------------------------
// STG_GraphPinTextureDescriptorWidget
//------------------------------------------------------------------------------

SLATE_IMPLEMENT_WIDGET(STG_GraphPinTextureDescriptorWidget)
void STG_GraphPinTextureDescriptorWidget::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "TextureDescriptor", TextureDescriptorAttribute, EInvalidateWidgetReason::Layout)
	.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			//static_cast<STG_GraphPinTextureDescriptorWidget&>(Widget).CacheQueryList();
		}));
}

STG_GraphPinTextureDescriptorWidget::STG_GraphPinTextureDescriptorWidget()
	: TextureDescriptorAttribute(*this)
{

}

STG_GraphPinTextureDescriptorWidget::~STG_GraphPinTextureDescriptorWidget()
{
	/*if (bRegisteredForUndo)
	{
	GEditor->UnregisterForUndo(this);
	}*/
}

void STG_GraphPinTextureDescriptorWidget::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	OnGenerateWidthMenu.BindRaw(this, &STG_GraphPinTextureDescriptorWidget::OnGenerateWidthEnumMenu);
	GetWidthDelegate.BindRaw(this, &STG_GraphPinTextureDescriptorWidget::HandleWidthText);
	OnGenerateHeightMenu.BindRaw(this, &STG_GraphPinTextureDescriptorWidget::OnGenerateHeightEnumMenu);
	GetHeightDelegate.BindRaw(this, &STG_GraphPinTextureDescriptorWidget::HandleHeightText);
	OnGenerateFormatMenu.BindRaw(this, &STG_GraphPinTextureDescriptorWidget::OnGenerateFormatEnumMenu);
	GetFormatDelegate.BindRaw(this, &STG_GraphPinTextureDescriptorWidget::HandleFormatText);

	GraphPinObj = InGraphPinObj;

	OnTextureDescriptorChanged = InArgs._OnTextureDescriptorChanged;

	TextureDescriptorAttribute.Assign(*this, InArgs._TextureDescriptor);

	const int UniformPadding = 2;

	ChildSlot
		[
			//Param Box hidden when pin is connected and Advanced view is collapsed
			SNew(SBox)
				.MinDesiredWidth(LabelSize * 2)
				.MaxDesiredWidth(LabelSize * 4)
				.Visibility(this, &STG_GraphPinTextureDescriptorWidget::ShowParameters)
				[
					SNew(SVerticalBox)
						//Separator
						+SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Center)
						[
							SNew(SSeparator)
								.Thickness(2)
						]

						//Path Width
						+SVerticalBox::Slot()
						.Padding(UniformPadding)
						[
							AddEnumComobox(LOCTEXT("OutputWidth", "Width"), GetWidthDelegate, OnGenerateWidthMenu)
						]

						//Path Height
						+ SVerticalBox::Slot()
						.Padding(UniformPadding)
						[
							AddEnumComobox(LOCTEXT("OutputHeight", "Height"), GetHeightDelegate, OnGenerateHeightMenu)
						]

						//Path Format
						+ SVerticalBox::Slot()
						.Padding(UniformPadding)
						[
							AddEnumComobox(LOCTEXT("OutputFormat", "Format"), GetFormatDelegate, OnGenerateFormatMenu)
						]

						//SRGB
						+ SVerticalBox::Slot()
						.Padding(UniformPadding)
						[
							AddSRGBWidget()
						]		

						//Separator
						+SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Center)
						[
							SNew(SSeparator)
								.Thickness(2)
						]
				]
		];
}

EVisibility STG_GraphPinTextureDescriptorWidget::ShowPinLabel() const
{
	return ShowParameters() == EVisibility::Collapsed ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility STG_GraphPinTextureDescriptorWidget::ShowParameters() const
{
	return (GraphPinObj->GetOwningNode()->AdvancedPinDisplay == ENodeAdvancedPins::Type::Hidden && GraphPinObj->LinkedTo.Num() > 0) ? EVisibility::Collapsed : EVisibility::Visible;
}

TSharedRef<SWidget> STG_GraphPinTextureDescriptorWidget::AddEnumComobox(FText Label, FGetTextDelegate GetText, FGenerateEnumMenu OnGenerateEnumMenu)
{
	return 
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.MinSize(LabelSize)
		.FillWidth(1.0f)
		[
			SNew(SBox)
				.MinDesiredWidth(LabelSize)
				.MaxDesiredWidth(LabelSize)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
						.Text(Label)
						.TextStyle(FAppStyle::Get(), TEXT("Graph.Node.PinName"))
				]
		]

		+ SHorizontalBox::Slot()
		.MaxWidth(LabelSize * 3)
		[
			SNew(SComboButton)
				.HAlign(HAlign_Right)
				.OnGetMenuContent(OnGenerateEnumMenu)
				.ButtonContent()
				[
					SNew(STextBlock)
						.Text_Lambda([GetText]() { return GetText.Execute(); })
				]
		];
}

TSharedRef<SWidget> STG_GraphPinTextureDescriptorWidget::AddSRGBWidget()
{
	return 
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.MinSize(LabelSize)
		.FillWidth(1.0f)
		[
			SNew(SBox)
				.MinDesiredWidth(LabelSize)
				.MaxDesiredWidth(LabelSize)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
						.Text(FText::FromString("sRGB"))
						.TextStyle(FAppStyle::Get(), TEXT("Graph.Node.PinName"))
				]
		]

		+ SHorizontalBox::Slot()
		.MaxWidth(LabelSize * 3)
		[
			SNew(SCheckBox)
				.IsChecked(this, &STG_GraphPinTextureDescriptorWidget::HandleSRGBIsChecked)
				.OnCheckStateChanged(this, &STG_GraphPinTextureDescriptorWidget::HandleSRGBExecute)
		];
}

ECheckBoxState STG_GraphPinTextureDescriptorWidget::HandleSRGBIsChecked() const
{
	auto Settings = GetSettings();
	return Settings.bIsSRGB ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void STG_GraphPinTextureDescriptorWidget::HandleSRGBExecute(ECheckBoxState InNewState)
{
	bSRGB = InNewState == ECheckBoxState::Checked;
	auto Settings = GetSettings();
	Settings.bIsSRGB = bSRGB;
	OnTextureDescriptorChanged.ExecuteIfBound(Settings);
}

FTG_TextureDescriptor STG_GraphPinTextureDescriptorWidget::GetSettings() const
{
	FString TextureDescriptorString = GraphPinObj->GetDefaultAsString();
	FTG_TextureDescriptor Settings;
	Settings.InitFromString(TextureDescriptorString);

	return Settings;
}

void STG_GraphPinTextureDescriptorWidget::GenerateStringsFromEnum(TArray<FString>& OutEnumNames, const FString& EnumPathName)
{
	UEnum* EnumPtr = FindObject<UEnum>(nullptr, *EnumPathName);
	if (EnumPtr)
	{
		for (int32 i = 0; i < EnumPtr->NumEnums() - 1; ++i)
		{
			if (!EnumPtr->HasMetaData(TEXT("Hidden"), i))
			{
				FString DisplayName = EnumPtr->GetDisplayNameTextByIndex(i).ToString();
				uint8 EnumValue = EnumPtr->GetValueByIndex(i);
				UE_LOGF(LogTemp, Warning, "Enum Value: %d, Display Name: %ls", EnumValue, *DisplayName);
				OutEnumNames.Add(DisplayName);
			}
		}
	}
}

template<typename T>
void STG_GraphPinTextureDescriptorWidget::GenerateValuesFromEnum(TArray<T>& OutEnumValues, const FString& EnumPathName) const
{
	UEnum* EnumPtr = FindObject<UEnum>(nullptr, *EnumPathName);
	if (EnumPtr)
	{
		for (int32 i = 0; i < EnumPtr->NumEnums() - 1; ++i)
		{
			if (!EnumPtr->HasMetaData(TEXT("Hidden"), i))
			{
				FString DisplayName = EnumPtr->GetDisplayNameTextByIndex(i).ToString();
				T EnumValue = EnumPtr->GetValueByIndex(i);
				UE_LOGF(LogTemp, Warning, "Enum Value: %d, Display Name: %ls", EnumValue, *DisplayName);
				OutEnumValues.Add(EnumValue);
			}
		}
	}
}

template<typename T>
int STG_GraphPinTextureDescriptorWidget::GetValueFromIndex(const FString& EnumPathName, int Index) const
{
	int Value = 0;
	TArray<T> EnumValues;
	UEnum* EnumPtr = FindObject<UEnum>(nullptr, *EnumPathName);
	if (EnumPtr)
	{
		GenerateValuesFromEnum<T>(EnumValues, EnumPathName);
		if (EnumValues.Num() > Index)
		{
			Value = EnumValues[Index];
		}
	}
	return Value;
}

FString STG_GraphPinTextureDescriptorWidget::GetEnumValueDisplayName(const FString& EnumPathName, int EnumValue) const
{
	UEnum* EnumPtr = FindObject<UEnum>(nullptr, *EnumPathName, EFindObjectFlags::ExactClass);
	if (EnumPtr)
	{
		FString DisplayName = EnumPtr->GetDisplayNameTextByValue(EnumValue).ToString();
		return DisplayName;
	}
	return FString();
}

TSharedRef<SWidget> STG_GraphPinTextureDescriptorWidget::OnGenerateWidthEnumMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	//Get list for Enum
	TArray<FString> ResolutionEnumItems;
	UEnum* Resolution = StaticEnum<EResolution>();
	GenerateStringsFromEnum(ResolutionEnumItems, Resolution->GetPathName());

	for (int i = 0; i < ResolutionEnumItems.Num(); i++)
	{
		auto Item = ResolutionEnumItems[i];
		MenuBuilder.AddMenuEntry(
			FText::FromString(Item),
			FText::FromString(Item),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STG_GraphPinTextureDescriptorWidget::HandleWidthChanged, Item, i),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, i]() {return SelectedWidthIndex == i; })
			));
	}

	return MenuBuilder.MakeWidget();
}

void STG_GraphPinTextureDescriptorWidget::HandleWidthChanged(FString Name, int Index)
{
	auto Settings = GetSettings();
	Settings.Width = (EResolution)GetValueFromIndex<int>(StaticEnum<EResolution>()->GetPathName(), Index);

	OnTextureDescriptorChanged.ExecuteIfBound(Settings);

	SelectedWidthIndex = Index;
	SelectedWidthName = Name;
}

FText STG_GraphPinTextureDescriptorWidget::HandleWidthText() const
{
	return FText::FromString(GetEnumValueDisplayName(StaticEnum<EResolution>()->GetPathName(), (int)GetSettings().Width));
}

TSharedRef<SWidget> STG_GraphPinTextureDescriptorWidget::OnGenerateHeightEnumMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	//Get list for Enum
	TArray<FString> ResolutionEnumItems;
	GenerateStringsFromEnum(ResolutionEnumItems, StaticEnum<EResolution>()->GetPathName());

	for (int i = 0; i < ResolutionEnumItems.Num(); i++)
	{
		auto Item = ResolutionEnumItems[i];
		MenuBuilder.AddMenuEntry(
			FText::FromString(Item),
			FText::FromString(Item),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STG_GraphPinTextureDescriptorWidget::HandleHeightChanged, Item, i),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, i]() {return SelectedHeightIndex == i; })
			));
	}

	return MenuBuilder.MakeWidget();
}

void STG_GraphPinTextureDescriptorWidget::HandleHeightChanged(FString Name, int Index)
{
	auto Settings = GetSettings();
	Settings.Height = (EResolution)GetValueFromIndex<int>(StaticEnum<EResolution>()->GetPathName(), Index);
	OnTextureDescriptorChanged.ExecuteIfBound(Settings);

	SelectedHeightIndex = Index;
}

FText STG_GraphPinTextureDescriptorWidget::HandleHeightText() const
{
	return FText::FromString(GetEnumValueDisplayName(StaticEnum<EResolution>()->GetPathName(), (int)GetSettings().Height));
}

TSharedRef<SWidget> STG_GraphPinTextureDescriptorWidget::OnGenerateFormatEnumMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	//Get list for Enum
	TArray<FString> ResolutionEnumItems;
	GenerateStringsFromEnum(ResolutionEnumItems, StaticEnum<ETG_TextureFormat>()->GetPathName());

	for (int i = 0; i < ResolutionEnumItems.Num(); i++)
	{
		auto Item = ResolutionEnumItems[i];
		MenuBuilder.AddMenuEntry(
			FText::FromString(Item),
			FText::FromString(Item),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STG_GraphPinTextureDescriptorWidget::HandleFormatChanged, Item, i),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, i]() {return SelectedFormatIndex == i; })
			));
	}

	return MenuBuilder.MakeWidget();
}

void STG_GraphPinTextureDescriptorWidget::HandleFormatChanged(FString Name, int Index)
{
	auto Settings = GetSettings();
	Settings.TextureFormat = (ETG_TextureFormat)GetValueFromIndex<uint8>(StaticEnum<ETG_TextureFormat>()->GetPathName(), Index);

	OnTextureDescriptorChanged.ExecuteIfBound(Settings);

	SelectedFormatIndex = Index;
}

FText STG_GraphPinTextureDescriptorWidget::HandleFormatText() const
{
	return FText::FromString(GetEnumValueDisplayName(StaticEnum<ETG_TextureFormat>()->GetPathName(), (int)GetSettings().TextureFormat));
}

void STG_GraphPinTextureDescriptorWidget::PostUndo(bool bSuccess)
{
}

void STG_GraphPinTextureDescriptorWidget::PostRedo(bool bSuccess)
{
}

#undef LOCTEXT_NAMESPACE
