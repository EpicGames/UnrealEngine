// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_TextureDescriptor.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "Widgets/Layout/SWrapBox.h"
#include "SGraphPinComboBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "TG_Graph.h"
#include "Expressions/TG_Expression.h"
#include "EdGraph/TG_EdGraphSchema.h"
#include "Widgets/Layout/SSeparator.h"
#include "TG_HelperFunctions.h"

#define LOCTEXT_NAMESPACE "STG_TextureDescriptor"
//------------------------------------------------------------------------------
// STG_TextureDescriptor
//------------------------------------------------------------------------------

void STG_TextureDescriptor::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	OnGenerateWidthMenu.BindRaw(this, &STG_TextureDescriptor::OnGenerateWidthEnumMenu);
	GetWidthDelegate.BindRaw(this, &STG_TextureDescriptor::HandleWidthText);
	OnGenerateHeightMenu.BindRaw(this, &STG_TextureDescriptor::OnGenerateHeightEnumMenu);
	GetHeightDelegate.BindRaw(this, &STG_TextureDescriptor::HandleHeightText);
	OnGenerateFormatMenu.BindRaw(this, &STG_TextureDescriptor::OnGenerateFormatEnumMenu);
	GetFormatDelegate.BindRaw(this, &STG_TextureDescriptor::HandleFormatText);

	GraphPinObj = InGraphPinObj;
	OnTextureDescriptorChanged = InArgs._OnTextureDescriptorChanged;

	TSharedPtr<SVerticalBox> VerticalBox = SNew(SVerticalBox);


	float UniformPadding = 5;
	float SeperatorMargin = 0;

	VerticalBox->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	.Padding(0, 3)
	[
		SNew(STextBlock)
		.Text(FText::FromString("Output Settings"))
		.TextStyle(FAppStyle::Get(), TEXT("Graph.Node.PinName"))
	];

	VerticalBox->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SSeparator)
		.Thickness(2)
	];
	//Width
	VerticalBox->AddSlot()
	.Padding(UniformPadding)
	[
		AddEnumComobox(LOCTEXT("OutputWidth", "Width"), GetWidthDelegate, OnGenerateWidthMenu)
	];

	AddVerticalSeperation(VerticalBox, SeperatorMargin);
	//Height
	VerticalBox->AddSlot()
	.Padding(UniformPadding)
	[
		AddEnumComobox(LOCTEXT("OutputHeight", "Height"), GetHeightDelegate, OnGenerateHeightMenu)
	];

	AddVerticalSeperation(VerticalBox, SeperatorMargin);
	//Format
	VerticalBox->AddSlot()
	.Padding(UniformPadding)
	[
		AddEnumComobox(LOCTEXT("OutputFormat", "TextureFormat"), GetFormatDelegate, OnGenerateFormatMenu)
	];

	AddVerticalSeperation(VerticalBox, SeperatorMargin);

	ChildSlot
	[
		VerticalBox.ToSharedRef()
	];
}

void STG_TextureDescriptor::AddVerticalSeperation(TSharedPtr<SVerticalBox> VBox, FMargin Padding)
{
	VBox->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	[
		SNew(SSeparator)
		.Thickness(2)
	];
}

TSharedRef<SWidget> STG_TextureDescriptor::AddEditBox(FText Label, FGetTextDelegate GetText, FTextCommitted OnTextCommitted)
{
	return SNew(SBox)
	.MinDesiredWidth(100)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.FillWidth(1.0)
		.Padding(0, 0, 5, 0)
		[
			SNew(STextBlock)
			.Text(Label)
			.TextStyle(FAppStyle::Get(), TEXT("Graph.Node.PinName"))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(250)
		[

			SNew(SEditableTextBox)
			.Text_Lambda([GetText]() { return GetText.Execute(); })
			.SelectAllTextWhenFocused(true)
			.SelectAllTextOnCommit(true)
			.OnTextCommitted_Lambda([OnTextCommitted](const FText& InText, ETextCommit::Type InCommitType) {
					OnTextCommitted.ExecuteIfBound(InText, InCommitType);
			})
		]
	];
}

TSharedRef<SWidget> STG_TextureDescriptor::AddEnumComobox(FText Label, FGetTextDelegate GetText, FGenerateEnumMenu OnGenerateEnumMenu)
{
	return SNew(SBox)
	.MinDesiredWidth(100)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		.FillWidth(1.0)
		.Padding(0, 0, 5, 0)
		[
			SNew(STextBlock)
			.Text(Label)
			.TextStyle(FAppStyle::Get(), TEXT("Graph.Node.PinName"))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(250)
		[
			SNew(SComboButton)
			.OnGetMenuContent(OnGenerateEnumMenu)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text_Lambda([GetText]() { return GetText.Execute(); })
			]
		]
	];
}

FTG_TextureDescriptor STG_TextureDescriptor::GetTextureDescriptor() const
{
	const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(GraphPinObj->GetSchema());
	UTG_Pin* TGPin = Schema->GetTGPinFromEdPin(GraphPinObj);
	check(TGPin->GetArgument().IsTexture());
	FTG_Texture OutTexture;
	TGPin->GetValue(OutTexture);

	return OutTexture.Descriptor;
}

void STG_TextureDescriptor::SetValue(FTG_TextureDescriptor Descriptor)
{
	const FScopedTransaction Transaction(NSLOCTEXT("STG_TextureDescriptor", "SetValue", "Output Settings Pin Changed"));

	const UTG_EdGraphSchema* Schema = Cast<const UTG_EdGraphSchema>(GraphPinObj->GetSchema());
	UTG_Pin* TGPin = Schema->GetTGPinFromEdPin(GraphPinObj);
	check(TGPin->GetArgument().IsTexture());
	TGPin->Modify();

	TGPin->SetValue(Descriptor);

	Schema->TrySetDefaultObject(*GraphPinObj, TGPin);
}

void STG_TextureDescriptor::GenerateStringsFromEnum(TArray<FString>& OutEnumNames,const FString& EnumPathName)
{
	UEnum* EnumPtr = FindObject<UEnum>(nullptr, *EnumPathName);
	if (EnumPtr)
	{
		for (int32 i = 0; i < EnumPtr->NumEnums() - 1; ++i)
		{
			if (!EnumPtr->HasMetaData(TEXT("Hidden"), i))
			{
				FString DisplayName = EnumPtr->GetDisplayNameTextByIndex(i).ToString();
				uint8 EnumValue = EnumPtr->GetValueByIndex(i);
				UE_LOGF(LogTemp, Warning, "Enum Value: %d, Display Name: %ls", EnumValue, *DisplayName);
				OutEnumNames.Add(DisplayName);
			}
		}
	}
}

int STG_TextureDescriptor::GetValueFromIndex(const FString& EnumPathName, int Index) const
{
	int Value = 0;
	UEnum* EnumPtr = FindObject<UEnum>(nullptr, *EnumPathName);
	if (EnumPtr)
	{
		Value = EnumPtr->GetValueByIndex(Index);
	}
	return Value;
}

FString STG_TextureDescriptor::GetEnumValueDisplayName(const FString& EnumPathName, int EnumValue) const
{
	UEnum* EnumPtr = FindObject<UEnum>(nullptr, *EnumPathName, EFindObjectFlags::ExactClass);
	if (EnumPtr)
	{
		FString DisplayName = EnumPtr->GetDisplayNameTextByValue(EnumValue).ToString();
		return DisplayName;
	}
	return FString();
}

TSharedRef<SWidget> STG_TextureDescriptor::OnGenerateWidthEnumMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	//Get list for Enum
	TArray<FString> ResolutionEnumItems;
	UEnum* Resolution = StaticEnum<EResolution>();
	GenerateStringsFromEnum(ResolutionEnumItems, Resolution->GetPathName());

	for (int i =0;i< ResolutionEnumItems.Num();i++)
	{
		auto Item = ResolutionEnumItems[i];
		MenuBuilder.AddMenuEntry(
			FText::FromString(Item),
			FText::FromString(Item),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STG_TextureDescriptor::HandleWidthChanged, Item , i),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, i]() {return SelectedWidthIndex == i; })
			));
	}

	return MenuBuilder.MakeWidget();
}

void STG_TextureDescriptor::HandleWidthChanged(FString Name,int Index)
{
	auto TextureDescriptor = GetTextureDescriptor();
	TextureDescriptor.Width = (EResolution)GetValueFromIndex(StaticEnum<EResolution>()->GetPathName(), Index);
	
	SetValue(TextureDescriptor);
	OnTextureDescriptorChanged.ExecuteIfBound(TextureDescriptor);

	SelectedWidthIndex = Index;
	SelectedWidthName = Name;
}

FText STG_TextureDescriptor::HandleWidthText() const
{
	return FText::FromString(GetEnumValueDisplayName(StaticEnum<EResolution>()->GetPathName(), (int)GetTextureDescriptor().Width));
}

TSharedRef<SWidget> STG_TextureDescriptor::OnGenerateHeightEnumMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	//Get list for Enum
	TArray<FString> ResolutionEnumItems;
	GenerateStringsFromEnum(ResolutionEnumItems, StaticEnum<EResolution>()->GetPathName());

	for (int i = 0; i < ResolutionEnumItems.Num(); i++)
	{
		auto Item = ResolutionEnumItems[i];
		MenuBuilder.AddMenuEntry(
			FText::FromString(Item),
			FText::FromString(Item),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STG_TextureDescriptor::HandleHeightChanged, Item, i),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, i]() {return SelectedHeightIndex == i; })
			));
	}

	return MenuBuilder.MakeWidget();
}

void STG_TextureDescriptor::HandleHeightChanged(FString Name, int Index)
{
	auto TextureDescriptor = GetTextureDescriptor();
	TextureDescriptor.Height = (EResolution)GetValueFromIndex(StaticEnum<EResolution>()->GetPathName(), Index);

	SetValue(TextureDescriptor);
	OnTextureDescriptorChanged.ExecuteIfBound(TextureDescriptor);

	SelectedHeightIndex = Index;
}

FText STG_TextureDescriptor::HandleHeightText() const
{
	return FText::FromString(GetEnumValueDisplayName(StaticEnum<EResolution>()->GetPathName(), (int)GetTextureDescriptor().Height));
}

TSharedRef<SWidget> STG_TextureDescriptor::OnGenerateFormatEnumMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	//Get list for Enum
	TArray<FString> TGTextureFormatEnumItems;
	GenerateStringsFromEnum(TGTextureFormatEnumItems, StaticEnum<ETG_TextureFormat>()->GetPathName());
	
	for (int i = 0; i < TGTextureFormatEnumItems.Num(); i++)
	{
		auto Item = TGTextureFormatEnumItems[i];
		MenuBuilder.AddMenuEntry(
			FText::FromString(Item),
			FText::FromString(Item),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STG_TextureDescriptor::HandleFormatChanged, Item, i),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, i]() {return SelectedFormatIndex == i; })
			));
	}

	return MenuBuilder.MakeWidget();
}

void STG_TextureDescriptor::HandleFormatChanged(FString Name, int Index)
{
	auto TextureDescriptor = GetTextureDescriptor();
	TextureDescriptor.TextureFormat = (ETG_TextureFormat)GetValueFromIndex(StaticEnum<ETG_TextureFormat>()->GetPathName(), Index);

	SetValue(TextureDescriptor);
	OnTextureDescriptorChanged.ExecuteIfBound(TextureDescriptor);

	SelectedFormatIndex = Index;
}

FText STG_TextureDescriptor::HandleFormatText() const
{
	return FText::FromString(GetEnumValueDisplayName(StaticEnum<ETG_TextureFormat>()->GetPathName(), (int)GetTextureDescriptor().TextureFormat));
}

void STG_TextureDescriptor::PostUndo(bool bSuccess)
{
	
}

void STG_TextureDescriptor::PostRedo(bool bSuccess)
{

}

#undef LOCTEXT_NAMESPACE
