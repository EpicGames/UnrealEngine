// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "TG_Texture.h"
#include "EditorUndoClient.h"
#include "EdGraph/EdGraphPin.h"

class IPropertyHandle;
class SHorizontalBox;
class SVerticalBox;

/**
 * Widget for editing TextureDescriptor.
 */
class STG_TextureDescriptor : public SCompoundWidget, public FEditorUndoClient
{	
private:
	DECLARE_DELEGATE_RetVal(FText, FGetTextDelegate);
	DECLARE_DELEGATE_TwoParams(FTextCommitted, const FText&, ETextCommit::Type);
	DECLARE_DELEGATE_RetVal(TSharedRef<SWidget>, FGenerateEnumMenu);

public:

	DECLARE_DELEGATE_OneParam(FOnTextureDescriptorChanged, const FTG_TextureDescriptor& /*TextureDescriptor*/)

	SLATE_BEGIN_ARGS(STG_TextureDescriptor)
		:
		_DescriptionMaxWidth(250.0f)
		,_ShowingAsPin(true)
		, _PropertyHandle(nullptr)
	{}

		/** Maximum with of the query description field. */
		SLATE_ARGUMENT(float, DescriptionMaxWidth)

		/** Showing as a pin */
		SLATE_ARGUMENT(bool, ShowingAsPin)

		/** If set, the TextureDescriptor is read from the property, and the property is update when TextureDescriptor is edited. */ 
		SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, PropertyHandle)

		SLATE_EVENT(FOnTextureDescriptorChanged, OnTextureDescriptorChanged)
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);
	TSharedRef<SWidget> AddEditBox(FText Label, FGetTextDelegate GetText, FTextCommitted OnTextCommitted);
	TSharedRef<SWidget> AddEnumComobox(FText Label, FGetTextDelegate GetText, FGenerateEnumMenu OnGenerateEnumMenu);
	FTG_TextureDescriptor GetTextureDescriptor() const;
	void SetValue(FTG_TextureDescriptor Settings);
	void GenerateStringsFromEnum(TArray<FString>& OutEnumNames, const FString& EnumPathName);
	int GetValueFromIndex(const FString& EnumPathName, int Index) const;
	FString GetEnumValueDisplayName(const FString& EnumPathName, int EnumValue) const;

protected:
	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FEditorUndoClient Interface

private:
	UEdGraphPin* GraphPinObj;
	
	FOnTextureDescriptorChanged OnTextureDescriptorChanged;
	FTG_TextureDescriptor CachedTextureDescriptor;
	FString SelectedWidthName;

	int SelectedWidthIndex;
	int SelectedHeightIndex;
	int SelectedFormatIndex;

	bool ShowingAsPin;

	FGenerateEnumMenu OnGenerateWidthMenu;
	FGetTextDelegate GetWidthDelegate;
	FGenerateEnumMenu OnGenerateHeightMenu;
	FGetTextDelegate GetHeightDelegate;
	FGenerateEnumMenu OnGenerateFormatMenu;
	FGetTextDelegate GetFormatDelegate;

	TSharedRef<SWidget> OnGenerateWidthEnumMenu();
	void HandleWidthChanged(FString OuputName, int Index);
	FText HandleWidthText() const;

	TSharedRef<SWidget> OnGenerateHeightEnumMenu();
	void HandleHeightChanged(FString OuputName, int Index);
	FText HandleHeightText() const;

	TSharedRef<SWidget> OnGenerateFormatEnumMenu();
	void HandleFormatChanged(FString OuputName, int Index);
	FText HandleFormatText() const;

	void AddVerticalSeperation(TSharedPtr<SVerticalBox> VBox, FMargin Padding);
};
