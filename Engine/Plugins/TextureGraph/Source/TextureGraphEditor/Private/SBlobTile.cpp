// Copyright Epic Games, Inc. All Rights Reserved.
#include "SBlobTile.h"

#include "TextureGraphEngine.h"
#include "Device/DeviceManager.h"
#include "Device/FX/DeviceBuffer_FX.h"

#include "Widgets/Layout/SScaleBox.h"
#include "Materials/Material.h"
#include <Materials/MaterialInstanceDynamic.h>
#include <Styling/SlateBrush.h>
#include <Widgets/SOverlay.h>
#include <Widgets/Images/SImage.h>
#include <Widgets/Text/STextBlock.h>
#include <Widgets/Layout/SBox.h>
#include <Widgets/SBoxPanel.h>
#include <Brushes/SlateDynamicImageBrush.h>

void SBlobTile::Construct(const FArguments& InArgs)
{
	bool withToolTip = InArgs._withToolTip;
	bool withDescription = InArgs._withDescription;
	SetPadding(InArgs._padding);
	SetBorderBackgroundColor(InArgs._borderColor);

	uint32_t SIMAGE_WIDTH = 512;
	uint32_t SIMAGE_HEIGHT = 512;

	ImageWidth = SIMAGE_WIDTH;
	ImageHeight = SIMAGE_HEIGHT;

	// Now let's go find the true live DeviceBuffer ptr
	BlobPtr Blob = InArgs._blob;
	auto Padding = InArgs._padding;
	bool bShowChecker = Blob == nullptr;
	// allocate the material brush where we can assign the texture (or a copy of ) async, or just reflect the status
	MakeWidgetMaterial(bShowChecker);

	// Build the basic widget eventually showing the checker board if no buffer found
	TSharedPtr<SOverlay> overlay;
	ChildSlot
		[
			SAssignNew(overlay, SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(Brush.Get())
			]
		];

	// Hopefully found a device buffer if not early exit
	if (!Blob)
		return;
	
	DeviceBufferPtr Buffer = Blob->GetBufferRef().GetPtr();

	// Hopefully found a device buffer if not early exit
	if (!Buffer)
		return;

	bool blobContentInBrush = false;

	ImageWidth = std::min(Blob->GetWidth(), SIMAGE_WIDTH);
	ImageHeight = std::min(Blob->GetHeight(), SIMAGE_HEIGHT);

	auto DynamicBrush = StaticCastSharedPtr<FSlateDynamicImageBrush>(Brush);
	DynamicBrush->SetImageSize(FVector2D(ImageWidth, ImageHeight));

	SetPercentageBorderSize(Padding);

	auto FXBuffer = std::static_pointer_cast<DeviceBuffer_FX>(Buffer);
	 // Got the description, now need to retreive the true live buffer and the texture to configure the brush
	 {
		// Hopefully found a device buffer
		if (FXBuffer)
		{
			// Let's create a texture from the raw data
			//auto tex = std::make_shared<Tex>();

			if (!FXBuffer->IsNull())
			{
				auto Tex = FXBuffer->GetTexture();
				check(Tex);
				Tex->SetFilter(TextureFilter::TF_Nearest);
				UTexture* BlobTexture = Tex->GetTexture();
				CreateWidgetBrush(BlobTexture);
			}
			else
			{
				UE_LOGF(LogTemp, Warning, "Buffer is not an FXBuffer");
			}
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "BlobTexture failed to find the buffer ?");
			// no buffer ?
		}
	}
}

void SBlobTile::CreateWidgetBrush(UTexture* blobTexture)
{
	if (blobTexture && Brush && BrushMaterial) {
		BrushMaterial->SetTextureParameterValue("BlobTex", blobTexture);
	}
}

void SBlobTile::MakeWidgetMaterial(bool bShowChecker) 
{
	Brush = MakeShareable(
		new FSlateDynamicImageBrush(
			(UTexture2D*) nullptr,
			FVector2D(ImageWidth, ImageHeight),
			//desiredSize,
			FName(ImageName),
			FLinearColor(1.0f, 1.0f, 1.0f, 1.0f),
			ESlateBrushTileType::NoTile,
			ESlateBrushImageType::Linear
		)
	);

	if (bShowChecker)
	{
		UMaterial* material = LoadObject<UMaterial>(nullptr, TEXT("/TextureGraph/UI/Materials/CheckerPattern"));
		BrushMaterial = (UMaterialInstanceDynamic::Create(material, nullptr));
	}
	else
	{
		UMaterial* material = LoadObject<UMaterial>(nullptr, TEXT("/TextureGraph/UI/Materials/RHSView"));
		BrushMaterial = (UMaterialInstanceDynamic::Create(material, nullptr));
	}
	Brush->SetResourceObject(BrushMaterial);
}

void SBlobTile::SetPercentageBorderSize(FMargin margin)
{
	const float Left = ImageWidth * (margin.Left / 100.0f);
	const float Top = ImageHeight * (margin.Top / 100.0f);
	float Margin = FMath::Min(Left, Top);
	SetPadding(FMargin(Margin));
}