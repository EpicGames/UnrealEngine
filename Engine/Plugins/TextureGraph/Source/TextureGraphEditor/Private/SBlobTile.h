// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//#include "SlateBasics.h"
#include "2D/Tex.h"
#include <UObject/GCObject.h>
#include <Widgets/Layout/SBorder.h>

class UMaterialInstanceDynamic;

class SBlobTile : public SBorder, public FGCObject /// Need FGCObject to control garbage collection of objects
{
public:
	SLATE_BEGIN_ARGS(SBlobTile) :
		_withToolTip(false),
		_withDescription(false),
		_padding(FMargin(0,0,0,0)),
		_borderColor(FLinearColor(1.0f, 0.0f, 0.0f, 1.0f)),
		_blob(nullptr) {}
		SLATE_ARGUMENT(bool, withToolTip)
		SLATE_ARGUMENT(bool, withDescription)
		SLATE_ARGUMENT(FMargin, padding)
		SLATE_ARGUMENT(FLinearColor, borderColor)
		SLATE_ARGUMENT(BlobPtr , blob)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args);

	void CreateWidgetBrush(UTexture* BlobTexture);
	void MakeWidgetMaterial(bool bShowChecker);

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(BrushMaterial);
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("SBlobTile");
	}

private:

	void SetPercentageBorderSize(FMargin margin);

	uint32 ImageWidth = 1;
	uint32 ImageHeight = 1;
	FString ImageName;

	// Needs our own brush to display the blob content
	TSharedPtr<FSlateBrush> Brush;
	TObjectPtr<UMaterialInstanceDynamic> BrushMaterial = nullptr;
};
