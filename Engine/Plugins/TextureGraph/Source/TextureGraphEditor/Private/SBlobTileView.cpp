// Copyright Epic Games, Inc. All Rights Reserved.
#include "SBlobTileView.h"
#include "SBlobTile.h"
#include "Device/DeviceManager.h"
#include "Device/FX/DeviceBuffer_FX.h"

#include "Widgets/Layout/SScaleBox.h"
#include "Materials/Material.h"
#include <Materials/MaterialInstanceDynamic.h>
#include <Styling/SlateBrush.h>
#include <Widgets/SOverlay.h>
#include <Widgets/Images/SImage.h>
#include <Widgets/Text/STextBlock.h>
#include <Widgets/Layout/SBox.h>
#include <Widgets/SBoxPanel.h>
#include <Brushes/SlateDynamicImageBrush.h>
#include <Brushes/SlateColorBrush.h>

void SBlobTileView::Construct(const FArguments& InArgs)
{
	BlobPtr Blob = InArgs._blob;

	_brush = MakeShareable(new FSlateColorBrush(FLinearColor(1, 1, 1, 1)));
	_brush->ImageSize = FVector2D(100.0f, 100.0f);
	_brush->DrawAs = ESlateBrushDrawType::Image;
	
	TSharedPtr<SUniformGridPanel> GridPanel;
	TSharedPtr<SOverlay> OverLaySlot;
	
	ChildSlot
	[
			SAssignNew(OverLaySlot, SOverlay)
			+ SOverlay::Slot()
		[
			SAssignNew(GridPanel, SUniformGridPanel)
		]
	];

	if (Blob == nullptr)
	{

		GridPanel->AddSlot(0, 0)
		[
				SNew(SBlobTile)
		];
		return;
	}

	if (Blob->IsTiled())
	{
		if (!InArgs._finalized)
		{
			Blob->OnFinalise().then([this, InArgs, Blob, GridPanel]()
			{
				// OnFinalise can sometimes occur after the editor is closed and thus can potentially
				// deallocate all corresponding slate objects
				if (DoesSharedInstanceExist())
				{		
					InArgs._OnFinalizeBlob.ExecuteIfBound();
					CreateTiles(Blob, GridPanel, InArgs._padding);
				}
			});
		}
		else
		{
			CreateTiles(Blob, GridPanel, InArgs._padding);
		}
		
	}
	else
	{
		GridPanel->AddSlot(0, 0)
		[
			SNew(SBlobTile)
			.blob(Blob)
			.padding(FMargin(0))
		];
	}
	
}

void SBlobTileView::CreateTiles(BlobPtr Blob, TSharedPtr<SUniformGridPanel> GridPanel,FMargin Padding)
{
	TiledBlobPtr tiledBlob = std::static_pointer_cast<TiledBlob>(Blob);

	for (int i = 0; i < tiledBlob->Rows(); i++)
	{
		for (int j = 0; j < tiledBlob->Cols(); j++)
		{
			GridPanel->AddSlot(i, j)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(_brush.Get())
				]

				+ SOverlay::Slot()
				[
					SNew(SBlobTile)
					.borderColor(FLinearColor(0, 0, 0, 0.0))
					.padding(Padding)
					.blob(tiledBlob->GetTile(i, j))
				]
			];
		}
	}
}