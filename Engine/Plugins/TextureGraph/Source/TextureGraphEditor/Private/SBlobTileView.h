// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//#include "SlateBasics.h"
#include "2D/Tex.h"
#include "Data/Blobber.h"
#include <Widgets/Layout/SBorder.h>
#include <Widgets/Layout/SUniformGridPanel.h>
#include <UObject/GCObject.h>

class UMaterialInstanceDynamic;

DECLARE_DELEGATE(FOnFinalizeBlobEvent);

class SBlobTileView : public SBorder, public FGCObject /// Need FGCObject to control garbage collection of objects
{
public:
	SLATE_BEGIN_ARGS(SBlobTileView) :
		_withToolTip(false),
		_withDescription(false),
		_finalized(true),
		_padding(FMargin(0, 0, 0, 0)),
		_blob(nullptr),
		_OnFinalizeBlob() {}
		SLATE_ARGUMENT(bool, withToolTip)
		SLATE_ARGUMENT(bool, withDescription)
		SLATE_ARGUMENT(bool, finalized)
		SLATE_ARGUMENT(FMargin, padding)
		SLATE_ARGUMENT(BlobPtr , blob)
		SLATE_EVENT(FOnFinalizeBlobEvent, OnFinalizeBlob)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args);

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(_brushMaterial);
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("SBlobTileView");
	}

private:
	void CreateTiles(BlobPtr Blob, TSharedPtr<SUniformGridPanel> GridPanel, FMargin Padding);
	// Needs our own brush to display the blob content
	TSharedPtr<FSlateBrush> _brush;
	TObjectPtr<UMaterialInstanceDynamic> _brushMaterial = nullptr;
};
