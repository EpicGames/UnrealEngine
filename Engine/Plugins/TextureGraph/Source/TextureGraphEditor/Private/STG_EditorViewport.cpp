// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_EditorViewport.h"
#include "AdvancedPreviewScene.h"
#include "AdvancedPreviewSceneMenus.h"
#include "AssetViewerSettings.h"
#include "Brushes/SlateBorderBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "ComponentAssetBroker.h"
#include "Components/MeshComponent.h"
#include "Dialogs/Dialogs.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/PostProcessVolume.h"
#include "Engine/Selection.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/TextureCube.h"
#include "SceneInterface.h"
#include "Framework/Layout/ScrollyZoomy.h"
#include "ISettingsModule.h"
#include "ITG_Editor.h"
#include "ImageUtils.h"
#include "PreviewProfileController.h"
#include "MaterialEditor/MaterialEditorMeshComponent.h"
#include "Model/Mix/MixInterface.h"
#include "Modules/ModuleManager.h"
#include "STG_EditorViewportToolBar.h"
#include "Slate/SceneViewport.h"
#include "SlateMaterialBrush.h"
#include "TG_Editor.h"
#include "TG_EditorCommands.h"
#include "UnrealEdGlobals.h"
#include "Model/Mix/MixSettings.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SViewport.h"
#include "Widgets/Text/STextBlock.h"
#include "Materials/Material.h"
#include "Model/Mix/ViewportSettings.h"
#include "TextureGraph.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"


class FTG_EditorViewportClient : public FEditorViewportClient
{
public:
	FTG_EditorViewportClient(TObjectPtr<UTextureGraphBase> InTextureGraphPtr, FAdvancedPreviewScene& InPreviewScene, const TSharedRef<STG_EditorViewport>& InTG_AssetEditorViewport);

	// FEditorViewportClient interface
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	virtual bool InputAxis(const FInputKeyEventArgs& EventArgs) override;

	virtual FLinearColor GetBackgroundColor() const override;

	virtual void Tick(float DeltaSeconds) override;
	virtual void Draw(FViewport* Viewport,FCanvas* Canvas) override;
	virtual bool ShouldOrbitCamera() const override;

	/**
	* Focuses the viewport to the center of the bounding box/sphere ensuring that the entire bounds are in view
	*
	* @param Bounds   The bounds to focus
	* @param bInstant Whether or not to focus the viewport instantly or over time
	*/
	void FocusViewportOnBounds(const FBoxSphereBounds Bounds, bool bInstant = false);

private:

	/** Pointer to the Edited Texture Graph */
	TObjectPtr<UTextureGraphBase> TextureGraphPtr;

	/** Preview Scene - uses advanced preview settings */
	class FAdvancedPreviewScene* AdvancedPreviewScene;
};


FTG_EditorViewportClient::FTG_EditorViewportClient(TObjectPtr<UTextureGraphBase> InTextureGraphPtr, FAdvancedPreviewScene& InPreviewScene, const TSharedRef<STG_EditorViewport>& InTG_EditorViewport)
	: FEditorViewportClient(nullptr, &InPreviewScene, StaticCastSharedRef<SEditorViewport>(InTG_EditorViewport))
	, TextureGraphPtr(InTextureGraphPtr)
{
	bUsesDrawHelper = true;
	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = true;
	DrawHelper.GridColorAxis = FColor(80, 80, 80);
	DrawHelper.GridColorMajor = FColor(72, 72, 72);
	DrawHelper.GridColorMinor = FColor(64, 64, 64);
	DrawHelper.PerspectiveGridSize = static_cast<float>(HALF_WORLD_MAX1);

	//SetViewMode(VMI_Lit);

	EngineShowFlags.DisableAdvancedFeatures();
	EngineShowFlags.SetSnap(0);
	EngineShowFlags.SetSeparateTranslucency(true);

	//OverrideNearClipPlane(1.0f);
	bUsingOrbitCamera = true;

	// Don't want to display the widget in this viewport
	//Widget->SetDefaultVisibility(false);

	AdvancedPreviewScene = &InPreviewScene;

}



void FTG_EditorViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);

	// Tick the preview scene world.
	if (!GIntraFrameDebuggingGameThread)
	{
		PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
	}
}


void FTG_EditorViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	FEditorViewportClient::Draw(InViewport, Canvas);
}

bool FTG_EditorViewportClient::ShouldOrbitCamera() const
{
	// Should always orbit around the preview object to keep it in view.
	return true;
}

bool FTG_EditorViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	bool bHandled = FEditorViewportClient::InputKey(EventArgs);

	// Handle viewport screenshot.
	bHandled |= InputTakeScreenshot(EventArgs.Viewport, EventArgs.Key, EventArgs.Event);

	bHandled |= AdvancedPreviewScene->HandleInputKey(EventArgs);

	return bHandled;
}

bool FTG_EditorViewportClient::InputAxis(const FInputKeyEventArgs& EventArgs)
{
	bool bResult = true;

	if (!bDisableInput)
	{
		bResult = AdvancedPreviewScene->HandleViewportInput(EventArgs.Viewport, EventArgs.InputDevice, EventArgs.Key, EventArgs.AmountDepressed, EventArgs.DeltaTime, EventArgs.NumSamples, EventArgs.IsGamepad());
		if (bResult)
		{
			Invalidate();
		}
		else
		{
			bResult = FEditorViewportClient::InputAxis(EventArgs);
		}
	}

	return bResult;
}

FLinearColor FTG_EditorViewportClient::GetBackgroundColor() const
{
	if (AdvancedPreviewScene != nullptr)
	{
		return AdvancedPreviewScene->GetBackgroundColor();
	}
	else
	{
		FLinearColor BackgroundColor = FColor(64, 64, 64);			
		
		return BackgroundColor;
	}
}

void FTG_EditorViewportClient::FocusViewportOnBounds(const FBoxSphereBounds Bounds, bool bInstant /*= false*/)
{
	const FVector Position = Bounds.Origin;
	float Radius = Bounds.SphereRadius;

	float AspectToUse = AspectRatio;
	FIntPoint ViewportSize = Viewport->GetSizeXY();
	if (!bUseControllingActorViewInfo && ViewportSize.X > 0 && ViewportSize.Y > 0)
	{
		AspectToUse = Viewport->GetDesiredAspectRatio();
	}

	const bool bEnable = false;
	ToggleOrbitCamera(bEnable);

	/**
	* We need to make sure we are fitting the sphere into the viewport completely, so if the height of the viewport is less
	* than the width of the viewport, we scale the radius by the aspect ratio in order to compensate for the fact that we have
	* less visible vertically than horizontally.
	*/
	if (AspectToUse > 1.0f)
	{
		Radius *= AspectToUse;
	}

	/**
	* Now that we have a adjusted radius, we are taking half of the viewport's FOV,
	* converting it to radians, and then figuring out the camera's distance from the center
	* of the bounding sphere using some simple trig.  Once we have the distance, we back up
	* along the camera's forward vector from the center of the sphere, and set our new view location.
	*/
	const float HalfFOVRadians = FMath::DegreesToRadians(ViewFOV / 2.0f);
	const float DistanceFromSphere = Radius / FMath::Sin(HalfFOVRadians);
	FViewportCameraTransform& ViewTransform = GetViewTransform();
	FVector CameraOffsetVector = ViewTransform.GetRotation().Vector() * -DistanceFromSphere;

	ViewTransform.SetLookAt(Position);
	ViewTransform.TransitionToLocation(Position + CameraOffsetVector, EditorViewportWidget, bInstant);

	// Tell the viewport to redraw itself.
	Invalidate();
}




void STG_EditorViewport::Construct(const FArguments& InArgs)
{
	TextureGraphPtr = InArgs._InTextureGraph;

	PreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues()));

	// restore last used feature level
	UWorld* World = PreviewScene->GetWorld();
	if (World != nullptr && GWorld != nullptr && GWorld->Scene != nullptr)
	{
		World->ShaderPlatformChanged(GWorld->Scene->GetShaderPlatform());
	}

	UEditorEngine* Editor = CastChecked<UEditorEngine>(GEngine);
	PreviewFeatureLevelChangedHandle = Editor->OnPreviewShaderPlatformChanged().AddLambda([this](EShaderPlatform NewShaderPlatform)
		{
			PreviewScene->GetWorld()->ShaderPlatformChanged(NewShaderPlatform);
		});

	CurrentViewMode = VMI_Lit;

	FTG_EditorCommands::Register();

	SEditorViewport::Construct(SEditorViewport::FArguments());

	PreviewMeshComponent = nullptr;

	UAssetViewerSettings* Settings = UAssetViewerSettings::Get();
	const int32 ProfileIndex = PreviewScene->GetCurrentProfileIndex();
	if (Settings->Profiles.IsValidIndex(ProfileIndex))
	{
		PreviewScene->SetEnvironmentVisibility(Settings->Profiles[ProfileIndex].bShowEnvironment, true);
	}

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &STG_EditorViewport::OnObjectPropertyChanged);
	
	UE::AdvancedPreviewScene::BindDefaultOnSettingsChangedHandler(PreviewScene, EditorViewportClient);
	
	GenerateRendermodeToolbar();
}

void STG_EditorViewport::SetTextureGraph(const TObjectPtr<UTextureGraphBase>& InTextureGraph)
{
	TextureGraphPtr = InTextureGraph;
	GenerateRendermodeToolbar();
}

STG_EditorViewport::STG_EditorViewport()
	: PreviewScene(MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues())))
{

}

STG_EditorViewport::~STG_EditorViewport()
{
	if (PreviewFeatureLevelChangedHandle.IsValid())
	{
		if (UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine))
		{
			EditorEngine->OnPreviewShaderPlatformChanged().Remove(PreviewFeatureLevelChangedHandle);
		}
	}
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

void STG_EditorViewport::OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!ensure(ObjectBeingModified))
	{
		return;
	}
}

void STG_EditorViewport::RefreshViewport()
{
	// Invalidate the viewport's display.
	SceneViewport->Invalidate();
}

FEditorViewportClient& STG_EditorViewport::GetViewportClient()
{
	return *EditorViewportClient;
}

TSharedRef<SEditorViewport> STG_EditorViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedRef<FEditorViewportClient> STG_EditorViewport::MakeEditorViewportClient()
{
	EditorViewportClient = MakeShareable(new FTG_EditorViewportClient(TextureGraphPtr, *PreviewScene.Get(), SharedThis(this)));
	EditorViewportClient->SetViewLocation(FVector::ZeroVector);
	EditorViewportClient->SetViewRotation(FRotator(-25.0f, -135.0f, 0.0f));
	EditorViewportClient->SetViewLocationForOrbiting(FVector::ZeroVector, 500);
	EditorViewportClient->bSetListenerPosition = false;
	EditorViewportClient->EngineShowFlags.EnableAdvancedFeatures();
	EditorViewportClient->EngineShowFlags.SetGrid(false);
	EditorViewportClient->EngineShowFlags.SetLighting(true);
	EditorViewportClient->EngineShowFlags.SetIndirectLightingCache(true);
	EditorViewportClient->EngineShowFlags.SetPostProcessing(true);
	EditorViewportClient->Invalidate();
	//_editorViewportClient->VisibilityDelegate.BindSP(this, &STG_EditorViewport::IsVisible);

	return EditorViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> STG_EditorViewport::BuildViewportToolbar()
{
	const FName ToolbarName = FName("TextureGraph.ViewportToolbar");
	
	if (!UToolMenus::Get()->IsMenuRegistered(ToolbarName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(ToolbarName);
		Menu->MenuType = EMultiBoxType::SlimHorizontalToolBar;
		Menu->StyleName = "ViewportToolbar";
		
		Menu->AddSection("Left");
		
		FToolMenuSection& RightSection = Menu->AddSection("Right");
		RightSection.Alignment = EToolMenuSectionAlign::Last;
		
		RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(UE::UnrealEd::FViewportCameraMenuOptions().ShowAll()));
		RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
		
		RightSection.AddEntry(UE::UnrealEd::CreateShowSubmenu(
			FNewToolMenuDelegate::CreateLambda([](UToolMenu* Submenu)
			{
				auto Commands = FTG_EditorCommands::Get();
				FToolMenuSection& Section = Submenu->AddSection(NAME_None);
				Section.AddMenuEntry(Commands.TogglePreviewGrid);
				Section.AddMenuEntry(Commands.TogglePreviewBackground);
			})
		));
		
		RightSection.AddEntry(UE::UnrealEd::CreatePerformanceAndScalabilitySubmenu());
		RightSection.AddEntry(UE::UnrealEd::CreateAssetViewerProfileSubmenu());
	}
	
	FToolMenuContext Context;
	{
		Context.AppendCommandList(GetCommandList());
		UUnrealEdViewportToolbarContext* ContextObject = UE::UnrealEd::CreateViewportToolbarDefaultContext(SharedThis(this));
		ContextObject->IsViewModeSupported.BindLambda([](EViewModeIndex ViewModeIndex)
		{
			switch (ViewModeIndex)
			{
			case VMI_PrimitiveDistanceAccuracy:
			case VMI_MeshUVDensityAccuracy:
			case VMI_RequiredTextureResolution:
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			case VMI_Lit_Wireframe:
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
				return false;
			default:
				return true;
			}
		});
		Context.AddObject(ContextObject);
	}
	
	return UToolMenus::Get()->GenerateWidget(ToolbarName, Context);
}

TSharedPtr<IPreviewProfileController> STG_EditorViewport::CreatePreviewProfileController()
{
	return MakeShared<FPreviewProfileController>();
}

void STG_EditorViewport::PopulateViewportOverlays(TSharedRef<class SOverlay> Overlay)
{
	Overlay->AddSlot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Right)
		.Padding(2.0f)
		[
			SAssignNew(RenderModeToolBar,STG_EditorViewportRenderModeToolBar, SharedThis(this))
		];

	Overlay->AddSlot()
		.VAlign(VAlign_Bottom)
		[
			SNew(STG_EditorViewportPreviewShapeToolBar, SharedThis(this))
		];

	// add the feature level display widget
	Overlay->AddSlot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Right)
		.Padding(5.0f)
		[
			BuildShaderPlatformWidget()
		];
}

void STG_EditorViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	const FTG_EditorCommands& Commands = FTG_EditorCommands::Get();

	// Add the commands to the toolkit command list so that the toolbar buttons can find them
	CommandList->MapAction(
		Commands.SetCylinderPreview,
		FExecuteAction::CreateSP(this, &STG_EditorViewport::OnSetPreviewPrimitive, TPT_Cylinder, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &STG_EditorViewport::IsPreviewPrimitiveChecked, TPT_Cylinder));

	CommandList->MapAction(
		Commands.SetSpherePreview,
		FExecuteAction::CreateSP(this, &STG_EditorViewport::OnSetPreviewPrimitive, TPT_Sphere, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &STG_EditorViewport::IsPreviewPrimitiveChecked, TPT_Sphere));

	CommandList->MapAction(
		Commands.SetPlanePreview,
		FExecuteAction::CreateSP(this, &STG_EditorViewport::OnSetPreviewPrimitive, TPT_Plane, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &STG_EditorViewport::IsPreviewPrimitiveChecked, TPT_Plane));

	CommandList->MapAction(
		Commands.SetCubePreview,
		FExecuteAction::CreateSP(this, &STG_EditorViewport::OnSetPreviewPrimitive, TPT_Cube, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &STG_EditorViewport::IsPreviewPrimitiveChecked, TPT_Cube));

	CommandList->MapAction(
		Commands.SetPreviewMeshFromSelection,
		FExecuteAction::CreateSP(this, &STG_EditorViewport::OnSetPreviewMeshFromSelection),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &STG_EditorViewport::IsPreviewMeshFromSelectionChecked));

	CommandList->MapAction(
		Commands.TogglePreviewGrid,
		FExecuteAction::CreateSP(this, &STG_EditorViewport::TogglePreviewGrid),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &STG_EditorViewport::IsTogglePreviewGridChecked));

	CommandList->MapAction(
		Commands.TogglePreviewBackground,
		FExecuteAction::CreateSP(this, &STG_EditorViewport::TogglePreviewBackground),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &STG_EditorViewport::IsTogglePreviewBackgroundChecked));
}

bool STG_EditorViewport::IsRenderModeEnabled(FName InRenderModeName) const
{
	return RenderModeName == InRenderModeName;
}

void STG_EditorViewport::SetRenderMode(FName InRenderModeName)
{
	check(RenderModeMgr);
	
	RenderModeName = InRenderModeName;
	RenderModeMgr->ChangeRenderMode(RenderModeName, TextureGraphPtr);
	
	// set editor viewport view mode
#if WITH_EDITOR
	const bool bIsLit =  RenderModeMgr->IsCurrentRenderModelLit();

	if (bIsLit)
	{
		Client->SetViewMode(EViewModeIndex::VMI_Lit);
	}
	else
	{
		Client->SetViewMode(EViewModeIndex::VMI_Unlit);
	}
#endif
}

void STG_EditorViewport::InitRenderModes(UTextureGraphBase* InTextureGraph)
{
	if (!RenderModeMgr)
		RenderModeMgr = MakeShared<TG_RenderModeManager>();
	
	TextureGraphPtr = InTextureGraph;
	RenderModeMgr->Clear();
	UpdateRenderMode();
}

void STG_EditorViewport::UpdateRenderMode()
{
	SetRenderMode(GetRenderModeFName());
}

void STG_EditorViewport::InitPreviewMesh()
{
	UStaticMesh* Primitive = GUnrealEd->GetThumbnailManager()->EditorCube;
	SetPreviewAsset(Primitive);
			
}

bool STG_EditorViewport::IsPreviewPrimitiveChecked(EThumbnailPrimType PrimType) const
{
	return PreviewPrimType == PrimType;
}

void STG_EditorViewport::OnSetPreviewPrimitive(EThumbnailPrimType PrimType, bool bInitialLoad)
{
	if (SceneViewport.IsValid())
	{
		UStaticMesh* Primitive = nullptr;
		switch (PrimType)
		{
		case TPT_Cylinder: Primitive = GUnrealEd->GetThumbnailManager()->EditorCylinder; break;
		case TPT_Sphere: Primitive = GUnrealEd->GetThumbnailManager()->EditorSphere; break;
		case TPT_Plane: Primitive = GUnrealEd->GetThumbnailManager()->EditorPlane; break;
		case TPT_Cube: Primitive = GUnrealEd->GetThumbnailManager()->EditorCube; break;
		}

		if (Primitive != nullptr)
		{
			SetPreviewAsset(Primitive);
			
			RefreshViewport();
		}
	}
}

void STG_EditorViewport::OnSetPreviewMeshFromSelection()
{
	bool bFoundPreviewMesh = false;
	FEditorDelegates::LoadSelectedAssetsIfNeeded.Broadcast();

	UMixSettings* TextureGraphSettings = TextureGraphPtr->GetSettings();

	// Look for a selected asset that can be converted to a mesh component
	for (FSelectionIterator SelectionIt(*GEditor->GetSelectedObjects()); SelectionIt && !bFoundPreviewMesh; ++SelectionIt)
	{
		UObject* TestAsset = *SelectionIt;
		if (TestAsset->IsAsset())
		{
			if (TSubclassOf<UActorComponent> ComponentClass = FComponentAssetBrokerage::GetPrimaryComponentForAsset(TestAsset->GetClass()))
			{
				if (ComponentClass->IsChildOf(UMeshComponent::StaticClass()))
				{
					SetPreviewAsset(TestAsset);
					bFoundPreviewMesh = true;
				}
			}
		}
	}

	if (bFoundPreviewMesh)
	{
		RefreshViewport();
	}
	else
	{
		FSuppressableWarningDialog::FSetupInfo Info(NSLOCTEXT("UnrealEd", "Warning_NoPreviewMeshFound_Message", "You need to select a mesh-based asset in the content browser to preview it."),
			NSLOCTEXT("UnrealEd", "Warning_NoPreviewMeshFound", "Warning: No Preview Mesh Found"), "Warning_NoPreviewMeshFound");
		Info.ConfirmText = NSLOCTEXT("UnrealEd", "Warning_NoPreviewMeshFound_Confirm", "Continue");

		FSuppressableWarningDialog NoPreviewMeshWarning(Info);
		NoPreviewMeshWarning.ShowModal();
	}
}

bool STG_EditorViewport::IsPreviewMeshFromSelectionChecked() const
{
	return (PreviewPrimType == TPT_None && PreviewMeshComponent != nullptr);
}

void STG_EditorViewport::TogglePreviewGrid()
{
	EditorViewportClient->SetShowGrid();
	RefreshViewport();
}

bool STG_EditorViewport::IsTogglePreviewGridChecked() const
{
	return EditorViewportClient->IsSetShowGridChecked();
}

void STG_EditorViewport::TogglePreviewBackground()
{
	UAssetViewerSettings* Settings = UAssetViewerSettings::Get();
	const int32 ProfileIndex = PreviewScene->GetCurrentProfileIndex();
	if (Settings->Profiles.IsValidIndex(ProfileIndex))
	{
		PreviewScene->SetEnvironmentVisibility(!Settings->Profiles[ProfileIndex].bShowEnvironment);
	}
	RefreshViewport();
}

bool STG_EditorViewport::IsTogglePreviewBackgroundChecked() const
{
	UAssetViewerSettings* Settings = UAssetViewerSettings::Get();
	const int32 ProfileIndex = PreviewScene->GetCurrentProfileIndex();
	if (Settings->Profiles.IsValidIndex(ProfileIndex))
	{
		return Settings->Profiles[ProfileIndex].bShowEnvironment;
	}
	return false;
}

void STG_EditorViewport::GenerateRendermodeToolbar()
{
	GenerateRenderModesList();
	
	RenderModeToolBar->Init();
}

TSharedPtr<FExtender> STG_EditorViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void STG_EditorViewport::OnFloatingButtonClicked()
{
}

void STG_EditorViewport::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(PreviewMeshComponent);
}

bool STG_EditorViewport::SetPreviewAssetByName(const TCHAR* InAssetName)
{
	bool bSuccess = false;
	if ((InAssetName != nullptr) && (*InAssetName != 0))
	{
		if (UObject* Asset = LoadObject<UObject>(nullptr, InAssetName))
		{
			bSuccess = SetPreviewAsset(Asset);
		}
	}
	return bSuccess;
}

bool STG_EditorViewport::SetPreviewAsset(UObject* InAsset)
{
	// Unregister the current component
	if (PreviewMeshComponent != nullptr)
	{
		PreviewScene->RemoveComponent(PreviewMeshComponent);
		PreviewMeshComponent = nullptr;
	}

	FTransform Transform = FTransform::Identity;

	if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(InAsset))
	{
		// Special case handling for static meshes, to use more accurate bounds via a subclass
		UStaticMeshComponent* NewSMComponent = NewObject<UMaterialEditorMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
		NewSMComponent->SetStaticMesh(StaticMesh);

		PreviewMeshComponent = NewSMComponent;

		// Update the toolbar state implicitly through _previewPrimType.
		if (StaticMesh == GUnrealEd->GetThumbnailManager()->EditorCylinder)
		{
			PreviewPrimType = TPT_Cylinder;
		}
		else if (StaticMesh == GUnrealEd->GetThumbnailManager()->EditorCube)
		{
			PreviewPrimType = TPT_Cube;
		}
		else if (StaticMesh == GUnrealEd->GetThumbnailManager()->EditorSphere)
		{
			PreviewPrimType = TPT_Sphere;
		}
		else if (StaticMesh == GUnrealEd->GetThumbnailManager()->EditorPlane)
		{
			PreviewPrimType = TPT_Plane;
		}
		else
		{
			PreviewPrimType = TPT_None;
		}

		// Update the rotation of the plane mesh so that it is front facing to the viewport camera's default forward view.
		if (PreviewPrimType == TPT_Plane)
		{
			const FRotator PlaneRotation(-90.0f, 180.0f, 0.0f);
			Transform.SetRotation(FQuat(PlaneRotation));
		}
		if (TextureGraphPtr)
		{
			TextureGraphPtr->GetSettings()->SetPreviewMesh(StaticMesh);
		}
	}
	else if (InAsset != nullptr)
	{
		// Fall back to the component asset broker
		if (TSubclassOf<UActorComponent> ComponentClass = FComponentAssetBrokerage::GetPrimaryComponentForAsset(InAsset->GetClass()))
		{
			if (ComponentClass->IsChildOf(UMeshComponent::StaticClass()))
			{
				PreviewMeshComponent = NewObject<UMeshComponent>(GetTransientPackage(), ComponentClass, NAME_None, RF_Transient);
				FComponentAssetBrokerage::AssignAssetToComponent(PreviewMeshComponent, InAsset);
				PreviewPrimType = TPT_None;
			}
		}
	}

	// Add the new component to the scene
	if (PreviewMeshComponent != nullptr)
	{
		if (GEditor->PreviewPlatform.GetEffectivePreviewFeatureLevel() <= ERHIFeatureLevel::ES3_1)
		{
			PreviewMeshComponent->SetMobility(EComponentMobility::Static);
		}
		PreviewScene->AddComponent(PreviewMeshComponent, Transform);
		PreviewScene->SetFloorOffset(-PreviewMeshComponent->Bounds.Origin.Z + PreviewMeshComponent->Bounds.BoxExtent.Z);

	}
	if (IsValid(TextureGraphPtr))
	{
		try
		{
			TextureGraphPtr->SetEditorMesh(Cast<UStaticMeshComponent>(PreviewMeshComponent), PreviewScene->GetWorld())
				.then([this]()
					{
						this->InitRenderModes(TextureGraphPtr);
					})
				.fail([this]()
					{
						int a = 10;
					});
		}
		catch (...)
		{
			int a = 10;
		}
	}
	return (PreviewMeshComponent != nullptr);
}

void STG_EditorViewport::GenerateRenderModesList()
{
	if (IsValid(TextureGraphPtr))
	{
		UMixSettings* Settings = TextureGraphPtr->GetSettings();
	
		if(ensure(Settings->GetViewportSettings().Material))
		{
			const FName MaterialName = Settings->GetViewportSettings().GetMaterialName();

			CurrentMaterialName = MaterialName;
			RenderModeName = MaterialName;

			RenderModesList.Empty();
			RenderModesList.Add(MaterialName);

			for(FMaterialMappingInfo MaterialMappingInfo : Settings->GetViewportSettings().MaterialMappingInfos)
			{
				RenderModesList.Add(MaterialMappingInfo.MaterialInput);
			}
		}
	}
}

FReply STG_EditorViewport::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// assign dropped asset as preview mesh
	OnSetPreviewMeshFromSelection();
	return SAssetEditorViewport::OnDrop(MyGeometry, DragDropEvent);
}
