// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "SAssetEditorViewport.h"
#include "Styling/StyleColors.h"

#include "AdvancedPreviewScene.h"
#include "EditorViewportClient.h"
#include "TG_RenderModeManager.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "UObject/GCObject.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UTextureGraphBase;
class SVerticalBox;
class SRichTextBlock;
class ITG_Editor;
class UTextureGraph;
class UMeshComponent;

#define LOCTEXT_NAMESPACE "TG_Editor"

class STG_EditorViewport : public SAssetEditorViewport, public FGCObject, public ICommonEditorViewportToolbarInfoProvider
{
public:
	SLATE_BEGIN_ARGS(STG_EditorViewport)
	{}
	SLATE_ARGUMENT(UTextureGraphBase*, InTextureGraph)
		SLATE_END_ARGS()


private:

	FName									CurrentMaterialName;
	FName									RenderModeName;	///Current Selected render mode name
	TArray<FName>							RenderModesList;
	TSharedPtr<class FEditorViewportClient> EditorViewportClient;	/// Editor viewport client
	TObjectPtr<UTextureGraphBase>		TextureGraphPtr;		/// Pointer to the Texture Graph being shown
	
	EViewModeIndex							CurrentViewMode;		/// The currently selected view mode
	TSharedPtr<FAdvancedPreviewScene>		PreviewScene;			/// The scene for this viewport
	TObjectPtr<UMeshComponent>				PreviewMeshComponent;	/// Component for the preview mesh
	EThumbnailPrimType						PreviewPrimType;		/// The preview primitive we are using

	FDelegateHandle							PreviewFeatureLevelChangedHandle;	/// Handle to the registered OnPreviewFeatureLevelChanged delegate
	TSharedPtr<class STG_EditorViewportRenderModeToolBar>
											RenderModeToolBar;

	TSharedPtr<TG_RenderModeManager>		RenderModeMgr;
	
protected:
	void									GenerateRenderModesList();
	
	virtual FReply							OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
public:
	void									Construct(const FArguments& InArgs);
	void									SetTextureGraph(const TObjectPtr<UTextureGraphBase>& InTextureGraph);

	STG_EditorViewport();
	virtual									~STG_EditorViewport() override;

	void									RefreshViewport();
	bool									SetPreviewAsset(UObject* InAsset);
	void									OnSetPreviewPrimitive(EThumbnailPrimType PrimType, bool bInitialLoad = false);
	bool									IsPreviewPrimitiveChecked(EThumbnailPrimType PrimType) const;
	void									OnSetPreviewMeshFromSelection();
	bool									IsPreviewMeshFromSelectionChecked() const;
	void									TogglePreviewGrid();
	bool									IsTogglePreviewGridChecked() const;
	void									TogglePreviewBackground();
	bool									IsTogglePreviewBackgroundChecked() const;

	void									GenerateRendermodeToolbar();
	TArray<FName>							GetRenderModesList() const { return RenderModesList;};
	bool									IsRenderModeEnabled(FName RenderModeTypeArg) const;
	void									SetRenderMode(FName RenderModeTypeArg);
	void									InitRenderModes(UTextureGraphBase* InTextureGraph);
	void									UpdateRenderMode();
	FText									GetRenderModeName() { return FText::FromName(RenderModeName); }
	FName									GetRenderModeFName() { return RenderModeName; }

	void									InitPreviewMesh();

	/** @return The editor viewport client */
	class FEditorViewportClient&			GetViewportClient();
	void									OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);

	/** Returns the preview scene being rendered in the viewport */
	TSharedRef<FAdvancedPreviewScene>		GetPreviewScene() { return PreviewScene.ToSharedRef(); }

	// ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender>			GetExtenders() const override;
	virtual void							OnFloatingButtonClicked() override;
	// End of ICommonEditorViewportToolbarInfoProvider interface

	// FGCObject interface
	virtual void							AddReferencedObjects(FReferenceCollector& Collector) override;
	bool									SetPreviewAssetByName(const TCHAR* InAssetName);

	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget>				BuildViewportToolbar() override;
	virtual TSharedPtr<IPreviewProfileController> CreatePreviewProfileController() override;
	virtual void							PopulateViewportOverlays(TSharedRef<class SOverlay> Overlay) override;
	virtual void							BindCommands() override;



	FORCEINLINE virtual FString				GetReferencerName() const override { return TEXT("STG_EditorViewport"); } /// Overriding ReferencerName as it should be unique name
};

#undef LOCTEXT_NAMESPACE
