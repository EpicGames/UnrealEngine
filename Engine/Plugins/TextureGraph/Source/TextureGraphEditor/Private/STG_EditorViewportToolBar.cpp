// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_EditorViewportToolBar.h"

#include "ITG_Editor.h"
#include "Widgets/Layout/SBorder.h"
#include "PreviewProfileController.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "SEditorViewportToolBarMenu.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "TG_EditorCommands.h"
#include "STG_EditorViewport.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"


#define LOCTEXT_NAMESPACE "TG_EditorViewportToolBar"

///////////////////////////////////////////////////////////
// STG_EditorViewportPreviewShapeToolBar

void STG_EditorViewportPreviewShapeToolBar::Construct(const FArguments& InArgs, TSharedPtr<class STG_EditorViewport> InViewport)
{
	// Force this toolbar to have small icons, as the preview panel is only small so we have limited space
	const bool bForceSmallIcons = true;
	FToolBarBuilder ToolbarBuilder(InViewport->GetCommandList(), FMultiBoxCustomization::None, nullptr, bForceSmallIcons);

	// Use a custom style
	ToolbarBuilder.SetStyle(&FAppStyle::Get(), "LegacyViewportMenu");
	ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);
	ToolbarBuilder.SetIsFocusable(false);
	
	ToolbarBuilder.BeginSection("Viewport");
	{
		ToolbarBuilder.AddToolBarButton(FTG_EditorCommands::Get().SetCylinderPreview);
		ToolbarBuilder.AddToolBarButton(FTG_EditorCommands::Get().SetSpherePreview);
		ToolbarBuilder.AddToolBarButton(FTG_EditorCommands::Get().SetPlanePreview);
		ToolbarBuilder.AddToolBarButton(FTG_EditorCommands::Get().SetCubePreview);
		ToolbarBuilder.AddToolBarButton(FTG_EditorCommands::Get().SetPreviewMeshFromSelection);
	}
	ToolbarBuilder.EndSection();

	static const FName DefaultForegroundName("DefaultForeground");

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.ForegroundColor(FAppStyle::GetSlateColor(DefaultForegroundName))
		.HAlign(HAlign_Right)
		[
			ToolbarBuilder.MakeWidget()
		]
	];

	SViewportToolBar::Construct(SViewportToolBar::FArguments());
}

///////////////////////////////////////////////////////////

void STG_EditorViewportRenderModeToolBar::Construct(const FArguments& InArgs, TSharedPtr<class STG_EditorViewport> InViewport)
{
#if 0
	ViewportRef = InViewport;

	const FMargin ToolbarSlotPadding(4.0f, 1.0f);
	ChildSlot
		[
			SAssignNew(HorizontalBox,SHorizontalBox)
		];
	
	SViewportToolBar::Construct(SViewportToolBar::FArguments());
#endif 
}

void STG_EditorViewportRenderModeToolBar::HandleOnRenderModeChange(FName UpdatedRenderMode)
{
	if (ViewportRef)
		ViewportRef->SetRenderMode(UpdatedRenderMode);
}

TSharedRef<SWidget> STG_EditorViewportRenderModeToolBar::GenerateRenderModes()
{
	FMenuBuilder MenuBuilder(true, nullptr);

#if 0
	auto RenderModesList = ViewportRef.Get()->GetRenderModesList();

	for (FName RenderMode : RenderModesList)
	{
		MenuBuilder.AddMenuEntry(
			FText::Format(NSLOCTEXT("STG_EditorViewportToolbar", "RenderMode", "{0}"), FText::FromName(RenderMode)),
			FText::Format(NSLOCTEXT("STG_EditorViewportToolbar", "RenderMode", "{0}"), FText::FromName(RenderMode)),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &STG_EditorViewportRenderModeToolBar::HandleOnRenderModeChange, RenderMode),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, RenderMode]() {return CurrentRenderMode == RenderMode; })
			)
		);
	}
#endif 	

	return MenuBuilder.MakeWidget();
}

FText STG_EditorViewportRenderModeToolBar::GetRenderModeLabel() const
{
	return ViewportRef->GetRenderModeName();
}

void STG_EditorViewportRenderModeToolBar::Init()
{
#if 0
	if (RenderModeToolBar != nullptr)
	{
		HorizontalBox->RemoveSlot(RenderModeToolBar.ToSharedRef());
	}
	const FMargin ToolbarSlotPadding(4.0f, 1.0f);

	StaticCastSharedPtr<SHorizontalBox>(HorizontalBox)->AddSlot()
			.AutoWidth()
			.Padding(ToolbarSlotPadding)
			.HAlign(HAlign_Right)
			[
				SAssignNew(RenderModeToolBar,SEditorViewportToolbarMenu)
				.Label(this, &STG_EditorViewportRenderModeToolBar::GetRenderModeLabel)
				.Cursor(EMouseCursor::Default)
				.ParentToolBar(SharedThis(this))
				.OnGetMenuContent(this, &STG_EditorViewportRenderModeToolBar::GenerateRenderModes)				
			];

	// For now disabled render modes functionality.
	RenderModeToolBar->SetVisibility(EVisibility::Visible);

	//MenuAnchor
	TSharedRef<SMenuAnchor> menuAnchor = StaticCastSharedRef<SMenuAnchor>( RenderModeToolBar->GetChildren()->GetChildAt(0));
	menuAnchor->SetMenuPlacement(EMenuPlacement::MenuPlacement_BelowRightAnchor);
#endif 
}

#undef LOCTEXT_NAMESPACE
