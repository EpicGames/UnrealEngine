// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SViewportToolBar.h"
#include "SCommonEditorViewportToolbarBase.h"

class SEditorViewportToolbarMenu;

///////////////////////////////////////////////////////////
// STG_EditorViewportPreviewShapeToolBar

class STG_EditorViewportPreviewShapeToolBar : public SViewportToolBar
{
public:
	SLATE_BEGIN_ARGS(STG_EditorViewportPreviewShapeToolBar){}
	SLATE_END_ARGS()

	void								Construct(const FArguments& InArgs, TSharedPtr<class STG_EditorViewport> InViewport);
};

class STG_EditorViewportRenderModeToolBar : public SViewportToolBar
{
public:
	SLATE_BEGIN_ARGS(STG_EditorViewportRenderModeToolBar) {}
	SLATE_END_ARGS()

	TSharedPtr<SEditorViewportToolbarMenu> RenderModeToolBar;
	TSharedPtr<SHorizontalBox>			HorizontalBox;
	TSharedPtr<class STG_EditorViewport>ViewportRef; ///Holds the reference of view port
	FName								CurrentRenderMode;
	void								Construct(const FArguments& InArgs, TSharedPtr<class STG_EditorViewport> InViewport);
	void								HandleOnRenderModeChange(FName UpdatedRenderMode);
	TSharedRef<SWidget>					GenerateRenderModes();
	FText								GetRenderModeLabel() const;
	void								Init();
};
