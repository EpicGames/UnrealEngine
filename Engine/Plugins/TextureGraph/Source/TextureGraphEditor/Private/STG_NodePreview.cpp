// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_NodePreview.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "EditorModeManager.h"
#include "SImageViewport.h"
#include "Texture2DPreview.h"
#include "TextureResource.h"
#include "TG_Node.h"
#include "2D/Tex.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include "EdGraph/TG_EdGraphNode.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Engine/Texture2D.h"

#define LOCTEXT_NAMESPACE "STG_NodePreview"
UE::ImageWidgets::IImageViewer::FImageInfo FNodeViewer::GetCurrentImageInfo() const
{
	if (NodeTexture)
	{
		return {{}, {(int32)NodeTexture->GetSurfaceWidth(), (int32)NodeTexture->GetSurfaceHeight()}, GetNodeTextureNumMips(), true};
	}

	return {{}, FIntPoint::ZeroValue, 0, false};
}

void FNodeViewer::UpdateTexture()
{
	if (!CurrentBlob)
		return;

	if (CurrentBlob->IsTiled())
	{
		CurrentBlob->OnFinalise()
			.then([this]()
				{
					const TiledBlobPtr BlobTiled = std::static_pointer_cast<TiledBlob>(CurrentBlob);
					return BlobTiled->CombineTiles(false, false);
				})
			.then([this]()
				{
					FIntPoint ImageSize = FIntPoint {(int32)CurrentBlob->GetWidth(), (int32)CurrentBlob->GetHeight()};
					SetTexture(CurrentBlob);
				});
	}
	else
	{
		CurrentBlob->OnFinalise().then([this]()
			{
				SetTexture(CurrentBlob);
			});
	}
}

void FNodeViewer::DrawCurrentImage(FViewport*, FCanvas* Canvas, const FDrawProperties& Properties)
{
	check(NodeTexture);
	if (NodeTexture)
	{
		FTextureResource* TextureResource = NodeTexture->GetResource();
		if (TextureResource != nullptr)
		{
			TextureResource->bGreyScaleFormat = IsSingleChannel();
			TextureResource->bSRGB = IsSRGB();

			DrawTexture(TextureResource, Canvas, Properties.Placement, Properties.Mip);
			return;
		}
	}
	Util::OnGameThread([this]()
		{
			UpdateTexture();
		});
}


TOptional<TVariant<FColor, FLinearColor>> FNodeViewer::GetCurrentImagePixelColor(FIntPoint PixelCoords, int32 MipIndex) const
{
	const uint64 PixelIndex = PixelCoords.Y * NodeDescriptor.Width + PixelCoords.X;
	if (NodeTexture && CurrentBlob && CurrentBlob->IsValid())
	{
		if (!CurrentBlob->GetBufferRef()->HasRaw())
		{
			/// If it's already fetching raw, then we don't want to do it again
			if (CurrentBlob->GetBufferRef()->IsFetchingRaw())
				return {};

			CurrentBlob->GetBufferRef()->Raw();
		}
		else 
		{
			const FLinearColor LinearColor = CurrentBlob->GetBufferRef()->Raw_Now()->GetAsLinearColor(PixelIndex);

			if (NodeDescriptor.Format == BufferFormat::Byte)
			{
				return TVariant<FColor, FLinearColor>(TInPlaceType<FColor>(), LinearColor.ToFColor(IsSRGB()));
			}
			return TVariant<FColor, FLinearColor>(TInPlaceType<FLinearColor>(), LinearColor);
		}
	}

	return {};
}

UE::ImageWidgets::SImageViewport::FDrawSettings FNodeViewer::GetDrawSettings() const
{
	return DrawSettings;
}

FText FNodeViewer::GetFormatLabelText() const
{
	if (NodeTexture != nullptr)
	{
		return FText::Format(FTextFormat::FromString("{0}_{1} {2}"),
		                     FText::FromString(TextureHelper::GetChannelsTextFromItemsPerPoint(NodeDescriptor.ItemsPerPoint)),
		                     FText::FromString(BufferDescriptor::FormatToString(NodeDescriptor.Format)),
		                     FText::FromString(NodeDescriptor.bIsSRGB ? "(sRGB)" : "(Linear)"));
	}

	return LabelText;
}

bool FNodeViewer::IsSingleChannel() const
{
	return NodeDescriptor.ItemsPerPoint == 1;
}

void FNodeViewer::SetTexture(const BlobPtr& InBlob, const FLinearColor InClearColor /*= FLinearColor(0.1, 0.1, 0.1, 1)*/)
{
	NodeTexture = nullptr;
	CurrentBlob = InBlob;

	NodeDescriptor = {};
	DrawSettings.ClearColor = InClearColor;

	if (InBlob)
	{
		const DeviceBufferPtr Buffer = InBlob->GetBufferRef().GetPtr();
		check(Buffer);

		NodeTexture = GetTextureFromBuffer(Buffer);
		NodeDescriptor = Buffer->Descriptor();

		check(NodeDescriptor.Width > 0 && NodeDescriptor.Height > 0);
	}
}

void FNodeViewer::SetLabelText(FText InText)
{
	LabelText = InText;
}

void FNodeViewer::SetRGBA(bool bR, bool bG, bool bB, bool bA)
{
	bRGBA[0] = bR;
	bRGBA[1] = bG;
	bRGBA[2] = bB;
	bRGBA[3] = bA;
}

void FNodeViewer::SetDrawSettings(const UE::ImageWidgets::SImageViewport::FDrawSettings& InDrawSettings)
{
	DrawSettings = InDrawSettings;
}

void FNodeViewer::DrawTexture(const FTextureResource* TextureResource, FCanvas* Canvas, const FDrawProperties::FPlacement& Placement,
                              const FDrawProperties::FMip& Mip) const
{
	check(GMaxRHIFeatureLevel >= ERHIFeatureLevel::SM5);

	const bool bIsNormalMap = NodeTexture->IsNormalMap();
	const bool bIsVirtualTexture = NodeTexture->IsCurrentlyVirtualTextured();

	const TRefCountPtr BatchedElementParameters = new FBatchedElementTexture2DPreviewParameters(
		Mip.MipLevel, 0, 0, bIsNormalMap, false, false, bIsVirtualTexture, false, Placement.ZoomFactor >= 2.0/*Mip.bUsePointSampling*/);

	FCanvasTileItem Tile(Placement.Offset, TextureResource, Placement.Size, FLinearColor::White);
	Tile.BatchedElementParameters = BatchedElementParameters.GetReference();
	Tile.BlendMode = GetBlendMode();
	Canvas->DrawItem(Tile);
}

ESimpleElementBlendMode FNodeViewer::GetBlendMode() const
{
	if (NodeTexture)
	{
		const TEnumAsByte<TextureCompressionSettings>& CompressionSettings = NodeTexture->CompressionSettings;
		if (CompressionSettings == TC_Grayscale || CompressionSettings == TC_Alpha)
		{
			return SE_BLEND_Opaque;
		}
	}

	int32 Result = SE_BLEND_RGBA_MASK_START;

	if (IsSingleChannel())
	{
		Result += bRGBA[0] * 0b00111;
	}
	else
	{
		Result += bRGBA[0] * 0b00001;
		Result += bRGBA[1] * 0b00010;
		Result += bRGBA[2] * 0b00100;
		Result += bRGBA[3] * 0b01000;
	}

	return static_cast<ESimpleElementBlendMode>(Result);
}

UTexture* FNodeViewer::GetTextureFromBuffer(const DeviceBufferPtr& Buffer) const
{
	UTexture* OutTexture = nullptr;

	if (const std::shared_ptr<DeviceBuffer_FX> FXBuffer = std::static_pointer_cast<DeviceBuffer_FX>(Buffer))
	{
		if (!FXBuffer->IsNull())
		{
			const TexPtr Tex = FXBuffer->GetTexture();
			check(Tex);
			UTexture* BufferTexture = Tex->GetTexture();
			if (UTextureRenderTarget2D* RenderTarget = Cast<UTextureRenderTarget2D>(BufferTexture))
			{
				OutTexture = RenderTarget;
			}
			else if (UTexture2D* Texture2D = Cast<UTexture2D>(BufferTexture))
			{
				OutTexture = Texture2D;
			}
			else
			{
				UE_LOGF(LogTemp, Warning, "Texture is not a UTexture2D | UTextureRenderTarget2D.");
			}
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "Buffer is not an FXBuffer.");
		}
	}
	else
	{
		UE_LOGF(LogTemp, Warning, "BlobTexture failed to find the buffer.");
	}

	return OutTexture;
}

int32 FNodeViewer::GetNodeTextureNumMips() const
{
	//Enable this when we can switch mip levels
#if 0
	if (UTexture2D* Texture2D = Cast<UTexture2D>(NodeTexture))
	{
		return Texture2D->GetNumMips();
	}
	else if(UTextureRenderTarget2D* RenderTarget = Cast<UTextureRenderTarget2D>(NodeTexture))
	{
		return RenderTarget->GetNumMips();
	}
#endif
	return 0;
}

bool FNodeViewer::IsSRGB() const
{
	return NodeDescriptor.bIsSRGB;
}

class FPreviewViewerCommands : public TCommands<FPreviewViewerCommands>
{
public:
	FPreviewViewerCommands()
		: TCommands(TEXT("NodePreview"), LOCTEXT("ContextDescription", "Node Preview"), NAME_None, FAppStyle::Get().GetStyleSetName())
	{
	}

	virtual void RegisterCommands() override
	{
		UI_COMMAND(ToggleLock, "Toggle Node Preview Lock", "Toggles the node preview lock.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::L));
	}

	TSharedPtr<FUICommandInfo> ToggleLock;
};

STG_NodePreviewWidget::~STG_NodePreviewWidget()
{
	FPreviewViewerCommands::Unregister();
}

UE::ImageWidgets::SImageViewport::FDrawSettings STG_NodePreviewWidget::GetDrawSettings() const
{
	return NodeViewer->GetDrawSettings();
}

void STG_NodePreviewWidget::Construct(const FArguments& InArgs)
{
	NodeViewer = MakeShared<FNodeViewer>();

	NodeViewer->SetDrawSettings(UE::ImageWidgets::SImageViewport::FDrawSettings{
					.ClearColor = FVector3f(0.1f),
					.bBorderEnabled = false,
					.bBackgroundColorEnabled = true,
					.BackgroundColor = FLinearColor::Black,
					.bBackgroundCheckerEnabled = true});

	OnNodeBlobChanged = InArgs._OnNodeBlobChanged;

	FPreviewViewerCommands::Register();
	const FPreviewViewerCommands Commands = FPreviewViewerCommands::Get();
	CommandList = MakeShared<FUICommandList>();
	CommandList->MapAction(Commands.ToggleLock,
	                       FExecuteAction::CreateSP(this, &STG_NodePreviewWidget::ToggleLock),
	                       FCanExecuteAction::CreateLambda([this] { return NodeViewer->GetCurrentImageInfo().bIsValid; }),
	                       FIsActionChecked::CreateLambda([this] { return LockedNode != nullptr; }));

	const TSharedPtr<FExtender> ToolbarExtender = MakeShared<FExtender>();
	ToolbarExtender->AddToolBarExtension("ToolbarLeft", EExtensionHook::Before, CommandList,
	                                     FToolBarExtensionDelegate::CreateSP(this, &STG_NodePreviewWidget::AddLockButton));

	ToolbarExtender->AddToolBarExtension("ToolbarRight", EExtensionHook::After, CommandList,
	                                     FToolBarExtensionDelegate::CreateSP(this, &STG_NodePreviewWidget::AddRGBAButtons));

	const TSharedPtr<UE::ImageWidgets::SImageViewport::FStatusBarExtender> StatusBarExtender = MakeShared<
		UE::ImageWidgets::SImageViewport::FStatusBarExtender>();
	StatusBarExtender->AddExtension("StatusBarLeft", EExtensionHook::After, CommandList,
	                                UE::ImageWidgets::SImageViewport::FStatusBarExtender::FDelegate::CreateSP(this, &STG_NodePreviewWidget::AddFormatLabel));

	if (Viewport)
	{
		if (const TSharedPtr<FEditorViewportClient>& ViewportClient = Viewport->GetViewportClient())
		{
			if (FEditorModeTools* const ModeTools = ViewportClient->GetModeTools())
			{
				ModeTools->SetSupportsViewportITF(false);
			}
		}
	}

	ChildSlot
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Fill)
		[
			SAssignNew(Viewport, UE::ImageWidgets::SImageViewport, NodeViewer.ToSharedRef())
				.ToolbarExtender(ToolbarExtender)
				.StatusBarExtender(StatusBarExtender)
				.DrawSettings(this,&STG_NodePreviewWidget::GetDrawSettings)
				.ControllerSettings(UE::ImageWidgets::SImageViewport::FControllerSettings{
					.DefaultZoomMode = UE::ImageWidgets::SImageViewport::FControllerSettings::EDefaultZoomMode::Fill
				})
		];
}

void STG_NodePreviewWidget::SelectionChanged(UTG_Node* Node)
{
	const bool bUpdatePreview = LockedNode == nullptr && SelectedNode != Node;

	SelectedNode = Node;

	if (bUpdatePreview)
	{
		Update();
	}
}

void STG_NodePreviewWidget::NodeDeleted(const UTG_Node* Node)
{
	const bool bPreviewNodeDeleted = LockedNode == Node || (LockedNode == nullptr && SelectedNode == Node);

	if (LockedNode == Node)
	{
		LockedNode = nullptr;
	}

	if (SelectedNode == Node)
	{
		SelectedNode = nullptr;
	}

	if (bPreviewNodeDeleted)
	{
		Update();
	}
}

void STG_NodePreviewWidget::Update() const
{
	FTG_Variant Variant;
	BlobPtr Blob;
	bool bValidVariant = GetOutputVariantFromNode(Variant);

	if (Variant)
	{
		FIntPoint ImageSize = FIntPoint::ZeroValue;

		if (Variant.IsTexture() && Variant.GetTexture() && Variant.GetTexture().RasterBlob)
		{
			Blob = Variant.GetTexture().RasterBlob;
			ImageSize = FIntPoint {(int32)Blob->GetWidth(), (int32)Blob->GetHeight()};
			TWeakPtr<FNodeViewer> WeakViewer = NodeViewer;
			TWeakPtr<UE::ImageWidgets::SImageViewport> WeakViewport = Viewport;
			if (Blob->IsTiled())
			{
				Blob->OnFinalise()
					.then([Blob]()
						{
							const TiledBlobPtr BlobTiled = std::static_pointer_cast<TiledBlob>(Blob);
							return BlobTiled->CombineTiles(false, false);
						})
					.then([WeakViewer, WeakViewport, Blob]()
						{
							Util::OnGameThread([WeakViewer, WeakViewport, Blob]()
							{
								const auto PinnedViewer   = WeakViewer.Pin();
								const auto PinnedViewport = WeakViewport.Pin();
								if (!PinnedViewer || !PinnedViewport)
								{
									return; // widget/editor closed; nothing to do
								}
								FIntPoint ImageSize = FIntPoint {(int32)Blob->GetWidth(), (int32)Blob->GetHeight()};
								PinnedViewer->SetTexture(Blob);
								PinnedViewport->ResetZoom(ImageSize);
							});
						});
			}
			else
			{
				Blob->OnFinalise().then([WeakViewer, Blob]()
				{
					Util::OnGameThread([WeakViewer, Blob]()
						{
							if (const auto PinnedViewer   = WeakViewer.Pin())
							{
								PinnedViewer->SetTexture(Blob);
							}
						});
				});
			}
		}
		else if(Variant.IsColor())
		{
			NodeViewer->SetTexture(nullptr, Variant.GetColor());
		}
		else
		{
			NodeViewer->SetTexture(nullptr, FLinearColor::Black);
		}

		NodeViewer->SetLabelText(GetLabelText(Variant, bValidVariant));
		Viewport->ResetZoom(ImageSize);
		[[maybe_unused]] bool Result = OnNodeBlobChanged.ExecuteIfBound(Blob);

		return;
	}

	/// If we get to here then we need to set it to null
	NodeViewer->SetTexture(nullptr);

	// Update preview blob and trigger related external updates.
	[[maybe_unused]] bool Result = OnNodeBlobChanged.ExecuteIfBound(Blob);
}

bool STG_NodePreviewWidget::GetOutputVariantFromNode(FTG_Variant& OutVariant) const
{
	const UTG_Node* PreviewNode = LockedNode ? LockedNode : SelectedNode;
	
	// Get the Variant if preview node is valid or assign a nullptr if not.
	TArray<FTG_Variant> OutVariants;
	TArray<FName>* OutNames = nullptr;

	if(PreviewNode)
	{
		PreviewNode->GetAllOutputValues(OutVariants, OutNames);

		if (!OutVariants.IsEmpty())
		{
			OutVariant = OutVariants[0];
			return true;
		}	
	}

	return false;
}

FText STG_NodePreviewWidget::GetLabelText(FTG_Variant& InVariant, bool bValidVariant) const
{
	const UTG_Node* PreviewNode = LockedNode ? LockedNode : SelectedNode;
	const bool bInValidTexture = InVariant.IsTexture() && InVariant.GetTexture() && !InVariant.GetTexture()->IsValid();
	const FText PreviewNotAvailable = FText::FromString("Node preview is not available");

	if (!PreviewNode)
	{
		return FText::FromString("Select a node to preview");
	}
	else if (!bValidVariant)
	{
		return PreviewNotAvailable;
	}
	else if (InVariant.IsColor())
	{
		return FText::FromString("Color " + InVariant.GetColor().ToFColor(false).ToString());
	}
	else if (InVariant.IsVector())
	{
		return FText::FromString("Vector (" + InVariant.GetVector().ToString() + ")");
	}
	else if (InVariant.IsScalar())
	{
		return FText::FromString(FString::Printf(TEXT("Scalar (%0.3f)"), InVariant.GetScalar()));
	}
	else if (bInValidTexture)
	{
		return FText::FromString("Texture is not valid");
	}

	return PreviewNotAvailable;
}

FReply STG_NodePreviewWidget::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void STG_NodePreviewWidget::AddFormatLabel(SHorizontalBox& HorizontalBox)
{
	HorizontalBox.AddSlot()
				 .VAlign(VAlign_Center)
	[
		SNew(STextBlock)
			.Text_Raw(NodeViewer.Get(), &FNodeViewer::GetFormatLabelText)
	];
}

void STG_NodePreviewWidget::AddLockButton(FToolBarBuilder& ToolbarBuilder) const
{
	auto GetLockButtonImage = [this]
	{
		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), LockedNode ? "PropertyWindow.Locked" : "PropertyWindow.Unlocked");
	};

	ToolbarBuilder.AddToolBarButton(FPreviewViewerCommands::Get().ToggleLock, NAME_None, TAttribute<FText>(), TAttribute<FText>(),
	                                TAttribute<FSlateIcon>::CreateLambda(GetLockButtonImage));

	ToolbarBuilder.AddSeparator();
}

void STG_NodePreviewWidget::AddRGBAButtons(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.AddSeparator();

	auto GetTextButton = [this](const FString& Label, const FCheckBoxStyle* ButtonStyle, bool& bChecked)
	{
		return SNew(SCheckBox)
			.Style(ButtonStyle)
			.IsEnabled_Lambda([this, &bChecked]
		                      {
			                      return NodeViewer->GetCurrentImageInfo().bIsValid && (&bChecked == &bRGBA[0] ||
				                      !NodeViewer->IsSingleChannel());
		                      })
			.IsChecked(bChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked)
			.OnCheckStateChanged_Lambda([&bChecked, this](const ECheckBoxState State)
		                      {
			                      bChecked = State == ECheckBoxState::Checked;
			                      NodeViewer->SetRGBA(bRGBA[0], bRGBA[1], bRGBA[2], bRGBA[3]);
		                      })
		[
			SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("EditorViewportToolBar.Font"))
					.Text(FText::FromString(Label))
		];
	};

	const FCheckBoxStyle* ButtonStyleStart = &FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.Start");
	const FCheckBoxStyle* ButtonStyleMiddle = &FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.Middle");
	const FCheckBoxStyle* ButtonStyleEnd = &FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.End");

	const TSharedRef<SHorizontalBox> RGBA = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			GetTextButton("R", ButtonStyleStart, bRGBA[0])
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			GetTextButton("G", ButtonStyleMiddle, bRGBA[1])
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			GetTextButton("B", ButtonStyleMiddle, bRGBA[2])
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			GetTextButton("A", ButtonStyleEnd, bRGBA[3])
		];

	ToolbarBuilder.AddToolBarWidget(RGBA);
}

void STG_NodePreviewWidget::ToggleLock()
{
	if (LockedNode)
	{
		const bool bUpdatePreview = SelectedNode != LockedNode;

		LockedNode = nullptr;

		if (bUpdatePreview)
		{
			Update();
		}
	}
	else
	{
		LockedNode = SelectedNode;
	}
}

#undef LOCTEXT_NAMESPACE