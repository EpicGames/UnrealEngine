// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IImageViewer.h"
#include "SImageViewport.h"
#include "Data/Blob.h"
#include "Widgets/SCompoundWidget.h"

struct FTG_Variant;

namespace UE::ImageWidgets
{
	class SImageViewport;
}

class FToolBarBuilder;
class FUICommandList;
class SHorizontalBox;
class UTG_Node;

/**
 * Image viewer implementation that holds the node buffer information and draws the node texture.
 */
class FNodeViewer final : public UE::ImageWidgets::IImageViewer
{
public:
	// IImageViewer overrides - begin
	virtual FImageInfo GetCurrentImageInfo() const override;
	virtual void DrawCurrentImage(FViewport* Viewport, FCanvas* Canvas, const FDrawProperties& Properties) override;
	virtual TOptional<TVariant<FColor, FLinearColor>> GetCurrentImagePixelColor(FIntPoint PixelCoords, int32 MipIndex) const override;
	virtual void OnImageSelected(const FGuid& ImageGuid) override {};
	virtual bool IsValidImage(const FGuid& Guid) const override { return true; }
	virtual FText GetImageName(const FGuid& Guid) const override { return {}; }
	// IImageViewer overrides - end

	/** Returns the format label for the status bar */
	FText GetFormatLabelText() const;

	/** Indicates that a node is single channel, i.e. grayscale. */
	bool IsSingleChannel() const;

	/** Sets the node buffer to a given node. */
	void SetTexture(const BlobPtr& InBlob, FLinearColor InClearColor = FLinearColor(0.1f, 0.1f, 0.1f, 1));
	void UpdateTexture();

	/** Sets the label text. */
	void SetLabelText(FText InText);

	/** Toggles RGBA components for display. */
	void SetRGBA(bool bR, bool bG, bool bB, bool bA);

	/** Sets the required Draw settings for the viewport. */ 
	void SetDrawSettings(const UE::ImageWidgets::SImageViewport::FDrawSettings& InDrawSettings);

	/** Returns the Draw settings for the viewport. */ 
	UE::ImageWidgets::SImageViewport::FDrawSettings GetDrawSettings() const;
private:
	/** Draws the node texture in the viewport */
	void DrawTexture(const FTextureResource* TextureResource, FCanvas* Canvas, const FDrawProperties::FPlacement& TilePlacementInfo,
	                 const FDrawProperties::FMip& Mip) const;

	/** Determines the blend mode based on the node texture and the RGBA toggles. */
	ESimpleElementBlendMode GetBlendMode() const;

	/** Retrieves the texture  from a given node buffer. */
	UTexture* GetTextureFromBuffer(const DeviceBufferPtr& Buffer) const;

	/** Returns the num mips for node texture*/
	int32 GetNodeTextureNumMips() const;

	/** Indicates that a node is in sRGB format. */
	bool IsSRGB() const;

	/** The Blob for this node. */
	BlobPtr CurrentBlob;

	/** The texture for the node. */
	UTexture* NodeTexture = nullptr;

	/** Toggles for enabling RGBA components for drawing. */
	bool bRGBA[4] = {true, true, true, true};

	/** Node meta data.  */
	BufferDescriptor NodeDescriptor;

	/** Current draw settings for the viewport. */ 
	UE::ImageWidgets::SImageViewport::FDrawSettings DrawSettings;

	/** Label text for adding node description.*/
	FText LabelText;
};

/**
 * Widget for the Node Preview tab containing the image viewport.
 */
class STG_NodePreviewWidget : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnNodeBlobChanged, BlobPtr Blob);

	SLATE_BEGIN_ARGS(STG_NodePreviewWidget)
		{
		}

		/** Callback to notify about the node preview texture having changed. */
		SLATE_EVENT(FOnNodeBlobChanged, OnNodeBlobChanged)
	SLATE_END_ARGS()

	virtual ~STG_NodePreviewWidget() override;

	UE::ImageWidgets::SImageViewport::FDrawSettings GetDrawSettings() const;
	void Construct(const FArguments& InArgs);

	/** Notify the preview about the node selection having changed. */
	void SelectionChanged(UTG_Node* Node);

	/** Notify the preview about a node being deleted. */
	void NodeDeleted(const UTG_Node* Node);

	/** Trigger an update of the node preview after the displayed contents changed. */
	void Update() const;

	bool GetOutputVariantFromNode(FTG_Variant& OutVariant) const; 

	/** Get the label text for the node preview*/
	FText GetLabelText(FTG_Variant& InVariant, bool bValidVariant) const;

	// SWidget overrides - begin
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	// SWidget overrides - end

private:
	// Add toolbar and status bar extensions to the viewport.
	void AddFormatLabel(SHorizontalBox& HorizontalBox);
	void AddLockButton(FToolBarBuilder& ToolbarBuilder) const;
	void AddRGBAButtons(FToolBarBuilder& ToolbarBuilder);

	/** Toggles the node preview lock based on the lock button extension in the viewport toolbar. */
	void ToggleLock();

	/** The viewport widget for displaying the node preview. */
	TSharedPtr<UE::ImageWidgets::SImageViewport> Viewport;

	/** The image viewer implementation holding and drawing the actual image. */
	TSharedPtr<FNodeViewer> NodeViewer;

	/** The additional commands used for this widget. */
	TSharedPtr<FUICommandList> CommandList;

	/** Pointer to the currently selected node, which might be different if the node preview is locked to another node. */
	// UTG_EdGraphNode* SelectedNode = nullptr;
	UTG_Node* SelectedNode = nullptr;

	/** Pointer to the node the node preview is currently locked on. */
	// UTG_EdGraphNode* LockedNode = nullptr;
	UTG_Node* LockedNode = nullptr;

	/** Callback to notify about the node preview texture having changed to make sure that for example the histogram is synced with the node preview. */
	FOnNodeBlobChanged OnNodeBlobChanged;

	/** Flags for toggling RGBA channels in the preview, which are hooked up to the RGBA buttons in the viewport toolbar extension. */
	bool bRGBA[4] = {true, true, true, true};
};
