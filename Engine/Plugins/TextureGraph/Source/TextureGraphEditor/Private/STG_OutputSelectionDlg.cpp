// Copyright Epic Games, Inc. All Rights Reserved.
#include "STG_OutputSelectionDlg.h"
#include "Widgets/Layout/SBox.h"
#include "TG_Pin.h"
#include "TG_Graph.h"
#include "Widgets/Layout/SSeparator.h"
#include "SPrimaryButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Expressions/Output/TG_Expression_Output.h"
#include "Editor.h"
#include "EdGraph/STG_NodeThumbnail.h"
#include "STG_OutputSelector.h"
#include "TextureGraph.h"
#include "TG_HelperFunctions.h"
#include "EdGraph/TG_EdGraphNode.h"
#include "Widgets/Colors/SColorBlock.h"
#define LOCTEXT_NAMESPACE "STG_OutputSelectionDlg"
void STG_OutputSelectionDlg::Construct(const FArguments& InArgs)
{
	EdGraph = InArgs._EdGraph;
	SWindow::Construct(SWindow::FArguments()
		.Title(InArgs._Title)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		//.SizingRule( ESizingRule::Autosized )
		.ClientSize(FVector2D(350, 450))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot() // Add user input block
			.Padding(2, 2, 2, 4)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.FillHeight(1)
					[
						SAssignNew(ScrollBox,SScrollBox)
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(8.f, 16.f)
			[
				SNew(SUniformGridPanel)
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("Export", "Export"))
					.OnClicked(this, &STG_OutputSelectionDlg::OnButtonClick, EAppReturnType::Ok)
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.Text(LOCTEXT("Cancel", "Cancel"))
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &STG_OutputSelectionDlg::OnButtonClick, EAppReturnType::Cancel)
				]
			]
		]);
	AddExportItems();
}
void STG_OutputSelectionDlg::AddExportItems()
{
	ScrollBox->ClearChildren();
	UTG_EdGraph* TGEdGraph = Cast<UTG_EdGraph>(EdGraph);
	TGEdGraph->TextureGraph->Graph()->ForEachNodes([this](const UTG_Node* Node, uint32 Index)
	{
		UTG_Expression_Output* TargetExpression = Cast<UTG_Expression_Output>(Node->GetExpression());
		if (TargetExpression)
		{
			FTG_OutputSettings& OutputSetting = TargetExpression->OutputSettings;
			auto OutPinIds = Node->GetOutputPinIds();
			for (auto Id : OutPinIds)
			{
				//This is a work around for checking the type of the output
				//Probably we need to have a better solution for checking output type
				auto Pin = Node->GetGraph()->GetPin(Id);

				FName OutputName = Pin->GetAliasName();

				FTG_Variant Variant;
				Pin->GetValue(Variant);
				TSharedPtr<SWidget> ThumbnailWidget;
				if (Variant.IsTexture())
				{
					TSharedPtr<STG_NodeThumbnail> NodeThumbnail = SNew(STG_NodeThumbnail);
					TiledBlobPtr CachedThumb = EdGraph->GetCachedThumbBlob(Id);

					if(!CachedThumb)
					{
						CachedThumb = TextureHelper::GetBlack();						
					}
					
					if (CachedThumb->IsFinalised())
					{
						if (NodeThumbnail.IsValid())
						{
							NodeThumbnail->UpdateBlob(CachedThumb);
						}
					}

					ThumbnailWidget = NodeThumbnail;
				}
				// else if color do something
				else if (Variant.IsColor())
				{
					FLinearColor ColorValue;
					Pin->GetValue(ColorValue);
					ThumbnailWidget = SNew(SColorBlock)
					.Color(ColorValue);
				}
				else if(Variant.IsVector())
				{
					FVector4f Vector;
					Pin->GetValue(Vector);
					ThumbnailWidget = SNew(SColorBlock)
					.Color(Vector);
				}
				else if (Variant.IsScalar())
				{
					float Scalar;
					Pin->GetValue(Scalar);
					ThumbnailWidget = SNew(SColorBlock)
						.Color(FLinearColor(Scalar, Scalar, Scalar, 1));
				}

				ScrollBox->AddSlot()
				.Padding(5)
				[
					SNew(STG_OutputSelector)
					.Name(FText::FromString(OutputName.ToString()))
					.ThumbnailWidget(ThumbnailWidget)
					.OnOutputSelectionChanged(this, &STG_OutputSelectionDlg::OnOutputSelectionChanged)
					.bIsSelected(TargetExpression->GetShouldExport())
				];
				ScrollBox->AddSlot()
				[
					SNew(SSeparator)
					.Thickness(1)
				];
			}
		}
	});
}
FReply STG_OutputSelectionDlg::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;
	if (ButtonID == EAppReturnType::Cancel || ButtonID == EAppReturnType::Ok)
	{
		// Only close the window if canceling or if the ok
		RequestDestroyWindow();
	}
	else
	{
		// reset the user response in case the window is closed using 'x'.
		UserResponse = EAppReturnType::Cancel;
	}
	return FReply::Handled();
}
EAppReturnType::Type STG_OutputSelectionDlg::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}
void STG_OutputSelectionDlg::OnOutputSelectionChanged(const FString ItemName, ECheckBoxState NewState)
{
	UTG_EdGraph* TGEdGraph = Cast<UTG_EdGraph>(EdGraph);
	TGEdGraph->TextureGraph->Graph()->ForEachNodes([=](const UTG_Node* Node, uint32 Index)
	{
		UTG_Expression_Output* TargetExpression = Cast<UTG_Expression_Output>(Node->GetExpression());
		if (TargetExpression)
		{
			UTG_Pin* Pin = Node->GetPin(GET_MEMBER_NAME_CHECKED(UTG_Expression_Output, Output));

			FName OutputName = Pin->GetAliasName();

			if (OutputName == ItemName)
			{
				TargetExpression->SetShouldExport( NewState == ECheckBoxState::Checked);
			}
		}
	});
}
#undef LOCTEXT_NAMESPACE