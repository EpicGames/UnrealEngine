// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "Widgets/SBoxPanel.h"
#include "Input/Reply.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SScrollBox.h"
#include "TG_OutputSettings.h"
#include "EdGraph/TG_EdGraph.h"
#include "Transform/Layer/T_Thumbnail.h"

class STG_OutputSelectionDlg : public SWindow
{
	SLATE_BEGIN_ARGS(STG_OutputSelectionDlg)
	{}
	SLATE_ARGUMENT(FText, Title)
	SLATE_ARGUMENT(TObjectPtr<UTG_EdGraph>, EdGraph)
	SLATE_ARGUMENT(TArray<FText>, ExportItems)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	/** Displays the dialog in a blocking fashion */
	EAppReturnType::Type ShowModal();

protected:
	FReply OnButtonClick(EAppReturnType::Type ButtonID);
	EAppReturnType::Type UserResponse;

private:
	TSharedPtr<SScrollBox> ScrollBox;
	TObjectPtr<UTG_EdGraph> EdGraph;

	void AddExportItems();
	void OnOutputSelectionChanged(const FString ItemName, ECheckBoxState NewState);
};
