// Copyright Epic Games, Inc. All Rights Reserved.
#include "STG_OutputSelector.h"
#include "Widgets/Layout/SBox.h"
#include <Widgets/Text/STextBlock.h>

#define LOCTEXT_NAMESPACE "STG_OutputSelector"

void STG_OutputSelector::Construct(const FArguments& InArgs)
{
	OnOutputSelectionChanged = InArgs._OnOutputSelectionChanged;
	bIsSelected = InArgs._bIsSelected;
	auto& Name = InArgs._Name;

	auto OnSelectionChanged = [this, Name](ECheckBoxState NewState)
	{
		bIsSelected = NewState == ECheckBoxState::Checked;
		OnOutputSelectionChanged.ExecuteIfBound(Name.ToString(), NewState);
	};

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		[
			SNew(SCheckBox)
			.IsChecked(this, &STG_OutputSelector::OnCheckBoxState)
			.OnCheckStateChanged_Lambda(OnSelectionChanged)
			.Content()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(5)
				.AutoWidth()
				.HAlign(HAlign_Left)
				[
					SNew(SBox)
					.WidthOverride(32)
					.HeightOverride(32)
					[
						InArgs._ThumbnailWidget.ToSharedRef()
					]
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(5)
				[
					SNew(STextBlock)
					.Text(InArgs._Name)
				]
			]
		]
	];
}

ECheckBoxState STG_OutputSelector::OnCheckBoxState() const
{
	return bIsSelected ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

#undef LOCTEXT_NAMESPACE