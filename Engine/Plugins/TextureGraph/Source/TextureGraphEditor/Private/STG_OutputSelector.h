// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "Widgets/SBoxPanel.h"
#include "Input/Reply.h"
#include "TG_OutputSettings.h"
#include <Widgets/Input/SCheckBox.h>
#include "Widgets/SCompoundWidget.h"

class STG_OutputSelector : public SCompoundWidget
{
	DECLARE_DELEGATE_TwoParams(FOnOutputSelectionChanged, const FString ItemName, ECheckBoxState NewState);

	SLATE_BEGIN_ARGS(STG_OutputSelector)
	{}
	SLATE_ARGUMENT(FText, Name)
	SLATE_ARGUMENT(bool, bIsSelected)
	SLATE_ARGUMENT(TSharedPtr<SWidget>, ThumbnailWidget)
	SLATE_EVENT(FOnOutputSelectionChanged, OnOutputSelectionChanged)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	ECheckBoxState OnCheckBoxState() const;

	FOnOutputSelectionChanged OnOutputSelectionChanged;
private:
	bool bIsSelected;
};
