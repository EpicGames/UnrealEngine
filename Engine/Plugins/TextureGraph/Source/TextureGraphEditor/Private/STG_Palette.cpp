// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_Palette.h"
#include "Widgets/ActionMenuWidgets/STG_ActionMenuTileView.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SOverlay.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "EdGraph/TG_EdGraphSchema.h"
#include "EdGraph/TG_EdGraph.h"

#include "EditorWidgetsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "TG_Editor.h"
#include "Brushes/SlateImageBrush.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/SComboButton.h"
#include "TG_Style.h"

#define LOCTEXT_NAMESPACE "TGPalette"

void STG_PaletteItem::Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData)
{
	check(InCreateData->Action.IsValid());

	TSharedPtr<FEdGraphSchemaAction> GraphAction = InCreateData->Action;
	ActionPtr = InCreateData->Action;
	MouseButtonDownDelegate = InCreateData->MouseButtonDownDelegate;

	const FSlateBrush* IconBrush = GetIconBrush();
	FSlateColor IconColor = FSlateColor::UseForeground();
	FText IconToolTip = GraphAction->GetCategory();
	bool bIsReadOnly = false;

	TSharedRef<SWidget> IconWidget = CreateIconWidget(IconToolTip, IconBrush, IconColor);
	TSharedRef<SWidget> NameSlotWidget = CreateTextSlotWidget(InCreateData, bIsReadOnly);
	InlineRenameWidget->SetOverflowPolicy(ETextOverflowPolicy::Ellipsis);

	auto NameAreaBackground = FAppStyle::Get().GetBrush(FName("ContentBrowser.AssetTileItem.NameAreaBackground"));
	const int IconWidth = 22;
	const int IconHeight = 22;
	// Create the actual widget
	this->ChildSlot
		[
			SNew(SHorizontalBox)
			.ToolTipText(InCreateData->Action->GetTooltipDescription())

		//Icon
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		[
			SNew(SBox)
			.Padding(3)
		.WidthOverride(IconWidth)
		.HeightOverride(IconHeight)
		[
			IconWidget
		]
		]

	+ SHorizontalBox::Slot()
		.Padding(8, 0, 0, 0)
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			SNullWidget::NullWidget
		]

	+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
		.Text(InCreateData->Action->GetMenuDescription())
		//.HighlightText(InArgs._HighlightText)
		]
		];
}

FString STG_PaletteItem::GetIconBrushName()
{
	FString ActionName = ActionPtr.Pin()->GetCategory().ToString();
	return "TG_Editor.Palette." + ActionName + "_Colored";
}

FString STG_PaletteItem::GetDefaultIconBrushName()
{
	return "TG_Editor.Palette.Default_Colored";
}

const FSlateBrush* STG_PaletteItem::GetIconBrush()
{
	if (FTG_Style::Get().HasKey(FName(GetIconBrushName())))
	{
		const FSlateBrush* IconBrush = FTG_Style::Get().GetBrush(FName(GetIconBrushName()));
		return IconBrush;

	}
	return FTG_Style::Get().GetBrush(FName(GetDefaultIconBrushName()));
}

TSharedRef<SWidget> STG_PaletteItem::CreateHotkeyDisplayWidget(const TSharedPtr<const FInputChord> HotkeyChord)
{
	FText HotkeyText;
	if (HotkeyChord.IsValid())
	{
		HotkeyText = HotkeyChord->GetInputText();
	}
	return SNew(STextBlock)
		.Text(HotkeyText);
}

FText STG_PaletteItem::GetItemTooltip() const
{
	return ActionPtr.Pin()->GetTooltipDescription();
}

FReply STG_PaletteItem::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseButtonDownDelegate.IsBound() && MouseButtonDownDelegate.Execute(ActionPtr))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

////////////////////////////////////////////////////////////////////

void STG_PaletteTileItem::Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData)
{
	check(InCreateData->Action.IsValid());

	TSharedPtr<FEdGraphSchemaAction> GraphAction = InCreateData->Action;
	ActionPtr = InCreateData->Action;

	const FSlateBrush* IconBrush = GetIconBrush();
	FText IconToolTip = GraphAction->GetTooltipDescription();
	bool bIsReadOnly = true;

	IconWidget = SNew(SImage)
		.ToolTipText(IconToolTip)
		.Image(IconBrush)
		.ColorAndOpacity(FStyleColors::Foreground);

	TSharedRef<SWidget> NameSlotWidget = CreateTextSlotWidget(InCreateData, bIsReadOnly );
	InlineRenameWidget->SetOverflowPolicy(ETextOverflowPolicy::Ellipsis);

	FLinearColor Color = UTG_EdGraphSchema::GetCategoryColor(FName(GraphAction->GetCategory().ToString()));

	BackgroundBrush = *(FTG_Style::Get().GetBrush("TG.Palette.Background"));
	BackgroundBrush.OutlineSettings.Color = FSlateColor(Color);
	
	const int IconWidth = 16;
	const int IconHeight = 16;
	const int ItemHeight = 36;
	const int IconPadding = (ItemHeight - IconHeight) / 2;
	// Create the actual widget
	this->ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
		[
			SAssignNew(BackgroundArea, SBox)
			.Padding(FMargin(2))
			.HAlign(HAlign_Fill)
			.HeightOverride(ItemHeight)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ProjectBrowser.ProjectTile.DropShadow"))
				[
					SNew(SImage)
					.Image(&BackgroundBrush)
					.ColorAndOpacity(Color)
				]
			]
		]

		+SOverlay::Slot()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.Padding(IconPadding)
				.WidthOverride(ItemHeight)
				.HeightOverride(ItemHeight)
				[
					IconWidget.ToSharedRef()
				]
			]
			
			+ SHorizontalBox::Slot()
			.Padding(0,0,2,0)
			[
				NameSlotWidget
			]
		]
	];
}

void STG_PaletteTileItem::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SGraphPaletteItem::OnMouseEnter(MyGeometry, MouseEvent);
	BackgroundArea->SetPadding(FMargin(0));
	InlineRenameWidget->SetForegroundColor(FStyleColors::White);
	IconWidget->SetColorAndOpacity(FStyleColors::White);
}

void STG_PaletteTileItem::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SGraphPaletteItem::OnMouseLeave(MouseEvent);
	BackgroundArea->SetPadding(FMargin(2));
	InlineRenameWidget->SetForegroundColor(FStyleColors::Foreground);
	IconWidget->SetColorAndOpacity(FStyleColors::Foreground);
}

FString STG_PaletteTileItem::GetIconBrushName()
{
	FString ActionName = ActionPtr.Pin()->GetCategory().ToString();
	return "TG_Editor.Palette." + ActionName;
}

FString STG_PaletteTileItem::GetDefaultIconBrushName()
{
	return "TG_Editor.Palette.Default";
}

//////////////////////////////////////////////////////////////////////////

void STG_Palette::Construct(const FArguments& InArgs, TWeakPtr<FTG_Editor> InTGEditorPtr)
{
	TGEditorPtr = InTGEditorPtr;

	BackgroundBrush = FSlateRoundedBoxBrush(FStyleColors::Input, 4.0f);
	CheckedBrush = FSlateRoundedBoxBrush(FStyleColors::InputOutline, 4.0f); 
	BackgroundHoverBrush = FSlateRoundedBoxBrush(FStyleColors::Panel, 4.0f);

	// Create the asset discovery indicator
	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
	TSharedRef<SWidget> AssetDiscoveryIndicator = EditorWidgetsModule.CreateAssetDiscoveryIndicator(EAssetDiscoveryIndicatorScaleMode::Scale_Vertical);

	CategoryNames.Add(MakeShareable(new FString(TEXT("All"))));
	CategoryNames.Add(MakeShareable(new FString(TEXT("Expressions"))));
	CategoryNames.Add(MakeShareable(new FString(TEXT("Functions"))));

	CreatePaletteMenu();

	// Register with the Asset Registry to be informed when it is done loading up files.
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetAdded().AddSP(this, &STG_Palette::AddAssetFromAssetRegistry);
	AssetRegistryModule.Get().OnAssetRemoved().AddSP(this, &STG_Palette::RemoveAssetFromRegistry);
	AssetRegistryModule.Get().OnAssetRenamed().AddSP(this, &STG_Palette::RenameAssetFromRegistry);
}

void STG_Palette::CreatePaletteMenu()
{
	const int ToggleMenuHeight = 22;
	const int ToggleMenuWidth = 47;
	const int Padding = 5;

	TSharedPtr<SWidget> Menu = IsTileViewChecked ? CreatePaletteTileMenu() : CreatePaletteListMenu();
	
	TSharedPtr<SWidget> FoundSearchBox = FindWidgetInChildren(Menu, "SSearchBox");
	if (FoundSearchBox.IsValid())
	{
		TSharedPtr<SWidget> ParentWidget = FoundSearchBox->GetParentWidget();
		TSharedPtr<SVerticalBox> Box = StaticCastSharedPtr<SVerticalBox>(ParentWidget);
		if (Box)
		{
			Box->RemoveSlot(FoundSearchBox.ToSharedRef());
		}
	}

	if (!ToggleButtons)
	{
		ToggleButtons = CreateViewToggle();
	}

	this->ChildSlot
	[
		SNew(SBorder)
		.Padding(2.0f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.Padding(Padding)
			.HAlign(HAlign_Fill)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(0,0, Padding,0)
				[
					FoundSearchBox.IsValid() ? FoundSearchBox.ToSharedRef() : SNullWidget::NullWidget
				]

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(ToggleMenuWidth)
					.HeightOverride(ToggleMenuHeight)
					[
						ToggleButtons.ToSharedRef()
					]
				]
			]

			// Content list
			+SVerticalBox::Slot()
			[
				SNew(SOverlay)

				+SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					Menu.ToSharedRef()
				]
			]

		]
	];
}

TSharedPtr<SWidget> STG_Palette::FindWidgetInChildren(TSharedPtr<SWidget> Parent, FName ChildType) const
{
	TSharedPtr<SWidget> FoundWidget = TSharedPtr<SWidget>();
	for (int32 ChildIndex = 0; ChildIndex < Parent->GetAllChildren()->Num(); ++ChildIndex)
	{
		TSharedRef<SWidget> ChildWidget = Parent->GetAllChildren()->GetChildAt(ChildIndex);
		if (ChildWidget->GetType() == ChildType)
		{
			return ChildWidget.ToSharedPtr();
		}
		else if (ChildWidget->GetAllChildren()->Num() > 0)
		{
			FoundWidget = FindWidgetInChildren(ChildWidget, ChildType);
			if (FoundWidget.IsValid() && FoundWidget->GetType() == ChildType)
			{
				return FoundWidget;
			}
		}
	}
	return FoundWidget;
}

TSharedRef<SWidget> STG_Palette::CreatePaletteTileMenu()
{
	return SAssignNew(TGActionMenuTileView,STG_ActionMenuTileView)
		.OnActionDragged(this, &STG_Palette::OnActionDragged)
		.OnCreateWidgetForAction(this, &STG_Palette::OnCreateWidgetForAction)
		.OnCollectAllActions(this, &STG_Palette::CollectAllActions)
		.AutoExpandActionMenu(true);
}

TSharedRef<SWidget> STG_Palette::CreatePaletteListMenu()
{
	return SNew(STG_GraphActionMenu)
		.OnActionDragged(this, &STG_Palette::OnActionDragged)
		.AutoExpandActionMenu(true)
		.bSpwanOnSelect(false)
		.GraphObj(Cast<UTG_EdGraph>(TGEditorPtr.Pin()->TG_EdGraph));
}

TSharedRef<SWidget> STG_Palette::CreateViewToggle()
{
	auto OnToggleChanged = [this](ECheckBoxState NewState, bool IsTileButton)
	{
		if (IsTileButton)
		{
			IsTileViewChecked = NewState == ECheckBoxState::Checked;
			IsListViewChecked = !IsTileViewChecked;
		}
		else
		{
			IsListViewChecked = NewState == ECheckBoxState::Checked;
			IsTileViewChecked = !IsListViewChecked;
		}

		CreatePaletteMenu();
	};

	return SNew(SBorder)
	.BorderImage(&BackgroundBrush)
	[

		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		[
			SNew(SOverlay)

			+SOverlay::Slot()
			[
				SNew(SCheckBox)
				.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("DetailsView.ChannelToggleButton"))
				.Type(ESlateCheckBoxType::ToggleButton)
				.CheckedImage(&CheckedBrush)
				.CheckedHoveredImage(&CheckedBrush)
				.UncheckedImage(&BackgroundBrush)
				.UncheckedHoveredImage(&BackgroundHoverBrush)
				.OnCheckStateChanged_Lambda(OnToggleChanged, true)
				.IsChecked(this, &STG_Palette::OnGetToggleCheckState, true)
				.Visibility(this,&STG_Palette::GetVisibility,true)
			]
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(FTG_Style::Get().GetBrush("TG_Editor.TileIcon"))
				.Visibility(EVisibility::HitTestInvisible)
			]
		]

		+ SHorizontalBox::Slot()
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				SNew(SCheckBox)
				.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("DetailsView.ChannelToggleButton"))
				.Type(ESlateCheckBoxType::ToggleButton)
				.CheckedImage(&CheckedBrush)
				.CheckedHoveredImage(&CheckedBrush)
				.UncheckedImage(&BackgroundBrush)
				.UncheckedHoveredImage(&BackgroundHoverBrush)
				.OnCheckStateChanged_Lambda(OnToggleChanged, false)
				.IsChecked(this, &STG_Palette::OnGetToggleCheckState, false)
				.Visibility(this, &STG_Palette::GetVisibility, false)
			]
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(FTG_Style::Get().GetBrush("TG_Editor.ListIcon"))
				.Visibility(EVisibility::HitTestInvisible)
			]
		]
	];
//.ColorAndOpacity(Color)
}

EVisibility STG_Palette::GetVisibility(bool IsTileView) const
{
	if (IsTileView)
	{
		return IsTileViewChecked ? EVisibility::HitTestInvisible : EVisibility::Visible;
	}
	else return IsListViewChecked ? EVisibility::HitTestInvisible : EVisibility::Visible;
}

ECheckBoxState STG_Palette::OnGetToggleCheckState(bool IsTileView) const
{
	if (IsTileView)
	{
		return IsTileViewChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	else return IsListViewChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

TSharedRef<SWidget> STG_Palette::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	return	SNew(STG_PaletteTileItem, InCreateData);
}

void STG_Palette::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	const UTG_EdGraphSchema* Schema = GetDefault<UTG_EdGraphSchema>();

	FGraphActionMenuBuilder ActionMenuBuilder;

	// Determine all possible actions
	Schema->GetPaletteActions(ActionMenuBuilder, GetFilterCategoryName());

	//@TODO: Avoid this copy
	OutAllActions.Append(ActionMenuBuilder);
}

FString STG_Palette::GetFilterCategoryName() const
{
	if (CategoryComboBox.IsValid())
	{
		return *CategoryComboBox->GetSelectedItem();
	}
	else
	{
		return TEXT("All");
	}
}

void STG_Palette::CategorySelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	RefreshActionsList(true);
}

void STG_Palette::AddAssetFromAssetRegistry(const FAssetData& InAddedAssetData)
{
	RefreshAssetInRegistry(InAddedAssetData);
}

void STG_Palette::RemoveAssetFromRegistry(const FAssetData& InAddedAssetData)
{
	RefreshAssetInRegistry(InAddedAssetData);
}

void STG_Palette::RenameAssetFromRegistry(const FAssetData& InAddedAssetData, const FString& InNewName)
{
	RefreshAssetInRegistry(InAddedAssetData);
}

void STG_Palette::RefreshAssetInRegistry(const FAssetData& InAddedAssetData)
{
	
}

void STG_Palette::RefreshActionsList(bool bPreserveExpansion)
{
	TGActionMenuTileView->RefreshAllActions(bPreserveExpansion);
}

#undef LOCTEXT_NAMESPACE
