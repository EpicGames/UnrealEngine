// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Framework/Commands/InputChord.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/ActionMenuWidgets/STG_GraphActionMenu.h"
#include <Widgets/Input/SCheckBox.h>
#include "SGraphPalette.h"

#include "Brushes/SlateRoundedBoxBrush.h"

class FTG_Editor;
class STG_ActionMenuTileView;
class SBox;
class STGGraphActionMenu;

/* This class is the custom Graph action menu item for Texture Graph*/
class STG_PaletteItem : public SGraphPaletteItem
{
public:
	SLATE_BEGIN_ARGS(STG_PaletteItem) {};
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData);

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

protected:
	virtual FString GetIconBrushName();

	virtual FString GetDefaultIconBrushName();
	/* Create the hotkey display widget */
	TSharedRef<SWidget> CreateHotkeyDisplayWidget(const TSharedPtr<const FInputChord> HotkeyChord);

	const FSlateBrush* GetIconBrush();

	virtual FText GetItemTooltip() const override;

	/** Delegate executed when mouse button goes down */
	FCreateWidgetMouseButtonDown MouseButtonDownDelegate;
};

/** Widget for displaying a single item  */
class STG_PaletteTileItem : public STG_PaletteItem
{
public:
	SLATE_BEGIN_ARGS( STG_PaletteTileItem ) {};
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData);

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

protected:
	virtual FString GetIconBrushName() override;

	virtual FString GetDefaultIconBrushName() override;

private:
	TSharedPtr<SBox> BackgroundArea;
	TSharedPtr<SImage> IconWidget;
	FSlateBrush BackgroundBrush;
};

//////////////////////////////////////////////////////////////////////////

class STG_Palette : public SGraphPalette
{
public:
	SLATE_BEGIN_ARGS( STG_Palette ) {};
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FTG_Editor> InTGEditorPtr);

protected:
	// SGraphPalette Interface
	virtual TSharedRef<SWidget> OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData) override;
	virtual void CollectAllActions(FGraphActionListBuilderBase& OutAllActions) override;
	virtual void RefreshActionsList(bool bPreserveExpansion) override;
	// End of SGraphPalette Interface

	void CreatePaletteMenu();

	TSharedRef<SWidget> CreatePaletteTileMenu();

	TSharedPtr<SWidget> FindWidgetInChildren(TSharedPtr<SWidget> Parent, FName ChildType) const;

	TSharedRef<SWidget> CreatePaletteListMenu();

	TSharedRef<SWidget> CreateViewToggle();

	/** Get the currently selected category name */
	FString GetFilterCategoryName() const;

	/** Callback for when the selected category changes */
	void CategorySelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);

	/** Callback from the Asset Registry when a new asset is added. */
	void AddAssetFromAssetRegistry(const FAssetData& InAddedAssetData);

	/** Callback from the Asset Registry when an asset is removed. */
	void RemoveAssetFromRegistry(const FAssetData& InAddedAssetData);

	/** Callback from the Asset Registry when an asset is renamed. */
	void RenameAssetFromRegistry(const FAssetData& InAddedAssetData, const FString& InNewName);

	void RefreshAssetInRegistry(const FAssetData& InAddedAssetData);

	EVisibility GetVisibility(bool IsTileView) const;

	ECheckBoxState OnGetToggleCheckState(bool IsTileView) const;

protected:
	bool IsTileViewChecked = true;
	bool IsListViewChecked = false;
	FSlateBrush BackgroundBrush;
	FSlateBrush BackgroundHoverBrush;
	FSlateBrush CheckedBrush;
	/** Pointer back to the material editor that owns us */
	TWeakPtr<FTG_Editor> TGEditorPtr;

	/** List of available Category Names */
	TArray< TSharedPtr<FString> > CategoryNames;

	/** Combo box used to select category */
	TSharedPtr<STextComboBox> CategoryComboBox;

	TSharedPtr<SWidget> ToggleButtons;

	TSharedPtr<STG_ActionMenuTileView> TGActionMenuTileView;
};
