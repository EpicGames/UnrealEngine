// Copyright Epic Games, Inc. All Rights Reserved.
#include "STG_TextureDetails.h"
#include "Widgets/Layout/SBox.h"
#include "TG_Pin.h"
#include "TG_Graph.h"
#include "Transform/Utility/T_TextureHistogram.h"
#include "STextureHistogram.h"
#include "Widgets/Layout/SSeparator.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateBorderBrush.h"

void STG_TextureDetails::Construct(const FArguments& InArgs)
{
	CheckedBrush = new FSlateRoundedBoxBrush(FLinearColor(0.039f, 0.039f, 0.039f, 1), CoreStyleConstants::InputFocusRadius);

	ChildSlot
		[
			SAssignNew(VerticalBox, SVerticalBox)
		];

	MakeControls();
}

void STG_TextureDetails::MakeControls()
{
	const float TEXT_PADDING = 2.0;
	VerticalBox->AddSlot()
	.Padding(TEXT_PADDING)
	.AutoHeight()
	.HAlign(HAlign_Fill)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.Padding(5.0f, 1.0f, 3.0f, 1.0f)
		[
			SAssignNew(RGBAButtons,STG_RGBAButtons)
		]
	];
			

	AddHistogramWidget();

	VerticalBox->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Fill)
	[
		SNew(SSeparator)
		.Thickness(1)
	];
}

void STG_TextureDetails::AddHistogramWidget()
{
	VerticalBox->AddSlot()
	.Padding(5, 5, 5, 10)
	[
		SNew(SBox)
		.MinDesiredHeight(300) // Set your desired minimum height here
		[
				SNew(SOverlay)

				+


				SOverlay::Slot()
				.Padding(2, 10, 2, 2)
				[
					SAssignNew(HistogramBlobWidgetR, STG_HistogramBlob)
					.Curves(ETG_HistogramCurves::R)
					.Visibility(this, &STG_TextureDetails::ShowR)
				]

				+

				SOverlay::Slot()
				.Padding(2, 10, 2, 2)
				[
					SAssignNew(HistogramBlobWidgetG, STG_HistogramBlob)
					.Curves(ETG_HistogramCurves::G)
					.Visibility(this, &STG_TextureDetails::ShowG)
				]

				+

				SOverlay::Slot()
				.Padding(2, 10, 2, 2)
				[
					SAssignNew(HistogramBlobWidgetB, STG_HistogramBlob)
					.Curves(ETG_HistogramCurves::B)
					.Visibility(this, &STG_TextureDetails::ShowB)
				]

				+

				SOverlay::Slot()
				.Padding(2, 10, 2, 2)
				[
					SAssignNew(HistogramBlobWidgetLuma, STG_HistogramBlob)
					.Curves(ETG_HistogramCurves::Luma)
					.Visibility(this, &STG_TextureDetails::ShowLuma)
				]
			
		]
	];
}

EVisibility STG_TextureDetails::ShowR() const
{
	return RGBAButtons && RGBAButtons->GetIsRChannel() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility STG_TextureDetails::ShowG() const
{
	return RGBAButtons && RGBAButtons->GetIsGChannel() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility STG_TextureDetails::ShowB() const
{
	return RGBAButtons && RGBAButtons->GetIsBChannel() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility STG_TextureDetails::ShowLuma() const
{
	return RGBAButtons && RGBAButtons->GetIsAChannel() ? EVisibility::Visible : EVisibility::Collapsed;
}

void STG_TextureDetails::ClearHistogramWidgets()
{
	if (UseBlobWidget)
	{
		HistogramBlobWidgetR->Clear();
		HistogramBlobWidgetG->Clear();
		HistogramBlobWidgetB->Clear();
		HistogramBlobWidgetLuma->Clear();
	}
}

void STG_TextureDetails::CalculateHistogram(BlobPtr InBlob, UTextureGraph* InTextureGraph)
{
	/// If the given blob is invalid then we clear the histogram widget
	if (!InBlob)
	{
		ClearHistogramWidgets();
		//Early out no need to calculate histogram for null source blob
		return;
	}

	/// If it's a transient blob then we don't calculate histogram, but retain the existing one
	if (InBlob->IsTransient())
		return;

	InBlob->OnFinalise()
		.then([this, InTextureGraph, InBlob]() mutable
		{
			// OnFinalise can sometimes occur after the editor is closed and thus can potentially
			// deallocate all corresponding slate objects
			if (DoesSharedInstanceExist() && !InBlob->IsTransient())
			{
				T_TextureHistogram::CreateOnService(InTextureGraph, std::static_pointer_cast<TiledBlob>(InBlob), 0);
				return InBlob->GetHistogram()->OnFinalise();
			}
			return (AsyncBlobResultPtr)(cti::make_ready_continuable<const Blob*>(nullptr));
		})
		.then([this, InBlob](const Blob* Histogram) mutable
		{
			if (DoesSharedInstanceExist() && !InBlob->IsTransient())
			{
				TiledBlobPtr FinalizedHistogram = std::static_pointer_cast<TiledBlob>(InBlob->GetHistogram());
				if (HistogramBlobWidgetR.IsValid())
				{
					HistogramBlobWidgetR->Update(FinalizedHistogram);
				}

				if (HistogramBlobWidgetG.IsValid())
				{
					HistogramBlobWidgetG->Update(FinalizedHistogram);
				}
		
				if (HistogramBlobWidgetB.IsValid())
				{
					HistogramBlobWidgetB->Update(FinalizedHistogram);
				}
		
				if (HistogramBlobWidgetLuma.IsValid())
				{
					HistogramBlobWidgetLuma->Update(FinalizedHistogram);
				}
			}
		});
}
