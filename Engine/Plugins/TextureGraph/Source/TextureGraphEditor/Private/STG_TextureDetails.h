// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TextureGraph.h"
#include "Widgets/SBoxPanel.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Widgets/SCompoundWidget.h"
#include "STextureHistogram.h"
#include "Widgets/STG_HistogramBlob.h"
#include "Widgets/STG_RGBAButtons.h"

struct FHistogramResult;

class STG_TextureDetails : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(STG_TextureDetails)		
	{}
	
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	void CalculateHistogram(BlobPtr InBlob, UTextureGraph* InTextureGraph);
private:
	TiledBlobPtr HistogramResult;

	bool UseBlobWidget = true;
	void MakeControls();

	TSharedPtr<SVerticalBox> VerticalBox;
	TSharedPtr<STG_RGBAButtons> RGBAButtons;
	TSharedPtr<STextureHistogram> TextureHistogramWidget;
	TSharedPtr<STG_HistogramBlob> HistogramBlobWidgetR;
	TSharedPtr<STG_HistogramBlob> HistogramBlobWidgetG;
	TSharedPtr<STG_HistogramBlob> HistogramBlobWidgetB;
	TSharedPtr<STG_HistogramBlob> HistogramBlobWidgetLuma;
	bool bShowHistogram = true;
	FSlateRoundedBoxBrush* CheckedBrush;

	void ClearHistogramWidgets();
	void AddHistogramWidget();
	EVisibility ShowR() const;
	EVisibility ShowG() const;
	EVisibility ShowB() const;
	EVisibility ShowLuma() const;
};
