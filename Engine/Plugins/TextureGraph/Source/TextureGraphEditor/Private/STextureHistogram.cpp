// Copyright Epic Games, Inc. All Rights Reserved.
#include "STextureHistogram.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Materials/Material.h"
#include <Materials/MaterialInstanceDynamic.h>
#include <Styling/SlateBrush.h>
#include <Widgets/SOverlay.h>
#include <Widgets/Images/SImage.h>
#include <Widgets/Text/STextBlock.h>
#include <Widgets/Layout/SBox.h>
#include <Widgets/SBoxPanel.h>
#include <Brushes/SlateDynamicImageBrush.h>
#include "Framework/Application/SlateApplication.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(STextureHistogram)

void STextureHistogram::Construct(const FArguments& InArgs)
{
	bDrawCombined = InArgs._DrawLayout == ETextureHistogramLayout::Combined;
	bDrawBars = InArgs._DrawStyle == ETextureHistogramStyle::Bar;

	if (bDrawCombined)
	{
		if (bDrawBars)
		{
			DrawCombineBarHistogram(InArgs);
		}
		else DrawCombineHistogramCurves(InArgs);
	}
	else
	{
		if (bDrawBars)
		{
			DrawSplitBarHistogram(InArgs);
		}
		else DrawSplitHistogramCurves(InArgs);
	}
}

void STextureHistogram::DrawCombineHistogramCurves(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SOverlay)

		+

		SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(TextureHistogramUtils::Margin)
		[
			SAssignNew(ReferenceLinesCombine, STextureHistogramReferenceLines)
			.Color(InArgs._ScaleLineColor)
			.Height(InArgs._Height)
			.Label("RGB Channels, Luminance")
		]

		+

		SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(TextureHistogramUtils::Margin)
		[
			SAssignNew(HistogramCurveR, STextureHistogramCurve)
			.bShadeCurve(true)
			.LineColor(InArgs._RCurveColor)
			.Height(InArgs._Height)
		]

		+

		SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(TextureHistogramUtils::Margin)
		[
			SAssignNew(HistogramCurveG, STextureHistogramCurve)
			.bShadeCurve(true)
			.LineColor(InArgs._GCurveColor)
			.Height(InArgs._Height)
		]

		+

		SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(TextureHistogramUtils::Margin)
		[
			SAssignNew(HistogramCurveA, STextureHistogramCurve)
			.bShadeCurve(true)
			.LineColor(InArgs._ACurveColor)
			.Height(InArgs._Height)
		]

		+

		SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(TextureHistogramUtils::Margin)
		[
			SAssignNew(HistogramCurveB, STextureHistogramCurve)
			.bShadeCurve(true)
			.LineColor(InArgs._BCurveColor)
			.Height(InArgs._Height)
		]
	];
}

void STextureHistogram::DrawSplitHistogramCurves(const FArguments& InArgs)
{
	ChildSlot
	[
		SAssignNew(SplitBox,SVerticalBox)

		+

		SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.FillHeight(0.25f)
		.Padding(TextureHistogramUtils::Margin)
		[
			SNew(SOverlay)
			+

			SOverlay::Slot()
			[
				SAssignNew(ReferenceLinesR, STextureHistogramReferenceLines)
				.Color(InArgs._ScaleLineColor)
				.Height(InArgs._Height)
				.Label("R - Channel")
			]

			+

			SOverlay::Slot()
			[
				SAssignNew(HistogramCurveR, STextureHistogramCurve)
				.bShadeCurve(true)
				.LineColor(InArgs._RCurveColor)
				.Height(InArgs._Height)
			]
		]

		+

		SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.FillHeight(0.25f)
		.Padding(TextureHistogramUtils::Margin)
		[
			SNew(SOverlay)

			+

			SOverlay::Slot()
			[
				SAssignNew(ReferenceLinesG, STextureHistogramReferenceLines)
				.Color(InArgs._ScaleLineColor)
				.Height(InArgs._Height)
				.Label("G - Channel")
			]

			+

			SOverlay::Slot()
			[
				SAssignNew(HistogramCurveG, STextureHistogramCurve)
				.bShadeCurve(true)
				.LineColor(InArgs._GCurveColor)
				.Height(InArgs._Height)
			]
		]

		+

		SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.FillHeight(0.25f)
		.Padding(TextureHistogramUtils::Margin)
		[
			SNew(SOverlay)

			+

			SOverlay::Slot()
			[
				SAssignNew(ReferenceLinesB, STextureHistogramReferenceLines)
				.Color(InArgs._ScaleLineColor)
				.Height(InArgs._Height)
				.Label("B - Channel")
			]

			+

			SOverlay::Slot()
			[
				SAssignNew(HistogramCurveB, STextureHistogramCurve)
				.bShadeCurve(true)
				.LineColor(InArgs._BCurveColor)
				.Height(InArgs._Height)
			]
		]

		+

		SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.FillHeight(0.25f)
		.Padding(TextureHistogramUtils::Margin)
		[
			SNew(SOverlay)

			+

			SOverlay::Slot()
			[
				SAssignNew(ReferenceLinesA, STextureHistogramReferenceLines)
				.Color(InArgs._ScaleLineColor)
				.Height(InArgs._Height)
				.Label("Luminance")
			]

			+

			SOverlay::Slot()
			[
				SAssignNew(HistogramCurveA, STextureHistogramCurve)
				.bShadeCurve(true)
				.LineColor(InArgs._ACurveColor)
				.Height(InArgs._Height)
			]	
		]
	];
}

void STextureHistogram::DrawCombineBarHistogram(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SOverlay)

		+

		SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(TextureHistogramUtils::Margin)
		[
			SAssignNew(ReferenceLinesCombine, STextureHistogramReferenceLines)
			.Color(InArgs._ScaleLineColor)
			.Height(InArgs._Height)
			.Label("RGB Channels, Luminance")
		]

		+

		SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(TextureHistogramUtils::Margin)
		[
			SAssignNew(HistogramBarsR, STextureHistogramBars)
			.ShadeColor(InArgs._RCurveColor)
			.Height(InArgs._Height)
		]

		+

		SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(TextureHistogramUtils::Margin)
		[
			SAssignNew(HistogramBarsG, STextureHistogramBars)
			.ShadeColor(InArgs._GCurveColor)
			.Height(InArgs._Height)
		]

		+

		SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(TextureHistogramUtils::Margin)
		[
			SAssignNew(HistogramBarsB, STextureHistogramBars)
			.ShadeColor(InArgs._BCurveColor)
			.Height(InArgs._Height)
		]

		+

		SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(TextureHistogramUtils::Margin)
		[
			SAssignNew(HistogramBarsA, STextureHistogramBars)
			.ShadeColor(InArgs._ACurveColor)
			.Height(InArgs._Height)
		]
	];
}

void STextureHistogram::DrawSplitBarHistogram(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)

		+

		SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(TextureHistogramUtils::Margin)
		[
			SNew(SOverlay)

			+

			SOverlay::Slot()
			[
				SAssignNew(ReferenceLinesR, STextureHistogramReferenceLines)
				.Color(InArgs._ScaleLineColor)
				.Height(InArgs._Height)
				.Label("R - Channel")
			]
			
			+

			SOverlay::Slot()
			[
				SAssignNew(HistogramBarsR, STextureHistogramBars)
				.ShadeColor(InArgs._RCurveColor)
				.Height(InArgs._Height)
			]
		]

		+

		SVerticalBox::Slot()
		.Padding(TextureHistogramUtils::Margin)
		[
			SNew(SOverlay)

			+

			SOverlay::Slot()
			[
				SAssignNew(ReferenceLinesG, STextureHistogramReferenceLines)
				.Color(InArgs._ScaleLineColor)
				.Height(InArgs._Height)
				.Label("G - Channel")
			]

			+

			SOverlay::Slot()
			[
				SAssignNew(HistogramBarsG, STextureHistogramBars)
				.ShadeColor(InArgs._GCurveColor)
				.Height(InArgs._Height)
			]
		]

		+

		SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(TextureHistogramUtils::Margin)
		[
			SNew(SOverlay)

			+

			SOverlay::Slot()
			[
				SAssignNew(ReferenceLinesB, STextureHistogramReferenceLines)
				.Color(InArgs._ScaleLineColor)
				.Height(InArgs._Height)
				.Label("B - Channel")
			]

			+

			SOverlay::Slot()
			[
				SAssignNew(HistogramBarsB, STextureHistogramBars)
				.ShadeColor(InArgs._BCurveColor)
				.Height(InArgs._Height)
			]
		]

		+

		SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(TextureHistogramUtils::Margin)
		[
			SNew(SOverlay)

			+

			SOverlay::Slot()
			[
				SAssignNew(ReferenceLinesA, STextureHistogramReferenceLines)
				.Color(InArgs._ScaleLineColor)
				.Height(InArgs._Height)
				.Label("Luminance")
			]

			+

			SOverlay::Slot()
			[
				SAssignNew(HistogramBarsA, STextureHistogramBars)
				.ShadeColor(InArgs._ACurveColor)
				.Height(InArgs._Height)
			]
		]
	];
}

void STextureHistogram::SetHistogramData(TArray<FVector4> InHistogramData)
{ 
	HistogramData = InHistogramData;
	
	if (bDrawBars)
	{
		HistogramBarsR->SetHistogramData(GetRValues());
		HistogramBarsG->SetHistogramData(GetGValues());
		HistogramBarsB->SetHistogramData(GetBValues());
		HistogramBarsA->SetHistogramData(GetLuminanceValues());
		
		if (bDrawCombined)
		{
			int MaxBin = GetMaxBin();
			HistogramBarsR->SetMax(MaxBin);
			HistogramBarsG->SetMax(MaxBin);
			HistogramBarsB->SetMax(MaxBin);
			HistogramBarsA->SetMax(MaxBin);

			ReferenceLinesCombine->SetMax(MaxBin);
		}
		else
		{
			int MaxR = HistogramBarsR->GetMaxBin();
			int MaxG = HistogramBarsG->GetMaxBin();
			int MaxB = HistogramBarsB->GetMaxBin();
			int MaxA = HistogramBarsA->GetMaxBin();

			HistogramBarsR->SetMax(MaxR);
			HistogramBarsG->SetMax(MaxG);
			HistogramBarsB->SetMax(MaxB);
			HistogramBarsA->SetMax(MaxA);

			ReferenceLinesR->SetMax(MaxR);
			ReferenceLinesG->SetMax(MaxG);
			ReferenceLinesB->SetMax(MaxB);
			ReferenceLinesA->SetMax(MaxA);
		}
	}
	else
	{
		HistogramCurveR->SetHistogramData(GetRValues());
		HistogramCurveG->SetHistogramData(GetGValues());
		HistogramCurveB->SetHistogramData(GetBValues());
		HistogramCurveA->SetHistogramData(GetLuminanceValues());

		if (bDrawCombined)
		{
			int MaxBin = GetMaxBin();
			HistogramCurveR->SetMax(MaxBin);
			HistogramCurveG->SetMax(MaxBin);
			HistogramCurveB->SetMax(MaxBin);
			HistogramCurveA->SetMax(MaxBin);

			ReferenceLinesCombine->SetMax(MaxBin);
		}
		else
		{
			int MaxR = HistogramCurveR->GetMaxBin();
			int MaxG = HistogramCurveG->GetMaxBin();
			int MaxB = HistogramCurveB->GetMaxBin();
			int MaxA = HistogramCurveA->GetMaxBin();

			HistogramCurveR->SetMax(MaxR);
			HistogramCurveG->SetMax(MaxG);
			HistogramCurveB->SetMax(MaxB);
			HistogramCurveA->SetMax(MaxA);

			ReferenceLinesR->SetMax(MaxR);
			ReferenceLinesG->SetMax(MaxG);
			ReferenceLinesB->SetMax(MaxB);
			ReferenceLinesA->SetMax(MaxA);
		}
	}
}

int STextureHistogram::GetMaxBin()
{
	int Max = 0;
	for (auto Data : HistogramData)
	{
		Max = FMath::Max3(FMath::Max3(Data.X, Data.Y, Data.Z), Data.W,(double)Max);
	}

	return Max;
}

TArray<int32> STextureHistogram::GetRValues() const
{
	TArray<int32> RValues;
	for (auto Data : HistogramData)
	{
		RValues.Add(Data.X);
	}

	return RValues;
}

TArray<int32> STextureHistogram::GetGValues() const
{
	TArray<int32> GValues;
	for (auto Data : HistogramData)
	{
		GValues.Add(Data.Y);
	}

	return GValues;
}

TArray<int32> STextureHistogram::GetBValues() const
{
	TArray<int32> BValues;
	for (auto Data : HistogramData)
	{
		BValues.Add(Data.Z);
	}

	return BValues;
}

TArray<int32> STextureHistogram::GetLuminanceValues() const
{
	TArray<int32> AValues;
	for (auto Data : HistogramData)
	{
		AValues.Add(Data.W);
	}

	return AValues;
}

void STextureHistogramCurve::Construct(const FArguments& InArgs)
{
	bShadeCurve = InArgs._bShadeCurve;
	LineColor = InArgs._LineColor;
	ShadeColor = FColor(LineColor.R * 255, LineColor.G * 255, LineColor.B * 255, LineColor.A * 255);

	ChildSlot
	[
		SNew(SOverlay)

		+

		SOverlay::Slot()
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			[
				SNew(SBorder)
				.Padding(5.0f, InArgs._Height)
				.BorderImage(FCoreStyle::Get().GetBrush("GenericWhiteBox"))
				[
					SAssignNew(Canvas, SCanvas)
				]
			]
		]
	];
}

void STextureHistogramCurve::SetHistogramData(const TArray<int32>& InHistogramData)
{
	HistogramData = InHistogramData;
}

int STextureHistogramCurve::GetMaxBin()
{
	int MaxValue = 0;
	for (auto Data : HistogramData)
	{
		MaxValue = FMath::Max(Data,(double)MaxValue);
	}

	return MaxValue;
}

void STextureHistogramCurve::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	Canvas->Invalidate(EInvalidateWidget::Layout);
}

int32 STextureHistogramCurve::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FSlateWindowElementList& ElementList = OutDrawElements;
	
	TArray<int32> Values = HistogramData;

	DrawHistogramCurve(AllottedGeometry, ElementList, LayerId, Values);

	return LayerId;
}

void STextureHistogramCurve::DrawHistogramCurve(const FGeometry& AllottedGeometry, FSlateWindowElementList& ElementList, int32 LayerId, TArray<int32> Values) const
{
	if (Values.IsEmpty())
	{
		return;
	}

	// Histogram parameters
	int32 NumBins = Values.Num();
	float MaxBarHeight = AllottedGeometry.GetLocalSize().Y; // Maximum height of the histogram bars
	float BarWidth = AllottedGeometry.GetLocalSize().X / NumBins; // Width of each histogram bar

	// Normalize histogram counts
	int32 MaxCount = FMath::Max(100,Max);
	TArray<float> NormalizedCounts;
	for (int32 Count : Values)
	{
		float NormalizedCount = Count / static_cast<float>(MaxCount) * MaxBarHeight;
		NormalizedCounts.Add(NormalizedCount);
	}

	// Calculate data points for the line
	TArray<FVector2D> LinePoints;
	for (int32 BinIndex = 0; BinIndex < NumBins; ++BinIndex)
	{
		float X = BinIndex * BarWidth + BarWidth * 0.5f; // X-coordinate in the middle of each bar
		float Y = MaxBarHeight - NormalizedCounts[BinIndex]; // Y-coordinate based on the normalized count
		LinePoints.Add(FVector2D(X, Y));
	}
	// Calculate data points for shading the area under the curve
	TArray<FVector2D> ShadePoints = LinePoints;
	ShadePoints.Insert(FVector2D(LinePoints[0].X, MaxBarHeight), 0);
	ShadePoints.Add(FVector2D(LinePoints[NumBins-1].X, MaxBarHeight));

	// Draw histogram line
	FSlateDrawElement::MakeLines(
		ElementList,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		ShadePoints,
		ESlateDrawEffect::None,
		LineColor, // Line color
		true,
		2.0
	);

	if (bShadeCurve)
	{
		DrawShadeForCurve(AllottedGeometry, ElementList, LayerId, ShadePoints);
	}
}

void STextureHistogramCurve::DrawShadeForCurve(const FGeometry& AllottedGeometry, FSlateWindowElementList& ElementList, int32 LayerId,TArray<FVector2D> ShadePoints) const
{
	// Create a vertex buffer for the filled polygon
	TArray<FSlateVertex> Vertices;
	TArray<SlateIndex> Indices;
	Vertices.Reserve(ShadePoints.Num() * 2);
	Indices.Reserve((ShadePoints.Num()) * 6);

	int32 StartVertexIndex = 0;
	int32 BottomVertexIndex = ShadePoints.Num() - 1;

	FVector2D Offset = AllottedGeometry.GetAccumulatedRenderTransform().GetTranslation();

	for (const FVector2D& Point : ShadePoints)
	{
		auto AbsolutePosition = AllottedGeometry.LocalToAbsolute(Point);
		FSlateVertex Vertex;// = FSlateVertex::Make<ESlateVertexRounding::Disabled>(AllottedGeometry.GetAccumulatedRenderTransform(), +FVector2D(0.5f + SelectionX1 + ArrowSize, 0.5f + ArrowY + ArrowSize) * Scale, AtlasOffset + FVector2D(0.0f, 1.0f) * AtlasUVSize, ArrowFillColor));
		Vertex.Position = AbsolutePosition;//Offset.X + Point.X;
		//Vertex.Position[1] = Offset.Y + Point.Y;
		Vertex.TexCoords[0] = 0;
		Vertex.TexCoords[1] = 0;
		Vertex.TexCoords[2] = Vertex.TexCoords[3] = 1.0f;
		Vertex.Color = ShadeColor;
		Vertices.Emplace(Vertex);

		FSlateVertex Vertex2;
		FVector2D BasePosition = FVector2D(Point.X, AllottedGeometry.GetLocalSize().Y);
		Vertex2.Position = AllottedGeometry.LocalToAbsolute(BasePosition);
		Vertex2.TexCoords[0] = 0;
		Vertex2.TexCoords[1] = 0;
		Vertex2.TexCoords[2] = Vertex.TexCoords[3] = 1.0f;
		Vertex2.Color = ShadeColor;
		Vertices.Emplace(Vertex2);
	}

	// Add indices for the filled polygon
	for (int32 Index = 0; Index < Vertices.Num() - 1 ; Index ++)
	{
		Indices.Add(Index);
		Indices.Add(Index + 1);
		Indices.Add(Index + 2);
	}

	const FSlateBrush* MyBrush = FCoreStyle::Get().GetBrush("GenericWhiteBox");
	FSlateResourceHandle Handle = MyBrush->GetRenderingResource();

	// Draw filled area under the curve
	FSlateDrawElement::MakeCustomVerts(
		ElementList,
		LayerId,
		Handle,
		Vertices,
		Indices,
		nullptr,
		0,
		0
	);
}

//////////////////////////// STextureHistogramBars ///////////////////////////////////////

void STextureHistogramBars::Construct(const FArguments& InArgs)
{
	ShadeColor = InArgs._ShadeColor;

	ChildSlot
	[
		SNew(SOverlay)

		+

		SOverlay::Slot()
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			[
				SNew(SBorder)
				.Padding(5.0f, InArgs._Height)
				.BorderImage(FCoreStyle::Get().GetBrush("GenericWhiteBox"))
				[
					SAssignNew(Canvas, SCanvas)
				]
			]
		]
	];
}

void STextureHistogramBars::SetHistogramData(const TArray<int32>& InHistogramData)
{
	HistogramData = InHistogramData;
	Max = GetMaxBin();
}

int STextureHistogramBars::GetMaxBin()
{
	int MaxValue = 0;
	for (auto Data : HistogramData)
	{
		MaxValue = FMath::Max(Data, (double)MaxValue);
	}

	return MaxValue;
}

void STextureHistogramBars::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	Canvas->Invalidate(EInvalidateWidget::Layout);
}

int32 STextureHistogramBars::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FSlateWindowElementList& ElementList = OutDrawElements;

	TArray<int32> Values = HistogramData;

	DrawHistogramBars(AllottedGeometry, ElementList, LayerId, Values, ShadeColor);
	return LayerId;
}

void STextureHistogramBars::DrawHistogramBars(const FGeometry& AllottedGeometry, FSlateWindowElementList& ElementList, int32 LayerId, TArray<int32> Values, FLinearColor BarColor) const
{
	/// Histogram parameters
	int32 NumBins = Values.Num();
	float MaxBarHeight = AllottedGeometry.GetLocalSize().Y; // Maximum height of the histogram bars
	float BarWidth = AllottedGeometry.GetLocalSize().X  / NumBins; // Width of each histogram bar

	// Normalize histogram counts
	int32 MaxCount = FMath::Max(100, Max);//*TArray<TOptional<int32>>::Max(HistogramData);
	TArray<float> NormalizedCounts;
	for (int32 Count : Values)
	{
		float NormalizedCount = Count / static_cast<float>(MaxCount) * MaxBarHeight;
		NormalizedCounts.Add(NormalizedCount);
	}

	const FSlateBrush* MyBrush = FCoreStyle::Get().GetBrush("GenericWhiteBox");//new FSlateColorBrush(FColor(1,1,1, BarColor.A));
	FLinearColor BarTint = FLinearColor(BarColor.R, BarColor.G, BarColor.B, BarColor.A);

	// Draw histogram bars
	FVector2D BarPosition(0.0f, MaxBarHeight);
	for (int32 BinIndex = 0; BinIndex < NumBins; ++BinIndex)
	{
		FSlateDrawElement::MakeBox(
			ElementList,
			LayerId,
			AllottedGeometry.ToPaintGeometry( FVector2D(BarWidth, NormalizedCounts[BinIndex]), FSlateLayoutTransform(FVector2D(BarPosition.X, MaxBarHeight - NormalizedCounts[BinIndex]))),
			MyBrush,
			ESlateDrawEffect::None,
			BarTint // Bar color
		);

		BarPosition.X += BarWidth;
	}
}

////////////////// STextureHistogramReferenceLines ///////////////////////////////////////////////

void STextureHistogramReferenceLines::Construct(const FArguments& InArgs)
{
	Color = InArgs._Color;
	NumBins = InArgs._NumBins;
	NumHorizontalScaleLines = InArgs._NumHorizontalScaleLines;
	NumVerticalScaleLines = InArgs._NumVerticalScaleLines;
	bDraw = InArgs._bDraw;
	bDrawBackground = InArgs._bDrawBackground;
	Label = InArgs._Label;

	ChildSlot
	[
		SNew(SOverlay)

		+

		SOverlay::Slot()
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			[
				SNew(SBorder)
				.Padding(5.0f, InArgs._Height)
				.BorderImage(FCoreStyle::Get().GetBrush("GenericWhiteBox"))
				[
					SAssignNew(Canvas, SCanvas)
				]
			]
		]
	];
}

void STextureHistogramReferenceLines::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	Canvas->Invalidate(EInvalidateWidget::Layout);
}

int32 STextureHistogramReferenceLines::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FSlateWindowElementList& ElementList = OutDrawElements;

	if (bDraw)
	{
		DrawReferenceLines(AllottedGeometry, ElementList, LayerId);
	}

	return LayerId;
}

void STextureHistogramReferenceLines::DrawReferenceLines(const FGeometry& AllottedGeometry, FSlateWindowElementList& ElementList, int32 LayerId) const
{
	// Histogram parameters
	int32 TotalBins = NumBins;
	float MaxBarHeight = AllottedGeometry.GetLocalSize().Y; // Maximum height of the histogram bars
	float BarWidth = AllottedGeometry.GetLocalSize().X / TotalBins; // Width of each histogram bar

	// Normalize histogram counts
	int32 MaxCount = FMath::Max(100, Max);

	//Draw ReferenceLine
	float ReferenceValue1 = MaxBarHeight; 
	float ReferenceValue2 = MaxBarHeight;
	TArray<FVector2D> ReferenceLinePoints;
	ReferenceLinePoints.Add(FVector2D(0.0f, 0.0f));
	ReferenceLinePoints.Add(FVector2D(0.0f, ReferenceValue1));
	ReferenceLinePoints.Add(FVector2D(AllottedGeometry.GetLocalSize().X, ReferenceValue1));

	FSlateDrawElement::MakeLines(
		ElementList,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		ReferenceLinePoints,
		ESlateDrawEffect::None,
		FLinearColor::White // Reference line color
	);

	// Draw vertical small reference lines
	int32 NumReferenceLines = NumVerticalScaleLines; // Number of reference lines to draw
	for (int32 LineIndex = 0; LineIndex < NumReferenceLines; ++LineIndex)
	{
		int Count = (MaxCount * (LineIndex + 1) / NumReferenceLines);
		float Y = MaxBarHeight - (MaxBarHeight * (float)Count / MaxCount);

		TArray<FVector2D> Points =
		{
			FVector2D(0.0f, Y),
			FVector2D(-5, Y)
		};
		FSlateDrawElement::MakeLines(
			ElementList,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			Points,
			ESlateDrawEffect::None,
			FLinearColor::White // Reference line color
		);

		//Full fadded Background Line
		if (bDrawBackground)
		{
			auto FadeWhite = FLinearColor::White;
			FadeWhite.A = 0.1f;
			TArray<FVector2D> Points2 =
			{
				FVector2D(0.0f, Y),
				FVector2D(AllottedGeometry.GetLocalSize().X, Y)
			};
			FSlateDrawElement::MakeLines(
				ElementList,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				Points2,
				ESlateDrawEffect::PreMultipliedAlpha,
				FadeWhite // Reference line color
			);
		}


		FString UnitText = FString::Printf(TEXT("%d"), Count);
		
		//Format Count to use small values for large numbers
		if (Count / 1000 > 0)
		{
			UnitText = FString::Printf(TEXT("%d K"), Count / 1000);
		}

		FSlateFontInfo FontInfo = FCoreStyle::GetDefaultFontStyle("Regular", 6);
		FVector2D TextSize = FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->Measure(UnitText, FontInfo);
		FVector2D TextPosition = FVector2D( -TextSize.X - 5.0f, Y - TextSize.Y * 0.5f);
		FSlateDrawElement::MakeText(
			ElementList,
			LayerId,
			AllottedGeometry.ToPaintGeometry(TextSize,FSlateLayoutTransform(TextPosition)),
			UnitText,
			FontInfo,
			ESlateDrawEffect::None,
			FLinearColor::White // Text color
		);
	}

	// Draw Horizontal small reference lines
	NumReferenceLines = NumHorizontalScaleLines;//NumBins / 32; // Number of reference lines to draw
	for (int32 LineIndex = 0; LineIndex < NumReferenceLines; ++LineIndex)
	{
		float X = (AllottedGeometry.GetLocalSize().X * (LineIndex + 1) / NumReferenceLines);
		float Y = -5;
		TArray<FVector2D> Points =
		{
			FVector2D(X, MaxBarHeight),
			FVector2D(X, MaxBarHeight - Y)
		};
		FSlateDrawElement::MakeLines(
			ElementList,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			Points,
			ESlateDrawEffect::None,
			FLinearColor::White // Reference line color
		);

		//Full fadded Background Line

		if (bDrawBackground)
		{
			auto FadeWhite = FLinearColor::White;
			FadeWhite.A = 0.1f;
			TArray<FVector2D> Points2 =
			{
				FVector2D(X, MaxBarHeight),
				FVector2D(X, 0)
			};
			FSlateDrawElement::MakeLines(
				ElementList,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				Points2,
				ESlateDrawEffect::PreMultipliedAlpha,
				FadeWhite // Reference line color
			);
		}

		FString UnitText = FString::Printf(TEXT("%d"), (TotalBins * (LineIndex + 1) / NumReferenceLines));
		FSlateFontInfo FontInfo = FCoreStyle::GetDefaultFontStyle("Regular", 6);
		FVector2D TextSize = FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->Measure(UnitText, FontInfo);
		FVector2D TextPosition = FVector2D( X - (TextSize.X * 0.5f), MaxBarHeight - (Y * 2) - TextSize.Y * 0.5f);
		FSlateDrawElement::MakeText(
			ElementList,
			LayerId,
			AllottedGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextPosition)),
			UnitText,
			FontInfo,
			ESlateDrawEffect::None,
			FLinearColor::White // Text color
		);
	}

	//Label below the reference line
	FString UnitText = Label;
	FSlateFontInfo FontInfo = FCoreStyle::GetDefaultFontStyle("Bold", 8);
	FVector2D TextSize = FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->Measure(UnitText, FontInfo);
	FVector2D TextPosition = FVector2D((AllottedGeometry.GetLocalSize().X / 2) - (TextSize.X * 0.5f), MaxBarHeight + 25 - (TextSize.Y * 0.5f));
	FSlateDrawElement::MakeText(
		ElementList,
		LayerId,
		AllottedGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextPosition)),
		UnitText,
		FontInfo,
		ESlateDrawEffect::None,
		FLinearColor::White // Text color
	);
}
