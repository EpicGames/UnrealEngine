// Copyright Epic Games, Inc. All Rights Reserved.
#include "TG_Editor.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Expressions/Output/TG_Expression_Output.h"
#include "Misc/MessageDialog.h"

#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Dialogs/DlgPickAssetPath.h"


#include "TextureGraphEditorModule.h"

#include "TextureGraph.h"
#include "TG_Graph.h"
#include "EdGraph/TG_EdGraph.h"
#include "EdGraph/TG_EdGraphSchema.h"
#include "EdGraph/TG_EdGraphNode.h"
#include "EdGraphUtilities.h"
#include "TG_Node.h"
#include "TG_Hash.h"
#include "TG_OutputSettings.h"
#include "TG_HelperFunctions.h"

#include "TG_EditorTabs.h"
#include "TG_EditorCommands.h"
#include "STG_EditorViewport.h"
#include "SPrimaryButton.h"
#include "SSimpleButton.h"

#include "HAL/PlatformApplicationMisc.h"

#include "AdvancedPreviewSceneModule.h"
#include "AssetEditorViewportLayout.h"
#include "Components/MeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "DetailsViewArgs.h"
#include "Editor.h"
#include "EditorViewportTabContent.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
//#include "GraphEditor.h"
#include "SGraphPanel.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "PropertyEditorModule.h"
#include "STG_TextureDetails.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include <Toolkits/AssetEditorToolkit.h>

#include "EngineAnalytics.h"
#include "GraphEditorActions.h"
#include "IMessageLogListing.h"
#include "MessageLogInitializationOptions.h"
#include "MessageLogModule.h"
#include "EdGraph/TG_EditorErrorReporter.h"
#include "Misc/UObjectToken.h"
#include "EdGraph/TG_EdGraphSchemaActions.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Expressions/Input/TG_Expression_Graph.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"
#include "Model/Mix/ViewportSettings.h"
#include "UObject/MetaData.h"
#include "UObject/ObjectSaveContext.h"
#include "STG_NodePreview.h"

#include "Expressions/Input/TG_Expression_InputParam.h"
#include "STG_OutputSelectionDlg.h"
#include "Widgets/ActionMenuWidgets/STG_GraphActionMenu.h"

DEFINE_LOG_CATEGORY(LogTextureGraphEditor);
#define LOCTEXT_NAMESPACE "TG_Editor"

void FTG_Editor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_TG_Editor", "Texture Graph Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(FTG_EditorTabs::GraphEditorId, FOnSpawnTab::CreateSP(this, &FTG_Editor::SpawnTab_GraphEditor))
		.SetDisplayName(LOCTEXT("GraphTab", "Graph"))
		.SetGroup(WorkspaceMenuCategoryRef);

	InTabManager->RegisterTabSpawner(FTG_EditorTabs::ViewportTabId, FOnSpawnTab::CreateSP(this, &FTG_Editor::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("ViewportTab", "3D Preview"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(FTG_EditorTabs::PropertiesTabId, FOnSpawnTab::CreateSP(this, &FTG_Editor::SpawnTab_TG_Properties))
		.SetDisplayName(LOCTEXT("DetailsTab", "Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(FTG_EditorTabs::PaletteTabId, FOnSpawnTab::CreateSP(this, &FTG_Editor::SpawnTab_Palette))
		.SetDisplayName(LOCTEXT("PaletteTab", "Palette"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette"));

	/*InTabManager->RegisterTabSpawner(FTG_EditorTabs::FindTabId, FOnSpawnTab::CreateSP(this, &FTG_Editor::SpawnTab_Find))
		.SetDisplayName(LOCTEXT("FindTab", "Find Results"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.FindResults"));*/

	InTabManager->RegisterTabSpawner(FTG_EditorTabs::PreviewSceneSettingsTabId, FOnSpawnTab::CreateSP(this, &FTG_Editor::SpawnTab_PreviewSceneSettings))
		.SetDisplayName(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(FTG_EditorTabs::ParameterDefaultsTabId, FOnSpawnTab::CreateSP(this, &FTG_Editor::SpawnTab_ParameterDefaults))
		.SetDisplayName(LOCTEXT("ParametersTab", "Parameters"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(FTG_EditorTabs::NodePreviewTabId, FOnSpawnTab::CreateSP(this, &FTG_Editor::SpawnTab_NodePreview))
		.SetDisplayName(LOCTEXT("NodePreviewTab", "Node Preview"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	/*InTabManager->RegisterTabSpawner(FTG_EditorTabs::OutputTabId, FOnSpawnTab::CreateSP(this, &FTG_Editor::SpawnTab_Output))
		.SetDisplayName(LOCTEXT("OutputTab", "Output"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));*/

	InTabManager->RegisterTabSpawner(FTG_EditorTabs::PreviewSettingsTabId, FOnSpawnTab::CreateSP(this, &FTG_Editor::SpawnTab_PreviewSettings))
		.SetDisplayName(LOCTEXT("PreviewSettingsTab", "3D Preview Settings"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	// Tab for Errors/Warnings Tab
	InTabManager->RegisterTabSpawner(FTG_EditorTabs::ErrorsTabId, FOnSpawnTab::CreateSP(this, &FTG_Editor::SpawnTab_Errors))
		.SetDisplayName(LOCTEXT("ErrorsTab", "Errors"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "MaterialEditor.TogglePlatformStats.Tab"));

	InTabManager->RegisterTabSpawner(FTG_EditorTabs::TextureDetailsTabId, FOnSpawnTab::CreateSP(this, &FTG_Editor::SpawnTab_TextureDetails))
		.SetDisplayName(LOCTEXT("TextureDetailsTab", "Node Histogram"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));
}

void FTG_Editor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::GraphEditorId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::ViewportTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::PropertiesTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::PaletteTabId);
	//InTabManager->UnregisterTabSpawner(FTG_EditorTabs::FindTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::PreviewSceneSettingsTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::ParameterDefaultsTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::NodePreviewTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::OutputTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::PreviewSettingsTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::ErrorsTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::TextureDetailsTabId);

	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
}

void FTG_Editor::OnTextureGraphPreSave(UObject* Object, FObjectPreSaveContext SaveContext)
{
	if (EditedTextureGraph == Object)
		EditedTextureGraph->TriggerUpdate(false);
}

void FTG_Editor::InitEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UTextureGraph* InTextureGraph)
{
	TG_Parameters = NewObject<UTG_Parameters>();
	OriginalTextureGraph = InTextureGraph;
	// We create a duplicate TextureGraph. All the modifications will be done in it.
	// All the modifications are applied to original TextureGraph when asset is saved.
	// Propagate all object flags except for RF_Standalone, otherwise the preview material won't GC once
	// the TS editor releases the reference.
	// overwrite the original TextureGraph in place by constructing a new one with the same name
	FName RuntimeGraphName = FName(OriginalTextureGraph->GetFName().ToString() + TEXT("_Runtime"));
	FObjectDuplicationParameters Params = InitStaticDuplicateObjectParams(OriginalTextureGraph, OriginalTextureGraph->GetOuter(), RuntimeGraphName,
	~RF_Standalone, UTextureGraph::StaticClass(), EDuplicateMode::Normal, EInternalObjectFlags::None);
	
	EditedTextureGraph = Cast<UTextureGraph>(StaticDuplicateObjectEx(Params));
#ifdef UE_BUILD_DEBUG
	EditedTextureGraph->Graph()->IsRuntime = 1;
#endif
	FCoreUObjectDelegates::OnObjectPreSave.AddSP(this, &FTG_Editor::OnTextureGraphPreSave);

	//Editor gets notified when rendering is done
	EditedTextureGraph->OnRenderDone.BindRaw(this, &FTG_Editor::OnRenderingDone);
	EditedTextureGraph->OnBatchQueued.BindRaw(this, &FTG_Editor::OnBatchQueued);

	FViewportSettings& ViewportSettings = EditedTextureGraph->GetSettings()->GetViewportSettings();

	ViewportSettings.OnViewportMaterialChangedEvent.AddSP(this, &FTG_Editor::OnViewportMaterialChanged);
	ViewportSettings.OnMaterialMappingChangedEvent.AddSP(this, &FTG_Editor::OnMaterialMappingChanged);
	EditedTextureGraph->GetSettings()->OnPreviewMeshChangedEvent.AddSP(this, &FTG_Editor::OnPreviewMeshChangedEvent);

	TextureGraphEngine::RegisterErrorReporter(EditedTextureGraph, std::make_shared<FTG_EditorErrorReporter>(this));

	TG_EdGraph = NewObject<UTG_EdGraph>(EditedTextureGraph, UTG_EdGraph::StaticClass(), NAME_None, RF_Transactional);
	TG_EdGraph->Schema = UTG_EdGraphSchema::StaticClass();

	TG_EdGraph->InitializeFromTextureGraph(EditedTextureGraph, SharedThis(this));

	TG_EdGraph->PinSelectionManager.OnPinSelectionUpdated.AddSP(this, &FTG_Editor::OnPinSelectionUpdated);

	EditedTextureGraph->Graph()->OnGraphChangedDelegate.AddSP(this, &FTG_Editor::OnGraphChanged);
	EditedTextureGraph->Graph()->OnTGNodeAddedDelegate.AddSP(this, &FTG_Editor::OnNodeAdded);
	EditedTextureGraph->Graph()->OnTGNodeRemovedDelegate.AddSP(this, &FTG_Editor::OnNodeRemoved);
	EditedTextureGraph->Graph()->OnTGNodeRenamedDelegate.AddSP(this, &FTG_Editor::OnNodeRenamed);

	GraphEditorWidget = CreateGraphEditorWidget();
	NodePreview = CreateNodePreviewWidget();
	TextureDetails = CreateTextureDetailsWidget();
	Palette = SNew(STG_Palette, SharedThis(this));

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bShowObjectLabel = false;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.ColumnWidth = 0.70f;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->OnFinishedChangingProperties().AddSP(this, &FTG_Editor::OnFinishedChangingProperties);

	// Settings details view
	FDetailsViewArgs SettingsViewArgs;
	SettingsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	SettingsViewArgs.bHideSelectionTip = true;
	SettingsViewArgs.NotifyHook = this;
	PreviewSettingsView = PropertyEditorModule.CreateDetailView(SettingsViewArgs);

	PreviewSettingsView->OnFinishedChangingProperties().AddSP(this, &FTG_Editor::OnFinishedChangingSettings);

	// Output details view
	FDetailsViewArgs OutputViewArgs;
	OutputViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	OutputViewArgs.bHideSelectionTip = true;
	OutputViewArgs.NotifyHook = this;
	OutputViewArgs.ColumnWidth = 0.70f;
	OutputView = PropertyEditorModule.CreateDetailView(OutputViewArgs);

	OutputView->OnFinishedChangingProperties().AddSP(this, &FTG_Editor::OnFinishedChangingSettings);

	FDetailsViewArgs ParameterViewArgs;
	ParameterViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	ParameterViewArgs.bHideSelectionTip = true;
	ParameterViewArgs.NotifyHook = this;
	ParameterViewArgs.ColumnWidth = 0.70f;
	ParameterView = PropertyEditorModule.CreateDetailView(ParameterViewArgs);

	UpdateParametersUI();

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions LogOptions;
	// Show Pages so that user is never allowed to clear log messages
	LogOptions.bShowPages = false;
	LogOptions.bShowFilters = false; //TODO - Provide custom filters? E.g. "Critical Errors" vs "Errors" needed for materials?
	LogOptions.bAllowClear = false;
	LogOptions.MaxPageCount = 1;
	ErrorsListing = MessageLogModule.CreateLogListing("TextureGraphLog", LogOptions);
	ErrorsListing->OnMessageTokenClicked().AddSP(this, &FTG_Editor::OnMessageLogLinkActivated);
	ErrorsWidget = MessageLogModule.CreateLogListingWidget(ErrorsListing.ToSharedRef());

	PropertyEditorModule.NotifyCustomizationModuleChanged();

	BindCommands();
	RegisterToolbar();

	EditedTextureGraph->SetFlags(RF_Transactional);
	GEditor->RegisterForUndo(this);

	// Setup layout 
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_TG_Editor_Layout_v0.0.8")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)->SetSizeCoefficient(0.9f)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.25f)
					->Split
					(
						FTabManager::NewStack()
						->SetHideTabWell(true)
						->AddTab(FTG_EditorTabs::PaletteTabId, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(FTG_EditorTabs::PropertiesTabId, ETabState::OpenedTab)
						->AddTab(FTG_EditorTabs::ParameterDefaultsTabId, ETabState::OpenedTab)
						->SetForegroundTab(FTG_EditorTabs::PropertiesTabId)
					)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(1.0f)
					->SetHideTabWell(true)
					->AddTab(FTG_EditorTabs::GraphEditorId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.25f)
					->Split
					(
						FTabManager::NewStack()
						->SetHideTabWell(true)
						->AddTab(FTG_EditorTabs::NodePreviewTabId, ETabState::OpenedTab)
						->AddTab(FTG_EditorTabs::TextureDetailsTabId, ETabState::OpenedTab)
						->SetForegroundTab(FTG_EditorTabs::NodePreviewTabId)
					)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(FTG_EditorTabs::ViewportTabId, ETabState::OpenedTab)
						->AddTab(FTG_EditorTabs::PreviewSettingsTabId, ETabState::OpenedTab)
						->AddTab(FTG_EditorTabs::PreviewSceneSettingsTabId, ETabState::ClosedTab)
						->SetForegroundTab(FTG_EditorTabs::ViewportTabId)
					)
				)
			)
		);
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, TG_EditorAppIdentifier, StandaloneDefaultLayout, /*bCreateDefaultToolbar*/ true, /*bCreateDefaultStandaloneMenu*/ true, InTextureGraph);

	RegenerateMenusAndToolbars();

	auto CurrentViewportTab = GetTabManager()->FindExistingLiveTab(FTG_EditorTabs::ViewportTabId);
	bool bViewportIsOff = CurrentViewportTab == nullptr;
	
	// check here if 3d viewport is turned off, we need to turn it on temporarily to initialize our systems correctly
	if (bViewportIsOff)
	{
		CurrentViewportTab = GetTabManager()->TryInvokeTab(FTG_EditorTabs::ViewportTabId);	
	}
	
	// Set the preview mesh for the material.  This call must occur after the toolbar is initialized.
	SetViewportPreviewMesh();

	if (bViewportIsOff)
	{
		// turn the viewporttab back off
		CurrentViewportTab->RequestCloseTab();
	}

	EditedTextureGraph->GetPackage()->ClearDirtyFlag();

	// Add analytics tag
	if (FEngineAnalytics::IsAvailable())
	{
		SessionStartTime = FDateTime::Now();
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.TextureGraph.SessionStarted"));
	}
}

FTG_Editor::FTG_Editor()
	:EditedTextureGraph(nullptr)
{

}

FTG_Editor::~FTG_Editor()
{
	// cleanup events
	EditedTextureGraph->OnRenderDone.Unbind();
	EditedTextureGraph->OnBatchQueued.Unbind();
	//EditedTextureGraph->Graph()->OnGraphChangedDelegate.Remove();

	EditedTextureGraph = nullptr;
	OriginalTextureGraph = nullptr;

	TG_Parameters->Parameters.Empty();
	TG_Parameters = nullptr;

	DetailsView.Reset();
	ParameterView.Reset();

	GEditor->UnregisterForUndo(this);
}

FLinearColor FTG_Editor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

FName FTG_Editor::GetToolkitFName() const
{
	return FName("TG_Editor");
}

FText FTG_Editor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "TG_ Editor");
}

FString FTG_Editor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "TG_ ").ToString();
}

void FTG_Editor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(OriginalTextureGraph);
	Collector.AddReferencedObject(EditedTextureGraph);
	Collector.AddReferencedObject(TG_EdGraph);
	Collector.AddReferencedObject(TG_Parameters);
}

class UMixInterface* FTG_Editor::GetTextureGraphInterface() const
{
	return EditedTextureGraph;
}

void FTG_Editor::Tick(float DeltaTime)
{
	RefreshViewport();
	RefreshErrors();
}

TStatId FTG_Editor::GetStatId() const
{
	return TStatId();
}

FText FTG_Editor::GetOriginalObjectName() const
{
	return FText::FromString(GetEditingObjects()[0]->GetName());
}

TSharedPtr<IAssetReferenceFilter> FTG_Editor::MakeAssetReferenceFilter() const
{
	if (GEditor)
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;
		AssetReferenceFilterContext.AddReferencingAssets(GetEditingObjects());
		return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
	}

	return {};
}

UTG_Expression* FTG_Editor::CreateNewExpression(UClass* NewExpressionClass, const FVector2D& NodePos, bool bAutoSelect, bool bAutoAssignResource, const class UEdGraph* Graph /*= nullptr*/)
{
	check(NewExpressionClass->IsChildOf(UTG_Expression::StaticClass()));

	UTG_EdGraph* ExpressionGraph = Graph ? ToRawPtr(CastChecked<UTG_EdGraph>(const_cast<UEdGraph*>(Graph))) : ToRawPtr(TG_EdGraph);
	ExpressionGraph->Modify();

	// Create the new expression

	EditedTextureGraph->Modify();

	return nullptr;
}

void FTG_Editor::RegisterToolbar()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	FName ParentName;
	const FName MenuName = GetToolMenuToolbarName(ParentName);

	UToolMenu* ToolBar = ToolMenus->IsMenuRegistered(MenuName) ? ToolMenus->FindMenu(MenuName) : ToolMenus->RegisterMenu(MenuName, ParentName, EMultiBoxType::ToolBar);

	const FTG_EditorCommands& TG_EditorCommands = FTG_EditorCommands::Get();

	const FToolMenuInsert InsertAfterAssetSection("Asset", EToolMenuInsertType::After);
	{
		FToolMenuSection& Section = ToolBar->AddSection("TG_Toolbar", TAttribute<FText>(), InsertAfterAssetSection);
		
		Section.AddEntry(FToolMenuEntry::InitToolBarButton(
			FTG_EditorCommands::Get().ExportAsUAsset,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Export")));
		
		Section.AddEntry(FToolMenuEntry::InitToolBarButton(
			FTG_EditorCommands::Get().RunGraph,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "FontEditor.Update")));

		Section.AddEntry(FToolMenuEntry::InitToolBarButton(
			FTG_EditorCommands::Get().AutoUpdateGraph,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "FontEditor.Update")));

#if UE_BUILD_DEBUG
		Section.AddEntry(FToolMenuEntry::InitToolBarButton(
			FTG_EditorCommands::Get().LogGraph,
			TAttribute<FText>(),
			FText::FromString("Log current state of Texture Graph (only available in debug builds)"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "FontEditor.Update")));
#endif

		{
			FToolMenuSection& TabsSection = ToolBar->AddSection("Texture Graph Windows", TAttribute<FText>(), InsertAfterAssetSection);

			FToolMenuEntry TabsMenu = FToolMenuEntry::InitComboButton(
				"Windows",
				FUIAction(),
				FNewToolMenuDelegate::CreateLambda([this](UToolMenu* InSubMenu)
				{
					FToolMenuSection& Section = InSubMenu->FindOrAddSection("TextureGraphWindowsOptions");
					if (WorkspaceMenuCategory)
					{
						for (auto& item : WorkspaceMenuCategory->GetChildItems())
						{
							TSharedPtr<FTabSpawnerEntry> Spawner = item->AsSpawnerEntry();
							Section.AddMenuEntry(Spawner->GetTabType(),
								Spawner->GetDisplayName(),
								Spawner->GetTooltipText(),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(this, &FTG_Editor::HandleTabWindowSelected, Spawner->GetTabType(), Spawner->GetDisplayName().ToString()),
									FCanExecuteAction(),
									FIsActionChecked::CreateLambda([this,&item]() {return GetTabSelected(item->AsSpawnerEntry()->GetTabType()); })
								),
								EUserInterfaceActionType::Check);
						}
					}
				}),
				LOCTEXT("TextureGraphWindows_Label", "Windows"),
				LOCTEXT("TextureGraphWindows_Tooltip", "Show/Hide Texture Graph tab windows"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette")
			);
			TabsMenu.StyleNameOverride = "CalloutToolbar";
			TabsSection.AddEntry(TabsMenu);
		}
	}
}

void FTG_Editor::HandleTabWindowSelected(const FName TabID, const FString OuputName)
{
	TSharedPtr<SDockTab> Tab = GetTabManager()->FindExistingLiveTab(TabID);
	if (Tab)
	{
		Tab->RequestCloseTab();
	}
	else
	{
		GetTabManager()->TryInvokeTab(TabID);
	}
}

bool FTG_Editor::GetTabSelected(const FName TabID)
{
	TSharedPtr<SDockTab> Tab = GetTabManager()->FindExistingLiveTab(TabID);
	if (Tab)
	{
		return true;
	}

	return false;
}

bool FTG_Editor::IsShowingAutoUpdate() const
{
	return bAutoRunGraph;
}

void FTG_Editor::ToggleAutoUpdate()
{
	bAutoRunGraph = !bAutoRunGraph;
	if (bAutoRunGraph)
	{
		OnRunGraph_Clicked();
	}
}

bool FTG_Editor::CanEnableOnRun()
{
	return !bAutoRunGraph;
}

void FTG_Editor::PostInitAssetEditor()
{
	// (Copied from FUVEditorToolkit::PostInitAssetEditor)
	// We need the viewport client to start out focused, or else it won't get ticked until we click inside it.
	if (TSharedPtr<STG_EditorViewport> Viewport = GetEditorViewport())
	{
		FEditorViewportClient& ViewportClient = Viewport->GetViewportClient();
		ViewportClient.ReceivedFocus(ViewportClient.Viewport);
	}
}


void FTG_Editor::BindCommands()
{
	const FTG_EditorCommands& TGEditorCommands = FTG_EditorCommands::Get();

	ToolkitCommands->MapAction(
		TGEditorCommands.RunGraph,
		FExecuteAction::CreateSP(this, &FTG_Editor::OnRunGraph_Clicked),
		FCanExecuteAction::CreateSP(this, &FTG_Editor::CanEnableOnRun));

	ToolkitCommands->MapAction(
		TGEditorCommands.AutoUpdateGraph,
		FExecuteAction::CreateSP(this, &FTG_Editor::ToggleAutoUpdate),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FTG_Editor::IsShowingAutoUpdate));
	
	ToolkitCommands->MapAction(
		TGEditorCommands.LogGraph,
		FExecuteAction::CreateSP(this, &FTG_Editor::OnLogGraph_Clicked));

	ToolkitCommands->MapAction(
		TGEditorCommands.ExportAsUAsset,
		FExecuteAction::CreateSP(this, &FTG_Editor::ExportAsUAsset));

	ToolkitCommands->MapAction(
		FGenericCommands::Get().Undo,
		FExecuteAction::CreateSP(this, &FTG_Editor::UndoGraphAction));

	ToolkitCommands->MapAction(
		FGenericCommands::Get().Redo,
		FExecuteAction::CreateSP(this, &FTG_Editor::RedoGraphAction));
	
	GraphEditorCommands->MapAction(
		TGEditorCommands.ConvertInputParameterToConstant,
		FExecuteAction::CreateSP(this, &FTG_Editor::OnConvertInputParameterToFromConstant));

	GraphEditorCommands->MapAction(
		TGEditorCommands.ConvertInputParameterFromConstant,
		FExecuteAction::CreateSP(this, &FTG_Editor::OnConvertInputParameterToFromConstant));

	GraphEditorCommands->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &FTG_Editor::OnRenameNodeClicked),
		FCanExecuteAction::CreateSP(this, &FTG_Editor::CanRenameNode));

}

void FTG_Editor::OnRunGraph_Clicked()
{
	// Trigger and Evaluation of the TextureGraph
	if (EditedTextureGraph)
	{
		EditedTextureGraph->TriggerUpdate(false);
	}
}
void FTG_Editor::OnRenderingDone(UMixInterface* TextureGraph, const FInvalidationDetails* Details)
{
	if (TextureGraph != nullptr)
	{
	}
}

void FTG_Editor::OnBatchQueued(UMixInterface* TextureGraph, const FInvalidationDetails* Details)
{
	if (TextureGraph != nullptr)
	{
		GetEditorViewport()->UpdateRenderMode();
		RefreshNodePreview(GraphEditorWidget->GetSelectedNodes(), Details, true);
		RefreshViewport();
	}
}

void FTG_Editor::OnPreviewMeshChangedEvent()
{
	SetViewportPreviewMesh();
}

void FTG_Editor::OnViewportMaterialChanged()
{
	const UTG_Node* FirstTargetNode = nullptr;
	EditedTextureGraph->Graph()->ForEachNodes([&](const UTG_Node* node, uint32 index)
		{
			if (UTG_Expression_Output* OutputExpression = dynamic_cast<UTG_Expression_Output*>(node->GetExpression()))
			{
				if (!FirstTargetNode)
				{
					FirstTargetNode = node;
				}
			}
		});

	FViewportSettings& ViewportSettings = EditedTextureGraph->GetSettings()->GetViewportSettings();

	if (FirstTargetNode && ViewportSettings.MaterialMappingInfos.Num() > 0)
	{
		ViewportSettings.SetDefaultTarget(FirstTargetNode->GetNodeName());
	}

	GetEditorViewport()->GenerateRendermodeToolbar();
	GetEditorViewport()->InitRenderModes(EditedTextureGraph);
}

void FTG_Editor::OnMaterialMappingChanged()
{
	GetEditorViewport()->UpdateRenderMode();
}

void FTG_Editor::OnLogGraph_Clicked()
{
	// Log the TextureGraph for debug
	if (EditedTextureGraph)
	{
		EditedTextureGraph->Log();
	}
}

void FTG_Editor::UndoGraphAction()
{
	//FlushRenderingCommands();

	GEditor->UndoTransaction();
}

void FTG_Editor::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		if (TSharedPtr<SGraphEditor> FocusedGraphEd = GraphEditorWidget)
		{
			FocusedGraphEd->ClearSelectionSet();
		}

		OnGraphChanged(EditedTextureGraph->Graph(), nullptr, false);

		// Remove any tabs are that are pending kill or otherwise invalid UObject pointers.
		bool bNeedOpenGraphEditor = false;

		if (TSharedPtr<SGraphEditor> FocusedGraphEd = GraphEditorWidget)
		{
			// Active graph can change above, re-acquire ptr
			GraphEditorWidget->NotifyGraphChanged();
		}


		FSlateApplication::Get().DismissAllMenus();
	}
}

void FTG_Editor::RedoGraphAction()
{
	//FlushRenderingCommands();
	GEditor->RedoTransaction();
}

void FTG_Editor::ReplicateExtraNodes() const
{
	if (TG_EdGraph)
	{
		if (UTG_Graph* Graph = EditedTextureGraph->Graph())
		{
			TArray<TObjectPtr<const UObject>> ExtraNodes;
			for (const UEdGraphNode* GraphNode : TG_EdGraph->Nodes)
			{
				check(GraphNode);
				if (!GraphNode->IsA<UTG_EdGraphNode>())
				{
					ExtraNodes.Add(GraphNode);
				}

			}

			Graph->SetExtraEditorNodes(ExtraNodes);
		}
	}
}
void FTG_Editor::OnCreateComment()
{
	if (TG_EdGraph)
	{
		FTG_EdGraphSchemaAction_NewComment CommentAction;

		TSharedPtr<SGraphEditor> GraphEditorPtr = SGraphEditor::FindGraphEditorForGraph(TG_EdGraph);
		FVector2f Location = FVector2f::ZeroVector;
		if (GraphEditorPtr)
		{
			Location = GraphEditorPtr->GetPasteLocation2f();
		}

		CommentAction.PerformAction(TG_EdGraph, nullptr, Location);
	}
}

TArray<UTG_EdGraphNode*>	FTG_Editor::GetCurrentSelectedTG_EdGraphNodes() const
{
	TArray<UTG_EdGraphNode*> EdGraphNodes;
	if (GraphEditorWidget.IsValid())
	{
		auto SelectedNodes = GraphEditorWidget->GetSelectedNodes();
		for (const auto SelectedNode : SelectedNodes)
		{
			if (UTG_EdGraphNode* EdGraphNode = Cast<UTG_EdGraphNode>(SelectedNode))
			{
				check(EdGraphNode);
				EdGraphNodes.Add(EdGraphNode);
			}
		}
	}
	return EdGraphNodes;
}

void FTG_Editor::OnRenameNodeClicked()
{
	const FGraphPanelSelectionSet SelectedNodes = GraphEditorWidget->GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		UEdGraphNode* SelectedNode = Cast<UEdGraphNode>(*NodeIt);
		if (SelectedNode != nullptr && SelectedNode->GetCanRenameNode())
		{
			bool ToRename = true;
			GraphEditorWidget->IsNodeTitleVisible(SelectedNode, ToRename);
			break;
		}
	}
}

bool FTG_Editor::CanRenameNode() const
{
	TArray<UTG_EdGraphNode*> SelectedNodes = GetCurrentSelectedTG_EdGraphNodes();
	if ( SelectedNodes.Num() == 1)
	{
		return SelectedNodes[0]->bCanRenameNode;
	}

	return false;
}
void FTG_Editor::OnConvertInputParameterToFromConstant()
{
	TArray<UTG_EdGraphNode*> EdGraphNodes = GetCurrentSelectedTG_EdGraphNodes();
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "TGEditorConvertInputNode", "TS Editor: Convert Input Node"));

	for (auto EdNode : EdGraphNodes)
	{
		UTG_Node* TGNode = EdNode->GetNode();
		if (TGNode && TGNode->GetExpression())
		{
			UTG_Expression_InputParam* InputParamExp = Cast< UTG_Expression_InputParam>(TGNode->GetExpression());
			if (InputParamExp)
			{
				bool bIsConstant = InputParamExp->bIsConstant;
				if (bIsConstant)
					InputParamExp->SetbIsConstant(false);
				else
					InputParamExp->SetbIsConstant(true);
			}
		}
	}
}

bool FTG_Editor::OnVerifyNodeTextCommit(const FText& NewText, UEdGraphNode* NodeBeingChanged, FText& OutErrorMessage)
{
	bool bValid = true;

	if(UTG_EdGraphNode* EdNode = Cast<UTG_EdGraphNode>(NodeBeingChanged))
	{
		UTG_Node* TGNode = EdNode->GetNode();
		if( TGNode && NewText.ToString().Len() >= NAME_SIZE )
		{
			OutErrorMessage = FText::Format( LOCTEXT("TextureGraphNodeError_NameTooLong", "Node names must be less than {0} characters"), FText::AsNumber(NAME_SIZE));
			bValid = false;
		}
	}
	return bValid;
}

TSharedRef<class SGraphEditor> FTG_Editor::CreateGraphEditorWidget()
{
	if (!GraphEditorCommands)
	{
		GraphEditorCommands = MakeShareable(new FUICommandList);
	}

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FTG_Editor::OnSelectedNodesChanged);
	InEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &FTG_Editor::OnNodeDoubleClicked);
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &FTG_Editor::OnNodeTitleCommitted);
	InEvents.OnVerifyTextCommit = FOnNodeVerifyTextCommit::CreateSP(this, &FTG_Editor::OnVerifyNodeTextCommit);
	InEvents.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &FTG_Editor::OnCreateGraphActionMenu);
	//InEvents.OnSpawnNodeByShortcut = SGraphEditor::FOnSpawnNodeByShortcut::CreateSP(this, &FTG_Editor::OnSpawnGraphNodeByShortcut, static_cast<UEdGraph*>(InGraph));

	// Support the delete case
	GraphEditorCommands->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &FTG_Editor::DeleteSelectedNodes),
		FCanExecuteAction::CreateSP(this, &FTG_Editor::CanDeleteSelectedNodes));
	// Support the delete case
	GraphEditorCommands->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &FTG_Editor::CopySelectedNodes),
		FCanExecuteAction::CreateSP(this, &FTG_Editor::CanCopySelectedNodes));

	GraphEditorCommands->MapAction(FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &FTG_Editor::CutSelectedNodes),
		FCanExecuteAction::CreateSP(this, &FTG_Editor::CanCutSelectedNodes));

	GraphEditorCommands->MapAction(FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &FTG_Editor::DuplicateSelectedNodes),
		FCanExecuteAction::CreateSP(this, &FTG_Editor::CanDuplicateSelectedNodes));

	GraphEditorCommands->MapAction(FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &FTG_Editor::PasteNodes),
		FCanExecuteAction::CreateSP(this, &FTG_Editor::CanPasteNodes));

	GraphEditorCommands->MapAction(FGraphEditorCommands::Get().CreateComment,
		FExecuteAction::CreateSP(this, &FTG_Editor::OnCreateComment)
	);

	return SNew(SGraphEditor)
		.AdditionalCommands(GraphEditorCommands)
		.IsEditable(true)
		.Appearance(this, &FTG_Editor::GetGraphAppearance)
		.GraphToEdit(TG_EdGraph)
		.GraphEvents(InEvents)
		.ShowGraphStateOverlay(false)
		.AutoExpandActionMenu(true)
		.AssetEditorToolkit(this->AsShared());
	
}

FActionMenuContent FTG_Editor::OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
{
	TSharedRef<STG_GraphActionMenu> ActionMenu =
		SNew(STG_GraphActionMenu)
		.GraphObj(InGraph)
		.NewNodePosition(FDeprecateSlateVector2D(InNodePosition))
		.DraggedFromPins(InDraggedPins)
		.AutoExpandActionMenu(bAutoExpand)
		.OnClosedCallback(InOnMenuClosed);
		//.OnCloseReason(this, &FBlueprintEditor::OnGraphActionMenuClosed);

	return FActionMenuContent(ActionMenu, ActionMenu->GetFilterTextBox());
}

TSharedRef<STG_NodePreviewWidget> FTG_Editor::CreateNodePreviewWidget()
{
	return SNew(STG_NodePreviewWidget)
		.OnNodeBlobChanged(this, &FTG_Editor::OnSelectedBlobChanged);
}

TSharedRef<class STG_TextureDetails> FTG_Editor::CreateTextureDetailsWidget()
{
	return SNew(STG_TextureDetails);
}

FGraphAppearanceInfo FTG_Editor::GetGraphAppearance() const
{
	FGraphAppearanceInfo AppearanceInfo;

	AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_TG_", "TEXTURE GRAPH");

	return AppearanceInfo;
}

void FTG_Editor::BuildSubTools()
{
	FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");

	TArray<FAdvancedPreviewSceneModule::FDetailDelegates> Delegates;
	Delegates.Add({ OnPreviewSceneChangedDelegate });
	AdvancedPreviewSettingsWidget = AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(GetEditorViewport()->GetPreviewScene(), nullptr, TArray<FAdvancedPreviewSceneModule::FDetailCustomizationInfo>(),  TArray<FAdvancedPreviewSceneModule::FPropertyTypeCustomizationInfo>(), Delegates);

	if (PreviewSceneSettingsDockTab.IsValid())
	{
		PreviewSceneSettingsDockTab.Pin()->SetContent(AdvancedPreviewSettingsWidget.ToSharedRef());
	}
}

void FTG_Editor::OnEditorLayoutChanged()
{
	BuildSubTools();

	OnPreviewSceneChangedDelegate.Broadcast(GetEditorViewport()->GetPreviewScene());
}

TSharedRef<SDockTab> FTG_Editor::SpawnTab_GraphEditor(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::GraphEditorId);

	return SNew(SDockTab)
		.TabColorScale(GetTabColorScale())
		[
			GraphEditorWidget.ToSharedRef()
		];
}

void FTG_Editor::SetViewportPreviewMesh()
{
	TObjectPtr<UStaticMesh> PreviewMesh = EditedTextureGraph->GetSettings()->GetPreviewMesh();
	// Set the preview mesh for the material.  
	if (!PreviewMesh || !SetPreviewAsset(PreviewMesh))
	{
		// The material preview mesh couldn't be found or isn't loaded. Fallback to the one of the primitive types.
		GetEditorViewport()->InitPreviewMesh();
	}
}

TSharedRef<SDockTab> FTG_Editor::SpawnTab_Viewport(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::ViewportTabId);

	TSharedRef< SDockTab > DockableTab =
		SNew(SDockTab);
	MakeViewportFunc = [this](const FAssetEditorViewportConstructionArgs& InArgs)
	{
		return SNew(STG_EditorViewport)
			.InTextureGraph(EditedTextureGraph);
	};
	// Create a new tab
	ViewportTabContent = MakeShareable(new FEditorViewportTabContent());
	ViewportTabContent->OnViewportTabContentLayoutChanged().AddRaw(this, &FTG_Editor::OnEditorLayoutChanged);
	const FString LayoutId = FString("TG_EditorViewport");
	ViewportTabContent->Initialize(MakeViewportFunc, DockableTab, LayoutId);
	// This call must occur after the toolbar is initialized.
	SetViewportPreviewMesh();
	
	return DockableTab;
}

TSharedRef<SDockTab> FTG_Editor::SpawnTab_TG_Properties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::PropertiesTabId);

	TSharedPtr<SDockTab> DetailsTab = SNew(SDockTab)
		[
			DetailsView.ToSharedRef()
		];

	SpawnedDetailsTab = DetailsTab;

	return DetailsTab.ToSharedRef();
}

TSharedRef<SDockTab> FTG_Editor::SpawnTab_Palette(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::PaletteTabId);

	TSharedRef<SDockTab> DockableTab = SNew(SDockTab)
		[
			Palette.ToSharedRef()
		];
	PaletteTab = DockableTab;
	return DockableTab;
}

TSharedRef<SDockTab> FTG_Editor::SpawnTab_Find(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::FindTabId);

	return SNew(SDockTab);
}

TSharedRef<SDockTab> FTG_Editor::SpawnTab_PreviewSceneSettings(const FSpawnTabArgs& Args)
{
	check( Args.GetTabId() == FTG_EditorTabs::PreviewSceneSettingsTabId );
	return SAssignNew(PreviewSceneSettingsDockTab, SDockTab)
		.Label( LOCTEXT("TG_EditorPreviewSceneSettings_TabTitle", "Preview Scene Settings") )
		[
			AdvancedPreviewSettingsWidget.IsValid() ? AdvancedPreviewSettingsWidget.ToSharedRef() : SNullWidget::NullWidget
		];
}

TSharedRef<SDockTab> FTG_Editor::SpawnTab_ParameterDefaults(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::ParameterDefaultsTabId);

	return SNew(SDockTab)
		[
			SNew(SBox)
				[
					ParameterView.ToSharedRef()
				]
		];
}

TSharedRef<SDockTab> FTG_Editor::SpawnTab_NodePreview(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::NodePreviewTabId);

	return SNew(SDockTab)
		// .Label(LOCTEXT("NodeViewer", "Node Viewer"))
		.TabColorScale(GetTabColorScale())
		[
			NodePreview.ToSharedRef()
		];
}

TSharedRef<SDockTab> FTG_Editor::SpawnTab_Output(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::OutputTabId);

	TSharedPtr<SDockTab> OutputTab = SNew(SDockTab)
		.Label(LOCTEXT("Outputs", "Outputs"))
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.VAlign(VAlign_Fill)
						[
							OutputView.ToSharedRef()
						]
						+ SVerticalBox::Slot()
						.VAlign(VAlign_Bottom)
						.HAlign(HAlign_Fill)
						.AutoHeight()
						[
							SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.Padding(10)
								.FillWidth(0.8)
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Left)
								[
									SNew(STextBlock)
										.Text(FText::FromName("Export Texture Output"))
								]

								+ SHorizontalBox::Slot()
								.Padding(10)
								.FillWidth(0.2)
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Right)
								[
									SNew(SBox)
										.MinDesiredWidth(300)
										[
											SNew(SPrimaryButton)
												.Text(LOCTEXT("Export", "Export"))
												.OnClicked(this, &FTG_Editor::OnExportClick)
										]
								]
						]
				]
		];

	return OutputTab.ToSharedRef();
}

void FTG_Editor::ExportAsUAsset()
{
	OnExportClick();
}

FReply FTG_Editor::OnExportClick()
{
	TSharedPtr<STG_OutputSelectionDlg> ExportSelectionWidget =
		SNew(STG_OutputSelectionDlg)
		.Title(LOCTEXT("Export Selection", "Choose outputs to export"))
		.EdGraph(TG_EdGraph);

	if (ExportSelectionWidget->ShowModal() == EAppReturnType::Ok)
	{
		FTG_HelperFunctions::ExportAsync(EditedTextureGraph, "", "", this->TargetExportSettings, false);
	}
	return FReply::Handled();
}

TSharedRef<SDockTab> FTG_Editor::SpawnTab_PreviewSettings(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::PreviewSettingsTabId);

	TSharedPtr<SDockTab> PreviewSettingsTab = SNew(SDockTab)
		[
			PreviewSettingsView.ToSharedRef()
		];

	GetPreviewSettingsView()->SetObject(EditedTextureGraph->GetSettings(), true);

	return PreviewSettingsTab.ToSharedRef();
}

TSharedRef<SDockTab> FTG_Editor::SpawnTab_Errors(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::ErrorsTabId);

	TSharedPtr<SDockTab> ErrorTab = SNew(SDockTab)
		[
			ErrorsWidget.ToSharedRef()
		];

	return ErrorTab.ToSharedRef();
}

TSharedRef<SDockTab> FTG_Editor::SpawnTab_TextureDetails(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::TextureDetailsTabId);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.TabColorScale(GetTabColorScale())
		[
			TextureDetails.ToSharedRef()
		];

	NodeHistogramTab = SpawnedTab;
	return SpawnedTab;
}

TSharedPtr<STG_EditorViewport> FTG_Editor::GetEditorViewport() const
{
	if (ViewportTabContent.IsValid())
	{
		// we can use static cast here b/c we know in this editor we will have a static mesh viewport 
		return StaticCastSharedPtr<STG_EditorViewport>(ViewportTabContent->GetFirstViewport());
	}

	return nullptr;
}

void FTG_Editor::OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection)
{
	TArray<UObject*> SelectedObjects;

	UObject* EditObject = nullptr;//TODO: set EditObject to TextureGraph so we can use it for selection when no node is selected 

	//TODO: handle the selection cases
	//Case 1: when no node is selected
	//Case 2: when comments node is selected

	for (TSet<class UObject*>::TConstIterator SetIt(NewSelection); SetIt; ++SetIt)
	{
		if (UTG_EdGraphNode* GraphNode = Cast<UTG_EdGraphNode>(*SetIt))
		{
			SelectedObjects.Add(GraphNode->GetDetailsObject());

			// calling this here to set EditCondition metadata properly for Details panel
			// this is driven from the CanEditChange() method of the expression which can be customized as required
			GraphNode->UpdatePinsEditCondition();
		}  
	}
	
	GetDetailView()->SetObjects(SelectedObjects, true);
	FocusDetailsPanel();

	RefreshPreviewViewport();
	RefreshNodePreview(NewSelection, nullptr, false);
}

void FTG_Editor::OnNodeDoubleClicked(UEdGraphNode* Node)
{
	UTG_EdGraphNode* GraphNode = Cast<UTG_EdGraphNode>(Node);
	UTG_Node* TGNode = GraphNode ? GraphNode->GetNode() : nullptr;

	if(TGNode && TGNode->GetExpression())
	{
		const UTG_Expression_TextureGraph* TextureGraphExpression = Cast<UTG_Expression_TextureGraph>(TGNode->GetExpression());

		if (TextureGraphExpression && TextureGraphExpression->TextureGraph)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(TextureGraphExpression->TextureGraph);
		}
	}
}


void FTG_Editor::DeleteSelectedNodes()
{
	if (GraphEditorWidget.IsValid())
	{
		auto SelectedNodes = GraphEditorWidget->GetSelectedNodes();

		TArray<UEdGraphNode*> EdGraphNodes;
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			EdGraphNodes.Add(CastChecked<UEdGraphNode>(*NodeIt));
		}

		bool bChanged = DeleteNodes(EdGraphNodes);

		if (bChanged)
		{
			GraphEditorWidget->ClearSelectionSet();
			// TODO: Should we then notify the TextureGraph ?
		}
	}
}

bool FTG_Editor::DeleteNodes(TArray<UEdGraphNode*> NodesToDelete, bool ForceDelete /* = false */)
{
	UTG_Graph* TextureGraph = EditedTextureGraph->Graph();
	check(TG_EdGraph && TextureGraph);

	bool bGraphUpdated = false;

	if (NodesToDelete.Num() > 0)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "TGEditorDeleteNode", "TG Editor: Delete Node"));
		TG_EdGraph->Modify();

		for (UEdGraphNode* EdGraphNode : NodesToDelete)
		{
			if (EdGraphNode->CanUserDeleteNode() || ForceDelete)
			{
				if(const UTG_EdGraphNode* TGEdGraphNode = Cast<UTG_EdGraphNode>(EdGraphNode))
				{
					UTG_Node* TGNode = TGEdGraphNode->GetNode();
					check(TGNode);

					NodePreview->NodeDeleted(TGNode);
					TextureGraph->RemoveNode(TGNode);
				}
				EdGraphNode->DestroyNode();
				bGraphUpdated = true;	
			}
		}

		if (bGraphUpdated)
		{
			GraphEditorWidget->NotifyGraphChanged();
		}
	}

	return bGraphUpdated;
}

bool FTG_Editor::CanDeleteSelectedNodes() const
{
	return true;
}

void FTG_Editor::CopySelectedNodes()
{
	// Export the selected nodes and place the text on the clipboard
	const FGraphPanelSelectionSet SelectedNodes = GraphEditorWidget->GetSelectedNodes();

	FString ExportedText;

	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter))
		{
			Node->PrepareForCopying();
		}
	}

	FEdGraphUtilities::ExportNodesToText(SelectedNodes, /*out*/ ExportedText);
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);

	//Call PostCopyNode to do what ever is needed to do in Post Copy
	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(SelectedNodes); SelectedIter; ++SelectedIter)
	{
		if (UTG_EdGraphNode* Node = Cast<UTG_EdGraphNode>(*SelectedIter))
		{
			Node->PostCopyNode();
		}
	}
}

bool FTG_Editor::CanCopySelectedNodes() const
{
	return true;
}

void FTG_Editor::CutSelectedNodes()
{
	CopySelectedNodes();
	// Cut should only delete nodes that can be duplicated
	DeleteSelectedDuplicatableNodes();
}

bool FTG_Editor::CanCutSelectedNodes() const
{
	return CanCopySelectedNodes() && CanDeleteSelectedNodes();
}

void FTG_Editor::DuplicateSelectedNodes()
{
	// Copy and paste current selection
	CopySelectedNodes();
	PasteNodes();
}

bool FTG_Editor::CanDuplicateSelectedNodes() const
{
	return CanCopySelectedNodes();
}

void FTG_Editor::PasteNodes()
{
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = GraphEditorWidget)
	{
		PasteNodesHere(FocusedGraphEd->GetPasteLocation2f(), FocusedGraphEd->GetCurrentGraph());
	}
}

void FTG_Editor::PasteNodesHere(const FVector2D& Location, const class UEdGraph* Graph)
{
	// Undo/Redo support
	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "TG_EditorPaste", "Texture Graph Editor: Paste"));
	TG_EdGraph->Modify();
	EditedTextureGraph->Modify();

	UTG_EdGraph* ExpressionGraph = Graph ? ToRawPtr(CastChecked<UTG_EdGraph>(const_cast<UEdGraph*>(Graph))) : ToRawPtr(TG_EdGraph);
	ExpressionGraph->Modify();

	// Clear the selection set (newly pasted stuff will be selected)
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = GraphEditorWidget)
	{
		FocusedGraphEd->ClearSelectionSet();
	}

	// Grab the text to paste from the clipboard.
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	// Import the nodes
	TSet<UEdGraphNode*> PastedNodes;
	FEdGraphUtilities::ImportNodesFromText(ExpressionGraph, TextToImport, /*out*/ PastedNodes);
	ExpressionGraph->FixDuplicatedNodesPinConnections(PastedNodes);
	
	//Average position of nodes so we can move them while still maintaining relative distances to each other
	FVector2D AvgNodePosition(0.0f, 0.0f);

	for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
	{
		UEdGraphNode* Node = *It;
		AvgNodePosition.X += Node->NodePosX;
		AvgNodePosition.Y += Node->NodePosY;
	}

	if (PastedNodes.Num() > 0)
	{
		float InvNumNodes = 1.0f / float(PastedNodes.Num());
		AvgNodePosition.X *= InvNumNodes;
		AvgNodePosition.Y *= InvNumNodes;
	}

	for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
	{
		UEdGraphNode* Node = *It;

		// Select the newly pasted stuff
		if (TSharedPtr<SGraphEditor> FocusedGraphEd = GraphEditorWidget)
		{
			FocusedGraphEd->SetNodeSelection(Node, true);
		}

		Node->NodePosX = (Node->NodePosX - AvgNodePosition.X) + Location.X;
		Node->NodePosY = (Node->NodePosY - AvgNodePosition.Y) + Location.Y;

		Node->SnapToGrid(SNodePanel::GetSnapGridSize());

		// Give new node a different Guid from the old one
		Node->CreateNewGuid();
	}

	// Update UI
	if (TSharedPtr<SGraphEditor> FocusedGraphEd = GraphEditorWidget/*FocusedGraphEdPtr.Pin()*/)
	{
		FocusedGraphEd->NotifyGraphChanged();
	}
}

bool FTG_Editor::CanPasteNodes() const
{
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);

	return FEdGraphUtilities::CanImportNodesFromText(TG_EdGraph, ClipboardContent);
}

void FTG_Editor::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo,
	UEdGraphNode* NodeBeingChanged)
{
	if (NodeBeingChanged)
	{
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "TGEditorRenameNode", "TS Editor: Rename Node"));
		NodeBeingChanged->Modify();
		NodeBeingChanged->OnRenameNode(NewText.ToString());
	}
}

void FTG_Editor::OnSelectedBlobChanged(BlobPtr Blob)
{
	//TODO:Enable this once Histogram calculation is done on GPU
	TextureDetails->CalculateHistogram(Blob, EditedTextureGraph);
}

void FTG_Editor::DeleteSelectedDuplicatableNodes()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = GraphEditorWidget;
	if (!FocusedGraphEd)
	{
		return;
	}

	// Cache off the old selection
	const FGraphPanelSelectionSet OldSelectedNodes = GraphEditorWidget->GetSelectedNodes();

	// Clear the selection and only select the nodes that can be duplicated
	FGraphPanelSelectionSet RemainingNodes;
	FocusedGraphEd->ClearSelectionSet();

	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(OldSelectedNodes); SelectedIter; ++SelectedIter)
	{
		UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter);
		if ((Node != NULL) && Node->CanDuplicateNode())
		{
			FocusedGraphEd->SetNodeSelection(Node, true);
		}
		else
		{
			RemainingNodes.Add(Node);
		}
	}

	// Delete the duplicatable nodes
	DeleteSelectedNodes();

	// Reselect whatever's left from the original selection after the deletion
	FocusedGraphEd->ClearSelectionSet();

	for (FGraphPanelSelectionSet::TConstIterator SelectedIter(RemainingNodes); SelectedIter; ++SelectedIter)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*SelectedIter))
		{
			FocusedGraphEd->SetNodeSelection(Node, true);
		}
	}
}


void FTG_Editor::RefreshViewport()
{
	if (GetEditorViewport().IsValid())
	{
		GetEditorViewport()->RefreshViewport();
	}
}

void FTG_Editor::RefreshTool()
{
	RefreshViewport();
}

void FTG_Editor::RefreshDetailsView() const
{
	DetailsView->ForceRefresh();
}

void FTG_Editor::SaveAsset_Execute()
{
	UE_LOGF(LogTextureGraphEditor, Log, "Saving Texture Graph %ls", *GetEditingObjects()[0]->GetName());

	// Extra nodes are replicated on asset save, to be saved in the underlying TextureGraph
	ReplicateExtraNodes();

	if (UpdateOriginalTextureGraph())
	{
		ITG_Editor::SaveAsset_Execute();
	}
}

void FTG_Editor::SaveAssetAs_Execute()
{
	UE_LOGF(LogTextureGraphEditor, Log, "Saving Texture Graph As %ls", *GetEditingObjects()[0]->GetName());

	// Extra nodes are replicated on asset save, to be saved in the underlying TextureGraph
	ReplicateExtraNodes();

	UpdateOriginalTextureGraph();

	ITG_Editor::SaveAssetAs_Execute();
}
bool FTG_Editor::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	if (EditedTextureGraph->GetPackage()->IsDirty())
	{
		// find out the user wants to do with this dirty TextureGraph
		EAppReturnType::Type YesNoCancelReply = FMessageDialog::Open(EAppMsgType::YesNoCancel,
			FText::Format(
				NSLOCTEXT("UnrealEd", "Prompt_TextureGraphEditorClose", "Would you like to apply the changes of the modified TextureGraph to the original TextureGraph?\n{0}\n(Selecting 'No' will cause all changes to be lost!)"),
				FText::FromString(OriginalTextureGraph->GetPathName())));

		// act on it
		switch (YesNoCancelReply)
		{
		case EAppReturnType::Yes:
			// update TextureGraph and exit
			SaveAsset_Execute();
			break;

		case EAppReturnType::No:		// exit
			break;

		case EAppReturnType::Cancel:	// don't exit
			return false;
		}
	}

	return true;
}

void FTG_Editor::OnClose()
{
	// Add analytics tag
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Add(FAnalyticsEventAttribute(TEXT("TimeActive.Seconds"),  (FDateTime::Now() - SessionStartTime).GetTotalSeconds()));
				
		// Send Analytics event 
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.TextureGraph.SessionEnded"), Attributes);
				
	}
	ITG_Editor::OnClose();

	if (EditedTextureGraph)
	{
		/// We need to flush any invalidations coming for this graph. This is because if the user decided to save
		/// graph on exit then this queues a mix update that never gets finished as the engine is being shutdown
		/// and results in a cleanup assertion in Device.cpp
		EditedTextureGraph->FlushInvalidations();
	}
}

bool FTG_Editor::UpdateOriginalTextureGraph()
{
	// We should cancel saving when TextureGraph has certain errors.
	FTextureGraphErrorReporter* ErrorReporter = TextureGraphEngine::GetErrorReporter(EditedTextureGraph);
	if (ErrorReporter)
	{
		// see if there are recursive call errors, we warn user and not let them save in that state
		const auto CompileErrors = TextureGraphEngine::GetErrorReporter(EditedTextureGraph)->GetCompilationErrors();
		const TArray<FTextureGraphErrorReport>* RecursiveErrors = CompileErrors.Find((int)ETextureGraphErrorType::RECURSIVE_CALL);
		if (RecursiveErrors != nullptr && RecursiveErrors->Num() > 0)
		{
			const FText Message =  FText::FromString(TEXT("Cannot save as Recursive Errors found.\r\n") + (*RecursiveErrors)[0].ErrorMsg);
			FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, Message);
			return false;
		}
	}
	if (EditedTextureGraph->GetPackage()->IsDirty())
	{
		// Cache any metadata
		const TMap<FName, FString>* MetaData = FMetaData::GetMapForObject(OriginalTextureGraph);

		// overwrite the original TextureGraph in place by constructing a new one with the same name
		FObjectDuplicationParameters Params = InitStaticDuplicateObjectParams(EditedTextureGraph, OriginalTextureGraph->GetOuter(), OriginalTextureGraph->GetFName(),
		RF_AllFlags, UTextureGraph::StaticClass(), EDuplicateMode::Normal, EInternalObjectFlags::None);

		// UObject* NewAsset = StaticDuplicateObjectEx(Params);
		OriginalTextureGraph = Cast<UTextureGraph>(StaticDuplicateObjectEx(Params));
#ifdef UE_BUILD_DEBUG
		OriginalTextureGraph->Graph()->IsRuntime = 0;
#endif
		// Restore the metadata
		if (MetaData)
		{
			FMetaData& PackageMetaData = OriginalTextureGraph->GetPackage()->GetMetaData();
			PackageMetaData.SetObjectValues(OriginalTextureGraph, *MetaData);
		}

		// Restore RF_Standalone on the original TextureGraph, as it had been removed from the duplicated TextureGraph so that it could be GC'd.
		OriginalTextureGraph->SetFlags(RF_Standalone);

		// clear the dirty flag
		EditedTextureGraph->GetPackage()->ClearDirtyFlag();
		return true;
	}

	return false;
}

void FTG_Editor::FocusDetailsPanel()
{
	if (SpawnedDetailsTab.IsValid() && !SpawnedDetailsTab.Pin()->IsForeground())
	{
		SpawnedDetailsTab.Pin()->DrawAttention();
	}
}

void FTG_Editor::RefreshErrors()
{
	TArray< TSharedRef<class FTokenizedMessage> > Messages;

	FTextureGraphErrorReporter* ErrorReporter = TextureGraphEngine::GetErrorReporter(EditedTextureGraph);

	if (ErrorReporter)
	{
		// loop through array of messages/nodeRef and highlight them
		const auto CompileErrors = TextureGraphEngine::GetErrorReporter(EditedTextureGraph)->GetCompilationErrors();

		// compute error crc
		FString NewErrorHash;

		for (auto ErrorEntry : CompileErrors)
		{
			for (const FTextureGraphErrorReport& Error : ErrorEntry.Value)
			{
				NewErrorHash += FMD5::HashAnsiString(*Error.ErrorMsg);
			}
		}
		// If errors changed
		if (NewErrorHash != ErrorHash)
		{
			ErrorHash = NewErrorHash;
			for (auto ErrorEntry : CompileErrors)
			{
				// report all errors for this type
				for (FTextureGraphErrorReport& Error : ErrorEntry.Value)
				{
					FString ErrorString = Error.GetFormattedMessage();
					TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(EMessageSeverity::Error);
					if (Error.ReferenceObj != nullptr)
					{
						Line->SetMessageLink(FUObjectToken::Create(Error.ReferenceObj));
					}
					Line->AddToken(FTextToken::Create(FText::FromString(ErrorString)));

					Messages.Add(Line);
				}
			}

			ErrorsListing->ClearMessages();
			ErrorsListing->AddMessages(Messages);
		}
	}
	else
	{
		ErrorsListing->ClearMessages();
	}

}

void FTG_Editor::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	//TODO: Add code to refresh graph , view ports and invalidate
	bool bRefreshNodePreviews = false;
	if (PropertyChangedEvent.Property != nullptr && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		RefreshPreviewViewport();
	}
}

void FTG_Editor::OnFinishedChangingSettings(const FPropertyChangedEvent& PropertyChangedEvent)
{
	// TODO : Invalidate Texture TextureGraph here.
}

void FTG_Editor::OnFinishedChangingOutput(const FPropertyChangedEvent& PropertyChangedEvent)
{
	// TODO : Invalidate Texture TextureGraph here.
}

void FTG_Editor::UpdateParametersUI()
{
	auto Graph = EditedTextureGraph->Graph();
	auto Ids = Graph->GetParamIds();

	uint32 Hash = 0;
	for (const FTG_Id& Id : Ids)
	{
		Hash = HashCombineFast(Hash, GetTypeHash(Id.IndexRaw()));

		UTG_Pin* Pin = Graph->GetPin(Id);

		if (Pin)
		{
			Hash = HashCombineFast(Hash, GetTypeHash(Pin->GetAliasName()));
		}
	}
	// If Parameters list not changed, we early out
	if (Hash == ParametersHash)
	{
		return;
	}

	// Update the cached hash
	ParametersHash = Hash;

	// Create a new object to set for the view.
	TG_Parameters = NewObject<UTG_Parameters>();

	for (auto id : Ids)
	{
		UTG_Pin* Pin = Graph->GetPin(id);

		if (Pin && (Pin->IsInput() || Pin->IsSetting()))
		{
			FTG_ParameterInfo Info{ id, Pin->GetAliasName() };
			TG_Parameters->Parameters.Add(Info);
		}
	}
	TG_Parameters->TextureGraph = Graph;

	GetParameterView()->SetObject(TG_Parameters);
}

void FTG_Editor::RunGraph(bool Tweaking)
{
	if (EditedTextureGraph && bAutoRunGraph)
	{
		EditedTextureGraph->TriggerUpdate(Tweaking);
	}
}

void FTG_Editor::JumpToNode(const UEdGraphNode* Node)
{
	if (Node->GetGraph() == GraphEditorWidget->GetCurrentGraph())
	{
		GraphEditorWidget->JumpToNode(Node, false);
	}
}

void FTG_Editor::OnMessageLogLinkActivated(const TSharedRef<IMessageToken>& Token)
{
	const TSharedRef<FUObjectToken> UObjectToken = StaticCastSharedRef<FUObjectToken>(Token);
	if (UObjectToken->GetObject().IsValid())
	{
		UTG_Node* Node = Cast<UTG_Node>(UObjectToken->GetObject().Get());

		UTG_EdGraphNode* ReferenceEdObj = TG_EdGraph->GetViewModelNode(Node->GetId());
		if (ReferenceEdObj)
		{
			JumpToNode(ReferenceEdObj);
		}
	}
}

void FTG_Editor::OnGraphChanged(UTG_Graph* InGraph, UTG_Node* InNode, bool Tweaking)
{
	RunGraph(Tweaking);
	UpdateParametersUI();

	if (IsOutputNode(InNode))
	{
		UpdateMixSettings();
	}
	RefreshViewport();
}

void FTG_Editor::UpdateMixSettings()
{
	check(EditedTextureGraph);
	EditedTextureGraph->UpdateGlobalTGSettings();
}

void FTG_Editor::OnNodeAdded(UTG_Node* InNode)
{
	check(EditedTextureGraph);
	auto TargetExpression = Cast<UTG_Expression_Output>(InNode->GetExpression());
	if (TargetExpression != nullptr)
	{
		TargetExpression->InitializeOutputSettings();
	}
}

void FTG_Editor::OnNodeRemoved(UTG_Node* InNode, FName Title)
{
	check(EditedTextureGraph);
	if (IsOutputNode(InNode))
	{
		// Remove output node texture.
		for (int i = 0; i < EditedTextureGraph->GetSettings()->NumTargets(); i++)
		{
			TargetTextureSetPtr& TargetTextureSet = EditedTextureGraph->GetSettings()->Target(i);

			if (TargetTextureSet->ContainsTexture(Title))
			{
				TargetTextureSet->FreeAt(Title);
			}
		}

		FViewportSettings& ViewportSettings = EditedTextureGraph->GetSettings()->GetViewportSettings();
		
		// Update Viewport material mapping for deleted output node
		ViewportSettings.RemoveMaterialMappingForTarget(Title);

		if(ViewportSettings.MaterialMappingInfos.Num() > 0)
		{
			// We should have atleast one target assigned.
			const int AssignedTargets = ViewportSettings.NumAssignedTargets();

			// if We dont have a valid target then we will try to assign 
			if(AssignedTargets <= 0)
			{
				// Check if we have output node available
				const FName FirstOutputName;
				EditedTextureGraph->Graph()->ForEachParams([&](const UTG_Pin* Pin, uint32 index)
					{
						if(Pin->IsOutput())
						{
							if (FirstOutputName.IsNone())
							{
								ViewportSettings.SetDefaultTarget(Pin->GetAliasName());
								GetEditorViewport()->UpdateRenderMode();
							}
						}
					});
			}
		}
	}
}

void FTG_Editor::OnNodeRenamed(UTG_Node* InNode, FName OldName)
{
	if (IsOutputNode(InNode))
	{
		check(EditedTextureGraph);

		// In case of rename
		// Save texture against old name
		// Remove old name entry
		// Add the same texture with new name
		for (int i = 0; i < EditedTextureGraph->GetSettings()->NumTargets(); i++)
		{
			TargetTextureSetPtr& TargetTextureSet = EditedTextureGraph->GetSettings()->Target(i);

			/// If we cannot rename the texture then just abort the rename operation
			if (!TargetTextureSet->RenameTexture(OldName, InNode->GetNodeName()))
				return;
		}

		// Update Viewport target name
		EditedTextureGraph->GetSettings()->GetViewportSettings().OnTargetRename(OldName, InNode->GetNodeName());

		// update output 
	}

	// check on the parameters view, it could have changed
	UpdateParametersUI();
}

bool FTG_Editor::IsOutputNode(UTG_Node* InNode)
{
	if (InNode != nullptr)
	{
		auto TargetExpression = Cast<UTG_Expression_Output>(InNode->GetExpression());
		if (TargetExpression != nullptr)
		{
			return true;
		}
	}

	return false;
}

void FTG_Editor::OnPinSelectionUpdated(UEdGraphPin* Pin)
{
	if (Pin)
	{
		//When pin get updated we need to update its Node
		GraphEditorWidget->GetGraphPanel()->SelectionManager.SelectSingleNode(Pin->GetOwningNode());
	}
	else
	{
		GraphEditorWidget->GetGraphPanel()->SelectionManager.ClearSelectionSet();
	}
}

void FTG_Editor::RefreshPreviewViewport()
{
	//TODO: refresh the viewport here
}

void FTG_Editor::RefreshNodePreview(const TSet<class UObject*>& NewSelection, const FInvalidationDetails* Details, bool bUpdateOnly)
{
	TArray<UTG_EdGraphNode*> NodesForSelectionPreview;
	for (TSet<class UObject*>::TConstIterator SetIt(NewSelection); SetIt; ++SetIt)
	{
		if (UTG_EdGraphNode* GraphNode = Cast<UTG_EdGraphNode>(*SetIt))
		{
			NodesForSelectionPreview.Add(GraphNode);
		}
	}

	//TODO: Handle multiple selection case
	UTG_EdGraphNode* SelectedNode = NodesForSelectionPreview.Num() > 0 ? NodesForSelectionPreview[0] : nullptr;

	if (bUpdateOnly)
	{
		NodePreview->Update();
	}
	else
	{
		NodePreview->SelectionChanged(SelectedNode ? SelectedNode->GetNode() : nullptr);
	}
}

void FTG_Editor::SetMesh(class UMeshComponent* InPreviewMesh, class UWorld* InWorld)
{
	EditedTextureGraph->SetEditorMesh(Cast<UStaticMeshComponent>(InPreviewMesh), InWorld).then([this]()
		{
			GetEditorViewport()->InitRenderModes(EditedTextureGraph);
		});
}

bool FTG_Editor::SetPreviewAsset(UObject* InAsset)
{
	if (GetEditorViewport().IsValid())
	{
		return GetEditorViewport()->SetPreviewAsset(InAsset);
	}
	return false;
}

FString FTG_Editor::GetCurrentFolderInContentBrowser()
{
	IContentBrowserSingleton& ContentBrowser = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	const FContentBrowserItemPath CurrentPath = ContentBrowser.GetCurrentPath();
	return CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : FString();
}

#undef LOCTEXT_NAMESPACE
