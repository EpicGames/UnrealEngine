// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AdvancedPreviewSceneModule.h"
#include "ITG_Editor.h"
#include "EdGraph/EdGraphPin.h"
#include "EditorUndoClient.h"
#include "Misc/NotifyHook.h"
#include "AssetEditorViewportLayout.h"
#include "EditorViewportTabContent.h"
#include "IDetailsView.h"
#include "Export/TextureExporter.h"

#include "TG_Parameter.h"
#include "EdGraph/TG_EdGraphNode.h"
#include "STG_Palette.h"
#include "GraphEditor.h"

class UTextureGraph;
class UTG_Graph;
class UTG_Node;
class UTG_EdGraph;
class SDockTab;
class FSpawnTabArgs;
class FTabManager;

class UTG_Expression;
class FUICommandList;
class FObjectPreSaveContext;
struct FGraphAppearanceInfo;

DECLARE_LOG_CATEGORY_EXTERN(LogTextureGraphEditor, Log, All);

class FTG_Editor : public ITG_Editor, public FGCObject, public FTickableGameObject, public FEditorUndoClient, public FNotifyHook
{
public:
	virtual void									RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual void									UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	void											OnTextureGraphPreSave(UObject* Object, FObjectPreSaveContext SaveContext);
	/**
	 * Edits the specified static TSX Asset object
	 *
	 * @param	Mode								Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost						When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	ObjectToEdit						The TSX Asset to edit
	 */
	void											InitEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UTextureGraph* TG_Script);

	/** Constructor */
													FTG_Editor();

	virtual											~FTG_Editor() override;

	// Inherited via ITG_Editor
	virtual FLinearColor							GetWorldCentricTabColorScale() const override;
	virtual FName									GetToolkitFName() const override;
	virtual FText									GetBaseToolkitName() const override;
	virtual FString									GetWorldCentricTabPrefix() const override;
	
	// Inherited via FGCObject
	virtual void									AddReferencedObjects(FReferenceCollector& Collector) override;

	virtual FString									GetReferencerName() const override { return TEXT("FTextureScriptEditor");}

	class UMixInterface*							GetTextureGraphInterface() const override;

	// Inherited via FTickableGameObject
	virtual void									Tick(float DeltaTime) override;

	virtual ETickableTickType						GetTickableTickType() const override { return ETickableTickType::Always; }

	virtual bool									IsTickableWhenPaused() const override { return true; }

	virtual bool									IsTickableInEditor() const override { return true; }

	virtual TStatId									GetStatId() const override;

	virtual FText									GetOriginalObjectName() const override;

	virtual TSharedPtr<IAssetReferenceFilter>		MakeAssetReferenceFilter() const override;

	UTG_Expression*									CreateNewExpression(UClass* NewExpressionClass, const FVector2D& NodePos, bool bAutoSelect, bool bAutoAssignResource, const class UEdGraph* Graph = nullptr) override;
	
	// Widget Accessors
	TSharedRef<class IDetailsView>					GetDetailView() const { return DetailsView.ToSharedRef(); }
	
	// Widget Accessors
	TSharedRef<class IDetailsView>					GetPreviewSettingsView() const { return PreviewSettingsView.ToSharedRef(); }

	// Widget Accessors
	TSharedRef<class IDetailsView>					GetOutputView() const { return OutputView.ToSharedRef(); }

	// Widget Accessors parameters panel
	TSharedRef<class IDetailsView>					GetParameterView() const { return ParameterView.ToSharedRef(); }

	/** Called to bring focus to the details panel */
	void											FocusDetailsPanel();

	/** Refresh Errors and Warnings **/
	void											RefreshErrors();

	/** Refreshes the viewport containing the preview mesh. **/
	void											RefreshPreviewViewport();
	
	/** Called to update the selection view */
	void											RefreshNodePreview(const TSet<class UObject*>& NewSelection, const FInvalidationDetails* Details, bool bUpdateOnly);

	void											SetMesh(class UMeshComponent* InPreviewMesh, class UWorld* InWorld) override;
	bool 											SetPreviewAsset(UObject* InAsset);
	
	/** Force Refresh Details View **/
	void											RefreshDetailsView() const;

protected:
	TArray<UTG_EdGraphNode*>						GetCurrentSelectedTG_EdGraphNodes() const;


	/** Called when the selection changes in the GraphEditor */
	void											OnSelectedNodesChanged(const TSet<class UObject*>& NewSelection);
	/** Called when a node is double clicked*/
	void											OnNodeDoubleClicked(UEdGraphNode* EdGraphNode);

	virtual void									RefreshViewport() override;

	virtual void									RefreshTool() override;


	// ~Begin FAssetEditorToolkit interface
	/** Called when "Save" is clicked for this asset */
	virtual void									SaveAsset_Execute() override;
	virtual void									SaveAssetAs_Execute() override;
	virtual bool									OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	virtual void									OnClose() override;
	// ~End FAssetEditorToolkit interface

	bool											UpdateOriginalTextureGraph();
	
private:
	/** Register TG_ specific toolbar for the editor */
	void											RegisterToolbar();

	virtual void									PostInitAssetEditor() override;

	void 											OnRenameNodeClicked();
	bool 											CanRenameNode() const;
	/** Bind commands to delegates */
	void											BindCommands();
	void 											SetShowNodeHistogramView(bool bValue);
	void 											SetShowPaletteView(bool bValue);

	void											OnGraphChanged(UTG_Graph* InGraph, UTG_Node* InNode, bool Tweaking);

	void											OnNodeAdded(UTG_Node* InNode);
	void											OnNodeRemoved(UTG_Node* InNode, FName Name);
	void											OnNodeRenamed(UTG_Node* InNode, FName OldName);
	bool											IsOutputNode(UTG_Node* InNode);
	void											UpdateMixSettings();

	void											OnPinSelectionUpdated(UEdGraphPin* Pin);
	/** Run Graph by invoking the graph notifications  */
	void											OnRunGraph_Clicked();

	
	void											ToggleAutoUpdate();
	bool											IsShowingAutoUpdate() const;

	void											HandleTabWindowSelected(const FName TabID,const FString OuputName);
	bool											GetTabSelected(const FName TabID);

	void											OnRenderingDone(UMixInterface* TextureGraph, const FInvalidationDetails* Details);
	void											OnBatchQueued(UMixInterface* TextureGraph, const FInvalidationDetails* Details);
	void											OnPreviewMeshChangedEvent();
	void											OnViewportMaterialChanged();
	void											OnMaterialMappingChanged();
	/** Log Graph in console */
	void											OnLogGraph_Clicked();

	/** Called to undo the last action */
	void											UndoGraphAction();

	/** Called to redo the last undone action */
	void											RedoGraphAction();
	
	void											ReplicateExtraNodes() const;
	void											OnCreateComment();

	/** Called when converting an INputParm node from / to constant */
	void											OnConvertInputParameterToFromConstant();

	bool											OnVerifyNodeTextCommit(const FText& Text, UEdGraphNode* EdGraphNode, FText& OutErrorMessage);
	/** Create new graph editor widget */
	TSharedRef<class SGraphEditor>					CreateGraphEditorWidget();

	FActionMenuContent								OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed);

	/** Create Node preview widget */
	TSharedRef<class STG_NodePreviewWidget>			CreateNodePreviewWidget();

	/** Create Texture details widget */
	TSharedRef<class STG_TextureDetails>			CreateTextureDetailsWidget();

	/** Gets the current TG_ Graph's appearance */
	FGraphAppearanceInfo							GetGraphAppearance() const;

	/** Builds the sub-tools that are a part of the texture graph editor. */
	void											BuildSubTools();
	
	/** Called when the Viewport Layout has changed. */
	void											OnEditorLayoutChanged();

	void											OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);
	
	void											OnFinishedChangingSettings(const FPropertyChangedEvent& PropertyChangedEvent);

	void											OnFinishedChangingOutput(const FPropertyChangedEvent& PropertyChangedEvent);

	void											UpdateParametersUI();

	/*Get the Current selected folder from content browser*/
	FString											GetCurrentFolderInContentBrowser();
	/** Delete all selected nodes in the graph */
	void											DeleteSelectedNodes();
	/* Deletes the list of provided nodes. Force delete override CanUserDeleteNode check */
	bool											DeleteNodes(TArray<class UEdGraphNode*> NodesToDelete, bool ForceDelete = false);
	/** Whether we can delete all selected nodes */
	bool 											CanDeleteSelectedNodes() const;
	/** Copy all selected nodes in the graph*/
	void 											CopySelectedNodes();
	/** Whether we can copy all selected nodes*/
	bool 											CanCopySelectedNodes() const;

	void 											CutSelectedNodes();

	bool 											CanCutSelectedNodes() const;

	void 											DuplicateSelectedNodes();	
												
	bool 											CanDuplicateSelectedNodes() const;
	void 											PasteNodes();	
												
	void 											DeleteSelectedDuplicatableNodes();
	void 											PasteNodesHere(const FVector2D& Location, const class UEdGraph* Graph);	
												
	bool 											CanPasteNodes() const;

	/** Called when the title of a node is changed */
	void											OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);

	/** Called when blob gets updated for the Texture Selection Preview*/
	void											OnSelectedBlobChanged(BlobPtr Blob);

private:
	TSharedRef<SDockTab>							SpawnTab_GraphEditor(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_Viewport(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_TG_Properties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_Palette(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_Find(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_PreviewSettings(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_ParameterDefaults(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_NodePreview(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_Output(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_PreviewSceneSettings(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_Errors(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_TextureDetails(const FSpawnTabArgs& Args);

	void											ExportAsUAsset();
	FReply											OnExportClick();

	TSharedPtr<class STG_EditorViewport>			GetEditorViewport() const;	
	void 											SetViewportPreviewMesh();
	
	bool											CanEnableOnRun();

	void											UpdateGenerator();
	void											RunGraph(bool Tweaking);
	void											JumpToNode(const UEdGraphNode* Node);
	void											OnMessageLogLinkActivated(const class TSharedRef<IMessageToken>& Token);
	// FEditorUndoClient Interface
	virtual void 									PostUndo(bool bSuccess) override;
	virtual void 									PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
public:
	/** Original Texture Graph */
	TObjectPtr<UTextureGraph>						OriginalTextureGraph;

	/** Duplicated Texture Graph used in the editor */
	TObjectPtr<UTextureGraph>						EditedTextureGraph;
	TObjectPtr<UTG_EdGraph>							TG_EdGraph;
	FDateTime										SessionStartTime;

private:
	TObjectPtr<UTG_Parameters>						TG_Parameters;
	FExportSettings                                 TargetExportSettings;

	/** Property View */
	TSharedPtr<class IDetailsView>					DetailsView;
	
	/** Settings View */
	TSharedPtr<class IDetailsView>					PreviewSettingsView;

	/** Output View */
	TSharedPtr<class IDetailsView>					OutputView;

	/** Parameter View */
	TSharedPtr<class IDetailsView>					ParameterView;

	/** Graph editor widget being displayed*/
	TSharedPtr<SGraphEditor>						GraphEditorWidget;

	/** Node preview widget being displayed*/
	TSharedPtr<STG_NodePreviewWidget>				NodePreview;

	/** Texture details widget being displayed*/
	TSharedPtr<class STG_TextureDetails>			TextureDetails;

	/** Command list for this editor */
	TSharedPtr<FUICommandList>						GraphEditorCommands;

	/** Storage for our viewport creation function that will be passed to the viewport layout system*/
	AssetEditorViewportFactoryFunction				MakeViewportFunc;

	FAdvancedPreviewSceneModule::FOnPreviewSceneChanged OnPreviewSceneChangedDelegate;

	// Tracking the active viewports in this editor.
	TSharedPtr<class FEditorViewportTabContent>		ViewportTabContent;

	/** Tab that holds the details panel */
	TWeakPtr<SDockTab>								SpawnedDetailsTab;	
	TWeakPtr<SDockTab>								NodeHistogramTab;	
	TWeakPtr<SDockTab>								PaletteTab;	
	TWeakPtr<SDockTab>								PreviewSceneSettingsDockTab;
	
	/** Stats log, with the log listing that it reflects */
	TSharedPtr<class SWidget>						ErrorsWidget;
	TSharedPtr<class IMessageLogListing>			ErrorsListing;

	/** Scene preview settings widget */
	TSharedPtr<SWidget>								AdvancedPreviewSettingsWidget;

	/** Hashed error code use to refresh the error displaying widget when this will change */
	FString											ErrorHash;

	/** Hash for Graph Params to check if anything was added */
	uint32											ParametersHash;
	
	TSharedPtr<class IPropertyRowGenerator>			Generator;

	TSharedPtr<class STG_Palette>					Palette;
	
	bool											bAutoRunGraph = true;
};
