// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_EditorCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Styling/AppStyle.h"
#include "TG_Style.h"

#define LOCTEXT_NAMESPACE "TG_EditorCommands"

FTG_EditorCommands::FTG_EditorCommands()
	: TCommands<FTG_EditorCommands>(
		"TG_Editor",
		NSLOCTEXT("Contexts", "TG_Editor", "Texture Graph Editor"),
		NAME_None,

		FTG_Style::Get().GetStyleSetName())
{
}

void FTG_EditorCommands::RegisterCommands()
{
	UI_COMMAND(RunGraph, "Update", "Update the current graph.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AutoUpdateGraph, "Auto Update", "Toggles to update graph automatically or not.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(LogGraph, "Log", "Log the current graph.", EUserInterfaceActionType::Button, FInputChord(EKeys::L, EModifierKey::Alt));
	UI_COMMAND(ExportAsUAsset, "Export", "Export all the output nodes as Texture UAssets", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertInputParameterToConstant, "Convert input to Constant", "Convert input parameter to or from a constant.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertInputParameterFromConstant, "Convert input to Parameter", "Convert input parameter to or from a constant.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetCylinderPreview, "Cylinder", "Sets the preview mesh to a cylinder primitive.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetSpherePreview, "Sphere", "Sets the preview mesh to a sphere primitive.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetPlanePreview, "Plane", "Sets the preview mesh to a plane primitive.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetCubePreview, "Cube", "Sets the preview mesh to a cube primitive.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetPreviewMeshFromSelection, "Mesh", "Sets the preview mesh based on the current content browser selection.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(TogglePreviewGrid, "Grid", "Toggles the preview pane's grid.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(TogglePreviewBackground, "Background", "Toggles the preview pane's background.", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
