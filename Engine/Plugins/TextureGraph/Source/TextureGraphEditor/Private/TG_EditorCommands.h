// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

class FTG_EditorCommands : public TCommands<FTG_EditorCommands>
{
public:
	FTG_EditorCommands();

	// ~Begin TCommands<> interface
	virtual void RegisterCommands() override;
	// ~End TCommands<> interface

	TSharedPtr<FUICommandInfo> RunGraph;
	TSharedPtr<FUICommandInfo> AutoUpdateGraph;
	TSharedPtr<FUICommandInfo> LogGraph;
	TSharedPtr<FUICommandInfo> ExportAsUAsset;

	/**
	 * Node Actions
	 */
	TSharedPtr< FUICommandInfo > ConvertInputParameterToConstant;		/// Convert an input param to from constant
	TSharedPtr< FUICommandInfo > ConvertInputParameterFromConstant;		/// Convert an input param to from constant

	/**
	 * Viewport Commands
	 */
	TSharedPtr< FUICommandInfo > SetCylinderPreview;				/// Sets the preview mesh to a cylinder
	TSharedPtr< FUICommandInfo > SetSpherePreview;				/// Sets the preview mesh to a sphere
	TSharedPtr< FUICommandInfo > SetPlanePreview;				/// Sets the preview mesh to a plane
	TSharedPtr< FUICommandInfo > SetCubePreview;					/// Sets the preview mesh to a cube
	TSharedPtr< FUICommandInfo > SetPreviewMeshFromSelection;	// Sets the preview mesh to the current selection in the level editor
	TSharedPtr< FUICommandInfo > TogglePreviewGrid;				/// Toggles the preview pane's grid
	TSharedPtr< FUICommandInfo > TogglePreviewBackground;		/// Toggles the preview pane's background

};
