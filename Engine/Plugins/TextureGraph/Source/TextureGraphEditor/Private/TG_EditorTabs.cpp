// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_EditorTabs.h"

#include "HAL/Platform.h"
#include "UObject/NameTypes.h"

const FName FTG_EditorTabs::ViewportTabId				(TEXT("TG_Editor_Viewport"));
const FName FTG_EditorTabs::PropertiesTabId				(TEXT("TG_Editor_MaterialProperties"));
const FName FTG_EditorTabs::PaletteTabId				(TEXT("TG_Editor_Palette"));
const FName FTG_EditorTabs::FindTabId					(TEXT("TG_Editor_Find"));
const FName FTG_EditorTabs::GraphEditorId				(TEXT("TG_Editor_GraphEditor"));
const FName FTG_EditorTabs::PreviewSceneSettingsTabId	(TEXT("TG_Editor_PreviewSceneSettings"));
const FName FTG_EditorTabs::ParameterDefaultsTabId		(TEXT("TG_Editor_ParameterDefaults"));
const FName FTG_EditorTabs::NodePreviewTabId			(TEXT("TG_Editor_NodePreview"));
const FName FTG_EditorTabs::OutputTabId					(TEXT("TG_Editor_Output"));
const FName FTG_EditorTabs::PreviewSettingsTabId		(TEXT("TG_Editor_PreviewSettings"));
const FName FTG_EditorTabs::ErrorsTabId					(TEXT("TG_Editor_Stats"));
const FName FTG_EditorTabs::TextureDetailsTabId			(TEXT("TG_Editor_TextureDetails"));
const FName FTG_EditorTabs::TextureExporterTabId		(TEXT("TG_Editor_TextureExporter"));