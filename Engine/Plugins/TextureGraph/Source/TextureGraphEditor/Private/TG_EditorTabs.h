// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FName;

/**  TG_ editor tabs identifiers */
struct FTG_EditorTabs
{
	/**	The tab id for the TG_ fixture types tab */
	static const FName ViewportTabId;
	static const FName PropertiesTabId;
	static const FName PaletteTabId;
	static const FName FindTabId;
	static const FName GraphEditorId;
	static const FName PreviewSceneSettingsTabId;
	static const FName ParameterDefaultsTabId;
	static const FName NodePreviewTabId;
	static const FName OutputTabId;
	static const FName PreviewSettingsTabId;
	static const FName ErrorsTabId;
	static const FName TextureDetailsTabId;
	static const FName TextureExporterTabId;

	// Disable default constructor
	FTG_EditorTabs() = delete;
};