// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_Exporter.h"

#include "TG_EditorTabs.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "TextureGraph.h"
#include "TG_Node.h"
#include "TG_Graph.h"
#include "TG_Parameter.h"
#include "Expressions/Output/TG_Expression_Output.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/LayoutService.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "AssetEditorViewportLayout.h"
#include "STG_EditorViewport.h"
#include "EditorViewportTabContent.h"
#include "AdvancedPreviewSceneModule.h"
#include "SPrimaryButton.h"
#include "TG_OutputSettings.h"
#include "STG_NodePreview.h"
#include "TG_HelperFunctions.h"
#include "PropertyEditorModule.h"
#include "Components/StaticMeshComponent.h"
#include "DetailsViewArgs.h"
#include "IDetailsView.h"

class SWidget;
class SWindow;

#define LOCTEXT_NAMESPACE "TextureGraphExporter"


void FTG_ExporterCommands::RegisterCommands()
{
	UI_COMMAND(ShowOutputPreview, "Node Preview", "Toggles visibility of the Output Preview", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(Show3DPreview, "3D Preview", "Toggles visibility of the 3D Preview window", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(Show3DPreviewSettings, "3D Preview Settings", "Toggles visibility of the 3D Preview Settings window", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(ShowParameters, "Parameters", "Toggles visibility of the Parameters window", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(ShowExportSettings, "Export Settings", "Toggles visibility of the Export Settings window", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(ShowDetails, "Property Settings", "Toggles visibility of the Property Detail Settings window", EUserInterfaceActionType::Check, FInputChord());
}
FTG_InstanceImpl::FTG_InstanceImpl() :
	TargetExportSettings(MakeShared<FExportSettings>())
{
}

FTG_InstanceImpl::~FTG_InstanceImpl()
{
	Cleanup();
	DetailsView.Reset();
	ExportSettingsView.Reset();
	PreviewSettingsView.Reset();
	PreviewSceneSettingsView.Reset();
	ParametersView.Reset();

	// cleanup UI
	if (Parameters && Parameters->IsValidLowLevel())
	{
		Parameters->Parameters.Empty();
		Parameters = nullptr;
	}
	if (ExportSettings  && ExportSettings->IsValidLowLevel())
	{
		ExportSettings->OutputExpressionsInfos.Empty();
		ExportSettings = nullptr;
	}
	OutputNodesComboBoxWidget.Reset();

}

void FTG_InstanceImpl::Cleanup(bool bFlushMixInvalidations /*= true*/)
{
	if (TextureGraphInstance && TextureGraphInstance->IsValidLowLevel())
	{
		// cleanup events
		if (UMixSettings* Settings = TextureGraphInstance->GetSettings())
		{
			Settings->GetViewportSettings().OnViewportMaterialChangedEvent.RemoveAll(this);
			Settings->GetViewportSettings().OnMaterialMappingChangedEvent.RemoveAll(this);
			Settings->OnPreviewMeshChangedEvent.RemoveAll(this);
		}
		if (bFlushMixInvalidations)
		{
			TextureGraphInstance->FlushInvalidations();
		}
		if (UTG_Graph* Graph = TextureGraphInstance->Graph())
		{
			Graph->OnGraphChangedDelegate.RemoveAll(this);
		}
		TextureGraphInstance->OnRenderDone.Unbind();
		TextureGraphInstance = nullptr;
	}

	OutputNodesList.Empty();
	SelectedNode = nullptr;
}

void FTG_InstanceImpl::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if(PropertyThatChanged != nullptr &&
		PropertyThatChanged->GetName() == GET_MEMBER_NAME_CHECKED(UTextureGraphInstance, ParentTextureGraph))
	{
		TextureGraphInstance->SetParent(TextureGraphInstance->ParentTextureGraph);

		// clear out previous handles but don't flush mix invalidations as that would skip invalidating the mix with the new parent
		SetTextureGraphToExport(TextureGraphInstance, false);
	}
}

void FTG_InstanceImpl::RegisterTabSpawners(const TSharedPtr<FTabManager> InTabManager)
{
	InTabManager->RegisterTabSpawner(FTG_EditorTabs::ViewportTabId, FOnSpawnTab::CreateRaw(this, &FTG_InstanceImpl::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("ViewportTab", "3D Preview"))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));
	
	InTabManager->RegisterTabSpawner(FTG_EditorTabs::ParameterDefaultsTabId, FOnSpawnTab::CreateRaw(this, &FTG_InstanceImpl::SpawnTab_ParameterDefaults))
		.SetDisplayName(LOCTEXT("ParametersTab", "Parameters"))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(FTG_EditorTabs::NodePreviewTabId, FOnSpawnTab::CreateRaw(this, &FTG_InstanceImpl::SpawnTab_NodePreview))
		.SetDisplayName(LOCTEXT("NodePreviewTab", "Node Preview"))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(FTG_EditorTabs::PreviewSettingsTabId, FOnSpawnTab::CreateRaw(this, &FTG_InstanceImpl::SpawnTab_PreviewSettings))
		.SetDisplayName(LOCTEXT("PreviewSettingsTab", "3D Preview Settings"))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(FTG_EditorTabs::PreviewSceneSettingsTabId, FOnSpawnTab::CreateRaw(this, &FTG_InstanceImpl::SpawnTab_PreviewSceneSettings))
			.SetDisplayName(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings"))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
	
	InTabManager->RegisterTabSpawner(FTG_EditorTabs::OutputTabId, FOnSpawnTab::CreateRaw(this, &FTG_InstanceImpl::SpawnTab_ExportSettings))
		.SetDisplayName(LOCTEXT("ExportSettingsTab", "Export Settings"))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
	
	InTabManager->RegisterTabSpawner(FTG_EditorTabs::PropertiesTabId, FOnSpawnTab::CreateRaw(this, &FTG_InstanceImpl::SpawnTab_TG_Properties))
		.SetDisplayName(LOCTEXT("DetailsTab", "Details"))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
	
}

void FTG_InstanceImpl::UnregisterTabSpawners(const TSharedPtr<FTabManager> InTabManager)
{
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::ViewportTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::ParameterDefaultsTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::NodePreviewTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::OutputTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::PreviewSettingsTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::PreviewSceneSettingsTabId);
	InTabManager->UnregisterTabSpawner(FTG_EditorTabs::PropertiesTabId);
}

void FTG_InstanceImpl::SetMesh(class UMeshComponent* InPreviewMesh, class UWorld* InWorld)
{
	TextureGraphInstance->SetEditorMesh(Cast<UStaticMeshComponent>(InPreviewMesh), InWorld).then([this]()
		{
			GetEditorViewport()->InitRenderModes(TextureGraphInstance);
		});
}

bool FTG_InstanceImpl::SetPreviewAsset(UObject* InAsset)
{
	if (GetEditorViewport().IsValid())
	{
		return GetEditorViewport()->SetPreviewAsset(InAsset);
	}
	return false;
}
TSharedPtr<STG_EditorViewport> FTG_InstanceImpl::GetEditorViewport() const
{
	if (ViewportTabContent.IsValid())
	{
		// we can use static cast here b/c we know in this editor we will have a static mesh viewport 
		return StaticCastSharedPtr<STG_EditorViewport>(ViewportTabContent->GetFirstViewport());
	}

	return nullptr;
}
TSharedRef<SDockTab> FTG_InstanceImpl::SpawnTab_PreviewSettings(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::PreviewSettingsTabId);

	TSharedPtr<SDockTab> PreviewSettingsTab = SNew(SDockTab)
		[
			PreviewSettingsView.ToSharedRef()
		];

	if (TextureGraphInstance)
	{
		GetPreviewSettingsView()->SetObject(TextureGraphInstance->GetSettings(), true);
	}

	return PreviewSettingsTab.ToSharedRef();
}

TSharedRef<SDockTab> FTG_InstanceImpl::SpawnTab_PreviewSceneSettings(const FSpawnTabArgs& Args)
{
	check( Args.GetTabId() == FTG_EditorTabs::PreviewSceneSettingsTabId );
	return SAssignNew(PreviewSceneSettingsDockTab, SDockTab)
		.Label( LOCTEXT("TG_EditorPreviewSceneSettings_TabTitle", "Preview Scene Settings") )
		[
			AdvancedPreviewSettingsWidget.IsValid() ? AdvancedPreviewSettingsWidget.ToSharedRef() : SNullWidget::NullWidget
		];
}

TSharedRef<SDockTab> FTG_InstanceImpl::SpawnTab_TG_Properties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::PropertiesTabId);

	TSharedPtr<SDockTab> DetailsTab = SNew(SDockTab)
		[
			DetailsView.ToSharedRef()
		];

	if (TextureGraphInstance)
	{
		DetailsView->SetObject(TextureGraphInstance, true);
	}
	return DetailsTab.ToSharedRef();
}

TSharedRef<SDockTab> FTG_InstanceImpl::SpawnTab_ExportSettings(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::OutputTabId);

	TSharedPtr<SDockTab> SettingsTab = SNew(SDockTab)
		[
			SNew(SVerticalBox)
			
			+ SVerticalBox::Slot()
			.FillHeight(1)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SScrollBox)
						+SScrollBox::Slot()
						.VAlign(VAlign_Fill)
						.FillSize(1.0)
						[
							ExportSettingsView.ToSharedRef()
						]
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("Export", "Export"))
				.OnClicked_Raw(this, &FTG_InstanceImpl::OnExportClicked, EAppReturnType::Ok)
			]
			
		];

	if (TextureGraphInstance)
	{
		GetExportSettingsView()->SetObject(ExportSettings, true);
	}

	return SettingsTab.ToSharedRef();
}
TSharedRef<SDockTab> FTG_InstanceImpl::SpawnTab_Viewport(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::ViewportTabId);

	TSharedRef< SDockTab > DockableTab =
		SNew(SDockTab);

	AssetEditorViewportFactoryFunction MakeViewportFunc = [this](const FAssetEditorViewportConstructionArgs& InArgs)
	{
		return SNew(STG_EditorViewport)
			.InTextureGraph(TextureGraphInstance);
	};

	// Create a new tab
	ViewportTabContent = MakeShared<FEditorViewportTabContent>();
	ViewportTabContent->OnViewportTabContentLayoutChanged().AddRaw(this, &FTG_InstanceImpl::OnEditorLayoutChanged);

	const FString LayoutId = FString("TG_EditorViewport");
	ViewportTabContent->Initialize(MakeViewportFunc, DockableTab, LayoutId);

	// This call must occur after the toolbar is initialized.
	SetViewportPreviewMesh();
	
	return DockableTab;
}
void FTG_InstanceImpl::BuildSubTools()
{
	FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");
	
	TArray<FAdvancedPreviewSceneModule::FDetailDelegates> Delegates;
	Delegates.Add({ OnPreviewSceneChangedDelegate });
	AdvancedPreviewSettingsWidget = AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(GetEditorViewport()->GetPreviewScene(), nullptr, TArray<FAdvancedPreviewSceneModule::FDetailCustomizationInfo>(),  TArray<FAdvancedPreviewSceneModule::FPropertyTypeCustomizationInfo>(), Delegates);
	
	if (PreviewSceneSettingsDockTab.IsValid())
	{
		PreviewSceneSettingsDockTab.Pin()->SetContent(AdvancedPreviewSettingsWidget.ToSharedRef());
	}
}

void FTG_InstanceImpl::OnEditorLayoutChanged()
{
	BuildSubTools();
	
	OnPreviewSceneChangedDelegate.Broadcast(GetEditorViewport()->GetPreviewScene());
}
TSharedRef<SDockTab> FTG_InstanceImpl::SpawnTab_ParameterDefaults(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::ParameterDefaultsTabId);

	return SNew(SDockTab)
		[
			SNew(SBox)
				[
					ParametersView.ToSharedRef()
				]
		];
}


TSharedRef<SDockTab> FTG_InstanceImpl::SpawnTab_NodePreview(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FTG_EditorTabs::NodePreviewTabId);

	TSharedRef<STG_NodePreviewWidget> NodePreview = SNew(STG_NodePreviewWidget);
	NodePreviewPtr = NodePreview;
	
	return SNew(SDockTab)
		[
			SNew(SVerticalBox)
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(OutputNodesComboBoxWidget, SComboBox<TSharedPtr<FName>>)
				.OptionsSource(&OutputNodesList)
				.OnGenerateWidget_Raw(this, &FTG_InstanceImpl::GenerateOutputComboItem)
				.OnSelectionChanged_Raw(this, &FTG_InstanceImpl::OnOutputSelectionChanged)
				[
					SNew(STextBlock)
					.Text_Lambda([this] ()
					{
						FText ComboTitleText = FText::FromString(TEXT("No TextureGraph selected, or it has no Outputs")); 
						if (IsValid(SelectedNode))
						{
							ComboTitleText = FText::FromName(SelectedNode->GetExpression()->GetTitleName());
						}
						else if (!OutputNodesList.IsEmpty())
						{
							ComboTitleText = FText::FromName(*OutputNodesList[0]);
						}
						return ComboTitleText;
					})
				]
			]
			
			+ SVerticalBox::Slot()
			[
				NodePreview
			]
		];

}
void FTG_InstanceImpl::SetViewportPreviewMesh()
{
	if (TextureGraphInstance)
	{
		TObjectPtr<UStaticMesh> PreviewMesh = TextureGraphInstance->GetSettings()->GetPreviewMesh();
		// Set the preview mesh for the material.  
		if (!PreviewMesh || !SetPreviewAsset(PreviewMesh))
		{
			// The material preview mesh couldn't be found or isn't loaded. Fallback to the one of the primitive types.
			GetEditorViewport()->InitPreviewMesh();
		}
	}
}

void FTG_InstanceImpl::UpdatePreviewMesh()
{
	if (GetEditorViewport())
	{
		GetEditorViewport()->SetTextureGraph(TextureGraphInstance);
		// OnViewportMaterialChanged();
		SetViewportPreviewMesh();
	}
}

void FTG_InstanceImpl::CleanupSlateReferences()
{
	ViewportTabContent.Reset();
	AdvancedPreviewSettingsWidget.Reset();
}

// Function to generate combo box items
TSharedRef<SWidget> FTG_InstanceImpl::GenerateOutputComboItem(TSharedPtr<FName> InItem)
{
	return SNew(STextBlock).Text(FText::FromName(*InItem));
}

// Function called when the selection changes
void FTG_InstanceImpl::OnOutputSelectionChanged(TSharedPtr<FName> SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (SelectedItem.IsValid())
	{
		FName SelectedNodeName = *SelectedItem;
		FTG_Id SelectedNodeId = FTG_Id::INVALID;

		TextureGraphInstance->Graph()->ForEachNodes([this, &SelectedNodeId, SelectedNodeName](const UTG_Node* Node, uint32 Index)
		{
			if (Node && Node->GetExpression()->IsA(UTG_Expression_Output::StaticClass()))
			{
				// choose a default node
				if (!SelectedNodeId.IsValid())
					SelectedNodeId = Node->GetId();

				// check if this is our selected node
				if (Node->GetExpression()->GetTitleName() == SelectedNodeName)
				{
					SelectedNodeId = Node->GetId();
				}
			}
		});
		
		SelectedNode = TextureGraphInstance->Graph()->GetNode(SelectedNodeId);
		if (NodePreviewPtr.IsValid())
		{
			NodePreviewPtr.Pin()->SelectionChanged(SelectedNode);
		}
	
	}
}

FReply FTG_InstanceImpl::OnExportClicked(EAppReturnType::Type ButtonID)
{
	if (TextureGraphInstance)
	{
		FTG_HelperFunctions::ExportAsync(TextureGraphInstance, "", "", *TargetExportSettings, false);
	}
	return FReply::Handled();
}

void FTG_InstanceImpl::SetTextureGraphToExport(UTextureGraphInstance* InTextureGraph, bool bClearPreviousMixInvalidations)
{
	if (!InTextureGraph)
	{
		return;
	}
	
	Cleanup(bClearPreviousMixInvalidations);

	TextureGraphInstance = InTextureGraph;
	
	//Exporter gets notified when rendering is done
	TextureGraphInstance->OnRenderDone.BindRaw(this, &FTG_InstanceImpl::OnRenderingDone);
	
	DetailsView->SetObject(TextureGraphInstance, true);
	
	UpdateParametersUI();

	UpdateExportSettingsUI();

	// Update list of output nodes in 2d View
	OutputNodesList.Empty();
	if (UTG_Graph* Graph = TextureGraphInstance->Graph())
	{
		Graph->ForEachNodes(
		[&](const UTG_Node* node, uint32 index)
		{
			if(UTG_Expression_Output* OutputExpression = Cast<UTG_Expression_Output>(node->GetExpression()))
			{
				OutputNodesList.Add(MakeShared<FName>(OutputExpression->GetTitleName()));			
			}
		});
		
		Graph->OnGraphChangedDelegate.AddRaw(this, &FTG_InstanceImpl::OnGraphChanged);
	}
	if (OutputNodesComboBoxWidget.IsValid())
	{
		if (!OutputNodesList.IsEmpty())
		{
			OutputNodesComboBoxWidget->SetSelectedItem(OutputNodesList[0]);
		}
		OutputNodesComboBoxWidget->RefreshOptions();
	}
	
	FViewportSettings& ViewportSettings = TextureGraphInstance->GetSettings()->GetViewportSettings();

	ViewportSettings.OnViewportMaterialChangedEvent.AddRaw(this, &FTG_InstanceImpl::OnViewportMaterialChanged);
	ViewportSettings.OnMaterialMappingChangedEvent.AddRaw(this, &FTG_InstanceImpl::OnMaterialMappingChanged);
	TextureGraphInstance->GetSettings()->OnPreviewMeshChangedEvent.AddRaw(this, &FTG_InstanceImpl::SetViewportPreviewMesh);

	GetPreviewSettingsView()->SetObject(TextureGraphInstance->GetSettings(), true);

	UpdatePreviewMesh();
	TextureGraphEngine::RegisterErrorReporter(TextureGraphInstance, std::make_shared<FTextureGraphErrorReporter>());

}
void FTG_InstanceImpl::OnGraphChanged(UTG_Graph* InGraph, UTG_Node* InNode, bool Tweaking)
{
	if (TextureGraphInstance)
	{
		TextureGraphInstance->TriggerUpdate(Tweaking);
		if (InNode && InNode->IsA<UTG_Expression_Output>())
		{
			TextureGraphInstance->UpdateGlobalTGSettings();
		}
		RefreshViewport();
	}
}

TSharedRef<FTabManager::FLayout> FTG_InstanceImpl::GetDefaultLayout()
{
	return FTabManager::NewLayout("Standalone_TextureGraphExporter_Layout_v2")
	->AddArea
	(
		FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)->SetSizeCoefficient(0.9f)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.25f)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(FTG_EditorTabs::ParameterDefaultsTabId, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(FTG_EditorTabs::PropertiesTabId, ETabState::OpenedTab)
						->AddTab(FTG_EditorTabs::OutputTabId, ETabState::OpenedTab)
						->SetForegroundTab(FTG_EditorTabs::OutputTabId)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.25f)
					->Split
					(
						FTabManager::NewStack()
						->SetHideTabWell(true)
						->AddTab(FTG_EditorTabs::NodePreviewTabId, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(FTG_EditorTabs::ViewportTabId, ETabState::OpenedTab)
						->AddTab(FTG_EditorTabs::PreviewSettingsTabId, ETabState::OpenedTab)
						->SetForegroundTab(FTG_EditorTabs::ViewportTabId)
					)
				)
			)
		);
}

void FTG_InstanceImpl::Initialize()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs ParameterViewArgs;
	ParameterViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	ParameterViewArgs.bHideSelectionTip = true;
	ParameterViewArgs.ColumnWidth = 0.70f;
	ParametersView = PropertyEditorModule.CreateDetailView(ParameterViewArgs);

	// Settings details view
	FDetailsViewArgs ExportSettingsViewArgs;
	ExportSettingsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	ExportSettingsViewArgs.bHideSelectionTip = true;
	ExportSettingsView = PropertyEditorModule.CreateDetailView(ExportSettingsViewArgs);

	// Settings details view
	FDetailsViewArgs SettingsViewArgs;
	SettingsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	SettingsViewArgs.bHideSelectionTip = true;
	PreviewSettingsView = PropertyEditorModule.CreateDetailView(SettingsViewArgs);
	
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bShowObjectLabel = false;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.ColumnWidth = 0.70f;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	
}

void FTG_InstanceImpl::OnRenderingDone(UMixInterface* TextureGraph, const FInvalidationDetails* Details)
{
	if (TextureGraph != nullptr && TextureGraph == TextureGraphInstance && NodePreviewPtr.IsValid())
	{
		// refresh node preview
		NodePreviewPtr.Pin()->Update();
		GetEditorViewport()->UpdateRenderMode();
	}
}
void FTG_InstanceImpl::OnViewportMaterialChanged()
{
	const UTG_Node* FirstTargetNode = nullptr;
	TextureGraphInstance->Graph()->ForEachNodes([&](const UTG_Node* node, uint32 index)
		{
			if (UTG_Expression_Output* OutputExpression = dynamic_cast<UTG_Expression_Output*>(node->GetExpression()))
			{
				if (!FirstTargetNode)
				{
					FirstTargetNode = node;
				}
			}
		});

	FViewportSettings& ViewportSettings = TextureGraphInstance->GetSettings()->GetViewportSettings();

	if (FirstTargetNode && ViewportSettings.MaterialMappingInfos.Num() > 0)
	{
		ViewportSettings.SetDefaultTarget(FirstTargetNode->GetNodeName());
	}

	GetEditorViewport()->GenerateRendermodeToolbar();
	GetEditorViewport()->InitRenderModes(TextureGraphInstance);
}

void FTG_InstanceImpl::OnMaterialMappingChanged()
{
	GetEditorViewport()->UpdateRenderMode();
}

void FTG_InstanceImpl::UpdateExportSettingsUI()
{
	// recreate export settings UI
	ExportSettings = NewObject<UTG_ExportSettings>(TextureGraphInstance);
	
	if (UTG_Graph* Graph = TextureGraphInstance->Graph())
	{
		Graph->ForEachNodes([&](const UTG_Node* node, uint32 index)
		{
			if (UTG_Expression_Output* OutputExpression = dynamic_cast<UTG_Expression_Output*>(node->GetExpression()))
			{
				ExportSettings->OutputExpressionsInfos.Add(FOutputExpressionInfo{OutputExpression->GetTitleName(), node->GetId()});
			}
		});
	}
	GetExportSettingsView()->SetObject(ExportSettings);
}
void FTG_InstanceImpl::UpdateParametersUI()
{
	// Create a new object to set for the view.
	Parameters = NewObject<UTG_Parameters>();

	if (UTG_Graph* Graph = TextureGraphInstance->Graph())
	{
		FTG_Ids Ids = Graph->GetParamIds();

		for (const FTG_Id& id : Ids)
		{
			UTG_Pin* Pin = Graph->GetPin(id);

			if (Pin && (Pin->IsInput() || Pin->IsSetting()))
			{
				FTG_ParameterInfo Info{ id, Pin->GetAliasName() };
				Parameters->Parameters.Add(Info);
			}
		}
		Parameters->TextureGraph = Graph;
	}
	
	GetParametersView()->SetObject(Parameters);
}
void FTG_InstanceImpl::Tick(float DeltaTime)
{
	RefreshViewport();
}

TStatId FTG_InstanceImpl::GetStatId() const
{
	return TStatId();
}

void FTG_InstanceImpl::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Parameters);
	Collector.AddReferencedObject(TextureGraphInstance);
	Collector.AddReferencedObject(ExportSettings);
}

void FTG_InstanceImpl::RefreshViewport()
{
	if (GetEditorViewport().IsValid())
	{
		GetEditorViewport()->RefreshViewport();
	}
}

///////////////////////////////////////////////////////////////////////////////////
FTG_ExporterUtility::FTG_ExporterUtility()
	: Impl(MakeUnique<FTG_InstanceImpl>())
{
	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
	
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(FTG_EditorTabs::TextureExporterTabId, FOnSpawnTab::CreateRaw(this, &FTG_ExporterUtility::CreateTGExporterTab))
		.SetDisplayName(NSLOCTEXT("TextureGraphExporter", "TabTitle", "Texture Graph Exporter"))
		.SetTooltipText(NSLOCTEXT("TextureGraphExporter", "TooltipText", "Open the Texture Graph Exporter tab."))
		.SetGroup(MenuStructure.GetDeveloperToolsMiscCategory())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Texture2D"));

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnPreShutdown().AddRaw(this, &FTG_ExporterUtility::Cleanup);
	}
}

void FTG_ExporterUtility::Cleanup()
{
	if (FSlateApplication::IsInitialized())
	{
		if (TGExporterTabManager.IsValid())
		{
			Impl->CleanupSlateReferences();
			Impl->UnregisterTabSpawners(TGExporterTabManager);
		}
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(FTG_EditorTabs::TextureExporterTabId);
		
		FSlateApplication::Get().OnPreShutdown().RemoveAll(this);
	}
	
}

FTG_ExporterUtility::~FTG_ExporterUtility()
{
}

TSharedRef<SDockTab> FTG_ExporterUtility::CreateTGExporterTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> NomadTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label(NSLOCTEXT("TextureGraphExporter", "TabTitle", "Texture Graph Exporter"));

	TGExporterTabManager = FGlobalTabmanager::Get()->NewTabManager(NomadTab);
	// on persist layout will handle saving layout if the editor is shut down:
	TGExporterTabManager->SetOnPersistLayout(
		FTabManager::FOnPersistLayout::CreateStatic(
			[](const TSharedRef<FTabManager::FLayout>& InLayout)
			{
				if (InLayout->GetPrimaryArea().Pin().IsValid())
				{
					FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, InLayout);
				}
			}
		)
	);


	TWeakPtr<FTabManager> TGExporterTabManagerWeak = TGExporterTabManager;
	// On tab close will save the layout if the exporter window itself is closed,
	// this handler also cleans up any floating controls. If we don't close
	// all areas we need to add some logic to the tab manager to reuse existing tabs:
	NomadTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda(
		[this](TSharedRef<SDockTab> Self, TWeakPtr<FTabManager> TabManager)
		{
			TSharedPtr<FTabManager> OwningTabManager = TabManager.Pin();
			if (OwningTabManager.IsValid())
			{
				FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, OwningTabManager->PersistLayout());
				OwningTabManager->CloseAllAreas();
			}
			Impl->Cleanup();
		}
		, TGExporterTabManagerWeak
	));
	Impl->RegisterTabSpawners(TGExporterTabManager);
	Impl->Initialize();
	
	TGExporterLayout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, Impl->GetDefaultLayout());
	TSharedRef<SWidget> TabContents = TGExporterTabManager->RestoreFrom(TGExporterLayout.ToSharedRef(), TSharedPtr<SWindow>()).ToSharedRef();
	
	// build command list for tab restoration menu:
	TSharedPtr<FUICommandList> CommandList = MakeShareable(new FUICommandList());
	
	TWeakPtr<FTabManager> TGExportManagerWeak = TGExporterTabManager;
	
	const auto ToggleTabVisibility = [](TWeakPtr<FTabManager> InTGExportManagerWeak, FName InTabName)
	{
		TSharedPtr<FTabManager> InTGExportManager = InTGExportManagerWeak.Pin();
		if (InTGExportManager.IsValid())
		{
			TSharedPtr<SDockTab> ExistingTab = InTGExportManager->FindExistingLiveTab(InTabName);
			if (ExistingTab.IsValid())
			{
				ExistingTab->RequestCloseTab();
			}
			else
			{
				InTGExportManager->TryInvokeTab(InTabName);
			}
		}
	};
	
	const auto IsTabVisible = [](TWeakPtr<FTabManager> InTGExportManagerWeak, FName InTabName)
	{
		TSharedPtr<FTabManager> InTGExportManager = InTGExportManagerWeak.Pin();
		if (InTGExportManager.IsValid())
		{
			return InTGExportManager->FindExistingLiveTab(InTabName).IsValid();
		}
		return false;
	};

	auto CurrentViewportTab = TGExporterTabManager->FindExistingLiveTab(FTG_EditorTabs::ViewportTabId);
	bool bViewportIsOff = CurrentViewportTab == nullptr;
	
	// check here if 3d viewport is turned off, we need to turn it on temporarily to initialize our systems correctly
	if (bViewportIsOff)
	{
		CurrentViewportTab = TGExporterTabManager->TryInvokeTab(FTG_EditorTabs::ViewportTabId);	
	}
	
	Impl->SetViewportPreviewMesh();

	CommandList->MapAction(
		FTG_ExporterCommands::Get().Show3DPreview,
		FExecuteAction::CreateStatic(
			ToggleTabVisibility,
			TGExportManagerWeak,
			FTG_EditorTabs::ViewportTabId
		),
		FCanExecuteAction::CreateStatic(
			[]() { return true; }
		),
		FIsActionChecked::CreateStatic(
			IsTabVisible,
			TGExportManagerWeak,
			FTG_EditorTabs::ViewportTabId
		)
	);
	
	CommandList->MapAction(
		FTG_ExporterCommands::Get().Show3DPreviewSettings,
		FExecuteAction::CreateStatic(
			ToggleTabVisibility,
			TGExportManagerWeak,
			FTG_EditorTabs::PreviewSettingsTabId
		),
		FCanExecuteAction::CreateStatic(
			[]() { return true; }
		),
		FIsActionChecked::CreateStatic(
			IsTabVisible,
			TGExportManagerWeak,
			FTG_EditorTabs::PreviewSettingsTabId
		)
	);

	CommandList->MapAction(
			FTG_ExporterCommands::Get().ShowParameters,
			FExecuteAction::CreateStatic(
				ToggleTabVisibility,
				TGExportManagerWeak,
				FTG_EditorTabs::ParameterDefaultsTabId
			),
			FCanExecuteAction::CreateStatic(
				[]() { return true; }
			),
			FIsActionChecked::CreateStatic(
				IsTabVisible,
				TGExportManagerWeak,
				FTG_EditorTabs::ParameterDefaultsTabId
			)
		);

	CommandList->MapAction(
			FTG_ExporterCommands::Get().ShowOutputPreview,
			FExecuteAction::CreateStatic(
				ToggleTabVisibility,
				TGExportManagerWeak,
				FTG_EditorTabs::NodePreviewTabId
			),
			FCanExecuteAction::CreateStatic(
				[]() { return true; }
			),
			FIsActionChecked::CreateStatic(
				IsTabVisible,
				TGExportManagerWeak,
				FTG_EditorTabs::NodePreviewTabId
			)
		);

	CommandList->MapAction(
			FTG_ExporterCommands::Get().ShowExportSettings,
			FExecuteAction::CreateStatic(
				ToggleTabVisibility,
				TGExportManagerWeak,
				FTG_EditorTabs::OutputTabId
			),
			FCanExecuteAction::CreateStatic(
				[]() { return true; }
			),
			FIsActionChecked::CreateStatic(
				IsTabVisible,
				TGExportManagerWeak,
				FTG_EditorTabs::OutputTabId
			)
		);

	CommandList->MapAction(
			FTG_ExporterCommands::Get().ShowDetails,
			FExecuteAction::CreateStatic(
				ToggleTabVisibility,
				TGExportManagerWeak,
				FTG_EditorTabs::PropertiesTabId
			),
			FCanExecuteAction::CreateStatic(
				[]() { return true; }
			),
			FIsActionChecked::CreateStatic(
				IsTabVisible,
				TGExportManagerWeak,
				FTG_EditorTabs::PropertiesTabId
			)
		);
	
	FMenuBarBuilder MenuBarBuilder(CommandList);
	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("WindowMenuLabel", "Window"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateLambda([](FMenuBuilder& Builder) {
			Builder.AddMenuEntry(FTG_ExporterCommands::Get().ShowOutputPreview);
			Builder.AddMenuEntry(FTG_ExporterCommands::Get().ShowParameters);
			Builder.AddMenuEntry(FTG_ExporterCommands::Get().Show3DPreviewSettings);
			Builder.AddMenuEntry(FTG_ExporterCommands::Get().Show3DPreview);
			Builder.AddMenuEntry(FTG_ExporterCommands::Get().ShowExportSettings);
			Builder.AddMenuEntry(FTG_ExporterCommands::Get().ShowDetails);
			})
	);
	
	
	TSharedRef<SWidget> MenuBarWidget = MenuBarBuilder.MakeWidget();
	
	
	NomadTab->SetContent(
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			MenuBarWidget
		]
		+SVerticalBox::Slot()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
			.Padding(FMargin(0.f, 2.f))
			[
				TabContents
			]
		]
	);
	
	
	// Tell tab-manager about the multi-box for platforms with a global menu bar
	TGExporterTabManager->SetMenuMultiBox(MenuBarBuilder.GetMultiBox(), MenuBarWidget);

	TextureGraphInstance = NewObject<UTextureGraphInstance>();
	TextureGraphInstance->Construct(FString());
	Impl->SetTextureGraphToExport(TextureGraphInstance, true);
	return NomadTab;
}

void FTG_ExporterUtility::SetTextureGraphToExport(UTextureGraphBase* InTextureGraph)
{
	// force open export window
	FGlobalTabmanager::Get()->TryInvokeTab(FTG_EditorTabs::TextureExporterTabId);
	
	if (InTextureGraph->IsA<UTextureGraph>())
	{
		TextureGraphInstance = NewObject<UTextureGraphInstance>();
		TextureGraphInstance->Construct(FString());
		TextureGraphInstance->SetParent(Cast<UTextureGraph>(InTextureGraph));
	}
	else if (InTextureGraph->IsA<UTextureGraphInstance>())
	{
		TextureGraphInstance = DuplicateObject(Cast<UTextureGraphInstance>(InTextureGraph), GetTransientPackage());
		if (!InTextureGraph->Graph())
		{
			TextureGraphInstance->Construct(FString());
		}
		TextureGraphInstance->Initialize();
	}
	Impl->SetTextureGraphToExport(TextureGraphInstance, true);
}

#undef LOCTEXT_NAMESPACE 
