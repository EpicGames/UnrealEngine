// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_Factory.h"
#include "Model/ModelObject.h"
#include "TextureGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_Factory)

UTG_Factory::UTG_Factory()
{
    SupportedClass = UTextureGraph::StaticClass();
    bCreateNew = true;
    bEditAfterNew = true;
}

UObject* UTG_Factory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
    UTextureGraph* TextureGraph = NewObject<UTextureGraph>(InParent, Class, Name, Flags, Context);

    check(TextureGraph);

	TextureGraph->Construct(Name.GetPlainNameString());

    return TextureGraph;
}
