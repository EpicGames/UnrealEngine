// Copyright Epic Games, Inc. All Rights Reserved.
#include "TG_InstanceEditor.h"
#include "Misc/MessageDialog.h"

#include "TextureGraphEditorModule.h"
#include "TextureGraph.h"
#include "Components/MeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"

#include "Framework/Application/SlateApplication.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include <Toolkits/AssetEditorToolkit.h>

#include "Model/Mix/MixInterface.h"
#include "UObject/MetaData.h"
#include "UObject/ObjectSaveContext.h"
#include "TG_Exporter.h"
DEFINE_LOG_CATEGORY(LogTextureGraphInstanceEditor);
#define LOCTEXT_NAMESPACE "TG_InstanceEditor"

void FTG_InstanceEditor::RefreshViewport()
{
	if (InstanceImpl)
	{
		InstanceImpl->RefreshViewport();
	}
}

void FTG_InstanceEditor::RefreshTool()
{
	RefreshViewport();
}
void FTG_InstanceEditor::SetMesh(class UMeshComponent* InPreviewMesh, class UWorld* InWorld)
{
	// TextureGraphInstance->SetEditorMesh(Cast<UStaticMeshComponent>(InPreviewMesh), InWorld).then([this]()
	// 	{
	// 		InstanceImpl->GetEditorViewport()->InitRenderModes(TextureGraphInstance);
	// 	});
}

void FTG_InstanceEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
	if (InstanceImpl)
	{
		InstanceImpl->RegisterTabSpawners(InTabManager);
	}
}

void FTG_InstanceEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	if (InstanceImpl)
	{
		InstanceImpl->UnregisterTabSpawners(InTabManager);
	}
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
}

void FTG_InstanceEditor::OnTextureGraphPreSave(UObject* Object, FObjectPreSaveContext SaveContext)
{
	if (TextureGraphInstance == Object)
		TextureGraphInstance->TriggerUpdate(false);
}

void FTG_InstanceEditor::InitEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UTextureGraphInstance* InTextureGraph)
{
	if (!InTextureGraph)
	{
		return;
	}

	// Initialize the TextureGraphInstance to create the runtime graph
	InTextureGraph->Initialize();
		
	InstanceImpl = MakeUnique<FTG_InstanceImpl>();
	InstanceImpl->Initialize();

	TextureGraphInstance = InTextureGraph;
	if (!InTextureGraph->Graph())
	{
		TextureGraphInstance->Construct(FString());
	}
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, TG_InstanceEditorAppIdentifier, InstanceImpl->GetDefaultLayout(), /*bCreateDefaultToolbar*/ true, /*bCreateDefaultStandaloneMenu*/ true, TextureGraphInstance);
	
	InstanceImpl->SetTextureGraphToExport(TextureGraphInstance, true);
	

// 	// Add analytics tag
// 	if (FEngineAnalytics::IsAvailable())
// 	{
// 		SessionStartTime = FDateTime::Now();
// 		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.TextureGraph.SessionStarted"));
// 	}
}

FTG_InstanceEditor::FTG_InstanceEditor()
{
}

FTG_InstanceEditor::~FTG_InstanceEditor()
{
	InstanceImpl.Reset();
	TextureGraphInstance = nullptr;
}

FLinearColor FTG_InstanceEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

FName FTG_InstanceEditor::GetToolkitFName() const
{
	return FName("TG_InstanceEditor");
}

FText FTG_InstanceEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "TG_ InstanceEditor");
}

FString FTG_InstanceEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "TG_ ").ToString();
}

void FTG_InstanceEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(TextureGraphInstance);
}

UMixInterface* FTG_InstanceEditor::GetTextureGraphInterface() const
{
	return TextureGraphInstance;
}

void FTG_InstanceEditor::Tick(float DeltaTime)
{
	RefreshViewport();
}

TStatId FTG_InstanceEditor::GetStatId() const
{
	return TStatId();
}

FText FTG_InstanceEditor::GetOriginalObjectName() const
{
	return FText::FromString(GetEditingObjects()[0]->GetName());
}

TSharedPtr<IAssetReferenceFilter> FTG_InstanceEditor::MakeAssetReferenceFilter() const
{
	if (GEditor)
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;
		AssetReferenceFilterContext.AddReferencingAssets(GetEditingObjects());
		return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
	}

	return {};
}

void FTG_InstanceEditor::OnClose()
{
	// // Add analytics tag
	// if (FEngineAnalytics::IsAvailable())
	// {
	// 	TArray<FAnalyticsEventAttribute> Attributes;
	// 	Attributes.Add(FAnalyticsEventAttribute(TEXT("TimeActive.Seconds"),  (FDateTime::Now() - SessionStartTime).GetTotalSeconds()));
	// 			
	// 	// Send Analytics event 
	// 	FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.TextureGraph.SessionEnded"), Attributes);
	// 			
	// }
	if (TextureGraphInstance)
	{
		/// We need to flush any invalidations coming for this graph. This is because if the user decided to save
		/// graph on exit then this queues a mix update that never gets finished as the engine is being shutdown
		/// and results in a cleanup assertion in Device.cpp
		TextureGraphInstance->FlushInvalidations();
	}
	
	InstanceImpl = nullptr;
}

#undef LOCTEXT_NAMESPACE
