// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ITG_Editor.h"
#include "EdGraph/EdGraphPin.h"
#include "EditorUndoClient.h"
#include "Misc/NotifyHook.h"
#include "GraphEditor.h"

class UTextureGraphInstance;
class UTextureGraph;
class UTG_Graph;
class UTG_Node;
class UTG_EdGraph;
class SDockTab;
class FSpawnTabArgs;
class FTabManager;

class UTG_Expression;
class FUICommandList;
class FObjectPreSaveContext;
struct FGraphAppearanceInfo;

DECLARE_LOG_CATEGORY_EXTERN(LogTextureGraphInstanceEditor, Log, All);

class FTG_InstanceEditor : public ITG_Editor, public FGCObject, public FTickableGameObject, public FEditorUndoClient
{
protected:
	virtual void RefreshViewport() override;
	virtual void RefreshTool() override;

public:
	virtual void SetMesh(UMeshComponent* PreviewMesh, UWorld* World) override;

	virtual void									RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual void									UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	void											OnTextureGraphPreSave(UObject* Object, FObjectPreSaveContext SaveContext);
	
	TSharedRef<SDockTab>							SpawnTab_TG_Properties(const FSpawnTabArgs& Args);
	/**
	 * Edits the specified static TSX Asset object
	 *
	 * @param	Mode								Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost						When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	ObjectToEdit						The TSX Asset to edit
	 */
	void											InitEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UTextureGraphInstance* TG_Script);

	/** Constructor */
													FTG_InstanceEditor();

	virtual											~FTG_InstanceEditor() override;

	// Inherited via ITG_Editor
	virtual FLinearColor							GetWorldCentricTabColorScale() const override;
	virtual FName									GetToolkitFName() const override;
	virtual FText									GetBaseToolkitName() const override;
	virtual FString									GetWorldCentricTabPrefix() const override;
	
	// Inherited via FGCObject
	virtual void									AddReferencedObjects(FReferenceCollector& Collector) override;

	virtual FString									GetReferencerName() const override { return TEXT("FTextureScriptEditor");}

	virtual class UMixInterface*					GetTextureGraphInterface() const override;

	// Inherited via FTickableGameObject
	virtual void									Tick(float DeltaTime) override;

	virtual ETickableTickType						GetTickableTickType() const override { return ETickableTickType::Always; }

	virtual bool									IsTickableWhenPaused() const override { return true; }

	virtual bool									IsTickableInEditor() const override { return true; }

	virtual TStatId									GetStatId() const override;

	virtual FText									GetOriginalObjectName() const override;

	virtual TSharedPtr<IAssetReferenceFilter>		MakeAssetReferenceFilter() const override;

	// ~Begin FAssetEditorToolkit interface
	virtual void									OnClose() override;
	// ~End FAssetEditorToolkit interface

private:
	//	FDateTime										SessionStartTime;

private:
	
	TUniquePtr<struct FTG_InstanceImpl>				InstanceImpl;
	
	TObjectPtr<UTextureGraphInstance>				TextureGraphInstance;
	
};
