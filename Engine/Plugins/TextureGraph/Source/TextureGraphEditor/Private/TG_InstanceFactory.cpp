// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_InstanceFactory.h"
#include "Model/ModelObject.h"
#include "TextureGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_InstanceFactory)

UTG_InstanceFactory::UTG_InstanceFactory()
{
	SupportedClass = UTextureGraphInstance::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UTG_InstanceFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UTextureGraphInstance* TextureGraphInstance = NewObject<UTextureGraphInstance>(InParent, Class, Name, Flags, Context);

	check(TextureGraphInstance);
	TextureGraphInstance->Construct(Name.GetPlainNameString());
	TextureGraphInstance->SetParent(InitialParent);
	return TextureGraphInstance;
}
