// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "TG_InstanceFactory.generated.h"


UCLASS()
class UTG_InstanceFactory : public UFactory
{
	GENERATED_BODY()
public:
	UPROPERTY()
	TObjectPtr<class UTextureGraphBase> InitialParent;
	
	UTG_InstanceFactory();
	UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn);
};
