// Copyright Epic Games, Inc. All Rights Reserved.
#include "TG_RenderModeManager.h"
#include "FxMat/MaterialManager.h"
#include "FxMat/RenderMaterial_BP.h"
#include "TextureGraphEngine.h"
#include "TextureGraph.h"
#include "Model/Mix/Mix.h"
#include "Device/FX/Device_FX.h"
#include "2D/Tex.h" 
#include "AI/NavigationSystemBase.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"
#include "Materials/Material.h"
#include "Model/Mix/ViewportSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TG_RenderModeManager)

TG_RenderModeManager::TG_RenderModeManager()
{
}

TG_RenderModeManager::~TG_RenderModeManager()
{
	
}

void TG_RenderModeManager::ChangeRenderMode(FName NewRenderMode, UTextureGraphBase* TextureGraph)
{
	_lastRenderMode = _currentRenderMode;
	_currentRenderMode = NewRenderMode;
	UpdateRenderMode(TextureGraph);
}

void TG_RenderModeManager::UpdateRenderMode(UTextureGraphBase* TextureGraph)
{
	UMixSettings* settings = TextureGraph->GetSettings();
	size_t NumTargets = settings->NumTargets();
	const FName MaterialName = settings->GetViewportSettings().GetMaterialName();
	const int32 NumRenderModeMappings = _renderModeMaterials.Num();
	
	if (NumTargets != NumRenderModeMappings)
	{
		InitializeDefaultMaterials(NumTargets, TextureGraph);
	}

	for (size_t ti = 0; ti < NumTargets; ti++)
	{
		RenderMaterial_BPPtr renderMaterial = GetTargetRenderModeMaterial(ti,_currentRenderMode);
		
		const TargetTextureSetPtr& target = settings->Target(ti);

		if(MaterialName == _currentRenderMode)
		{
			target->BindTo( std::static_pointer_cast<RenderMaterial_BP>(renderMaterial), settings->GetViewportSettings().MaterialMappingInfos);
		}
		else
		{
			const FName Target = settings->GetViewportSettings().GetMaterialMappingInfo(_currentRenderMode);

			FMaterialMappingInfo MaterialMappingInfo;
			MaterialMappingInfo.MaterialInput = "Source";
			MaterialMappingInfo.Target = Target;
			
			target->BindTo( std::static_pointer_cast<RenderMaterial_BP>(renderMaterial), MaterialMappingInfo);
		}
		UMaterialInterface* umaterial = std::static_pointer_cast<RenderMaterial_BP>(renderMaterial)->Instance();
		target->GetMesh()->SetMaterial(umaterial);
	}	
}

void TG_RenderModeManager::BindBlobToMaterial(RenderMaterialPtr renderMaterial, TiledBlobPtr blobToBind, const FName& targetName)
{
	ResourceBindInfo bindInfo;
	bindInfo.Dev = Device_FX::Get();
	bindInfo.Target = targetName.ToString();
	blobToBind->OnFinalise().then([=]()
	{
		renderMaterial->Bind(blobToBind, bindInfo);
	});
	
}

bool TG_RenderModeManager::IsCurrentRenderModelLit(int targetId /* = 0*/)
{
	const RenderMaterial_BPPtr RenderMaterial = GetTargetRenderModeMaterial(targetId, GetCurrentRenderMode());

	if(RenderMaterial)
	{
		return RenderMaterial->GetMaterial()->GetShadingModels().IsLit();
	}

	return false;
}


void TG_RenderModeManager::InitializeDefaultMaterials(int totalTargets, UTextureGraphBase* TextureGraph)
{
	_renderModeMaterials.Empty();

	auto Settings = TextureGraph->GetSettings();
	
	const TObjectPtr<UMaterial> ViewportMaterial = Settings->GetViewportSettings().Material;
	const TArray<FMaterialMappingInfo>& MaterialMappingInfos = Settings->GetViewportSettings().MaterialMappingInfos;
	
	for (int targetId = 0; targetId < totalTargets; targetId++)
	{
		TMap<FName, RenderMaterial_BPPtr> TargetMaterialsMap;
		TargetMaterialsMap.Add(ViewportMaterial.GetFName(), TextureGraphEngine::GetMaterialManager()->CreateMaterial_BP(ViewportMaterial.GetName(), ViewportMaterial));

		for(const FMaterialMappingInfo& MaterialMappingInfo : MaterialMappingInfos)
		{
			TargetMaterialsMap.Add(MaterialMappingInfo.MaterialInput, TextureGraphEngine::GetMaterialManager()->CreateMaterial_BP(MaterialMappingInfo.MaterialInput.ToString(), TEXT("Scene/RendermodeSource")));
		}
		
		_renderModeMaterials.Add(targetId, TargetMaterialsMap);
	}
}

RenderMaterial_BPPtr TG_RenderModeManager::GetTargetRenderModeMaterial(int targetId, FName RenderMode)
{
	if(_renderModeMaterials.Num() > 0)
	{
		check(_renderModeMaterials.Contains(targetId));
		
		auto targetMaterials = _renderModeMaterials.Find(targetId);

		check(targetMaterials);

		if (targetMaterials != nullptr)
		{
			if(targetMaterials->Contains(RenderMode))
			{
				RenderMaterial_BPPtr renderModeMaterial = (*targetMaterials)[RenderMode];
		
				check(renderModeMaterial);
		
				return renderModeMaterial;	
			}
		}
	}

	return nullptr;
}

void TG_RenderModeManager::Clear()
{
	_renderModeMaterials.Empty();
}

