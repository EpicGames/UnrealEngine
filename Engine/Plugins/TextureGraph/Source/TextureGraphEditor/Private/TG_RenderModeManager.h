// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include <memory>
#include "TG_RenderModeManager.generated.h"

class UTextureGraphBase;

UENUM()
enum class ERenderModes : uint8
{
	PBRMetalness		 	UMETA(DisplayName = "PBR Metalness"),
	Albedo 					UMETA(DisplayName = "Albedo"),
	Normal					UMETA(DisplayName = "Normal"),
	Displacement			UMETA(DisplayName = "Displacement"),
	Roughness				UMETA(DisplayName = "Roughness"),
	Specular				UMETA(DisplayName = "Specular"),
	Metalness				UMETA(DisplayName = "Metalness"),
	LayerMask				UMETA(DisplayName = "Layer Mask"),
	ActiveMask				UMETA(DisplayName = "Active Mask"),
	UV						UMETA(DisplayName = "D: UV"),
	WorldNormals			UMETA(DisplayName = "D: WorldNormals"),
	WorldTangents			UMETA(DisplayName = "D: WorldTangents"),
	WorldPosition			UMETA(DisplayName = "D: WorldPosition"),
	WorldUVMask				UMETA(DisplayName = "D: WorldUVMask"),
	Default = PBRMetalness 	UMETA(DisplayName = "Default")
};

class RenderMaterial_BP;

typedef std::shared_ptr<RenderMaterial_BP> RenderMaterial_BPPtr;
typedef std::shared_ptr<class RenderMaterial>	RenderMaterialPtr;
typedef std::shared_ptr<class TiledBlob>	TiledBlobPtr;
typedef TMap<FName, RenderMaterial_BPPtr> MaterialMap;

class TG_RenderModeManager 
{
protected:
	FName							_lastRenderMode;			/// Last used render mode (used for unbinding)
	FName							_currentRenderMode;			/// Currently applied render mode	
	TMap<int, MaterialMap>			_renderModeMaterials;
	TArray<FName>					_renderModesString;

	void							InitializeDefaultMaterials(int totalTargets, UTextureGraphBase* TextureGraph);
	RenderMaterial_BPPtr			GetTargetRenderModeMaterial(int targetId, FName renderMode);
	void							BindBlobToMaterial(RenderMaterialPtr renderMaterial, TiledBlobPtr blobToBind, const FName& targetName);
	virtual void					UpdateRenderMode(UTextureGraphBase* TextureGraph);
public:
									TG_RenderModeManager();
	virtual							~TG_RenderModeManager();		
	virtual void					ChangeRenderMode(FName NewRenderMode, UTextureGraphBase* TextureGraph);
	void							Clear();
	bool							IsCurrentRenderModelLit(int targetId = 0);
	//////////////////////////////////////////////////////////////////////////
	//// Inline Functions
	//////////////////////////////////////////////////////////////////////////

	FORCEINLINE FName				GetCurrentRenderMode() { return _currentRenderMode; }
	FORCEINLINE FName				GetLastRenderMode() { return _lastRenderMode; }
};