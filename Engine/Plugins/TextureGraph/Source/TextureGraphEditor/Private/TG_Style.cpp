// Copyright Epic Games, Inc. All Rights Reserved.

#include "TG_Style.h"

#include "Styling/StarshipCoreStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManagerGeneric.h"
#include "Styling/StyleColors.h"

void FTG_Style::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FTG_Style::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

TArray<FString> FTG_Style::GetPaletteIconNames()
{
	FString FolderPath = FPaths::Combine(IPluginManager::Get().FindPlugin("TextureGraph")->GetBaseDir() , TEXT("Content/Style/Palette/") );
	FString FileExtension = ".svg";

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	TArray<FString> ImageFileNames;

	if (PlatformFile.DirectoryExists(*FolderPath))
	{
		IFileManager& FileManager = IFileManager::Get();

		// Get all files in the specified directory
		TArray<FString> FileNames;
		FileManager.FindFiles(FileNames, *FolderPath, *FileExtension);

		// Filter out only the image file names
		for (const FString& FileName : FileNames)
		{
			FString BaseName = FPaths::GetBaseFilename(FileName);
			ImageFileNames.Add(BaseName);
		}
	}

	return ImageFileNames;
}

const bool FTG_Style::HasKey(FName StyleName) const
{
	auto Key = GetStyleKeys().Find(FName(StyleName));
	return Key != nullptr;
}

FTG_Style::FTG_Style() : FSlateStyleSet("TG_Style")
{
	// Const icon sizes
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
	
	// TG_ editor
	{	
		FSlateFontInfo ChannelButtonFont = FStyleFonts::Get().NormalBold;
		ChannelButtonFont.Size = 10;
		Set("TG_Editor.ChannelButtonFont", ChannelButtonFont);

		Set("TG_Editor.SetCylinderPreview", new IMAGE_BRUSH("Icons/icon_MatEd_Cylinder_40x", Icon40x40));
		Set("TG_Editor.SetCylinderPreview.Small", new IMAGE_BRUSH("Icons/icon_MatEd_Cylinder_40x", Icon20x20));
		Set("TG_Editor.SetSpherePreview", new IMAGE_BRUSH("Icons/icon_MatEd_Sphere_40x", Icon40x40));
		Set("TG_Editor.SetSpherePreview.Small", new IMAGE_BRUSH("Icons/icon_MatEd_Sphere_40x", Icon20x20));
		Set("TG_Editor.SetPlanePreview", new IMAGE_BRUSH("Icons/icon_MatEd_Plane_40x", Icon40x40));
		Set("TG_Editor.SetPlanePreview.Small", new IMAGE_BRUSH("Icons/icon_MatEd_Plane_40x", Icon20x20));
		Set("TG_Editor.SetCubePreview", new IMAGE_BRUSH("Icons/icon_MatEd_Cube_40x", Icon40x40));
		Set("TG_Editor.SetCubePreview.Small", new IMAGE_BRUSH("Icons/icon_MatEd_Cube_40x", Icon20x20));
		Set("TG_Editor.SetPreviewMeshFromSelection", new IMAGE_BRUSH("Icons/icon_MatEd_Mesh_40x", Icon40x40));
		Set("TG_Editor.SetPreviewMeshFromSelection.Small", new IMAGE_BRUSH("Icons/icon_MatEd_Mesh_40x", Icon20x20));
		Set("TG_Editor.TogglePreviewGrid", new IMAGE_BRUSH("Icons/icon_MatEd_Grid_40x", Icon40x40));
		Set("TG_Editor.TogglePreviewGrid.Small", new IMAGE_BRUSH("Icons/icon_MatEd_Grid_40x", Icon20x20));
	}

	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	SetContentRoot(IPluginManager::Get().FindPlugin("TextureGraph")->GetBaseDir() / TEXT("Content"));
	
	const FVector2D PinSize(20.45f, 15.0f);
	Set(TSEditorStyleConstants::Pin_Generic_Image_C, new IMAGE_BRUSH_SVG("Style/TG_Pin_Generic_C", PinSize));
	Set(TSEditorStyleConstants::Pin_Generic_Image_DC, new IMAGE_BRUSH_SVG("Style/TG_Pin_Generic_DC", FVector2D(15,15)));
	Set(TSEditorStyleConstants::Pin_IN_Image_C, new IMAGE_BRUSH_SVG("Style/TG_Pin_IN_Image_C", PinSize));
	Set(TSEditorStyleConstants::Pin_IN_Image_DC, new IMAGE_BRUSH_SVG("Style/TG_Pin_IN_Image_Unplugged", PinSize));
	Set(TSEditorStyleConstants::Pin_IN_Vector_C, new IMAGE_BRUSH_SVG("Style/TG_Pin_IN_Vector_C", PinSize));
	Set(TSEditorStyleConstants::Pin_IN_Vector_DC, new IMAGE_BRUSH_SVG("Style/TG_Pin_IN_Vector_Unplugged", PinSize));
	Set(TSEditorStyleConstants::Pin_IN_Scalar_C, new IMAGE_BRUSH_SVG("Style/TG_Pin_IN_Scalar_C", PinSize));
	Set(TSEditorStyleConstants::Pin_IN_Scalar_DC, new IMAGE_BRUSH_SVG("Style/TG_Pin_IN_Scalar_Unplugged", PinSize));

	//TODO: Update Out images to be different than In ones
	Set(TSEditorStyleConstants::Pin_OUT_Image_C, new IMAGE_BRUSH_SVG("Style/TG_Pin_IN_Image_C", PinSize));
	Set(TSEditorStyleConstants::Pin_OUT_Image_DC, new IMAGE_BRUSH_SVG("Style/TG_Pin_IN_Image_Unplugged", PinSize));
	Set(TSEditorStyleConstants::Pin_OUT_Vector_C, new IMAGE_BRUSH_SVG("Style/TG_Pin_IN_Vector_C", PinSize));
	Set(TSEditorStyleConstants::Pin_OUT_Vector_DC, new IMAGE_BRUSH_SVG("Style/TG_Pin_IN_Vector_Unplugged", PinSize));
	Set(TSEditorStyleConstants::Pin_OUT_Scalar_C, new IMAGE_BRUSH_SVG("Style/TG_Pin_IN_Scalar_C", PinSize));
	Set(TSEditorStyleConstants::Pin_OUT_Scalar_DC, new IMAGE_BRUSH_SVG("Style/TG_Pin_IN_Scalar_Unplugged", PinSize));

	Set("TG_Editor.TileIcon", new IMAGE_BRUSH_SVG("Style/TileIcon", Icon20x20));
	Set("TG_Editor.ListIcon", new IMAGE_BRUSH_SVG("Style/ListIcon", Icon20x20));

	//Creating brush for every icon in the palatte folder
	for (auto PlatteIconName : GetPaletteIconNames())
	{
		FString Path = "Style/Palette/" + PlatteIconName;
		FString PropertyName = "TG_Editor.Palette." + PlatteIconName;

		Set(FName(PropertyName), new IMAGE_BRUSH_SVG(Path, Icon16x16));
	}
	
	const FLinearColor NoSpillColor(1, 1, 1, 1.0);
	const int BodyRadius = 10;
	const int NodeHeaderRadius = 7;
	const int PalleteRadius = 4;
	const int NodeTitleEdiboxRadius = 4;

	Set("TG.Graph.Node.BodyBackground", new FSlateRoundedBoxBrush(NoSpillColor, BodyRadius));
	Set("TG.Graph.Node.BodyBorder", new FSlateRoundedBoxBrush(NoSpillColor, BodyRadius));
	Set("TG.Graph.Node.AssetBackground", new FSlateRoundedBoxBrush(NoSpillColor, BodyRadius));

	Set("TG.Graph.Node.Body", new FSlateRoundedBoxBrush(NoSpillColor, BodyRadius, NoSpillColor, 2.0));
	Set("TG.Graph.Node.Header", new FSlateRoundedBoxBrush(NoSpillColor, NodeHeaderRadius, NoSpillColor, 2.0));
	Set("TG.Graph.Node.ShadowSelected", new BOX_BRUSH("Style/TG_shadow_selected", FMargin(18.0/64.0)));

	Set("TG.Palette.Background", new FSlateRoundedBoxBrush(NoSpillColor, PalleteRadius, NoSpillColor, 2.0));
	
	SetParentStyleName("EditorStyle");
	FTextBlockStyle NormalText = GetParentStyle()->GetWidgetStyle<FTextBlockStyle>("NormalText");

	Set("TG.Graph.Node.NodeTitleExtraLines", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Italic", 8))
		.SetColorAndOpacity(FLinearColor::FromSRGBColor(FColor::FromHex(TEXT("CBCBCB"))))
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.7f))
	);

	FTextBlockStyle GraphNodeTitleText = FTextBlockStyle()
		.SetColorAndOpacity(FLinearColor(1.0, 1.0,1.0))
		.SetFont(FCoreStyle::GetDefaultFontStyle("Bold", 12))
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.7f));
	Set("TG.Graph.Node.Title", GraphNodeTitleText);

	FEditableTextBoxStyle GraphActionNodeTitleEditableText = FEditableTextBoxStyle()
		.SetFont(NormalText.Font)
		.SetForegroundColor(FStyleColors::Input)
		.SetBackgroundImageNormal(FSlateRoundedBoxBrush(FStyleColors::Foreground, NodeTitleEdiboxRadius, FStyleColors::Secondary, 1.0f))
		.SetBackgroundImageHovered(FSlateRoundedBoxBrush(FStyleColors::Foreground, NodeTitleEdiboxRadius, FStyleColors::Hover, 1.0f))
		.SetBackgroundImageFocused(FSlateRoundedBoxBrush(FStyleColors::Foreground, NodeTitleEdiboxRadius, FStyleColors::Primary, 1.0f))
		.SetBackgroundImageReadOnly(FSlateRoundedBoxBrush(FStyleColors::Header, NodeTitleEdiboxRadius, FStyleColors::InputOutline, 1.0f))
		.SetForegroundColor(FStyleColors::Background)
		.SetBackgroundColor(FStyleColors::White)
		.SetReadOnlyForegroundColor(FStyleColors::Foreground)
		.SetFocusedForegroundColor(FStyleColors::Background);

	Set("TG.Graph.Node.NodeTitleEditableText", GraphActionNodeTitleEditableText);

	Set("TG.Graph.Node.NodeTitleInlineEditableText", FInlineEditableTextBlockStyle()
		.SetTextStyle(GraphNodeTitleText)
		.SetEditableTextBoxStyle(GraphActionNodeTitleEditableText)
	);
};

const FTG_Style& FTG_Style::Get()
{
	static FTG_Style Instance;
	return Instance;
}

