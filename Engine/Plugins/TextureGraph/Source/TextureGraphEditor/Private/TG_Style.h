// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

class FTG_Style
	: public FSlateStyleSet
{
public:
	static void Register();
	static void Unregister();
	static TArray<FString> GetPaletteIconNames();
	static const FTG_Style& Get();

	const bool HasKey(FName StyleName) const;
private:	
	FTG_Style();
};


namespace TSEditorStyleConstants
{
	const FName Pin_Generic_Image_C = TEXT("TS.Graph.Pin.Generic.Connected");
	const FName Pin_Generic_Image_DC = TEXT("TS.Graph.Pin.Generic.Disconnected");
	const FName Pin_IN_Image_C = TEXT("TS.Graph.In.Image.Connected");
	const FName Pin_IN_Image_DC = TEXT("TS.Graph.In.Image.Disconnected");
	const FName Pin_OUT_Image_C = TEXT("TS.Graph.Out.Image.Connected");
	const FName Pin_OUT_Image_DC = TEXT("TS.Graph.Out.Image.Disconnected");
	
	const FName Pin_IN_Vector_C = TEXT("TS.Graph.In.Vector.Connected");
	const FName Pin_IN_Vector_DC = TEXT("TS.Graph.In.Vector.Disconnected");
	const FName Pin_OUT_Vector_C = TEXT("TS.Graph.Out.Vector.Connected");
	const FName Pin_OUT_Vector_DC = TEXT("TS.Graph.Out.Vector.Disconnected");
	
	const FName Pin_IN_Scalar_C = TEXT("TS.Graph.In.Scalar.Connected");
	const FName Pin_IN_Scalar_DC = TEXT("TS.Graph.In.Scalar.Disconnected");
	const FName Pin_OUT_Scalar_C = TEXT("TS.Graph.Out.Scalar.Connected");
	const FName Pin_OUT_Scalar_DC = TEXT("TS.Graph.Out.Scalar.Disconnected");
}
	