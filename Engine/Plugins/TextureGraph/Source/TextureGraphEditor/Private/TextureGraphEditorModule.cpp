// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureGraphEditorModule.h"

#include "AssetToolsModule.h"
#include "EdGraph/TG_EditorGraphNodeFactory.h"
#include "EdGraph/TG_EditorGraphPanelPinFactory.h"
#include "TextureGraphEngine.h"
#include "AssetDefinition_TextureGraph.h"
#include "AssetDefinition_TextureGraphInstance.h"
#include "TG_Editor.h"
#include "TG_InstanceEditor.h"
#include "TG_EditorCommands.h"
#include "TG_Style.h"
#include "EdGraph/TG_EditorErrorReporter.h"
#include "Customizations/TG_ParameterCustomization.h"
#include "Customizations/TG_TextureCustomization.h"
#include "Customizations/TG_MaterialCustomization.h"
#include "Customizations/TG_VariantCustomization.h"
#include "Customizations/TG_ScalarCustomization.h"
#include "Customizations/TG_MaterialMappingInfoCustomization.h"
#include "Customizations/TG_ViewportSettingsCustomization.h"
#include "Customizations/TG_OutputSettingsCustomization.h"
#include "Customizations/TG_LevelsSettingsCustomization.h"

#define LOCTEXT_NAMESPACE "FTextureGraphEditorModule"

const FName TG_EditorAppIdentifier = FName(TEXT("TG_EditorApp"));
const FName TG_InstanceEditorAppIdentifier = FName(TEXT("TG_InstanceEditorApp"));

void FTextureGraphEditorModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	// Register all custom AssetTypeActions here.
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("TG_ParameterInfo", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTG_ParameterInfoCustomization::Create));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("TG_Texture", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTG_TextureCustomization::Create));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("TG_Material", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTG_MaterialCustomization::Create));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("TG_Variant", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTG_VariantCustomization::Create));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("FloatProperty", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTG_ScalarCustomization::Create), MakeShared<FTG_ScalarTypeIdentifier>());
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("MaterialMappingInfo", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTG_MaterialMappingInfoCustomization::Create));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("ViewportSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTG_ViewportSettingsCustomization::Create));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("OutputSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTG_OutputSettingsCustomization::Create));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("TG_LevelsSettings", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTG_LevelsSettingsCustomization::Create));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("OutputExpressionInfo", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTG_OutputExpressionInfoCustomization::Create));
	PropertyEditorModule.NotifyCustomizationModuleChanged();
	
	// Register slate style overrides
	FTG_Style::Register();
	FTG_EditorCommands::Register();
	FTG_ExporterCommands::Register();
	
	GraphNodeFactory = MakeShareable(new FTG_EditorGraphNodeFactory());
	FEdGraphUtilities::RegisterVisualNodeFactory(GraphNodeFactory);

	GraphPanelPinFactory = MakeShared<FTG_EditorGraphPanelPinFactory>();
	FEdGraphUtilities::RegisterVisualPinFactory(GraphPanelPinFactory);
	
	TG_Exporter = MakeUnique<FTG_ExporterUtility>();
	
	StartTextureGraphEngine();
}

void FTextureGraphEditorModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	
	UnRegisterAllAssetTypeActions();
	FTG_EditorCommands::Unregister();
	FTG_ExporterCommands::Unregister();
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	// Unregister the details customization
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("TG_ParameterInfo");
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("TG_Texture");
		auto ScalarIdentifier = MakeShared<FTG_ScalarTypeIdentifier>();
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("FloatProperty", ScalarIdentifier);
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("MaterialMappingInfo");
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("OutputSettings");
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("TG_LevelsSettings");
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("TG_OutputExpressionInfoCustomization");
		
		PropertyEditorModule.NotifyCustomizationModuleChanged();
	}
	FEdGraphUtilities::UnregisterVisualPinFactory(GraphPanelPinFactory);
	FEdGraphUtilities::UnregisterVisualNodeFactory(GraphNodeFactory);
	// Unregister slate style overrides
	FTG_Style::Unregister();
	
	ShutdownTextureGraphEngine();
}

void FTextureGraphEditorModule::StartTextureGraphEngine()
{
	if (!TextureGraphEngine::GetInstance())
	{
		// In case of editor, this is the place where we create a new Texture Graph engine.
		// This is done only once and will get destroyed when we shutdown unreal.
		TextureGraphEngine::Create(false);
		check(TextureGraphEngine::GetInstance());
	}
	TickDelegate = FTickerDelegate::CreateRaw(this, &FTextureGraphEditorModule::Tick);
	TickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(TickDelegate);
}
void FTextureGraphEditorModule::ShutdownTextureGraphEngine()
{
	if (TextureGraphEngine::GetInstance())
	{
		TextureGraphEngine::Destroy();
		check(!TextureGraphEngine::GetInstance());
		FTSTicker::RemoveTicker(TickDelegateHandle);
	}
}
bool FTextureGraphEditorModule::Tick(float deltaTime)
{
	if (TextureGraphEngine::GetInstance())
		TextureGraphEngine::Update(deltaTime);
	return true;
}
void FTextureGraphEditorModule::RegisterAssetTypeAction(IAssetTools& AssetTools, TSharedRef<IAssetTypeActions> Action)
{
	AssetTools.RegisterAssetTypeActions(Action);
	CreatedAssetTypeActions.Add(Action);
}
void FTextureGraphEditorModule::UnRegisterAllAssetTypeActions()
{
	if (FModuleManager::Get().IsModuleLoaded("AssetTools"))
	{
		IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
		for (int32 Index = 0; Index < CreatedAssetTypeActions.Num(); ++Index)
		{
			AssetTools.UnregisterAssetTypeActions(CreatedAssetTypeActions[Index].ToSharedRef());
		}
	}
	CreatedAssetTypeActions.Empty();
}
TSharedRef<ITG_Editor> FTextureGraphEditorModule::CreateTextureGraphEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UTextureGraph* InTextureGraph)
{
	TSharedRef<FTG_Editor> NewEditor(new FTG_Editor());
	NewEditor->InitEditor(Mode, InitToolkitHost, InTextureGraph);
	
	return NewEditor;
}
TSharedRef<ITG_Editor> FTextureGraphEditorModule::CreateTextureGraphInstanceEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UTextureGraphInstance* InTextureGraphInstance)
{
	TSharedRef<FTG_InstanceEditor> NewEditor(new FTG_InstanceEditor());
	NewEditor->InitEditor(Mode, InitToolkitHost, InTextureGraphInstance);
	
	return NewEditor;
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FTextureGraphEditorModule, TextureGraphEditor)
