// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_ActionMenuTileView.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "EditorCategoryUtils.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Views/ITypedTableView.h"
#include "GraphActionNode.h"
#include "GraphEditorDragDropAction.h"
#include "IDocumentation.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "STG_ActionMenuTileView"

const int STG_ActionMenuTileView::TileWidth = 169;
const int STG_ActionMenuTileView::TileHeight = 36;

class STG_CategoryHeaderExpanderArrow : public SExpanderArrow
{
	SLATE_BEGIN_ARGS(STG_CategoryHeaderExpanderArrow) {}

	SLATE_ARGUMENT(bool, LeftAligned)
		SLATE_ARGUMENT(bool, AlwaysVisible)

		SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<class ITableRow>& TableRow, const TSharedPtr< STreeView< TSharedPtr<FGraphActionNode> > >& TreeView, TSharedPtr<FGraphActionNode> Item)
	{
		bLeftAligned = InArgs._LeftAligned;
		bAlwaysVisible = InArgs._AlwaysVisible;
		OwnerTreeView = TreeView;
		OwnerItem = Item;

		SExpanderArrow::Construct(
			SExpanderArrow::FArguments()
			.BaseIndentLevel(1),
			TableRow);

		ExpanderArrow->SetOnClicked(FOnClicked::CreateSP(this, &STG_CategoryHeaderExpanderArrow::OnClick));

		// Override visibility so that groups can have an expander arrow despite having no children.
		ExpanderArrow->SetVisibility(
			TAttribute<EVisibility>(this, &STG_CategoryHeaderExpanderArrow::GetExpanderVisibility_Extended));
	}

	EVisibility GetExpanderVisibility_Extended() const
	{
		if (bAlwaysVisible)
		{
			return EVisibility::Visible;
		}
		return SExpanderArrow::GetExpanderVisibility();
	}

	FReply OnClick()
	{
		OwnerTreeView.Pin()->SetItemExpansion(OwnerItem.Pin(), !OwnerTreeView.Pin()->IsItemExpanded(OwnerItem.Pin()));

		return FReply::Handled();
	}

	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
	{
		return FCursorReply::Cursor(EMouseCursor::Default);
	}

	bool bLeftAligned;
	bool bAlwaysVisible;
	TWeakPtr< STreeView< TSharedPtr<FGraphActionNode> > > OwnerTreeView;
	TWeakPtr < FGraphActionNode > OwnerItem;
};

template<typename ItemType>
class STG_CategoryHeaderTableRow : public STableRow<ItemType>
{
public:
	SLATE_BEGIN_ARGS(STG_CategoryHeaderTableRow)
	{}
	SLATE_DEFAULT_SLOT(typename STG_CategoryHeaderTableRow::FArguments, Content)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		STableRow<ItemType>::ChildSlot
		.Padding(0.0f, 2.0f, .0f, 0.0f)
		[
			SAssignNew(ContentBorder, SBorder)
			.BorderImage(this, &STG_CategoryHeaderTableRow::GetBackgroundImage)
			.Padding(FMargin(3.0f, 5.0f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(5.0f)
				.AutoWidth()
				[
					SNew(SExpanderArrow, STableRow< ItemType >::SharedThis(this))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					InArgs._Content.Widget
				]
			]
		];

		STableRow < ItemType >::ConstructInternal(
			typename STableRow< ItemType >::FArguments()
			.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
			.ShowSelection(false),
			InOwnerTableView
		);
	}

	const FSlateBrush* GetBackgroundImage() const
	{
		if (STableRow<ItemType>::IsHovered())
		{
			return FAppStyle::Get().GetBrush("Brushes.Secondary");
		}
		else
		{
			return FAppStyle::Get().GetBrush("Brushes.Header");
		}
	}

	virtual void SetContent(TSharedRef< SWidget > InContent) override
	{
		ContentBorder->SetContent(InContent);
	}

	virtual void SetRowContent(TSharedRef< SWidget > InContent) override
	{
		ContentBorder->SetContent(InContent);
	}

	virtual const FSlateBrush* GetBorder() const
	{
		return nullptr;
	}

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			STableRow<ItemType>::ToggleExpansion();
			return FReply::Handled();
		}
		else
		{
			return FReply::Unhandled();
		}
	}
private:
	TSharedPtr<SBorder> ContentBorder;
};

class STG_GraphActionCategoryWidget : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(STG_GraphActionCategoryWidget)
	{}
	SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_EVENT(FOnTextCommitted, OnTextCommitted)
	SLATE_EVENT(FIsSelected, IsSelected)
	SLATE_ATTRIBUTE(bool, IsReadOnly)
	SLATE_END_ARGS()

	TWeakPtr<FGraphActionNode> ActionNode;
	TAttribute<bool> IsReadOnly;
public:
	TWeakPtr<SInlineEditableTextBlock> InlineWidget;

	void Construct(const FArguments& InArgs, TSharedPtr<FGraphActionNode> InActionNode)
	{
		ActionNode = InActionNode;

		FText CategoryTooltip;
		FString CategoryLink, CategoryExcerpt;
		FEditorCategoryUtils::GetCategoryTooltipInfo(*InActionNode->GetDisplayName().ToString(), CategoryTooltip, CategoryLink, CategoryExcerpt);

		TSharedRef<SToolTip> ToolTipWidget = IDocumentation::Get()->CreateToolTip(CategoryTooltip, NULL, CategoryLink, CategoryExcerpt);
		IsReadOnly = InArgs._IsReadOnly;

		this->ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SAssignNew(InlineWidget, SInlineEditableTextBlock)
				.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
				.Text(FEditorCategoryUtils::GetCategoryDisplayString(InActionNode->GetDisplayName()))
				.ToolTip(ToolTipWidget)
				.HighlightText(InArgs._HighlightText)
				.OnTextCommitted(InArgs._OnTextCommitted)
				.IsSelected(InArgs._IsSelected)
				.IsReadOnly(InArgs._IsReadOnly)
			]
		];
	}

	// SWidget interface
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		TSharedPtr<FGraphEditorDragDropAction> GraphDropOp = DragDropEvent.GetOperationAs<FGraphEditorDragDropAction>();
		if (GraphDropOp.IsValid())
		{
			GraphDropOp->DroppedOnCategory(ActionNode.Pin()->GetCategoryPath());
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		TSharedPtr<FGraphEditorDragDropAction> GraphDropOp = DragDropEvent.GetOperationAs<FGraphEditorDragDropAction>();
		if (GraphDropOp.IsValid())
		{
			GraphDropOp->SetHoveredCategoryName(ActionNode.Pin()->GetDisplayName());
		}
	}

	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override
	{
		TSharedPtr<FGraphEditorDragDropAction> GraphDropOp = DragDropEvent.GetOperationAs<FGraphEditorDragDropAction>();
		if (GraphDropOp.IsValid())
		{
			GraphDropOp->SetHoveredCategoryName(FText::GetEmpty());
		}
	}

	// End of SWidget interface
};


FString STG_ActionMenuTileView::LastUsedFilterText;

void STG_ActionMenuTileView::Construct(const FArguments& InArgs, bool bIsReadOnly/* = true*/)
{
	this->SelectedSuggestionScore = TNumericLimits<float>::Lowest();
	this->SelectedSuggestionSourceIndex = INDEX_NONE;
	this->SelectedSuggestion = INDEX_NONE;
	this->bIgnoreUIUpdate = false;
	this->bUseSectionStyling = InArgs._UseSectionStyling;
	this->bAllowPreselectedItemActivation = InArgs._bAllowPreselectedItemActivation;

	this->bAutoExpandActionMenu = InArgs._AutoExpandActionMenu;
	this->bShowFilterTextBox = InArgs._ShowFilterTextBox;
	this->bAlphaSortItems = InArgs._AlphaSortItems;
	this->bSortItemsRecursively = InArgs._SortItemsRecursively;
	this->OnActionSelected = InArgs._OnActionSelected;
	this->OnActionDoubleClicked = InArgs._OnActionDoubleClicked;
	this->OnActionDragged = InArgs._OnActionDragged;
	this->OnCategoryDragged = InArgs._OnCategoryDragged;
	this->OnCreateWidgetForAction = InArgs._OnCreateWidgetForAction;
	this->OnCreateCustomRowExpander = InArgs._OnCreateCustomRowExpander;
	this->OnGetActionList = InArgs._OnGetActionList;
	this->OnCollectAllActions = InArgs._OnCollectAllActions;
	this->OnCollectStaticSections = InArgs._OnCollectStaticSections;
	this->OnCategoryTextCommitted = InArgs._OnCategoryTextCommitted;
	this->OnCanRenameSelectedAction = InArgs._OnCanRenameSelectedAction;
	this->OnGetSectionTitle = InArgs._OnGetSectionTitle;
	this->OnGetSectionToolTip = InArgs._OnGetSectionToolTip;
	this->OnGetSectionWidget = InArgs._OnGetSectionWidget;
	this->FilteredRootAction = FGraphActionNode::NewRootNode();
	this->OnActionMatchesName = InArgs._OnActionMatchesName;
	this->DraggedFromPins = InArgs._DraggedFromPins;
	this->GraphObj = InArgs._GraphObj;

	// Default graph action list (also provides an empty source list to start with)
	AllActions = MakeShared<FGraphActionListBuilderBase>();

	if (OnGetActionList.IsBound())
	{
		// If we are obtaining a new action list at refresh time, ensure that the indirect collection delegate is unbound
		if (!ensureMsgf(!OnCollectAllActions.IsBound(), TEXT("The OnCollectAllActions delegate is bound, but will not be invoked, because OnGetActionList has also been bound and will be used to obtain the action list for this menu. To resolve this, one of these events should be removed from its construction.")))
		{
			OnCollectAllActions.Unbind();
		}
	}

	// If a delegate for filtering text is passed in, assign it so that it will be used instead of the built-in filter box
	if (InArgs._OnGetFilterText.IsBound())
	{
		this->OnGetFilterText = InArgs._OnGetFilterText;
	}

	TreeView = SNew(STreeView< TSharedPtr<FGraphActionNode> >)
		.TreeItemsSource(&(this->FilteredRootAction->Children))
		.OnGenerateRow(this, &STG_ActionMenuTileView::GenerateChildForTree, bIsReadOnly)
		.OnSelectionChanged(this, &STG_ActionMenuTileView::OnItemSelected)
		.OnMouseButtonDoubleClick(this, &STG_ActionMenuTileView::OnItemDoubleClicked)
		.OnContextMenuOpening(InArgs._OnContextMenuOpening)
		.OnGetChildren(this, &STG_ActionMenuTileView::OnGetChildrenForCategory)
		.SelectionMode(ESelectionMode::Single)
		.OnSetExpansionRecursive(this, &STG_ActionMenuTileView::OnSetExpansionRecursive)
		.HighlightParentNodesForSelection(true);

	this->ChildSlot
		[
			SNew(SVerticalBox)

			// FILTER BOX
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(FilterTextBox, SSearchBox)
				// If there is an external filter delegate, do not display this filter box
				.Visibility(InArgs._OnGetFilterText.IsBound() ? EVisibility::Collapsed : EVisibility::Visible)
				.OnTextChanged(this, &STG_ActionMenuTileView::OnFilterTextChanged)
				.OnTextCommitted(this, &STG_ActionMenuTileView::OnFilterTextCommitted)
				.DelayChangeNotificationsWhileTyping(false)
			]

			// ACTION LIST
			+ SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 2.0f, 0.0f, 0.0f))
			.VAlign(VAlign_Fill)
			.FillHeight(1.0)
			[
				SNew(SScrollBox/*, TreeView.ToSharedRef()*/)
				+SScrollBox::Slot()
				.VAlign(VAlign_Fill)
				.FillSize(1.0)
				[
					TreeView.ToSharedRef()
				]
			]
		];

	// When the search box has focus, we want first chance handling of any key down events so we can handle the up/down and escape keys the way we want
	FilterTextBox->SetOnKeyDownHandler(FOnKeyDown::CreateSP(this, &STG_ActionMenuTileView::OnKeyDown));

	if (!InArgs._ShowFilterTextBox)
	{
		FilterTextBox->SetVisibility(EVisibility::Collapsed);
	}

	// Get all actions.
	RefreshAllActions(false);
}

void STG_ActionMenuTileView::RefreshAllActions(bool bPreserveExpansion, bool bHandleOnSelectionEvent/*=true*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STG_ActionMenuTileView::RefreshAllActions);

	// Save Selection (of only the first selected thing)
	TArray< TSharedPtr<FGraphActionNode> > SelectedNodes = GetSelectedItems();
	TSharedPtr<FGraphActionNode> SelectedAction = SelectedNodes.Num() > 0 ? SelectedNodes[0] : nullptr;

	if (OnGetActionList.IsBound())
	{
		// Obtain the source action list directly.
		AllActions = OnGetActionList.Execute();
	}
	else
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(STG_ActionMenuTileView::CollectAllActions);

		// Collect actions into our local list context.
		AllActions->Empty();
		OnCollectAllActions.ExecuteIfBound(*AllActions);
	}

	GenerateFilteredItems(bPreserveExpansion);

	// Re-apply selection #0 if possible
	if (SelectedAction.IsValid())
	{
		// Clear the selection, we will be re-selecting the previous action
		TreeView->ClearSelection();

		if (bHandleOnSelectionEvent)
		{
			SelectItemByName(*SelectedAction->GetDisplayName().ToString(), ESelectInfo::OnMouseClick, SelectedAction->SectionID, SelectedNodes[0]->IsCategoryNode());
		}
		else
		{
			// If we do not want to handle the selection, set it directly so it will reselect the item but not handle the event.
			SelectItemByName(*SelectedAction->GetDisplayName().ToString(), ESelectInfo::Direct, SelectedAction->SectionID, SelectedNodes[0]->IsCategoryNode());
		}
	}
}

void STG_ActionMenuTileView::GetSectionExpansion(TMap<int32, bool>& SectionExpansion) const
{

}

void STG_ActionMenuTileView::SetSectionExpansion(const TMap<int32, bool>& InSectionExpansion)
{
	for (auto& PossibleSection : FilteredRootAction->Children)
	{
		if (PossibleSection->IsSectionHeadingNode())
		{
			const bool* IsExpanded = InSectionExpansion.Find(PossibleSection->SectionID);
			if (IsExpanded != nullptr)
			{
				TreeView->SetItemExpansion(PossibleSection, *IsExpanded);
			}
		}
	}
}

TSharedRef<SEditableTextBox> STG_ActionMenuTileView::GetFilterTextBox()
{
	return FilterTextBox.ToSharedRef();
}

void STG_ActionMenuTileView::GetSelectedActions(TArray< TSharedPtr<FEdGraphSchemaAction> >& OutSelectedActions) const
{
	OutSelectedActions.Empty();

	TArray< TSharedPtr<FGraphActionNode> > SelectedNodes = GetSelectedItems();
	if (SelectedNodes.Num() > 0)
	{
		for (int32 NodeIndex = 0; NodeIndex < SelectedNodes.Num(); NodeIndex++)
		{
			OutSelectedActions.Add(SelectedNodes[NodeIndex]->Action);
		}
	}
}

void STG_ActionMenuTileView::OnRequestRenameOnActionNode()
{
	TArray< TSharedPtr<FGraphActionNode> > SelectedNodes = GetSelectedItems();
	if (SelectedNodes.Num() > 0)
	{
		if (!SelectedNodes[0]->BroadcastRenameRequest())
		{
			TreeView->RequestScrollIntoView(SelectedNodes[0]);
		}
	}
}

bool STG_ActionMenuTileView::CanRequestRenameOnActionNode() const
{
	return false;
}

FString STG_ActionMenuTileView::GetSelectedCategoryName() const
{
	TArray< TSharedPtr<FGraphActionNode> > SelectedNodes = GetSelectedItems();
	return (SelectedNodes.Num() > 0) ? SelectedNodes[0]->GetDisplayName().ToString() : FString();
}

void STG_ActionMenuTileView::GetSelectedCategorySubActions(TArray<TSharedPtr<FEdGraphSchemaAction>>& OutActions) const
{
	TArray< TSharedPtr<FGraphActionNode> > SelectedNodes = GetSelectedItems();
	for (int32 SelectionIndex = 0; SelectionIndex < SelectedNodes.Num(); SelectionIndex++)
	{
		if (SelectedNodes[SelectionIndex].IsValid())
		{
			GetCategorySubActions(SelectedNodes[SelectionIndex], OutActions);
		}
	}
}

void STG_ActionMenuTileView::GetCategorySubActions(TWeakPtr<FGraphActionNode> InAction, TArray<TSharedPtr<FEdGraphSchemaAction>>& OutActions) const
{
	if (InAction.IsValid())
	{
		TSharedPtr<FGraphActionNode> CategoryNode = InAction.Pin();
		TArray<TSharedPtr<FGraphActionNode>> Children;
		CategoryNode->GetLeafNodes(Children);

		for (int32 i = 0; i < Children.Num(); ++i)
		{
			TSharedPtr<FGraphActionNode> CurrentChild = Children[i];

			if (CurrentChild.IsValid() && CurrentChild->IsActionNode())
			{
				OutActions.Add(CurrentChild->Action);
			}
		}
	}
}

bool STG_ActionMenuTileView::SelectItemByName(const FName& ItemName, ESelectInfo::Type SelectInfo, int32 SectionId/* = INDEX_NONE */, bool bIsCategory/* = false*/)
{
	if (ItemName != NAME_None)
	{
		TSharedPtr<FGraphActionNode> SelectionNode;

		TArray<TSharedPtr<FGraphActionNode>> GraphNodes;
		FilteredRootAction->GetAllNodes(GraphNodes);
		for (int32 i = 0; i < GraphNodes.Num() && !SelectionNode.IsValid(); ++i)
		{
			TSharedPtr<FGraphActionNode> CurrentGraphNode = GraphNodes[i];
			FEdGraphSchemaAction* GraphAction = CurrentGraphNode->GetPrimaryAction().Get();

			// If the user is attempting to select a category, make sure it's a category
			if (CurrentGraphNode->IsCategoryNode() == bIsCategory)
			{
				if (SectionId == INDEX_NONE || CurrentGraphNode->SectionID == SectionId)
				{
					if (GraphAction)
					{
						if ((OnActionMatchesName.IsBound() && OnActionMatchesName.Execute(GraphAction, ItemName)) /*|| GraphActionMenuHelpers::ActionMatchesName(GraphAction, ItemName)*/)
						{
							SelectionNode = GraphNodes[i];

							break;
						}
					}

					if (CurrentGraphNode->GetDisplayName().ToString() == FName::NameToDisplayString(ItemName.ToString(), false))
					{
						SelectionNode = CurrentGraphNode;

						break;
					}
				}
			}

			// One of the children may match
			for (int32 ChildIdx = 0; ChildIdx < CurrentGraphNode->Children.Num() && !SelectionNode.IsValid(); ++ChildIdx)
			{
				TSharedPtr<FGraphActionNode> CurrentChildNode = CurrentGraphNode->Children[ChildIdx];

				if(FEdGraphSchemaAction* ChildGraphAction = CurrentChildNode->Action.Get())
				{
					// If the user is attempting to select a category, make sure it's a category
					if (CurrentChildNode->IsCategoryNode() == bIsCategory)
					{
						if (SectionId == INDEX_NONE || CurrentChildNode->SectionID == SectionId)
						{
							if (ChildGraphAction)
							{
								if ((OnActionMatchesName.IsBound() && OnActionMatchesName.Execute(ChildGraphAction, ItemName)) /*|| GraphActionMenuHelpers::ActionMatchesName(ChildGraphAction, ItemName)*/)
								{
									SelectionNode = GraphNodes[i]->Children[ChildIdx];

									break;
								}
							}
							else if (CurrentChildNode->GetDisplayName().ToString() == FName::NameToDisplayString(ItemName.ToString(), false))
							{
								SelectionNode = CurrentChildNode;

								break;
							}
						}
					}
				}
			}
		}

		if (SelectionNode.IsValid())
		{
			// Expand the parent nodes
			for (TSharedPtr<FGraphActionNode> ParentAction = SelectionNode->GetParentNode().Pin(); ParentAction.IsValid(); ParentAction = ParentAction->GetParentNode().Pin())
			{
				TreeView->SetItemExpansion(ParentAction, true);
			}

			// Select the node
			TreeView->SetSelection(SelectionNode, SelectInfo);
			TreeView->RequestScrollIntoView(SelectionNode);
			return true;
		}
	}
	else
	{
		TreeView->ClearSelection();
		return true;
	}
	return false;
}

void STG_ActionMenuTileView::ExpandCategory(const FText& CategoryName)
{
	if (!CategoryName.IsEmpty())
	{
		TArray<TSharedPtr<FGraphActionNode>> GraphNodes;
		FilteredRootAction->GetAllNodes(GraphNodes);
		for (int32 i = 0; i < GraphNodes.Num(); ++i)
		{
			if (GraphNodes[i]->GetDisplayName().EqualTo(CategoryName))
			{
				GraphNodes[i]->ExpandAllChildren(TreeView);
			}
		}
	}
}

static bool CompareGraphActionNode(TSharedPtr<FGraphActionNode> A, TSharedPtr<FGraphActionNode> B)
{
	check(A.IsValid());
	check(B.IsValid());

	// First check grouping is the same
	if (A->GetDisplayName().ToString() != B->GetDisplayName().ToString())
	{
		return false;
	}

	if (A->SectionID != B->SectionID)
	{
		return false;
	}

	if (A->HasValidAction() && B->HasValidAction())
	{
		return A->GetPrimaryAction()->GetMenuDescription().CompareTo(B->GetPrimaryAction()->GetMenuDescription()) == 0;
	}
	else if (!A->HasValidAction() && !B->HasValidAction())
	{
		return true;
	}
	else
	{
		return false;
	}
}

template<typename ItemType, typename ComparisonType>
void RestoreExpansionState(TSharedPtr< STreeView<ItemType> > InTree, const TArray<ItemType>& ItemSource, const TSet<ItemType>& OldExpansionState, ComparisonType ComparisonFunction)
{
	check(InTree.IsValid());

	// Iterate over new tree items
	for (int32 ItemIdx = 0; ItemIdx < ItemSource.Num(); ItemIdx++)
	{
		ItemType NewItem = ItemSource[ItemIdx];

		// Look through old expansion state
		for (typename TSet<ItemType>::TConstIterator OldExpansionIter(OldExpansionState); OldExpansionIter; ++OldExpansionIter)
		{
			const ItemType OldItem = *OldExpansionIter;
			// See if this matches this new item
			if (ComparisonFunction(OldItem, NewItem))
			{
				// It does, so expand it
				InTree->SetItemExpansion(NewItem, true);
			}
		}
	}
}

void STG_ActionMenuTileView::UpdateForNewActions(int32 IdxStart)
{
	check(bAlphaSortItems && bSortItemsRecursively);

	FScoreResults Results = ScoreAndAddActions(IdxStart);
	UpdateActiveSelection(Results);

	if (ShouldExpandNodes())
	{
		// Expand all
		FilteredRootAction->ExpandAllChildren(TreeView);
	}

	TreeView->RequestTreeRefresh();
}

void STG_ActionMenuTileView::GenerateFilteredItems(bool bPreserveExpansion)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(STG_ActionMenuTileView::GenerateFilteredItems);

	SelectedSuggestionScore = TNumericLimits<float>::Lowest();
	SelectedSuggestionSourceIndex = INDEX_NONE;

	// First, save off current expansion state
	TSet< TSharedPtr<FGraphActionNode> > OldExpansionState;
	if (bPreserveExpansion)
	{
		TreeView->GetExpandedItems(OldExpansionState);
	}

	// Clear the filtered root action
	FilteredRootAction->ClearChildren();

	// Collect the list of always visible sections if any, and force the creation of those sections.
	if (OnCollectStaticSections.IsBound())
	{
		TArray<int32> StaticSectionIDs;
		OnCollectStaticSections.Execute(StaticSectionIDs);

		for (int32 i = 0; i < StaticSectionIDs.Num(); i++)
		{
			FilteredRootAction->AddSection(0, StaticSectionIDs[i]);
		}
	}

	FScoreResults Results = ScoreAndAddActions();

	FilteredRootAction->SortChildren(bAlphaSortItems, bSortItemsRecursively);

	TreeView->RequestTreeRefresh();

	UpdateActiveSelection(Results);

	if (ShouldExpandNodes())
	{
		// Expand all
		FilteredRootAction->ExpandAllChildren(TreeView);
	}
	else
	{
		// Get _all_ new nodes (flattened tree basically)
		TArray< TSharedPtr<FGraphActionNode> > AllNodes;
		FilteredRootAction->GetAllNodes(AllNodes);

		// Expand to match the old state
		RestoreExpansionState< TSharedPtr<FGraphActionNode> >(TreeView, AllNodes, OldExpansionState, CompareGraphActionNode);
	}
}

// Returns true if the tree should be autoexpanded
bool STG_ActionMenuTileView::ShouldExpandNodes() const
{
	// Expand all the categories that have filter results, or when there are only a few to show
	const bool bFilterActive = !GetFilterText().IsEmpty();
	const bool bOnlyAFewTotal = AllActions->GetNumActions() < 10;

	return bFilterActive || bOnlyAFewTotal || bAutoExpandActionMenu;
}

bool STG_ActionMenuTileView::CanRenameNode(TWeakPtr<FGraphActionNode> InNode) const
{
	return false;
}

void STG_ActionMenuTileView::OnFilterTextChanged(const FText& InFilterText)
{
	// Reset the selection if the string is empty
	if (InFilterText.IsEmpty() == true)
	{
		SelectedSuggestion = INDEX_NONE;
	}
	GenerateFilteredItems(false);
}

void STG_ActionMenuTileView::OnFilterTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		TryToSpawnActiveSuggestion();
	}
}

bool STG_ActionMenuTileView::TryToSpawnActiveSuggestion()
{
	TArray< TSharedPtr<FGraphActionNode> > SelectionList = GetSelectedItems();

	if (SelectionList.Num() == 1)
	{
		// This isnt really a keypress - its Direct, but its always called from a keypress function. (Maybe pass the selectinfo in ?)
		OnItemSelected(SelectionList[0], ESelectInfo::OnKeyPress);
		return true;
	}
	else if (FilteredActionNodes.Num() == 1)
	{
		OnItemSelected(FilteredActionNodes[0], ESelectInfo::OnKeyPress);
		return true;
	}

	return false;
}

void STG_ActionMenuTileView::OnGetChildrenForCategory(TSharedPtr<FGraphActionNode> InItem, TArray< TSharedPtr<FGraphActionNode> >& OutChildren)
{
	//Intentionally leaving it unimplemented as we do not want the children to spawn for tree
}

TSharedRef<ITableRow> STG_ActionMenuTileView::MakeWidget(TSharedPtr<FGraphActionNode> InItem, const TSharedRef<STableViewBase>& OwnerTable, bool bIsReadOnly, bool bMakeTiles)
{
	TSharedPtr<IToolTip> SectionToolTip;

	if (InItem->IsSectionHeadingNode())
	{
		if (OnGetSectionToolTip.IsBound())
		{
			SectionToolTip = OnGetSectionToolTip.Execute(InItem->SectionID);
		}
	}

	// In the case of FGraphActionNodes that have multiple actions, all of the actions will
	// have the same text as they will have been created at the same point - only the actual
	// action itself will differ, which is why parts of this function only refer to InItem->Actions[0]
	// rather than iterating over the array

	// Create the widget but do not add any content, the widget is needed to pass the IsSelectedExclusively function down to the potential SInlineEditableTextBlock widget
	TSharedPtr< STableRow< TSharedPtr<FGraphActionNode> > > TableRow;

	if (InItem->IsSectionHeadingNode())
	{
		TableRow = SNew(STG_CategoryHeaderTableRow< TSharedPtr<FGraphActionNode> >, OwnerTable)
			.ToolTip(SectionToolTip);
	}
	else
	{
		if (bMakeTiles)
		{
			TableRow = SNew(STableRow< TSharedPtr<FGraphActionNode> >, OwnerTable)
			.Style(FAppStyle::Get(), "ContentBrowser.AssetListView.TileTableRow")
			.OnDragDetected(this, &STG_ActionMenuTileView::OnItemDragDetected)
			.ShowSelection(!InItem->IsSeparator())
			.bAllowPreselectedItemActivation(bAllowPreselectedItemActivation)
			.Cursor( EMouseCursor::GrabHand);
		}
		else
		{
			TableRow = SNew(STableRow< TSharedPtr<FGraphActionNode> >, OwnerTable)
			.ShowSelection(!InItem->IsSeparator())
			.bAllowPreselectedItemActivation(bAllowPreselectedItemActivation);
		}
	}

	TSharedPtr<SHorizontalBox> RowContainer;
	TableRow->SetRowContent
	(
		SAssignNew(RowContainer, SHorizontalBox)
	);

	TSharedPtr<SWidget> RowContent;
	FMargin RowPadding = FMargin(0,4,0,0);

	if (InItem->IsActionNode())
	{
		//When called for tree we are disabling the Action Nodes, As we are using tree to only create the category nodes
		//Action nodes are spawned inside of the Tile view of Category node
		if (!bMakeTiles)
		{
			TableRow->SetVisibility(EVisibility::Collapsed);
		}
		
		check(InItem->HasValidAction());

		FCreateWidgetForActionData CreateData(&InItem->OnRenameRequest());
		CreateData.Action = InItem->GetPrimaryAction();
		CreateData.HighlightText = TAttribute<FText>(this, &STG_ActionMenuTileView::GetFilterText);
		CreateData.MouseButtonDownDelegate = FCreateWidgetMouseButtonDown::CreateSP(this, &STG_ActionMenuTileView::OnMouseButtonDownEvent);

		if (OnCreateWidgetForAction.IsBound() && bMakeTiles)
		{
			CreateData.IsRowSelectedDelegate = FIsSelected::CreateSP(TableRow.Get(), &STableRow< TSharedPtr<FGraphActionNode> >::IsSelected);
			CreateData.bIsReadOnly = bIsReadOnly;
			CreateData.bHandleMouseButtonDown = false;		//Default to NOT using the delegate. OnCreateWidgetForAction can set to true if we need it
			RowContent = OnCreateWidgetForAction.Execute(&CreateData);
			RowPadding = FMargin(0, 0);
		}
		else
		{
			RowPadding = FMargin(0, 0);
			RowContent = SNullWidget::NullWidget;
		}
	}
	else if (InItem->IsCategoryNode())
	{
		TWeakPtr< FGraphActionNode > WeakItem = InItem;
		// Hook up the delegate for verifying the category action is read only or not
		STG_GraphActionCategoryWidget::FArguments ReadOnlyArgument;
		if (bIsReadOnly)
		{
			ReadOnlyArgument.IsReadOnly(bIsReadOnly);
		}
		else
		{
			ReadOnlyArgument.IsReadOnly(this, &STG_ActionMenuTileView::CanRenameNode, WeakItem);
		}

		TSharedRef<STG_GraphActionCategoryWidget> CategoryWidget =
			SNew(STG_GraphActionCategoryWidget, InItem)
			.HighlightText(this, &STG_ActionMenuTileView::GetFilterText)
			.IsSelected(TableRow.Get(), &STableRow< TSharedPtr<FGraphActionNode> >::IsSelectedExclusively)
			.IsReadOnly(ReadOnlyArgument._IsReadOnly);

		if (!bIsReadOnly)
		{
			InItem->OnRenameRequest().BindSP(CategoryWidget->InlineWidget.Pin().Get(), &SInlineEditableTextBlock::EnterEditingMode);
		}

		FString ChildCount = FString::Format(TEXT("{0} Items"), { InItem->Children.Num() });

		RowContent = CategoryWidget;
		auto TileView = SNew(STileView<TSharedPtr<FGraphActionNode>>)
			.ListItemsSource(&InItem->Children)
			.OnGenerateTile(this, &STG_ActionMenuTileView::GenerateTileForChildItem)
			.OnSelectionChanged(this, &STG_ActionMenuTileView::OnTileItemSelected)
			.ItemWidth(this,&STG_ActionMenuTileView::GetTileViewItemWidth)
			.SelectionMode(ESelectionMode::Single)
			.ItemHeight(TileHeight)
			.ScrollbarVisibility(EVisibility::Collapsed);

		auto ChildWidget = SNew(SBorder)
			.BorderBackgroundColor(FStyleColors::Recessed)
			.BorderImage(FCoreStyle::Get().GetBrush("WhiteBrush"))
			.Padding(FMargin(6.0f, 2.0f, 5.0f, 2.0f))
			.Visibility_Lambda([=]() {
				if (TableRow->IsItemExpanded())
				{
					return EVisibility::Visible;
				}
				return EVisibility::Collapsed;
			})
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					TileView
				]
			];

		TreeTileViewMap.Emplace(InItem, TileView);

		TableRow->SetRowContent
		(
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.VAlign(VAlign_Top)
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("WhiteBrush"))
				.BorderBackgroundColor(EStyleColor::Recessed)
				[
					RowContainer.ToSharedRef()
				]
			]

			+ SVerticalBox::Slot()
			.VAlign(VAlign_Top)
			.AutoHeight()
			[
				ChildWidget
			]
		);
		
	}

	//Only add expander for the category node
	if (!bMakeTiles && InItem->IsCategoryNode())
	{
		TSharedPtr<STG_CategoryHeaderExpanderArrow> ExpanderWidget = 
			SNew(STG_CategoryHeaderExpanderArrow, TableRow , TreeView, InItem)
			.AlwaysVisible(true);

		RowContainer->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Right)
		[
			ExpanderWidget.ToSharedRef()
		];
	}

	RowContainer->AddSlot()
	.FillWidth(1.0)
	.Padding(RowPadding)
	[
		RowContent.ToSharedRef()
	];

	return TableRow.ToSharedRef();
}

void STG_ActionMenuTileView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	CalculateFillScale(AllottedGeometry);
}

void STG_ActionMenuTileView::CalculateFillScale(const FGeometry& AllottedGeometry)
{
	float ItemWidth = TileWidth;

	// Scrollbars are 16, but we add 1 to deal with half pixels.
	const float ScrollbarWidth = 16 + 1 + 8;//+8 for tile view left and right padding
	float TotalWidth = AllottedGeometry.GetLocalSize().X - (ScrollbarWidth);
	float Coverage = TotalWidth / ItemWidth;
	int32 Items = (int)(TotalWidth / ItemWidth);

	// If there isn't enough room to support even a single item, don't apply a fill scale.
	if (Items > 0)
	{
		float GapSpace = ItemWidth * (Coverage - (float)Items);
		float ExpandAmount = GapSpace / (float)Items;
		FillScale = (ItemWidth + ExpandAmount) / ItemWidth;
		FillScale = FMath::Max(1.0f, FillScale);
	}
	else
	{
		FillScale = 1.0f;
	}
}

float STG_ActionMenuTileView::GetTileViewItemWidth() const
{
	return (float)(TileWidth * FillScale);
}

TSharedRef<ITableRow> STG_ActionMenuTileView::GenerateChildForTree(TSharedPtr<FGraphActionNode> InItem, const TSharedRef<STableViewBase>& OwnerTable, bool bIsReadOnly)
{
	return MakeWidget(InItem, OwnerTable, bIsReadOnly, false);
}

TSharedRef<ITableRow> STG_ActionMenuTileView::GenerateTileForChildItem(TSharedPtr<FGraphActionNode> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return MakeWidget(InItem, OwnerTable, false, true);
}

FText STG_ActionMenuTileView::GetFilterText() const
{
	// If there is an external source for the filter, use that text instead
	if (OnGetFilterText.IsBound())
	{
		return OnGetFilterText.Execute();
	}

	return FilterTextBox->GetText();
}

void STG_ActionMenuTileView::OnItemSelected(TSharedPtr< FGraphActionNode > InSelectedItem, ESelectInfo::Type SelectInfo)
{
	if (!bIgnoreUIUpdate)
	{
		HandleSelection(InSelectedItem, SelectInfo);
	}
}

void STG_ActionMenuTileView::OnTileItemSelected(TSharedPtr< FGraphActionNode > InSelectedItem, ESelectInfo::Type SelectInfo)
{
	if (!bIgnoreUIUpdate)
	{
		if (InSelectedItem.IsValid())
		{
			for (auto TileViewPair : TreeTileViewMap)
			{
				if (TileViewPair.Key != InSelectedItem->GetParentNode())
					TileViewPair.Value->ClearSelection();
			}
			TreeView->ClearSelection();
		}
		HandleSelection(InSelectedItem, SelectInfo);
	}
}

TArray< TSharedPtr<FGraphActionNode> > STG_ActionMenuTileView::GetSelectedItems() const
{
	TArray< TSharedPtr<FGraphActionNode> > SelectedItems;
	for (auto TileViewPair : TreeTileViewMap)
	{
		for (auto SelectedItem : TileViewPair.Value->GetSelectedItems())
		{
			SelectedItems.Add(SelectedItem);
		}
	}
	return SelectedItems;
}

void STG_ActionMenuTileView::OnItemDoubleClicked(TSharedPtr< FGraphActionNode > InClickedItem)
{
	if (InClickedItem.IsValid() && !bIgnoreUIUpdate)
	{
		if (InClickedItem->IsActionNode())
		{
			OnActionDoubleClicked.ExecuteIfBound({InClickedItem->Action});
		}
		else if (InClickedItem->Children.Num())
		{
			TreeView->SetItemExpansion(InClickedItem, !TreeView->IsItemExpanded(InClickedItem));
		}
	}
}

FReply STG_ActionMenuTileView::OnItemDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Start a function-call drag event for any entry that can be called by kismet
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		TArray< TSharedPtr<FGraphActionNode> > SelectedNodes = GetSelectedItems();
		if (SelectedNodes.Num() > 0)
		{
			TSharedPtr<FGraphActionNode> Node = SelectedNodes[0];
			// Dragging a ctaegory
			if (Node.IsValid() && Node->IsCategoryNode())
			{
				if (OnCategoryDragged.IsBound())
				{
					return OnCategoryDragged.Execute(Node->GetCategoryPath(), MouseEvent);
				}
			}
			// Dragging an action
			else
			{
				if (OnActionDragged.IsBound())
				{
					TArray< TSharedPtr<FEdGraphSchemaAction> > Actions;
					GetSelectedActions(Actions);
					return OnActionDragged.Execute(Actions, MouseEvent);
				}
			}
		}
	}

	return FReply::Unhandled();
}

bool STG_ActionMenuTileView::OnMouseButtonDownEvent(TWeakPtr<FEdGraphSchemaAction> InAction)
{
	bool bResult = false;
	if ((!bIgnoreUIUpdate) && InAction.IsValid())
	{
		TArray< TSharedPtr<FGraphActionNode> > SelectionList = GetSelectedItems();
		TSharedPtr<FGraphActionNode> SelectedNode;
		if (SelectionList.Num() == 1)
		{
			SelectedNode = SelectionList[0];
		}
		else if (FilteredActionNodes.Num() == 1)
		{
			SelectedNode = FilteredActionNodes[0];
		}
		if (SelectedNode.IsValid() && SelectedNode->HasValidAction())
		{
			if (SelectedNode->GetPrimaryAction().Get() == InAction.Pin().Get())
			{
				bResult = HandleSelection(SelectedNode, ESelectInfo::OnMouseClick);
			}
		}
	}
	return bResult;
}

FReply STG_ActionMenuTileView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent)
{
	int32 SelectionDelta = 0;

	// Escape dismisses the menu without placing a node
	if (KeyEvent.GetKey() == EKeys::Escape)
	{
		FSlateApplication::Get().DismissAllMenus();
		return FReply::Handled();
	}
	else if ((KeyEvent.GetKey() == EKeys::Enter) && !bIgnoreUIUpdate)
	{
		return TryToSpawnActiveSuggestion() ? FReply::Handled() : FReply::Unhandled();
	}
	else if (!FilterTextBox->GetText().IsEmpty())
	{
		// Needs to be done here in order not to eat up the text navigation key events when list isn't populated
		if (FilteredActionNodes.Num() == 0)
		{
			return FReply::Unhandled();
		}

		if (KeyEvent.GetKey() == EKeys::Up)
		{
			SelectedSuggestion = FMath::Max(0, SelectedSuggestion - 1);
		}
		else if (KeyEvent.GetKey() == EKeys::Down)
		{
			SelectedSuggestion = FMath::Min(FilteredActionNodes.Num() - 1, SelectedSuggestion + 1);
		}
		else if (KeyEvent.GetKey() == EKeys::PageUp)
		{
			const int32 NumItemsInAPage = 15; // arbitrary jump because we can't get at the visible item count from here
			SelectedSuggestion = FMath::Max(0, SelectedSuggestion - NumItemsInAPage);
		}
		else if (KeyEvent.GetKey() == EKeys::PageDown)
		{
			const int32 NumItemsInAPage = 15; // arbitrary jump because we can't get at the visible item count from here
			SelectedSuggestion = FMath::Min(FilteredActionNodes.Num() - 1, SelectedSuggestion + NumItemsInAPage);
		}
		else if (KeyEvent.GetKey() == EKeys::Home && KeyEvent.IsControlDown())
		{
			SelectedSuggestion = 0;
		}
		else if (KeyEvent.GetKey() == EKeys::End && KeyEvent.IsControlDown())
		{
			SelectedSuggestion = FilteredActionNodes.Num() - 1;
		}
		else
		{
			return FReply::Unhandled();
		}

		MarkActiveSuggestion();
		return FReply::Handled();
	}
	else
	{
		// When all else fails, it means we haven't filtered the list and we want to handle it as if we were just scrolling through a normal tree view
		return TreeView->OnKeyDown(FindChildGeometry(MyGeometry, TreeView.ToSharedRef()), KeyEvent);
	}
}

void STG_ActionMenuTileView::MarkActiveSuggestion()
{
	TGuardValue<bool> PreventSelectionFromTriggeringCommit(bIgnoreUIUpdate, true);

	if (SelectedSuggestion >= 0)
	{
		TSharedPtr<FGraphActionNode>& ActionToSelect = FilteredActionNodes[SelectedSuggestion];

		TreeView->SetSelection(ActionToSelect);
		TreeView->RequestScrollIntoView(ActionToSelect);
	}
	else
	{
		TreeView->ClearSelection();
	}
}

void STG_ActionMenuTileView::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (int32 CurTypeIndex = 0; CurTypeIndex < AllActions->GetNumActions(); ++CurTypeIndex)
	{
		TSharedPtr<FEdGraphSchemaAction>& Action = AllActions->GetSchemaAction(CurTypeIndex);
		Action->AddReferencedObjects(Collector);
	}
}

FString STG_ActionMenuTileView::GetReferencerName() const
{
	return TEXT("STG_ActionMenuTileView");
}

bool STG_ActionMenuTileView::HandleSelection(TSharedPtr< FGraphActionNode >& InSelectedItem, ESelectInfo::Type InSelectionType)
{
	bool bResult = false;
	if (OnActionSelected.IsBound())
	{
		if (InSelectedItem.IsValid() && InSelectedItem->IsActionNode())
		{
			OnActionSelected.Execute({InSelectedItem->Action}, InSelectionType);
			bResult = true;
		}
		else
		{
			OnActionSelected.Execute(TArray< TSharedPtr<FEdGraphSchemaAction> >(), InSelectionType);
			bResult = true;
		}
	}
	return bResult;
}

void STG_ActionMenuTileView::OnSetExpansionRecursive(TSharedPtr<FGraphActionNode> InTreeNode, bool bInIsItemExpanded)
{
	if (InTreeNode.IsValid() && InTreeNode->Children.Num())
	{
		TreeView->SetItemExpansion(InTreeNode, bInIsItemExpanded);

		for (TSharedPtr<FGraphActionNode> Child : InTreeNode->Children)
		{
			OnSetExpansionRecursive(Child, bInIsItemExpanded);
		}
	}
}

STG_ActionMenuTileView::FScoreResults STG_ActionMenuTileView::ScoreAndAddActions(int32 StartingIndex)
{
	// Trim and sanitized the filter text (so that it more likely matches the action descriptions)
	FString TrimmedFilterString = FText::TrimPrecedingAndTrailing(GetFilterText()).ToString();

	// Remember the last filter string to that external clients can access it
	LastUsedFilterText = TrimmedFilterString;

	// Tokenize the search box text into a set of terms; all of them must be present to pass the filter
	TArray<FString> FilterTerms;
	TrimmedFilterString.ParseIntoArray(FilterTerms, TEXT(" "), true);
	for (FString& String : FilterTerms)
	{
		String = String.ToLower();
	}

	// Generate a list of sanitized versions of the strings
	TArray<FString> SanitizedFilterTerms;
	for (int32 iFilters = 0; iFilters < FilterTerms.Num(); iFilters++)
	{
		FString EachString = FName::NameToDisplayString(FilterTerms[iFilters], false);
		EachString = EachString.Replace(TEXT(" "), TEXT(""));
		SanitizedFilterTerms.Add(EachString);
	}
	ensure(SanitizedFilterTerms.Num() == FilterTerms.Num());// Both of these should match !

	const bool bRequiresFiltering = FilterTerms.Num() > 0;
	float BestMatchCount = SelectedSuggestionScore;
	int32 BestMatchIndex = SelectedSuggestionSourceIndex;

	// Get the schema of the graph that we are in so that we can correctly get the action weight
	const UEdGraphSchema* ActionSchema = GraphObj ? GraphObj->GetSchema() : GetDefault<UEdGraphSchema>();
	check(ActionSchema);

	bool bIsPartialBuild = StartingIndex != INDEX_NONE;
	const int32 NumActions = AllActions->GetNumActions();
	for (int32 CurTypeIndex = bIsPartialBuild ? StartingIndex : 0; CurTypeIndex < NumActions; ++CurTypeIndex)
	{
		TSharedPtr<FEdGraphSchemaAction> CurrentAction = AllActions->GetSchemaAction(CurTypeIndex);

		// If we're filtering, search check to see if we need to show this action
		bool bShowAction = true;
		float EachWeight = TNumericLimits<float>::Lowest();

		const FString& SearchText = CurrentAction->GetFullSearchText();
		for (int32 FilterIndex = 0; (FilterIndex < FilterTerms.Num()) && bShowAction; ++FilterIndex)
		{
			const bool bMatchesTerm = (SearchText.Contains(FilterTerms[FilterIndex], ESearchCase::CaseSensitive) || (SearchText.Contains(SanitizedFilterTerms[FilterIndex], ESearchCase::CaseSensitive) == true));
			bShowAction = bMatchesTerm;
		}

		if (!bShowAction)
		{
			continue;
		}

		if (bRequiresFiltering)
		{
			// Get the 'weight' of this in relation to the filter
			EachWeight = ActionSchema->GetActionFilteredWeight(*CurrentAction, FilterTerms, SanitizedFilterTerms, DraggedFromPins);
			// If this action has a greater relevance than others, cache its index.
			if (EachWeight > BestMatchCount)
			{
				BestMatchCount = EachWeight;
				BestMatchIndex = CurTypeIndex;
			}
		}

		if (bIsPartialBuild)
		{
			FilteredRootAction->AddChildAlphabetical(CurrentAction);
		}
		else
		{
			FilteredRootAction->AddChild(CurrentAction);
		}
	}

	return { BestMatchIndex, BestMatchCount };
}

void STG_ActionMenuTileView::UpdateActiveSelection(STG_ActionMenuTileView::FScoreResults ForResults)
{
	int32 BestMatchIndex = ForResults.BestMatchIndex;
	float BestMatchCount = ForResults.BestMatchScore;
	// Update the filtered list (needs to be done in a separate pass because the list is sorted as items are inserted)
	FilteredActionNodes.Reset();
	FilteredRootAction->GetLeafNodes(FilteredActionNodes);

	// If theres a BestMatchIndex find it in the actions nodes and select it (maybe this should check the current selected suggestion first ?)
	if (BestMatchIndex != INDEX_NONE)
	{
		TSharedPtr<FEdGraphSchemaAction>& Action = AllActions->GetSchemaAction(BestMatchIndex);
		if (Action.IsValid() == true)
		{
			for (int32 iNode = 0; iNode < FilteredActionNodes.Num(); iNode++)
			{
				if (FilteredActionNodes[iNode].Get()->GetPrimaryAction() == Action)
				{
					SelectedSuggestion = iNode;
					SelectedSuggestionScore = BestMatchCount;
					SelectedSuggestionSourceIndex = BestMatchIndex;
				}
			}
		}
	}

	// Make sure the selected suggestion stays within the filtered list
	if ((SelectedSuggestion >= 0) && (FilteredActionNodes.Num() > 0))
	{
		//@TODO: Should try to actually maintain the highlight on the same item if it survived the filtering
		SelectedSuggestion = FMath::Clamp<int32>(SelectedSuggestion, 0, FilteredActionNodes.Num() - 1);
		MarkActiveSuggestion();
	}
	else
	{
		SelectedSuggestionScore = TNumericLimits<float>::Lowest();
		SelectedSuggestionSourceIndex = INDEX_NONE;
		SelectedSuggestion = INDEX_NONE;
	}
}
#undef LOCTEXT_NAMESPACE
