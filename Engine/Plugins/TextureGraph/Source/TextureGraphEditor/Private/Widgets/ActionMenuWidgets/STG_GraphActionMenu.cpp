// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_GraphActionMenu.h"

#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformMath.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Types/SlateStructs.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "TG_Style.h"
#include "EdGraph/TG_EdGraphSchema.h"
#include "STG_Palette.h"

class SEditableTextBox;

STG_GraphActionMenu::~STG_GraphActionMenu()
{
	OnClosedCallback.ExecuteIfBound();
}

void STG_GraphActionMenu::Construct(const FArguments& InArgs)
{
	this->GraphObj = InArgs._GraphObj;
	this->bSpwanOnSelect = InArgs._bSpwanOnSelect;
	this->DraggedFromPins = InArgs._DraggedFromPins;
	this->NewNodePosition = InArgs._NewNodePosition;
	this->OnClosedCallback = InArgs._OnClosedCallback;
	this->AutoExpandActionMenu = InArgs._AutoExpandActionMenu;

	FLinearColor FillColor = UTG_EdGraphSchema::NodeBodyColor;

	if (bSpwanOnSelect)
	{
		Brush = *FTG_Style::Get().GetBrush("TG.Palette.Background");
		Brush.OutlineSettings.Color = FSlateColor(FillColor * UTG_EdGraphSchema::NodeOutlineColorMultiplier);
	}
	else
	{
		Brush = *FCoreStyle::Get().GetBrush("WhiteBrush");
		Brush.OutlineSettings.Color = FSlateColor(FillColor);
	}

	// Build the widget layout
	SBorder::Construct(SBorder::FArguments()
		.BorderImage(&Brush)
		.BorderBackgroundColor(FillColor)
		.Padding(2)
		[
			// Achieving fixed width by nesting items within a fixed width box.
			SNew(SBox)
			.WidthOverride(400)
			.HeightOverride(400)
			[
				SAssignNew(GraphActionMenu, SGraphActionMenu)
				.OnActionSelected(this, &STG_GraphActionMenu::OnActionSelected)
				.OnCollectAllActions(this, &STG_GraphActionMenu::CollectAllActions)
				.OnCreateWidgetForAction(this, &STG_GraphActionMenu::OnCreateWidgetForAction)
				.AutoExpandActionMenu(AutoExpandActionMenu)
				.DraggedFromPins(DraggedFromPins)
				.OnActionDragged(InArgs._OnActionDragged)
				.GraphObj(GraphObj)
				.DefaultRowExpanderBaseIndentLevel(10)
			]
		]);
}

void STG_GraphActionMenu::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	// Build up the context object
	FGraphContextMenuBuilder ContextMenuBuilder(GraphObj);
	if (DraggedFromPins.Num() > 0)
	{
		ContextMenuBuilder.FromPin = DraggedFromPins[0];
	}

	// Determine all possible actions
	GraphObj->GetSchema()->GetGraphContextActions(ContextMenuBuilder);

	// Copy the added options back to the main list
	//@TODO: Avoid this copy
	OutAllActions.Append(ContextMenuBuilder);
}

TSharedRef<SWidget> STG_GraphActionMenu::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	if (!bSpwanOnSelect)
	{
		//Dont need the mouse down event when not spawning on selection
		InCreateData->MouseButtonDownDelegate.Unbind();
	}
	
	return	SNew(STG_PaletteItem, InCreateData);
}

TSharedRef<SEditableTextBox> STG_GraphActionMenu::GetFilterTextBox()
{
	return GraphActionMenu->GetFilterTextBox();
}


void STG_GraphActionMenu::OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedAction, ESelectInfo::Type InSelectionType)
{
	if (bSpwanOnSelect && (InSelectionType == ESelectInfo::OnMouseClick || InSelectionType == ESelectInfo::OnKeyPress || SelectedAction.Num() == 0))
	{
		bool bDoDismissMenus = true;

		if (GraphObj != NULL)
		{
			for (int32 ActionIndex = 0; ActionIndex < SelectedAction.Num(); ActionIndex++)
			{
				TSharedPtr<FEdGraphSchemaAction> CurrentAction = SelectedAction[ActionIndex];

				if (CurrentAction.IsValid())
				{
					if (bDoDismissMenus)
					{
						FSlateApplication::Get().DismissAllMenus();
						bDoDismissMenus = false;
					}

					CurrentAction->PerformAction(GraphObj, DraggedFromPins, NewNodePosition);
				}
			}
		}
	}
}
