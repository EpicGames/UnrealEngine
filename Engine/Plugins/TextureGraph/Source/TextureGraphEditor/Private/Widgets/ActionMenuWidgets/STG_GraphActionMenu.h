// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "GraphEditor.h"
#include "HAL/PlatformCrt.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "SGraphActionMenu.h"
#include "SGraphPalette.h"

class SEditableTextBox;
class UEdGraph;
class UEdGraphPin;
struct FEdGraphSchemaAction;
struct FGraphActionListBuilderBase;
class SInlineEditableTextBlock;

/////////////////////////////////////////////////////////////////////////////////////////////////
/* This class implements the customization to the SGraphActionMenu for Texture Graph*/
class STG_GraphActionMenu : public SBorder
{
public:
	SLATE_BEGIN_ARGS(STG_GraphActionMenu)
		: _GraphObj(static_cast<UEdGraph*>(NULL))
		, _bSpwanOnSelect(true)
		, _NewNodePosition(FVector2D::ZeroVector)
		, _AutoExpandActionMenu(false)
	{}

	SLATE_ARGUMENT(UEdGraph*, GraphObj)
		SLATE_ARGUMENT(bool, bSpwanOnSelect)
		SLATE_ARGUMENT(FVector2D, NewNodePosition)
		SLATE_ARGUMENT(TArray<UEdGraphPin*>, DraggedFromPins)
		SLATE_ARGUMENT(SGraphEditor::FActionMenuClosed, OnClosedCallback)
		SLATE_ARGUMENT(bool, AutoExpandActionMenu)

		SLATE_EVENT(SGraphActionMenu::FOnActionDragged, OnActionDragged)
		SLATE_EVENT(SGraphActionMenu::FOnCollectAllActions, OnCollectAllActions)
		SLATE_EVENT(SGraphActionMenu::FOnCreateWidgetForAction, OnCreateWidgetForAction)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	~STG_GraphActionMenu();

	TSharedRef<SEditableTextBox> GetFilterTextBox();
	TSharedRef<SWidget> OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData);

protected:
	//From Right click menu action is executed when selected
	bool bSpwanOnSelect;

	FSlateBrush Brush;

	UEdGraph* GraphObj;

	TArray<UEdGraphPin*> DraggedFromPins;

	FDeprecateSlateVector2D NewNodePosition;

	bool AutoExpandActionMenu;

	SGraphEditor::FActionMenuClosed OnClosedCallback;

	TSharedPtr<SGraphActionMenu> GraphActionMenu;

	void OnActionSelected(const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedAction, ESelectInfo::Type InSelectionType);

	/** Callback used to populate all actions list in SGraphActionMenu */
	void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);
};
