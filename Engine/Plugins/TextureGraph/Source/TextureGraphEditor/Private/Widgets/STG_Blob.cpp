// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_Blob.h"

#include "2D/Tex.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include "Engine/Texture2D.h"
#include "Transform/Layer/T_Thumbnail.h"

#define LOCTEXT_NAMESPACE "TextureGraphEditor"

void STG_Blob::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(BrushMaterial);
}

FString STG_Blob::GetReferencerName() const
{
	return TEXT("STG_Blob");
}

/** Constructs this widget with InArgs */
void STG_Blob::Construct(const FArguments& InArgs)
{
	BrushName = InArgs._BrushName;
	Brush = MakeShareable(
		new FSlateDynamicImageBrush(
			(UTexture2D*) nullptr,
			FVector2D(InArgs._Width, InArgs._Height),
			//desiredSize,
			BrushName,
			FLinearColor(1.0f, 1.0f, 1.0f, 1.0f),
			ESlateBrushTileType::NoTile,
			ESlateBrushImageType::Linear
		)
	);
	UMaterial* material = GetMaterial();
	BrushMaterial = (UMaterialInstanceDynamic::Create(material, nullptr));

	Position = FVector2D(100.0f, 100.0f);
	UpdateBlob(InArgs._Blob);
}

UMaterial* STG_Blob::GetMaterial()
{
	return LoadObject<UMaterial>(nullptr, TEXT("/TextureGraph/Materials/Util/RoundedThumb"));
}

UTexture* STG_Blob::GetTextureFromBlob(BlobPtr InBlob)
{
	UTexture* BlobTexture = nullptr;
	//By now the Result should have valid DeviceBuffer
	if (InBlob)
	{
		check(InBlob->IsFinalised());
		DeviceBufferPtr Buffer = InBlob->GetBufferRef().GetPtr();
		check(Buffer);

		auto FXBuffer = std::static_pointer_cast<DeviceBuffer_FX>(Buffer);
		// Got the description, now need to retreive the true live buffer and the texture
		{
			// If the Buffer is an FX buffer we can extract the Texture from it, Can be a texture2D or RT
			if (FXBuffer)
			{
				auto Tex = FXBuffer->GetTexture();
				check(Tex);
				Tex->SetFilter(TextureFilter::TF_Nearest);
				BlobTexture = Tex->GetTexture();
				//PinAssetMap[Pin->GetId()].ThumbAsset->SetAsset(BlobTexture);
			}
		}
		// auto Texture = Cast<UTexture>(AssetThumbnail->GetAsset());
	}

	return BlobTexture;
}

void STG_Blob::UpdateParams(BlobPtr InBlob)
{
	UTexture* BlobTexture = GetTextureFromBlob(InBlob);

	//FRHITexture* SlateTexture = (FRHITexture*)(AssetThumbnail->GetViewportRenderTargetTexture());
	UTexture2D* Texture2D = Cast<UTexture2D>(BlobTexture);
	UTextureRenderTarget2D* TextureRT2D = Cast<UTextureRenderTarget2D>(BlobTexture);

	float ShowChecker = 1.0;
	if (BlobTexture)
	{
		ShowChecker = 0.0;
		BrushMaterial->SetTextureParameterValue("ThumbTex", BlobTexture);
	}

	float SingleChannel = 0.0;

	if (Texture2D)
	{
		SingleChannel = GPixelFormats[Texture2D->GetPixelFormat()].NumComponents == 1 ? 1.0 : 0.0;
	}

	if (TextureRT2D)
	{
		SingleChannel = GPixelFormats[TextureRT2D->GetFormat()].NumComponents == 1 ? 1.0 : 0.0;
	}

	BrushMaterial->SetScalarParameterValue("ShowChecker", ShowChecker);
	BrushMaterial->SetScalarParameterValue("SingleChannel", SingleChannel);
	Brush->SetResourceObject(BrushMaterial);
}

void STG_Blob::UpdateBlob(BlobPtr InBlob)
{
	UpdateParams(InBlob);

	TSharedPtr<SOverlay> overlay;
	ChildSlot
	[
		SAssignNew(overlay, SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SImage)
			.Image(Brush.Get())
		]
	];
}

//////////////////////////////////////////////////////////////////////////
#undef LOCTEXT_NAMESPACE