// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/Material.h"
#include "AssetThumbnail.h"
#include "Data/TiledBlob.h"
#include "FxMat/RenderMaterial_Thumbnail.h"

class STG_Blob : public SCompoundWidget, public FGCObject
{
public:
	SLATE_BEGIN_ARGS(STG_Blob)
		:_Blob(nullptr)
		,_BrushName("Blob")
		,_Width(RenderMaterial_Thumbnail::GThumbWidth)
		,_Height(RenderMaterial_Thumbnail::GThumbHeight)
	{}

	SLATE_ARGUMENT(TiledBlobPtr , Blob)
	SLATE_ARGUMENT(FName , BrushName)
	SLATE_ARGUMENT(int , Width)
	SLATE_ARGUMENT(int, Height)
	SLATE_END_ARGS()

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	void UpdateBlob(BlobPtr InBlob);
	virtual void UpdateParams(BlobPtr InBlob);
	virtual UMaterial* GetMaterial();
	UTexture* GetTextureFromBlob(BlobPtr InBlob);

protected:
	FName BrushName;
	TSharedPtr<FSlateBrush> Brush;
	TObjectPtr<UMaterialInstanceDynamic> BrushMaterial = nullptr;

private:
	FVector2D Position;
};
