// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_HistogramBlob.h"

#include "2D/Tex.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include "Engine/Texture2D.h"
#include "Transform/Layer/T_Thumbnail.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(STG_HistogramBlob)

#define LOCTEXT_NAMESPACE "TextureGraphEditor"

void STG_HistogramBlob::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(BrushMaterial);
}

FString STG_HistogramBlob::GetReferencerName() const
{
	return TEXT("STG_HistogramBlob");
}
/** Constructs this widget with InArgs */
void STG_HistogramBlob::Construct(const FArguments& InArgs)
{
	auto BlobArgs = STG_Blob::FArguments();
	BlobArgs._Height = InArgs._Height;
	BlobArgs._Width = InArgs._Width;
	BlobArgs._Blob = InArgs._HistogramResult;
	BlobArgs._BrushName = InArgs._BrushName;

	//HistogramResult = InArgs._HistogramResult;
	Curves = InArgs._Curves;

	STG_Blob::Construct(BlobArgs);
	Clear();
}

void STG_HistogramBlob::Clear()
{
	//Do not want to show any thing when no data is calculated
	ChildSlot
	[
		SNew( SOverlay)
	];
}

UMaterial* STG_HistogramBlob::GetMaterial()
{
	return LoadObject<UMaterial>(nullptr, TEXT("/TextureGraph/Materials/Util/HistogramView"));
}

void STG_HistogramBlob::UpdateParams(BlobPtr InBlob)
{
	UTexture* BlobTexture = GetTextureFromBlob(InBlob);

	if (BlobTexture)
	{
		BrushMaterial->SetTextureParameterValue("HistogramTexture", BlobTexture);
	}

	BrushMaterial->SetScalarParameterValue("ShowR", Curves == ETG_HistogramCurves::R || Curves == ETG_HistogramCurves::RGB ? 1.0 : 0.0);
	BrushMaterial->SetScalarParameterValue("ShowG", Curves == ETG_HistogramCurves::G || Curves == ETG_HistogramCurves::RGB ? 1.0 : 0.0);
	BrushMaterial->SetScalarParameterValue("ShowB", Curves == ETG_HistogramCurves::B || Curves == ETG_HistogramCurves::RGB ? 1.0 : 0.0);
	BrushMaterial->SetScalarParameterValue("ShowLuma", Curves == ETG_HistogramCurves::Luma );
	BrushMaterial->SetScalarParameterValue("GraphOpcaity", 0.75);
	//BrushMaterial->SetScalarParameterValue("ShowChecker", ShowChecker);
	//BrushMaterial->SetScalarParameterValue("SingleChannel", SingleChannel);

	/*if (InBlob && InBlob->HasMinMax())
	{
		BlobPtr MinMaxBlob = std::static_pointer_cast<TiledBlob>(InBlob->GetMinMaxBlob())->GetTile(0,0);

		UTexture* MinMaxTexture = GetTextureFromBlob(MinMaxBlob);
		BrushMaterial->SetTextureParameterValue("MinMaxTexture", MinMaxTexture);
	}*/
	
	Brush->SetResourceObject(BrushMaterial);
}

void STG_HistogramBlob::Update(TiledBlobPtr InHistogramResult)
{
	UpdateBlob(InHistogramResult);
}

//////////////////////////////////////////////////////////////////////////
#undef LOCTEXT_NAMESPACE
