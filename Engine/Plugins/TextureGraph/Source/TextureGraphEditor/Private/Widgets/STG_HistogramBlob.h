// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/Material.h"
#include "AssetThumbnail.h"
#include "Data/Blob.h"
#include "STG_Blob.h"
#include "Transform/Utility/T_TextureHistogram.h"
#include "STG_HistogramBlob.generated.h"

UENUM(BlueprintType)
enum class ETG_HistogramCurves : uint8
{
	R = 0,
	G,
	B,
	Luma,
	RGB
};

class STG_HistogramBlob : public STG_Blob
{
public:
	SLATE_BEGIN_ARGS(STG_HistogramBlob)
		:_HistogramResult(nullptr)
		,_BrushName("HistogramBlob")
		, _Width(512)
		, _Height(512)
		, _Curves(ETG_HistogramCurves::RGB)
	{}

	SLATE_ARGUMENT(TiledBlobPtr, HistogramResult)
	SLATE_ARGUMENT(FName , BrushName)
	SLATE_ARGUMENT(int, Width)
	SLATE_ARGUMENT(int, Height)
	SLATE_ARGUMENT(ETG_HistogramCurves, Curves)
	SLATE_END_ARGS()

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	void Clear();
	virtual void UpdateParams(BlobPtr InBlob) override;
	virtual UMaterial* GetMaterial() override;

	void Update(TiledBlobPtr InHistogramResult);

private:
	//TiledBlobPtr HistogramResult;
	ETG_HistogramCurves Curves;
};
