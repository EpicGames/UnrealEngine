// Copyright Epic Games, Inc. All Rights Reserved.

#include "STG_RGBAButtons.h"
#include "Widgets/Input/SComboButton.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/StyleColors.h"
#define LOCTEXT_NAMESPACE "TextureGraphEditor"
/** Constructs this widget with InArgs */
void STG_RGBAButtons::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		MakeChannelControlWidget()
	];
}

TSharedRef<SWidget> STG_RGBAButtons::MakeChannelControlWidget()
{
	CheckedBrush = new FSlateRoundedBoxBrush(FLinearColor(0.039f, 0.039f, 0.039f, 1), CoreStyleConstants::InputFocusRadius);

	TSharedRef<SWidget> ChannelControl =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(1.0f)
		.AutoWidth()
		[
			CreateChannelWidget(ETSChannelButton::Red, "R", LOCTEXT("RBAButton_R_Channel_ToolTip", "Toggle the preview of R Channel of the texture"))
		]
	+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(1.0f)
		.AutoWidth()
		[
			CreateChannelWidget(ETSChannelButton::Green, "G", LOCTEXT("RBAButton_G_Channel_ToolTip", "Toggle the preview of G Channel of the texture"))
		]

	+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(1.0f)
		.AutoWidth()
		[
			CreateChannelWidget(ETSChannelButton::Blue, "B", LOCTEXT("RBAButton_B_Channel_ToolTip", "Toggle the preview of B Channel of the texture"))
		]
	+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(1.0f)
		.AutoWidth()
		[
			CreateChannelWidget(ETSChannelButton::Alpha, "A", LOCTEXT("RBAButton_A_Channel_ToolTip", "Toggle the preview of A Channel of the texture"))
		]
	;

	return ChannelControl;
}

TSharedRef<SWidget> STG_RGBAButtons::CreateChannelWidget(ETSChannelButton Type, FString Name, FText TooltipText)
{
	auto OnChannelCheckStateChanged = [this](ECheckBoxState NewState, ETSChannelButton Button)
	{
		OnChannelButtonCheckStateChanged(Button);
	};

	return SNew(SBox)
		.MinDesiredWidth(24)
		.MaxDesiredWidth(24)
		.MinDesiredHeight(24)
		.MinDesiredHeight(24)
		.Padding(0)
		[
			SNew(SCheckBox)
			.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("DetailsView.ChannelToggleButton"))
			.Type(ESlateCheckBoxType::ToggleButton)
			.CheckedImage(CheckedBrush)
			.CheckedHoveredImage(CheckedBrush)
			.Padding(FMargin(0, 4))
			.OnCheckStateChanged_Lambda(OnChannelCheckStateChanged, Type)
			.IsChecked(this, &STG_RGBAButtons::OnGetChannelButtonCheckState, Type)
			//.IsEnabled(this, &STG_SelectionPreview::IsChannelButtonEnabled, ETSChannelButton::Red)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("TG_Editor.ChannelButtonFont"))
			.Justification(ETextJustify::Center)
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			.Text(FText::FromString(Name))
			]
		]
	.ToolTipText(TooltipText);
}

void STG_RGBAButtons::OnChannelButtonCheckStateChanged(ETSChannelButton Button)
{
	switch (Button)
	{
	case ETSChannelButton::Red:
		bIsRedChannel = !bIsRedChannel;
		break;
	case ETSChannelButton::Green:
		bIsGreenChannel = !bIsGreenChannel;
		break;
	case ETSChannelButton::Blue:
		bIsBlueChannel = !bIsBlueChannel;
		break;
	case ETSChannelButton::Alpha:
		bIsAlphaChannel = !bIsAlphaChannel;
		break;
	default:
		check(false);
		break;
	}
}

ECheckBoxState STG_RGBAButtons::OnGetChannelButtonCheckState(ETSChannelButton Button) const
{
	switch (Button)
	{
	case ETSChannelButton::Red:
		return bIsRedChannel ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	case ETSChannelButton::Green:
		return bIsGreenChannel ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	case ETSChannelButton::Blue:
		return bIsBlueChannel ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	case ETSChannelButton::Alpha:
		return bIsAlphaChannel ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	default:
		check(false);
		return ECheckBoxState::Unchecked;
	}
}

//////////////////////////////////////////////////////////////////////////
#undef LOCTEXT_NAMESPACE