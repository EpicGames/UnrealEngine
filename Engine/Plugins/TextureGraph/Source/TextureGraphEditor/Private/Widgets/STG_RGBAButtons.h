// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include <Widgets/Input/SCheckBox.h>
#include "Widgets/SBoxPanel.h"
#include "CoreMinimal.h"

enum class ETSChannelButton : uint8
{
	Red,
	Green,
	Blue,
	Alpha
};

class STG_RGBAButtons : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STG_RGBAButtons)
	{}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
	bool GetIsRChannel() const { return bIsRedChannel; }
	bool GetIsGChannel() const { return bIsGreenChannel; }
	bool GetIsBChannel() const { return bIsBlueChannel; }
	bool GetIsAChannel() const { return bIsAlphaChannel; }
private:
	bool bIsRedChannel = true;
	bool bIsGreenChannel = true;
	bool bIsBlueChannel = true;
	bool bIsAlphaChannel = true;
	FSlateRoundedBoxBrush* CheckedBrush;

	TSharedRef<SWidget> MakeChannelControlWidget();
	TSharedRef<SWidget> CreateChannelWidget(ETSChannelButton Type, FString Name, FText ToolTipText);
	void OnChannelButtonCheckStateChanged(ETSChannelButton Button);
	ECheckBoxState OnGetChannelButtonCheckState(ETSChannelButton Button) const;
};
