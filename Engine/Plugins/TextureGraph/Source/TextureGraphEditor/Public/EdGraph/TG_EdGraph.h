// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"

#include "TG_SystemTypes.h"
#include "TG_PinSelectionManager.h"

#include "TG_EdGraph.generated.h"

class UTextureGraph;
class UTG_Node;
class UTG_Graph;
class UEdGraphNode;
class UTG_EdGraphNode;
struct FTG_EvaluationContext;

UCLASS()
class UTG_EdGraph : public UEdGraph
{
	GENERATED_BODY()

public:
    /**
     * Initialize the EdGrpah from the model TextureGraph
     * Generate the viewmodel graph matching the source model
     *
     * @param TextureGraph 
     */
    void InitializeFromTextureGraph(UTextureGraph* InTextureGraph, TWeakPtr<class FTG_Editor> InTGEditor);

	/**
	 * Add a TG_Node to the Graph and create a EdGraphNode controlling it
	 *
	 * @param	ModelNode	Ts_Node to be added
	 *
	 * @return	UTG_EdGraphNode*	Newly created ViewModelNode holding a pointer to ModelNode and representing it in the graph
	 */
	UTG_EdGraphNode* AddModelNode(UTG_Node* ModelNode, bool bUserInvoked, const FVector2D& Location);

	/**
	 * Find the TG_EdGraphNode from the FTG_Id representing a TG_Node
	 *
	 * @param NodeId 
	 * @return ViewModelNode or nullptr if not found
	 */
	UTG_EdGraphNode* GetViewModelNode(FTG_Id NodeId);

	/**
	 * Force refresh Editor details
	 */
	void RefreshEditorDetails() const;

	/**
	 * Tells the node to create a thumbnail for its output pins
	 * @param Node 
	 * @param EvaluationContext 
	 */
	void OnNodeCreateThumbnail(UTG_Node* Node, const FTG_EvaluationContext* EvaluationContext);
	

	UPROPERTY()
	TObjectPtr<UTextureGraph> TextureGraph = nullptr;
	
	TWeakPtr<FTG_Editor> TGEditor = nullptr;
	FTG_PinSelectionManager PinSelectionManager;

	void CacheThumbBlob(FTG_Id PinId, TiledBlobPtr InBlob);
	TiledBlobPtr GetCachedThumbBlob(FTG_Id PinId);

private:
	// Generate the viewmodel matching the Script
	void BuildEdGraphViewmodel();
	bool CreateViewModelLinkFromModelPins(const UTG_Pin* pinFrom, const UTG_Pin* pinTo);

	void OnNodeSignatureChanged(UTG_Node* InNode);
	void OnNodePostEvaluation(UTG_Node* InNode, const FTG_EvaluationContext* Context);

	void GraphChanged(UTG_Graph* InGraph, UTG_Node* InNode, bool Tweaking);
	TMap<FTG_Id, TiledBlobPtr> PinThumbBlobMap;

public:
	void FixDuplicatedNodesPinConnections(TSet<UEdGraphNode*>& PastedNodes);
};
