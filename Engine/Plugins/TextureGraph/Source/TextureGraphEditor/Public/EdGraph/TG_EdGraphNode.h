// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TG_SystemTypes.h"
#include "Data/Blob.h"
#include "Data/TiledBlob.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/TG_EdGraphSchema.h"

#include "TG_EdGraphNode.generated.h"

//UCLASS()
//class UParamData: public UObject
//{
//	GENERATED_BODY()
//
//public:
//
//	
//
//};

class UTG_Pin;
class UTG_Node;
struct FTG_EvaluationContext;

UCLASS()
class UTG_EdGraphNode : public UEdGraphNode
{
	GENERATED_BODY()

public:
	
	DECLARE_MULTICAST_DELEGATE(FOnTSEditorGraphNodeChanged);
	FOnTSEditorGraphNodeChanged OnNodeReconstructDelegate;
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNodePostEvaluateDelegate, const FTG_EvaluationContext*)
	FOnNodePostEvaluateDelegate OnNodePostEvaluateDelegate;
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPinSelectionChangeDelegate, UEdGraphPin*)
	FOnPinSelectionChangeDelegate OnPinSelectionChangeDelegate;

	void Construct(UTG_Node* InNode);

	// ~Begin UObject interface
	void BeginDestroy() override;
	void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;

	// ~End UObject interface

	
	// ~Begin UEdGraphNode interface
	FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void OnRenameNode(const FString& NewName) override;
	void GetNodeContextMenuActions(UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	FLinearColor GetNodeTitleColor() const override;
	FLinearColor GetNodeBodyTintColor() const;
	void AllocateDefaultPins() override;
	FName GetPinCategory(UTG_Pin* Pin, TWeakObjectPtr<UObject>& PinSubCategoryObject);
	bool CanUserDeleteNode() const override;
	void PrepareForCopying() override;
	void PostCopyNode();
	virtual void PostPasteNode() override;

#if WITH_EDITOR
	void UpdatePinVisibility(UEdGraphPin* Pin, UTG_Pin* TGPin) const;
	void UpdateInputPinsVisibility() const;
	void UpdatePinsEditCondition() const;
#endif
	
	/** Called when the DefaultValue of one of the pins of this node is changed in the editor */
	virtual void PinDefaultValueChangedWithTweaking(UEdGraphPin* Pin, bool bIsTweaking);
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	// ~End UEdGraphNode interface

	virtual void ReconstructNode() override;
	virtual FString GetTitleDetail();
	void OnNodeChanged(UTG_Node* InNode);
	
	UTG_Node* GetNode() const { return Node; }
	virtual FText GetTooltipText() const override; 


	UObject* GetDetailsObject();

	/*
	* Sets the selected pin for Node
	*/
	void SelectPin(UEdGraphPin* Pin, bool IsSelected);

	const UEdGraphPin* GetSelectedPin() const;


	TArray<UEdGraphPin*> GetOutputPins() const;
	TArray<UEdGraphPin*> GetTextureOutputPins() const;

protected:
	friend class UTG_EdGraph;

	// Copy the edNode position to the model node
	void UpdatePosition();

	void OnNodePostEvaluate(const FTG_EvaluationContext* Fts_EvaluationContext);
	bool UpdateEdPinDefaultValue(UEdGraphPin* EdPin, const UTG_EdGraphSchema* Schema);
	void UpdatePinsAndReconstructNode();

	void OnUpdateCommentText(const FString& NewComment) override;
	void OnCommentBubbleToggled(bool bInCommentBubbleVisible) override;
	void UpdateCommentBubblePinned();
	FLinearColor GetTitleColor() const;
	float GetNodeAlpha() const;
	//** Model TS Node this Editor graph node is representing
	UPROPERTY()
		TObjectPtr<UTG_Node> Node;

private:
	UEdGraphPin* SelectedPin = nullptr;
};

