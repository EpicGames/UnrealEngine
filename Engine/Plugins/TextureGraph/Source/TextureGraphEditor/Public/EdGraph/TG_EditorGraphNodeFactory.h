// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"

class SGraphNode;
class UEdGraphNode;

class FTG_EditorGraphNodeFactory : public FGraphPanelNodeFactory
{
	virtual TSharedPtr<SGraphNode> CreateNode(UEdGraphNode* InNode) const override;
};
