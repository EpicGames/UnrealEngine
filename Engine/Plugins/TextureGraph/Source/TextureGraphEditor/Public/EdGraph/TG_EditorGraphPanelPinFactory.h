// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraphUtilities.h"
#include "Templates/SharedPointer.h"

class FTG_EditorGraphPanelPinFactory :
	public FGraphPanelPinFactory
{
public:
	// FGraphPanelPinFactory interface
	virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* InPin) const override;

private:
	TSharedPtr<class SGraphPin> CreateEnumPin(class UEdGraphPin* InPin) const;
	TSharedPtr<SWidget> FindWidgetInChildren(TSharedPtr<SWidget> Parent, FName ChildType) const;
};
