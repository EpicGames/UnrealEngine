// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "GraphEditor.h"
#include "TG_Pin.h"

#include "TG_SystemTypes.h"

class UTG_Script;
class UTG_Node;
class UTG_Graph;
class UEdGraphNode;
class UTG_EdGraphNode;
struct FTG_EvaluationContext;

class FTG_PinSelectionManager : public FGCObject
{
public:
	void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FTG_PinSelectionManager");
	}

	void UpdateSelection(UEdGraphPin* Pin);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPinSelectionUpdated, UEdGraphPin* /*Pin*/)
	FOnPinSelectionUpdated OnPinSelectionUpdated;

private:
};
