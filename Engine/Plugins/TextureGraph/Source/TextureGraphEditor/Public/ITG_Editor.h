// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

class UTG_Expression;

/**
 * Public interface to TS Editor
 */
class ITG_Editor : public FAssetEditorToolkit
{
protected:
	/**
	 * Refreshes the TS Editor's viewport.
	 */
	virtual void					RefreshViewport() = 0;

	/**
	 * Refreshes everything in the TS Editor.
	 */
	virtual void					RefreshTool() = 0;			

public:

	/**
	 * Creates a new material expression of the specified class.
	 *
	 * @param	NewExpressionClass		The type of mix expression to add.  Must be a child of UTG_Expression.
	 * @param	NodePos					Position of the new node.
	 * @param	bAutoSelect				If true, deselect all expressions and select the newly created one.
	 * @param	bAutoAssignResource		If true, assign resources to new expression.
	 * @param	Graph            		Graph to create new expression within.
	 *
	 * @return	UMaterialExpression*	Newly created material expression
	 */
	virtual UTG_Expression*			CreateNewExpression(UClass* NewExpressionClass, const FVector2D& NodePos, bool bAutoSelect, bool bAutoAssignResource, const class UEdGraph* Graph = nullptr) { return nullptr; }


	/**
	 * Gets the name of the mix name that we are editing
	 */
	virtual FText					GetOriginalObjectName() const { return FText::GetEmpty(); }

	/**
	 * Creates an IAssetReferenceFilter that can be used to check if an asset is valid to be used with this texture graph.
	 */
	virtual TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter() const { return {}; }

	/** Gets the TextureGraph edited */
	virtual class UMixInterface*	GetTextureGraphInterface() const = 0;

	/**
	 * This mesh will be used to display the rendering
	 */
	virtual void					SetMesh(class UMeshComponent* PreviewMesh, class UWorld* World) = 0;
};
