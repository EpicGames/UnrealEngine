// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//#include "SlateBasics.h"
#include "CoreMinimal.h"
#include <UObject/GCObject.h>
#include <Widgets/Layout/SBorder.h>
#include <Widgets/SBoxPanel.h>
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SCanvas.h"
#include "Fonts/FontMeasure.h"
#include "STextureHistogram.generated.h"

UENUM(BlueprintType)
enum class ETextureHistogramStyle : uint8
{
	/** Draw the histogram in descrete bar shape */
	Bar = 0,

	/**  Draw the histogram in continous line curve shader */
	Curve,
};

UENUM(BlueprintType)
enum class ETextureHistogramLayout : uint8
{
	/** Draw RGBA curves in histogram in one graph */
	Combined = 0,

	/**  Draw RGBA curves in histogram in vertically seperated graphs */
	Split,
};

struct TextureHistogramUtils
{
public:
	inline static const FLinearColor LinearGray = FLinearColor(0.047f, 0.047f, 0.047f, 1);
	inline static const FLinearColor LinearRed = FLinearColor(1.0f, 0.0f, 0.0f, 0.4f);
	inline static const FLinearColor LinearGreen = FLinearColor(0.0f, 1.0f, 0.0f, 0.4f);
	inline static const FLinearColor LinearBlue = FLinearColor(0.0f, 0.0f, 1.0f, 0.4f);
	inline static const FLinearColor LinearWhite = FLinearColor(1.0f, 1.0f, 1.0f, 0.4f);
	
	inline static const float UniformMargin = 15.0f;
	inline static const FMargin Margin = FMargin(35, 10, 25, 35);
};

class UMaterialInstanceDynamic;
class STextureHistogramCurve;
class STextureHistogramBars;
class STextureHistogramReferenceLines;

//Texture Histogram widget used to draw the histogram of a texture
//Require data in an TArray of FVector4
class STextureHistogram : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextureHistogram) :
	_NumBins(256),
	_NumVerticalScaleLines(3),
	_NumHorizontalScaleLines(8),
	_bDrawScaleLines(true),
	_bDrawBackground(true),
	_ScaleLineColor(TextureHistogramUtils::LinearWhite),
	_RCurveColor(TextureHistogramUtils::LinearRed),
	_GCurveColor(TextureHistogramUtils::LinearGreen),
	_BCurveColor(TextureHistogramUtils::LinearBlue),
	_ACurveColor(TextureHistogramUtils::LinearWhite),
	_DrawLayout(ETextureHistogramLayout::Split),
	_DrawStyle(ETextureHistogramStyle::Curve),
	_Height(20.0)
	{}
	SLATE_ARGUMENT(int32, NumBins)
	SLATE_ARGUMENT(int, NumVerticalScaleLines)
	SLATE_ARGUMENT(int, NumHorizontalScaleLines)
	SLATE_ARGUMENT(bool, bDrawScaleLines)
	SLATE_ARGUMENT(bool, bDrawBackground)
	SLATE_ARGUMENT(FLinearColor, ScaleLineColor)
	SLATE_ARGUMENT(FLinearColor, RCurveColor)
	SLATE_ARGUMENT(FLinearColor, GCurveColor)
	SLATE_ARGUMENT(FLinearColor, BCurveColor)
	SLATE_ARGUMENT(FLinearColor, ACurveColor)
	SLATE_ARGUMENT(ETextureHistogramLayout, DrawLayout)
	SLATE_ARGUMENT(ETextureHistogramStyle, DrawStyle)
	SLATE_ARGUMENT(float, Height)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args);
	void SetHistogramData(TArray<FVector4> InHistogramData);

private:

	void DrawCombineHistogramCurves(const FArguments& InArgs);
	void DrawSplitHistogramCurves(const FArguments& InArgs);
	void AddCurveWidget(const FArguments& InArgs,TSharedRef<SWidget> ReferenceLineWidget, TSharedRef<SWidget> CurveWidget);
	void DrawCombineBarHistogram(const FArguments& InArgs);
	void DrawSplitBarHistogram(const FArguments& InArgs);

	int GetMaxBin();
	TArray<int32> GetRValues() const;
	TArray<int32> GetGValues() const;
	TArray<int32> GetBValues() const;
	TArray<int32> GetLuminanceValues() const;

	bool bDrawCombined;
	bool bDrawBars;
	TArray<FVector4> HistogramData;
	TSharedPtr<SVerticalBox> SplitBox;
	TSharedPtr<STextureHistogramCurve> HistogramCurveR;
	TSharedPtr<STextureHistogramCurve> HistogramCurveG;
	TSharedPtr<STextureHistogramCurve> HistogramCurveB;
	TSharedPtr<STextureHistogramCurve> HistogramCurveA;

	TSharedPtr<STextureHistogramBars> HistogramBarsR;
	TSharedPtr<STextureHistogramBars> HistogramBarsG;
	TSharedPtr<STextureHistogramBars> HistogramBarsB;
	TSharedPtr<STextureHistogramBars> HistogramBarsA;

	TSharedPtr<STextureHistogramReferenceLines> ReferenceLinesR;
	TSharedPtr<STextureHistogramReferenceLines> ReferenceLinesG;
	TSharedPtr<STextureHistogramReferenceLines> ReferenceLinesB;
	TSharedPtr<STextureHistogramReferenceLines> ReferenceLinesA;
	TSharedPtr<STextureHistogramReferenceLines> ReferenceLinesCombine;
};

//Widget to Draw the Curve Style Histogram
class STextureHistogramCurve : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextureHistogramCurve):
	_bShadeCurve(true),
	_LineColor(FLinearColor::Red),
	_Height(20.0)
	{}
	SLATE_ARGUMENT(bool, bShadeCurve)
	SLATE_ARGUMENT(FLinearColor, LineColor)
	SLATE_ARGUMENT(float, Height)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args);
	void SetHistogramData(const TArray<int32>& InHistogramData);
	void SetMax(int MaxValue) { Max = MaxValue; }

	int GetMaxBin();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	void DrawHistogramCurve(const FGeometry& AllottedGeometry, FSlateWindowElementList& ElementList, int32 LayerId, TArray<int32> Values) const;
	void DrawShadeForCurve(const FGeometry& AllottedGeometry, FSlateWindowElementList& ElementList, int32 LayerId, TArray<FVector2D> ShadePoints) const;

private:
	int Max;
	bool bShadeCurve;
	FLinearColor LineColor;
	FColor ShadeColor;
	TArray<int32> HistogramData;
	TSharedPtr<SCanvas> Canvas;
};

//Widget to Draw the Bars for Histogram
class STextureHistogramBars : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextureHistogramBars) :
		_ShadeColor(FLinearColor::Red),
		_Height(20.0)
	{}
		SLATE_ARGUMENT(FLinearColor, ShadeColor)
		SLATE_ARGUMENT(float, Height)
		SLATE_END_ARGS()

	void Construct(const FArguments& Args);
	void SetHistogramData(const TArray<int32>& InHistogramData);
	void SetMax(int MaxValue) { Max = MaxValue; }
	int GetMaxBin();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	void DrawHistogramBars(const FGeometry& AllottedGeometry, FSlateWindowElementList& ElementList, int32 LayerId, TArray<int32> Values, FLinearColor BarColor) const;

private:
	int Max;
	FLinearColor ShadeColor;
	TArray<int32> HistogramData;
	TSharedPtr<SCanvas> Canvas;
};

//Widget to Draw the Reference Lines for Histogram
class STextureHistogramReferenceLines : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextureHistogramReferenceLines) :
		_Color(FLinearColor::White),
		_NumBins(256),
		_NumVerticalScaleLines(3),
		_NumHorizontalScaleLines(8),
		_bDraw(true),
		_bDrawBackground(true),
		_Height(20.0),
		_Label("Histogram")
	{}
	SLATE_ARGUMENT(FLinearColor, Color)
		SLATE_ARGUMENT(int32, NumBins)
		SLATE_ARGUMENT(int, NumVerticalScaleLines)
		SLATE_ARGUMENT(int, NumHorizontalScaleLines)
		SLATE_ARGUMENT(bool, bDraw)
		SLATE_ARGUMENT(bool, bDrawBackground)
		SLATE_ARGUMENT(float, Height)
		SLATE_ARGUMENT(FString, Label)
		SLATE_END_ARGS()

	void Construct(const FArguments& Args);
	void SetMax(int MaxValue) { Max = MaxValue; }

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	void DrawReferenceLines(const FGeometry& AllottedGeometry, FSlateWindowElementList& ElementList, int32 LayerId) const;

private:
	int Max;
	int32 NumBins;
	int	NumVerticalScaleLines;
	int	NumHorizontalScaleLines;
	bool bDraw;
	bool bDrawBackground;
	FString Label;
	FLinearColor Color;
	TSharedPtr<SCanvas> Canvas;
};
