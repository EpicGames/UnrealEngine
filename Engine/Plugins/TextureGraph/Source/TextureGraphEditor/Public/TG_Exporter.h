// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedPreviewSceneModule.h"
#include "Templates/UniquePtr.h"
#include "Tickable.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Input/SComboBox.h"
#include "Misc/NotifyHook.h"
#define LOCTEXT_NAMESPACE "TextureGraphExporter"
class UTextureGraphInstance;
class UTextureGraphBase;
class FUICommandInfo;
struct FTG_ExporterUtility
{
	// Initializes the global state of the exporter (commands, tab spawners, etc):
	FTG_ExporterUtility();
	void Cleanup();

	// Destructor declaration purely so that we can pimpl:
	~FTG_ExporterUtility();

	/** Sets the current texture graph to be used with the exporter */
	void SetTextureGraphToExport(UTextureGraphBase* InTextureGraph);

	/** Function registered with tab manager to create the Texture Graph Exporter */
	TSharedRef<SDockTab> CreateTGExporterTab(const FSpawnTabArgs& Args);
	

private:
	TUniquePtr<struct FTG_InstanceImpl>	Impl;
	TObjectPtr<UTextureGraphInstance> TextureGraphInstance;
	TSharedPtr<FTabManager> TGExporterTabManager;
	TSharedPtr<FTabManager::FLayout> TGExporterLayout;
	
	// prevent copying:
	FTG_ExporterUtility(const FTG_ExporterUtility&);
	FTG_ExporterUtility(FTG_ExporterUtility&&);
	FTG_ExporterUtility& operator=(FTG_ExporterUtility const&);
	FTG_ExporterUtility& operator=(FTG_ExporterUtility&&);
};


struct FTG_ExporterCommands : public TCommands<FTG_ExporterCommands>
{
	FTG_ExporterCommands()
		: TCommands<FTG_ExporterCommands>(
			TEXT("TextureGraphExporter"), // Context name for fast lookup
			LOCTEXT("TextureGraphExporter", "Texture Graph Exporter"), // Localized context name for displaying
			NAME_None, // Parent
			FCoreStyle::Get().GetStyleSetName() // Icon Style Set
		)
	{
	}

	// TCommand<> interface
	virtual void RegisterCommands() override;
	// End of TCommand<> interface

	TSharedPtr<FUICommandInfo> ShowOutputPreview;
	TSharedPtr<FUICommandInfo> Show3DPreview;
	TSharedPtr<FUICommandInfo> Show3DPreviewSettings;
	TSharedPtr<FUICommandInfo> ShowParameters;
	TSharedPtr<FUICommandInfo> ShowExportSettings;
	TSharedPtr<FUICommandInfo> ShowDetails;

};
class UTG_Node;
class UTG_Graph;

struct FTG_InstanceImpl: public FTickableGameObject, public FGCObject, public FNotifyHook
{
	friend class FTG_Exporter;
	friend class FTG_InstanceEditor;
	friend class UTextureGraphInstance;
public:
													FTG_InstanceImpl();
	virtual											~FTG_InstanceImpl() override;

	void											Cleanup(bool bFlushMixInvalidations = true);
	/** Sets the Texture Graph to be exported in the Exporter */
	void 											SetTextureGraphToExport(UTextureGraphInstance* InTextureGraph, bool bClearPreviousMixInvalidations);
	void 											OnGraphChanged(UTG_Graph* InGraph, UTG_Node* InNode, bool Tweaking);
	TSharedRef<FTabManager::FLayout>				GetDefaultLayout();
	void 											Initialize();
	void 											RegisterTabSpawners(const TSharedPtr<FTabManager>);
	void 											UnregisterTabSpawners(const TSharedPtr<FTabManager>);
	void 											SetMesh(class UMeshComponent* InPreviewMesh, class UWorld* InWorld);
	void 											SetViewportPreviewMesh();
	void											UpdatePreviewMesh();
	void											CleanupSlateReferences();

private:
	TObjectPtr<UTextureGraphInstance>				TextureGraphInstance;
	TWeakPtr<class STG_NodePreviewWidget>			NodePreviewPtr;
	
	TSharedPtr<class IDetailsView>					DetailsView;
	TSharedPtr<class IDetailsView>					ParametersView;
	TSharedPtr<class IDetailsView>					ExportSettingsView;
	TSharedPtr<class IDetailsView>					PreviewSettingsView;
	TSharedPtr<class IDetailsView>					PreviewSceneSettingsView;
	
	/** Scene preview settings widget */
	TSharedPtr<SWidget>								AdvancedPreviewSettingsWidget;
	
	TWeakPtr<SDockTab>								PreviewSceneSettingsDockTab;
	
	FAdvancedPreviewSceneModule::FOnPreviewSceneChanged OnPreviewSceneChangedDelegate;
	
// Tracking the active viewports in this editor.
	TSharedPtr<class FEditorViewportTabContent>		ViewportTabContent;

	TObjectPtr<class UTG_Parameters>				Parameters;
	TObjectPtr<class UTG_ExportSettings>			ExportSettings;
	TSharedPtr<class FExportSettings>				TargetExportSettings;
	
	TArray<TSharedPtr<FName>>						OutputNodesList;
	TSharedPtr<SComboBox<TSharedPtr<FName>>>		OutputNodesComboBoxWidget;
	UTG_Node*										SelectedNode = nullptr;
	// Inherited via FTickableGameObject
	virtual void									Tick(float DeltaTime) override;

	virtual ETickableTickType						GetTickableTickType() const override { return ETickableTickType::Always; }

	virtual bool									IsTickableWhenPaused() const override { return true; }

	virtual bool									IsTickableInEditor() const override { return true; }

	virtual TStatId									GetStatId() const override;

	/** Post edit change notify for properties. */
	virtual void									NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged ) override;
	
	// Inherited via FGCObject
	virtual void									AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString									GetReferencerName() const override { return TEXT("FTextureGraphExporter");}

private:
	
	/** Builds the sub-tools that are a part of the texture graph editor. */
	void											BuildSubTools();
	
	/** Called when the Viewport Layout has changed. */
	void											OnEditorLayoutChanged();
	
	TSharedRef<SDockTab>							SpawnTab_Viewport(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_ParameterDefaults(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_NodePreview(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_ExportSettings(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_TG_Properties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_PreviewSettings(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab>							SpawnTab_PreviewSceneSettings(const FSpawnTabArgs& Args);
	TSharedPtr<class STG_EditorViewport>			GetEditorViewport() const;
	
	bool 											SetPreviewAsset(UObject* InAsset);
	TSharedRef<class IDetailsView>					GetDetailsView() const { return DetailsView.ToSharedRef(); }
	TSharedRef<class IDetailsView>					GetPreviewSettingsView() const { return PreviewSettingsView.ToSharedRef(); }
	TSharedRef<class IDetailsView>					GetPreviewSceneSettingsView() const { return PreviewSceneSettingsView.ToSharedRef(); }
	TSharedRef<class IDetailsView>					GetExportSettingsView() const { return ExportSettingsView.ToSharedRef(); }
	TSharedRef<class IDetailsView>					GetParametersView() const { return ParametersView.ToSharedRef(); }
	void											OnViewportMaterialChanged();
	void											OnMaterialMappingChanged();
	void											UpdateExportSettingsUI();
	void											UpdateParametersUI();
	virtual void									RefreshViewport();
	void											OnRenderingDone(class UMixInterface* TextureGraph, const struct FInvalidationDetails* Details);
	TSharedRef<SWidget>								GenerateOutputComboItem(TSharedPtr<FName> String);
	void											OnOutputSelectionChanged(TSharedPtr<FName> String, ESelectInfo::Type Arg);
	FReply											OnExportClicked(EAppReturnType::Type ButtonID);

	// prevent copying:
	FTG_InstanceImpl(const FTG_InstanceImpl&);
	FTG_InstanceImpl(FTG_InstanceImpl&&);
	FTG_InstanceImpl& operator=(FTG_InstanceImpl const&);
	FTG_InstanceImpl& operator=(FTG_InstanceImpl&&);
};

#undef LOCTEXT_NAMESPACE 
