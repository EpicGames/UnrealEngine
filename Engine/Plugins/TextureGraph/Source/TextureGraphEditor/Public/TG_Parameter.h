// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TG_SystemTypes.h"


#include "TG_Parameter.generated.h"




USTRUCT()
struct FTG_ParameterInfo
{
	GENERATED_BODY()
public:
	FTG_Id Id;
	FName Name;
};

UCLASS()
class UTG_Parameters : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, EditFixedSize, Category = "TextureGraph", meta=(EditFixedOrder, ShowOnlyInnerProperties))
	TArray<FTG_ParameterInfo> Parameters;

	TObjectPtr<class UTG_Graph> TextureGraph;

};

