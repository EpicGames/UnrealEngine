// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Containers/Ticker.h"
#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/IToolkit.h"
#include "TG_Exporter.h"

#define UE_API TEXTUREGRAPHEDITOR_API
class IAssetTools;
class IAssetTypeActions;
class ITG_Editor;
class UTextureGraph;
class UTextureGraphInstance;
class FTG_EditorGraphNodeFactory;
class FTG_EditorGraphPanelPinFactory;
typedef TArray<TSharedPtr<IAssetTypeActions>> AssetTypeActionsArray;
extern const FName TG_EditorAppIdentifier;
extern const FName TG_InstanceEditorAppIdentifier;

class FTextureGraphEditorModule : public IModuleInterface
{
private:
	/** All created asset type actions.  Cached here so that we can unregister them during shutdown. */
	AssetTypeActionsArray			CreatedAssetTypeActions;
	/** Delegate to run the Tick method in charge of running TextureGraphEngine update*/
	FTickerDelegate					TickDelegate;
	/** Handle of the delegate to run the Tick method in charge of running TextureGraphEngine update*/
	FTSTicker::FDelegateHandle		TickDelegateHandle;
	TUniquePtr<FTG_ExporterUtility>		TG_Exporter;

protected:
	TSharedPtr<FTG_EditorGraphNodeFactory> GraphNodeFactory;
	TSharedPtr<FTG_EditorGraphPanelPinFactory>	GraphPanelPinFactory;
public:
	/** IModuleInterface implementation */
	UE_API virtual void					StartupModule() override;
	UE_API virtual void					ShutdownModule() override;

	
	UE_API virtual void					StartTextureGraphEngine();
	UE_API virtual void					ShutdownTextureGraphEngine();
	UE_API bool							Tick(float deltaTime);
	UE_API void							RegisterAssetTypeAction(IAssetTools& AssetTools, TSharedRef<IAssetTypeActions> Action);
	UE_API void							UnRegisterAllAssetTypeActions();
	static UE_API TSharedRef<ITG_Editor>	CreateTextureGraphEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UTextureGraph* InTextureGraph);
	static UE_API TSharedRef<ITG_Editor>	CreateTextureGraphInstanceEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UTextureGraphInstance* InTextureGraphInstance);

	/** Returns a reference to the Blueprint Debugger state object */
	const TUniquePtr<FTG_ExporterUtility>& GetTextureExporter() const { return TG_Exporter; }
};

#undef UE_API
