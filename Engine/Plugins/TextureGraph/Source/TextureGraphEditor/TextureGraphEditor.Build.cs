// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;
public class TextureGraphEditor : ModuleRules
{
	// Flag that enables the new node preview.
	public TextureGraphEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		bEnableExceptions = true;
		bDisableAutoRTFMInstrumentation = true;

		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
            }
			);
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
				"TextureGraph",
				"TextureGraphEngine",
				"Renderer",
				"RenderCore",
				"RHI",
				"Projects",
				"ToolWidgets",
				"KismetWidgets",
				"WorkspaceMenuStructure"
				// ... add other public dependencies that you statically link with here ...
			}
			);
		if (Target.bBuildEditor == true)
		{
			//reference the module "MyModule"
			PublicDependencyModuleNames.AddRange(new string[]
			{
				"AssetTools",
				"UnrealEd",
				"EditorFramework",
				"ToolMenus",
				"AdvancedPreviewScene",
				"PropertyEditor",
				"ContentBrowser",
				"ContentBrowserData"
			});
		}
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"ApplicationCore",
				"Engine",
				"Slate",
				"SlateCore",
				"InputCore",
				"GraphEditor", 
				"MessageLog",
				"EditorWidgets",
				"AssetDefinition",
				"ImageWidgets"
				// ... add private dependencies that you statically link with here ...	
			}
			);
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}