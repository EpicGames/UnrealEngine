// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "BlendModes.generated.h"

UENUM(BlueprintType)
namespace EBlendModes
{
	enum Type : int
	{
		Normal						UMETA(DisplayName = "Copy"),
		Add							UMETA(DisplayName = "Add"),
		Subtract					UMETA(DisplayName = "Subtract"),
		Multiply					UMETA(DisplayName = "Multiply"),
		Divide						UMETA(DisplayName = "Divide"),
		Difference					UMETA(DisplayName = "Difference"),
		Max 						UMETA(DisplayName = "Max"),
		Min 						UMETA(DisplayName = "Min"),
		Step						UMETA(DisplayName = "Step"),
		Overlay						UMETA(DisplayName = "Overlay"),
		// Distort						UMETA(DisplayName = "Distort")
	};
}
