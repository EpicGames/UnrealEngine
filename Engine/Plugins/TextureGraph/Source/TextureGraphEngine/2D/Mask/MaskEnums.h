// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "MaskEnums.generated.h"

UENUM()
enum class MaskType : uint8
{
	PaintMask					UMETA(DisplayName = "Paint"),
	SolidMask					UMETA(DisplayName = "Solid"),
	ImageMask					UMETA(DisplayName = "Image"),
	NoiseMask					UMETA(DisplayName = "Noise"),
	PatternMask					UMETA(DisplayName = "Pattern"),
	NormalMask					UMETA(DisplayName = "Normal"),
	CurvatureMask				UMETA(DisplayName = "Curvature"),
	PositionGradient			UMETA(DisplayName = "Position Gradient"),
	MasksTypeCount				UMETA(DisplayName = "Total mask types"),
};

UENUM()
enum class TransformType : uint8
{
	Circular      UMETA(DisplayName = "Circular"),
	Projection     UMETA(DisplayName = "Projection"),
};

UENUM()
enum MaskModifierType : int
{
	BrightnessMaskModifier		UMETA(DisplayName = "Brightness/Contrast"),
	ClampMaskModifier			UMETA(DisplayName = "Clamp"),
	InvertMaskModifier			UMETA(DisplayName = "Invert"),
	NormalizeMaskModifier		UMETA(DisplayName = "Normalize"),
	GradientRemapMaskModifier	UMETA(DisplayName = "Gradient Remap"),
	PosterizeMaskModifier		UMETA(DisplayName = "Posterize"),
	ScatterMaskModifier			UMETA(DisplayName = "Scatter"),
	MaskModifiersTypeCount		UMETA(DisplayName = "Total mask modifier types"),
};