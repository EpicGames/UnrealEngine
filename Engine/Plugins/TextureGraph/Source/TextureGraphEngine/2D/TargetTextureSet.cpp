// Copyright Epic Games, Inc. All Rights Reserved.
#include "TargetTextureSet.h"
#include "TextureHelper.h"
#include "2D/TextureHelper.h"
#include "Helper/ColorUtil.h"
#include "Device/FX/Device_FX.h"
#include "TextureGraphEngineGameInstance.h"
#include "Data/Blobber.h"
#include "2D/Tex.h"
#include "Model/Mix/MixSettings.h"
#include "Misc/ScopeLock.h"

/// Defined in TextureSet.cpp: the shared lock that serializes all texture-map access across the
/// render and game threads (see the note there). Declared here so this translation unit locks the
/// same instance. The maps are protected; callbacks are always fired outside the lock because they
/// self-defer to the game thread and touch material/RHI state.
extern FCriticalSection& GetTextureSetMapCS();

TargetTextureSet::TargetTextureSet(int32 InId, const FString& InName, RenderMeshPtr InMesh, int32 InWidth, int32 InHeight)
	: TextureSet(InWidth, InHeight)
	, Id(InId)
	, Name(InName)
	, Mesh(InMesh)
{
	
}

TargetTextureSet::~TargetTextureSet()
{
	FScopeLock Lock(&GetTextureSetMapCS());
	Callbacks.Empty();
}

void TargetTextureSet::InitTex(int32 InTypeIndex)
{
	// This is being used in multiple places in old Texture Graph code base.
	// For now, just removing the code.
	// We will do the refactor later.
	// TODO: We need to remove this function
	check(false);
}

AsyncBufferResultPtr TargetTextureSet::BindTo(RenderMaterial_BPPtr Material, TArray<FMaterialMappingInfo> MaterialMappingInfo)
{
	RenderCount++;

	{
		// clear previous callbacks
		FScopeLock Lock(&GetTextureSetMapCS());
		Callbacks.Empty();
	}

	/// RegisterCallback (reached via BindOnTextureUpdate) takes the lock itself and fires its
	/// immediate notification outside it, so we must not hold the lock across this loop.
	for(const FMaterialMappingInfo MappingInfo : MaterialMappingInfo)
	{
		BindOnTextureUpdate(Material, MappingInfo);
	}

	return cti::make_ready_continuable(std::make_shared<BufferResult>());
}

AsyncBufferResultPtr TargetTextureSet::BindTo(RenderMaterial_BPPtr Material, FMaterialMappingInfo MaterialMappingInfo)
{
	RenderCount++;

	BindOnTextureUpdate(Material, MaterialMappingInfo);
	
	return cti::make_ready_continuable(std::make_shared<BufferResult>());
}

AsyncBufferResultPtr TargetTextureSet::UnbindFrom(RenderMaterial_BPPtr Material)
{
	ResourceBindInfo BindInfo;
	BindInfo.Dev = Device_FX::Get();

	FScopeLock Lock(&GetTextureSetMapCS());

	for(auto TextureEntry : TexturesMap)
	{
		if (TexturesMap.Contains(TextureEntry.Key) && !TexturesMap[TextureEntry.Key].expired())
		{

			BindInfo.Target = TextureEntry.Key.ToString();
			TexturesMap[TextureEntry.Key].lock()->Unbind(Material.get(), BindInfo);
		}
	}

	return cti::make_ready_continuable(std::make_shared<BufferResult>());
}

void TargetTextureSet::SetMesh(RenderMeshPtr InMesh)
{
	Mesh = InMesh;
}

void TargetTextureSet::SetTexture(FName TextureName, TiledBlobRef InTexture)
{
	/// This runs on the render thread (from T_UpdateTargets) while the game thread may be in
	/// BindTo/FreeAt. The lock serializes the map mutations; the callbacks are snapshotted here and
	/// fired below, after releasing the lock (UE-376270).
	TArray<TextureReadyCallback> CallbacksToFire;

	{
		FScopeLock Lock(&GetTextureSetMapCS());

		TextureSet::SetTexture(TextureName, InTexture);

		if(BoundTexturesMap.Contains(TextureName))
		{
			BoundTexturesMap[TextureName] = InTexture;
		}
		else
		{
			BoundTexturesMap.Add(TextureName, InTexture);
		}

		if (Callbacks.Contains(TextureName))
		{
			CallbacksToFire = Callbacks[TextureName];
		}
	}

	/// Fire outside the lock: each callback self-defers to the game thread and touches material/RHI
	/// state, which must not run while the texture-map lock is held.
	for (TextureReadyCallback& Callback : CallbacksToFire)
	{
		Callback(InTexture);
	}
}

void TargetTextureSet::FreeAt(FName TextureName)
{
	/// Recursive lock held across UnRegisterCallback and base FreeAt (both re-lock) so the removal
	/// is atomic vs. a concurrent render-thread SetTexture.
	FScopeLock Lock(&GetTextureSetMapCS());

	UnRegisterCallback(TextureName);

	if(BoundTexturesMap.Contains(TextureName))
	{
		BoundTexturesMap.Remove(TextureName);
	}

	TextureSet::FreeAt(TextureName);
}

void TargetTextureSet::Free()
{
	FScopeLock Lock(&GetTextureSetMapCS());

	for(auto TextureEntry : BoundTexturesMap)
	{
		BoundTexturesMap[TextureEntry.Key] = TiledBlobRef();
	}

	BoundTexturesMap.Empty();

	TextureSet::Free();
}

void TargetTextureSet::BindOnTextureUpdate(RenderMaterial_BPPtr InMaterial, FMaterialMappingInfo MaterialMappingInfo) const
{
	TextureReadyCallback bindCallback = [this, InMaterial, MaterialMappingInfo](TiledBlobRef Texture) mutable
	{
		Util::OnGameThread([this, Texture, InMaterial, MaterialMappingInfo]()
			{
				ResourceBindInfo BindInfo;
				BindInfo.Dev = Device_FX::Get();
				if (Texture)
				{
					BindInfo.Target = MaterialMappingInfo.MaterialInput.ToString();
					//TODO: Need to move it into the Resource Bind Info

					// Linear textures are now being displayed in linear gamma.
					// So turning off the conversion to match UE's convention.
					// InMaterial->SetInt(TEXT("sRGB"), Texture.get()->GetDescriptor().bIsSRGB ? 1 : 0);
					FString GrayScaleParam = "Is" + BindInfo.Target + "GrayScale";
					
					InMaterial->SetInt(FName(GrayScaleParam), Texture.get()->GetDescriptor().ItemsPerPoint == 1 ? 1 : 0);

					Texture->OnFinalise().then([=]()
					{
						Texture->Bind(InMaterial.get(), BindInfo);
					});
				}
			});
	};

	if(MaterialMappingInfo.HasTarget())
	{
		RegisterCallback(bindCallback, MaterialMappingInfo);
	}
}

void TargetTextureSet::RegisterCallback(TextureReadyCallback Callback, FMaterialMappingInfo MaterialMappingInfo) const
{
	TiledBlobRef ExistingTexture;
	bool bHasExisting = false;

	{
		FScopeLock Lock(&GetTextureSetMapCS());

		//If we have something in bound, snapshot it so the callback can be told below
		if (BoundTexturesMap.Contains(MaterialMappingInfo.Target))
		{
			ExistingTexture = BoundTexturesMap[MaterialMappingInfo.Target];
			bHasExisting = true;
		}

		/// And also put it in an array incase it gets updated
		if(Callbacks.Contains(MaterialMappingInfo.Target))
		{
			Callbacks[MaterialMappingInfo.Target].Add(Callback);
		}
		else
		{
			Callbacks.Add(MaterialMappingInfo.Target, TArray<TextureReadyCallback>{Callback});
		}
	}

	/// Fire outside the lock (the callback self-defers to the game thread and touches material/RHI)
	if (bHasExisting)
	{
		Callback(ExistingTexture);
	}
}

void TargetTextureSet::UnRegisterCallback(FName TextureName) const
{
	FScopeLock Lock(&GetTextureSetMapCS());

	if(Callbacks.Contains(TextureName))
	{
		Callbacks.Remove(TextureName);
	}
}

