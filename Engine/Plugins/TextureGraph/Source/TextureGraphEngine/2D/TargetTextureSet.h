// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "TextureSet.h"
#include "2D/TextureType.h"
#include "FxMat/RenderMaterial_BP.h"

#define UE_API TEXTUREGRAPHENGINE_API

struct FMaterialMappingInfo;
typedef TFunction<void(TiledBlobPtr)> TextureReadyCallback;

class RenderMesh;
typedef std::shared_ptr<RenderMesh>	RenderMeshPtr;


class TargetTextureSet : public TextureSet
{
protected:

	mutable TMap<FName, TArray<TextureReadyCallback>> Callbacks;		/// The callbacks to call when texture is set

	int32							Id = -1;					/// The ID/index into the list of target texture sets
	FString							Name;						/// Display name of this target set

	RenderMeshPtr					Mesh = nullptr;				/// The render mesh that this this texture set is targetting. 
																/// One texture set can only target one render mesh at a time.
	

	TMap<FName, TiledBlobRef>		BoundTexturesMap;			///Textures currently bound and dont want deallocation until new textures assigned
	int32							RenderCount = 0;			/// The number of times this layer has been rendered to

	UE_API virtual void					InitTex(int32 InTypeIndex) override;
	UE_API virtual void					BindOnTextureUpdate(RenderMaterial_BPPtr InMaterial, FMaterialMappingInfo MaterialMappingInfo) const;
	UE_API virtual void					RegisterCallback(TextureReadyCallback Callback, FMaterialMappingInfo MaterialMappingInfo) const;
	UE_API virtual void					UnRegisterCallback(FName TextureName) const;
public:
									UE_API TargetTextureSet(int32 InId, const FString& InName, RenderMeshPtr InMesh, int32 InWidth, int32 InHeight);
	UE_API virtual							~TargetTextureSet() override;

	UE_API void							SetMesh(RenderMeshPtr InMesh);
	
	UE_API AsyncBufferResultPtr			BindTo(RenderMaterial_BPPtr Material, TArray<struct FMaterialMappingInfo> MaterialMappingInfos);
	UE_API AsyncBufferResultPtr			BindTo(RenderMaterial_BPPtr Material, struct FMaterialMappingInfo MaterialMappingInfo);
	UE_API AsyncBufferResultPtr			UnbindFrom(RenderMaterial_BPPtr Material);
	
	UE_API virtual void					SetTexture(FName TextureName, TiledBlobRef InTexture) override;
	UE_API virtual void					FreeAt(FName TextureName) override;
	UE_API virtual void					Free() override;
	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE int32				GetId() const { return Id; }
	FORCEINLINE FString				GetName() const { return Name; }
	FORCEINLINE RenderMeshPtr		GetMesh() const { return Mesh; }
	FORCEINLINE int32				GetRenderCount() const { return RenderCount; }
	
};

typedef std::unique_ptr<TargetTextureSet>	TargetTextureSetPtr;
typedef std::vector<TargetTextureSetPtr>	TargetTextureSetPtrVec;

#undef UE_API
