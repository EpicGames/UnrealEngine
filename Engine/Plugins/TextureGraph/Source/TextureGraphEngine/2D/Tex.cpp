// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tex.h" 
#include "Async/Async.h"
#include "Containers/EnumAsByte.h"
#include "Data/Blobber.h"
#include "Device/FX/Device_FX.h"
#include "Engine/Canvas.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "EngineModule.h"
#include "FxMat/MaterialManager.h"
#include "Helper/Promise.h"  
#include "Helper/Util.h"
#include "Helper/Util.h"
#include "ImageCore.h"
#include "ImageCoreUtils.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h" 
#include "ImageUtils.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "Materials/Material.h"
#include "Misc/FileHelper.h"
#include "TextureGraphEngine.h"
#include "TextureGraphEngineGameInstance.h" 
#include "Modules/ModuleManager.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIUtilities.h"
#include "Serialization/BufferArchive.h"
#include "Slate/SceneViewport.h"
#include "TextureHelper.h" 
#include "UnrealClient.h"
#include "Widgets/SViewport.h"
#include <Rendering/Texture2DResource.h>
#include "Math/GuardedInt.h"

#include "Profiling/StatGroup.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include "CoreGlobals.h"
#include "Framework/Application/SlateApplication.h"

#pragma push_macro("DLLEXPORT")
#undef DLLEXPORT
#undef WITH_LIBJPEGTURBO
#include "turbojpeg.h"
#include "TextureCompiler.h"
#pragma pop_macro("DLLEXPORT")

DECLARE_CYCLE_STAT(TEXT("Tex_InitRT"), STAT_Tex_InitRT, STATGROUP_TextureGraphEngine);

TexDescriptor::TexDescriptor()
{
}

TexDescriptor::TexDescriptor(uint32 InWidth, uint32 InHeight, EPixelFormat InFormat)
	: Name(Util::RandomID())
	, Width(InWidth)
	, Height(InHeight)
	, Format(InFormat)
	, NumChannels(TextureHelper::GetChannelsFromPixelFormat(InFormat))
{
}

TexDescriptor::TexDescriptor(uint32 InWidth, uint32 InHeight, EPixelFormat InFormat, bool bInSRGB)
	: Name(Util::RandomID())
	, Width(InWidth)
	, Height(InHeight)
	, Format(InFormat)
	, NumChannels(TextureHelper::GetChannelsFromPixelFormat(InFormat))
	, bIsSRGB(bInSRGB)
{
}

TexDescriptor::TexDescriptor(uint32 InWidth, uint32 InHeight, EPixelFormat InFormat, bool bInMipMaps, bool bInAutoGenerateMipMaps, bool bInSRGB)
	: Name(Util::RandomID())
	, Width(InWidth)
	, Height(InHeight)
	, Format(InFormat)
	, NumChannels(TextureHelper::GetChannelsFromPixelFormat(InFormat))
	, bMipMaps(bInMipMaps)
	, bAutoGenerateMipMaps(bInAutoGenerateMipMaps)
	, bIsSRGB(bInSRGB)
{
}

TexDescriptor::TexDescriptor(UTexture2D* TextureObj)
	: Name(TextureObj->GetName())
	, Width(TextureObj->GetSizeX())
	, Height(TextureObj->GetSizeY())
	, Format(TextureObj->GetPixelFormat())
	, NumChannels(TextureHelper::GetChannelsFromPixelFormat(Format))
	, bMipMaps(TextureObj->GetNumMips() > 1)
	, bIsSRGB(TextureObj->SRGB != 0)
	, bCompress(!TextureObj->IsUncompressed())
	, ClearColor(FLinearColor::Black)
{
}

TexDescriptor::TexDescriptor(UTextureRenderTarget2D* RT)
	: Name(RT->GetName())
	, Width((uint32)RT->GetSurfaceWidth())
	, Height((uint32)RT->GetSurfaceHeight())
	, Format(TextureHelper::GetPixelFormatFromRenderTargetFormat(RT->RenderTargetFormat))
	, NumChannels(TextureHelper::GetChannelsFromPixelFormat(Format))
	, bMipMaps(RT->GetNumMips() > 1)
	, bIsSRGB(RT->IsSRGB())
	, ClearColor(RT->ClearColor)		//Required for Hash calculation from descriptor
{
}

TexDescriptor::TexDescriptor(const BufferDescriptor& InDesc)
	: Name(InDesc.Name)
	, Width(InDesc.Width)
	, Height(InDesc.Height)
	, Format(InDesc.PixelFormat())
	, NumChannels(InDesc.ItemsPerPoint)
	, bMipMaps(InDesc.bMipMaps)
	, bIsSRGB(InDesc.bIsSRGB)
	, bUAV(InDesc.RequiresUAV())
	, ClearColor(InDesc.DefaultValue)
{
	/// TODO
}

HashType TexDescriptor::HashValue() const 
{
	HashTypeVec Sources = 
	{
		Format_HashValue(),
		MX_HASH_VAL_DEF(ClearColor.R),
		MX_HASH_VAL_DEF(ClearColor.G),
		MX_HASH_VAL_DEF(ClearColor.B),
		MX_HASH_VAL_DEF(ClearColor.A),
	};

	return DataUtil::Hash(Sources);
}

HashType TexDescriptor::Format_HashValue() const
{
	HashTypeVec Sources =
	{
		MX_HASH_VAL_DEF(Width),
		MX_HASH_VAL_DEF(Height),
		MX_HASH_VAL_DEF(Format),
		MX_HASH_VAL_DEF(NumChannels),
		MX_HASH_VAL_DEF(bIsSRGB),
		MX_HASH_VAL_DEF(bCompress),
	};

	return DataUtil::Hash(Sources);
}

BufferDescriptor TexDescriptor::ToBufferDescriptor(uint32 NewWidth /* = 0 */, uint32 NewHeight /* = 0 */) const
{
	if (!NewWidth)
		NewWidth = Width;
	if (!NewHeight)
		NewHeight = Height;

	BufferFormat BufferFormat = BufferDescriptor::BufferFormatFromPixelFormat(Format);
	uint32 ItemsPerPoint = NumChannels;

	BufferDescriptor Desc(NewWidth, NewHeight, ItemsPerPoint, BufferFormat, ClearColor, BufferType::Image, bMipMaps, bIsSRGB);
	Desc.Name = Name;
	if (bUAV)
		Desc.AddMetadata(RawBufferMetadataDefs::G_FX_UAV);

	return Desc;
}

size_t TexDescriptor::GetPitch() const
{
	return Width * TextureHelper::GetBppFromPixelFormat(Format) / 8;
}

//////////////////////////////////////////////////////////////////////////
EObjectFlags Tex::Flags = (RF_Public | RF_Standalone);

Tex::Tex(int32 Width, int32 Height, EPixelFormat PixelFormat)
	: Desc(Width, Height, PixelFormat)
	, bNoCache(TextureGraphEngine::GetBlobber() && !TextureGraphEngine::GetBlobber()->IsCacheEnabled())
{
	Free();
	InitRT();
}

Tex::Tex(const TexDescriptor& InDesc)
	: Desc(InDesc)
	, bNoCache(TextureGraphEngine::GetBlobber() && !TextureGraphEngine::GetBlobber()->IsCacheEnabled())
{
}

Tex::Tex(RawBufferPtr RawObj)
	: Desc(RawObj->GetDescriptor())
	, bNoCache(TextureGraphEngine::GetBlobber() && !TextureGraphEngine::GetBlobber()->IsCacheEnabled())
{
	LoadRaw(RawObj);
}

Tex::Tex(UTexture2D* Texture) 
	: Texture(Texture)
	, Desc(Texture)
	, bNoCache(TextureGraphEngine::GetBlobber() && !TextureGraphEngine::GetBlobber()->IsCacheEnabled())
{
}

Tex::Tex(UTextureRenderTarget2D* RT)
	: RT(RT)
	, Desc(RT)
	, bNoCache(TextureGraphEngine::GetBlobber() && !TextureGraphEngine::GetBlobber()->IsCacheEnabled())
{
}

Tex::Tex() : bNoCache(TextureGraphEngine::GetBlobber() && !TextureGraphEngine::GetBlobber()->IsCacheEnabled())
{
}

Tex::~Tex()
{
	Free();
}

void Tex::FreeTexture(UTexture2D** Texture)
{
	if (Texture && *Texture)
	{
		(*Texture) = nullptr;
	}
}

void Tex::FreeGenericTexture(UTexture** Texture)
{
	if (Texture && *Texture)
	{
		UE_LOGF(LogTexture, Verbose, "Deleting Texture: %ls [Ptr = 0x%llx]", *((*Texture)->GetName()), (*Texture));
		FTextureResource* TextureResource = (*Texture)->GetResource();

		if (!TextureGraphEngine::IsDestroying() && TextureResource && TextureResource->IsInitialized())
			(*Texture)->ReleaseResource();

		(*Texture) = nullptr;
	}
}

void Tex::FreeRT(UTextureRenderTarget2D** RT)
{
	if (RT && *RT)
	{
		UE_LOGF(LogTexture, Verbose, "Deleting render target: %ls [Ptr = 0x%llx]", *((*RT)->GetName()), (*RT));
		FTextureResource* RTResource = (*RT)->GetResource();

		if (!TextureGraphEngine::IsDestroying() && RTResource && RTResource->IsInitialized())
			(*RT)->ReleaseResource();

		(*RT) = nullptr;
	}
}

void Tex::Free()
{
	check(IsInGameThread());

#if WITH_EDITOR
	/// Make it insensitive to the order of mixer engine destruction
	if (TextureGraphEngine::IsDestroying())
		return;
#endif 
	FreeTexture(ToRawPtr(MutableView(Texture)));
	FreeRT(ToRawPtr(MutableView(RT)));
}

void Tex::InvalidateCached()
{
}

void Tex::SetFilter(TextureFilter FilterValue)
{
	Filter = FilterValue;

	if (RT)
		RT->Filter = TEnumAsByte<TextureFilter>(Filter);

	if (Texture)
		Texture->Filter = TEnumAsByte<TextureFilter>(Filter);
}

void Tex::InitTexture(const uint8* SrcPixels, size_t Length)
{
	check(IsInGameThread());

	Free();

	// Create the Texture
	auto Package = Util::GetTexturesPackage();
	FName Name	= *Desc.Name;

	if (Desc.Name.IsEmpty())
		Name = MakeUniqueObjectName(Package, UTexture2D::StaticClass()); 

	Texture = CreateTexture(Desc.Width, Desc.Height, Desc.Format, Desc.bIsSRGB, Package); /// NewObject<UTexture2D>(Package, Name, Tex::s_flags);

	/// only update if we have been passed a valid set of pixels
	if (SrcPixels)
		UpdateRaw(SrcPixels, Length);
}

void Tex::UpdateRaw(const uint8* Data, size_t Length)
{
	auto Package = Util::GetTexturesPackage();

	check(Texture);
	check(Texture->GetPlatformData());
	check(Texture->GetPlatformData()->Mips.Num() > 0);

	uint8* MipData = static_cast<uint8*>(Texture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE));
	size_t DataSize = Texture->GetPlatformData()->Mips[0].BulkData.GetBulkDataSize();

	check(Length >= DataSize);

	// Bulk Data was already allocated for the correct size when we called CreateTransient above
	FMemory::Memcpy(MipData, Data, DataSize);

	Texture->GetPlatformData()->Mips[0].BulkData.Unlock();

	SetFilter(Filter);

	Texture->UpdateResource();
}

void Tex::UpdateRaw(RawBufferPtr RawObj)
{
	check(IsInGameThread());

	Desc = TexDescriptor(RawObj->GetDescriptor());

	const uint8* Data = RawObj->GetData();
	const size_t Length = RawObj->GetLength();

	UpdateRaw(Data, Length);
}

void Tex::InitRT(bool ForceFloat /* = false */)
{
	SCOPE_CYCLE_COUNTER(STAT_Tex_InitRT)
	check(IsInGameThread());
	check(Desc.Width > 0 && Desc.Height > 0);

	UE_LOGF(LogTexture, VeryVerbose, "InitRT: Freeing existing Texture: %ls", *Desc.Name)

	Free();

	UE_LOGF(LogTexture, VeryVerbose, "InitRT: Existing Texture freed: %ls", *Desc.Name)

	if (ForceFloat)
	{
		BufferDescriptor BuffDesc = Desc.ToBufferDescriptor();

		if (BuffDesc.Format != BufferFormat::Float)
		{
			BuffDesc.Format = BufferFormat::Float;
			Desc.Format = BuffDesc.PixelFormat();
		}
	}

	FString DescName = Desc.Name.Left(std::min(NAME_SIZE / 2, Desc.Name.Len()));
	const FName Name = *FString::Printf(TEXT("%s [RT]"), *DescName);
	const auto Package = Util::GetRenderTargetPackage();
	const FName UniqueName = MakeUniqueObjectName(Package, UTextureRenderTarget2D::StaticClass(), Name);
	RT = NewObject<UTextureRenderTarget2D>(Package, UniqueName);

	UE_LOGF(LogTexture, VeryVerbose, "InitRT: Creating new render target: %ls", *Name.ToString());
	
	check(RT);
	
	RT->ClearColor = Desc.ClearColor;

	if (Desc.bMipMaps && Desc.bAutoGenerateMipMaps) 
	{
		RT->bAutoGenerateMips = true;
	}
	
	RT->MipsSamplerFilter = Filter;
	RT->SRGB = Desc.bIsSRGB;
	RT->bForceLinearGamma = !Desc.bIsSRGB;
	RT->bCanCreateUAV = Desc.bUAV;
	RT->RenderTargetFormat = TextureHelper::GetRenderTargetFormatFromPixelFormat(Desc.Format);
	RT->OverrideFormat = Desc.Format;
	RT->LODBias = 0;
	RT->SizeX = Desc.Width;
	RT->SizeY = Desc.Height;
	RT->UpdateResource();
}

FString Tex::GetReferencerName() const
{
	return Desc.Name;
}

void Tex::AddReferencedObjects(FReferenceCollector& Collector)
{
	/// Don't do anything if caching is disabled
	if (!TextureGraphEngine::GetBlobber()->IsCacheEnabled() || bNoCache)
	{
		// Clear these pointers so that they don't become dangling if the objects are deleted
		RT = nullptr;
		Texture = nullptr;
		return;
	}

	if (RT)
		Collector.AddReferencedObject(RT);

	if (Texture)
		Collector.AddReferencedObject(Texture);
}

AsyncActionResultPtr Tex::LoadRaw(RawBufferPtr RawObj)
{
	check(IsInGameThread());

	if (RawObj)
	{
		Desc = TexDescriptor(RawObj->GetDescriptor());
		Desc.Name += FString::Printf(TEXT(".%llu"), RawObj->Hash()->Value());
	}

	/// TODO: Figure out the best compression scheme based on the type of the Texture.
	/// Perhaps in the longer run, we need to encode this information in the Buffer?
	//_desc.compress = true;

	const uint8* Data = RawObj ? RawObj->GetData() : nullptr;
	const size_t DataLength = RawObj ? RawObj->GetLength() : 0;

	InitTexture(Data, DataLength);
	
	/// return ready Promise
	return cti::make_ready_continuable(std::make_shared<ActionResult>(nullptr));
}

bool Tex::IsNull() const
{
	if (RT || Texture)
		return false;

	//UTexture* tex = Texture();
	//return !tex || !tex->GetResource() || !tex->GetResource()->TextureRHI;

	return true;
}

TArray<FColor> Tex::ReadPixels()
{
	TArray<FColor> Colors;
	if(RT)
	{
		FTextureRenderTargetResource* RenderTarget = RT->GameThread_GetRenderTargetResource();
		RenderTarget->ReadPixels(Colors);
	}

	if(Texture)
	{
		// TODO : Read pixels code for texture 2D
		check(false);
	}
	
	return Colors; 
}

AsyncActionResultPtr Tex::LoadFlat()
{
	check(IsInGameThread());

	if(!IsValid(RT))
		InitRT(false);

	//float pixels[4] =
	//{
	//	_desc.clearColor.R,
	//	_desc.clearColor.G,
	//	_desc.clearColor.B,
	//	_desc.clearColor.A,
	//};

	//InitTexture((const uint8*)pixels);
	//Promise.set_value(std::make_shared<ActionResult>(nullptr));

	return cti::make_continuable<ActionResultPtr>([this](auto&& Promise) mutable
	{
		try
		{
			Device_FX::Get()->Use().then([this, Promise = std::forward<decltype(Promise)>(Promise)]() mutable
			{
				Clear(Device_FX::Get()->RHI());

				Util::OnGameThread([this, Promise = std::forward<decltype(Promise)>(Promise)]() mutable
				{
					/// Go back to the game thread
					Promise.set_value(std::make_shared<ActionResult>(nullptr));
				});
			});

		}
		catch (...)
		{
			Promise.set_exception(std::current_exception());
		}

	});
}

UTexture2D* Tex::CreateTexture(uint32 Width, uint32 Height, EPixelFormat Format, bool sRGB, UObject* Package)
{
	UTexture2D* NewTexture = nullptr;

	if (Width > 0 && Height > 0 &&
		(Width % GPixelFormats[Format].BlockSizeX) == 0 &&
		(Height % GPixelFormats[Format].BlockSizeY) == 0)
	{
		if (!Package)
			Package = Util::GetTexturesPackage();

		const FName UniqueName = MakeUniqueObjectName(Package, UTexture2D::StaticClass(), *Desc.Name);
		NewTexture = NewObject<UTexture2D>(Package, UniqueName);
		NewTexture->SRGB = sRGB;

		const int32 NumBlocksX = Width / GPixelFormats[Format].BlockSizeX;
		const int32 NumBlocksY = Height / GPixelFormats[Format].BlockSizeY;

		if (Width <= 0 || Height <= 0)
		{
			UE_LOGF(LogTexture, Warning, "Negative size specified for UTexture2D::CreateTransient()");
			return nullptr;
		}

		if ((Width % GPixelFormats[Format].BlockSizeX) ||
			(Height % GPixelFormats[Format].BlockSizeY))
		{
			UE_LOGF(LogTexture, Warning, "Size specified isn't valid for block-based pixel format in UTexture2D::CreateTransient()");
			return nullptr;
		}

		FGuardedInt64 BytesForImageValidation = FGuardedInt64(NumBlocksX) * NumBlocksY * GPixelFormats[Format].BlockBytes;
		if (BytesForImageValidation.IsValid() == false)
		{
			UE_LOGF(LogTexture, Warning, "Size specified overflows in UTexture2D::CreateTransient()");
			return nullptr;
		}

		int64 BytesForImage = BytesForImageValidation.Get(0);

		NewTexture->SetPlatformData(new FTexturePlatformData());
		NewTexture->GetPlatformData()->SizeX = Width;
		NewTexture->GetPlatformData()->SizeY = Height;
		NewTexture->GetPlatformData()->SetNumSlices(1);
		NewTexture->GetPlatformData()->PixelFormat = Format;

		// Allocate first mipmap.
		FTexture2DMipMap* Mip = new FTexture2DMipMap(Width, Height, 1);
		NewTexture->GetPlatformData()->Mips.Add(Mip);
		Mip->BulkData.Lock(LOCK_READ_WRITE);
		void* DestImageData = Mip->BulkData.Realloc(BytesForImage);
		memset(DestImageData, 0, BytesForImage);
		Mip->BulkData.Unlock();

		NewTexture->UpdateResource();

		return NewTexture;

	}
	else
	{
		UE_LOGF(LogTexture, Warning, "Invalid parameters specified for UTexture2D::CreateTransient()");
	}

	return NewTexture;
}

/*
 Not used at the moment yet
 THis could be use to allocate the mipmaps for a UTexture2D if provided from sysmem
 template <class T>
bool isPowerOf2(const T& value)
{
	return value > 0 && (value & (value - 1)) == 0;
}

bool Tex::AllocateTextureMips(UTexture2D* Texture) {

	// Allocate mipmaps only if size are power of 2
	int32 Width = Texture->GetSizeX();
	int32 Height = Texture->GetSizeY();
	EPixelFormat Format = Texture->GetPixelFormat();

	bool bGenerateMips = isPowerOf2(Width) && isPowerOf2(Height);
	if (bGenerateMips) {
		Texture->MipGenSettings = TMGS_Sharpen4;

		int priorwidth = Width;
		int priorheight = Height;

		while ((priorwidth > 1) && (priorheight > 1)) {

			int mipwidth = priorwidth >> 1;
			int mipheight = priorheight >> 1;

			int32 NumBlocksX = mipwidth / GPixelFormats[Format].BlockSizeX;
			int32 NumBlocksY = mipheight / GPixelFormats[Format].BlockSizeY;
			FTexture2DMipMap* Mip = new FTexture2DMipMap();
			Texture->PlatformData->Mips.Add(Mip);
			Mip->SizeX = mipwidth;
			Mip->SizeY = mipheight;
			Mip->BulkData.Lock(LOCK_READ_WRITE);
			Mip->BulkData.Realloc(NumBlocksX * NumBlocksY * GPixelFormats[Format].BlockBytes);
			Mip->BulkData.Unlock();

			priorwidth = mipwidth;
			priorheight = mipheight;
		}
	}

	return bGenerateMips;
}
*/

UTexture2D* Tex::InitTextureHDR(const TArray<uint8>& Buffer, UPackage* Package)
{
	// this code has nothing to do with HDR, it's just a generic image loader
	//	this function is not actually used
	//	this function is now better than the used path, which does bad/deprecated manual use of ImageWrappers
	//	use this instead

	FImage Image;
	if ( ! FImageUtils::DecompressImage(Buffer.GetData(),Buffer.Num(),Image) )
	{
		return nullptr;
	}

	ERawImageFormat::Type NewFormat;
	EPixelFormat PixelFormat = FImageCoreUtils::GetPixelFormatForRawImageFormat(Image.Format,&NewFormat);
	if ( Image.Format != NewFormat )
	{
		// PixelFormat isn't identical to Image.Format, so we must convert so we can blit
		Image.ChangeFormat(NewFormat, ERawImageFormat::GetDefaultGammaSpace(NewFormat) );

		PixelFormat = FImageCoreUtils::GetPixelFormatForRawImageFormat(Image.Format);
	}

	if (Desc.Name.IsEmpty())
	{
		Desc.Name = MakeUniqueObjectName(Package, UTexture2D::StaticClass()).ToString();
	}

	const int32 Width = Image.GetWidth();
	const int32 Height = Image.GetHeight();

	UTexture2D* NewTexture = CreateTexture(Width, Height, PixelFormat, Desc.bIsSRGB, Package);

	if ( !NewTexture)
	{
		return nullptr;
	}

	void * MipData = NewTexture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE);
	check(MipData);

	// Bulk Data was already allocated for the correct size when we called CreateTransient above
	int64 BulkDataSize = NewTexture->GetPlatformData()->Mips[0].BulkData.GetBulkDataSize();
	check(BulkDataSize == Image.RawData.Num());

	if ( BulkDataSize == Image.RawData.Num())
	{
		FMemory::Memcpy(MipData, Image.RawData.GetData(), BulkDataSize);
	}

	NewTexture->GetPlatformData()->Mips[0].BulkData.Unlock();

	// If mipmaps are required, this Tex will be turned into a rendertarget and autogenerate mips from mip0
	// If needed, We could allocate and fill the mipmaps from sys mem Data in the flow here
	// if (_desc.mipmaps)
	//		AllocateTextureMips(_texture, some_init_data_for_the_mipmaps);

	NewTexture->UpdateResource();

	return NewTexture;
}

UTexture2D* Tex::InitTextureDefault(int32 Width, int32 Height, EPixelFormat PixelFormat, const uint8* UncompressedData, size_t UncompressedDataSize, UObject* Package)
{
	check(IsInGameThread());
	check(UncompressedData);

	Desc.Name += MakeUniqueObjectName(Package, UTexture2D::StaticClass()).ToString(); // , FName());

	UTexture2D* NewTexture = CreateTexture(Width, Height, PixelFormat, Desc.bIsSRGB, Package);

	if (NewTexture)
	{
#if 0
		check(UncompressedData && UncompressedDataSize);

		NewTexture->SetPlatformData(new FTexturePlatformData());
		NewTexture->GetPlatformData()->SizeX = Width;
		NewTexture->GetPlatformData()->SizeY = Height;
		NewTexture->GetPlatformData()->SetNumSlices(1);
		NewTexture->GetPlatformData()->PixelFormat = PixelFormat;

		// Allocate first mipmap.
		FTexture2DMipMap* Mip = new FTexture2DMipMap(Width, Height, 1);
		NewTexture->GetPlatformData()->Mips.Add(Mip);

		Mip->BulkData.Lock(LOCK_READ_WRITE);
		void* DestImageData = Mip->BulkData.Realloc(UncompressedDataSize);
		FMemory::Memcpy(DestImageData, UncompressedData, UncompressedDataSize);
		Mip->BulkData.Unlock();
		NewTexture->UpdateResource();
#endif 

		uint8* MipData = static_cast<uint8*>(NewTexture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE));
		int64 MipSize = NewTexture->GetPlatformData()->Mips[0].BulkData.GetBulkDataSize();

		check(MipSize == UncompressedDataSize);

		// Bulk Data was already allocated for the correct size when we called CreateTransient above
		FMemory::Memcpy(MipData, UncompressedData, MipSize);
		NewTexture->GetPlatformData()->Mips[0].BulkData.Unlock();

		NewTexture->UpdateResource();
	}

	return NewTexture;
}

bool UncompressJpeg(const ERGBFormat Format, int32 BitDepth, int32 Width, int32 Height, TArray<uint8>& compressedData, TArray<uint8>& UncompressedData)
{
	// there's no need for this, just let ImageWrapper do the loading ; use IImageWrapperModule::DecompressImage or FImageUtils::DecompressImage

	int TJPixelFormat;
	switch (Format)
	{
	case ERGBFormat::BGRA:
		TJPixelFormat = TJPF_BGRA;
		break;
	case ERGBFormat::Gray:
		TJPixelFormat = TJPF_GRAY;
		break;
	case ERGBFormat::RGBA:
		TJPixelFormat = TJPF_RGBA;
		break;
	default:
		TJPixelFormat = TJPF_RGBA;
		break;
	}

	check(UncompressedData.Num() == 0);
	
	// Get the number of NumChannels we need to extract
	int NumChannels = 0;
	if ((Format == ERGBFormat::RGBA || Format == ERGBFormat::BGRA))
	{
		NumChannels = 4;
	}
	else if (Format == ERGBFormat::Gray)
	{
		NumChannels = 1;
	}
	else
	{
		return false;
	}

	void* Decompressor(tjInitDecompress());
	check(Decompressor); //-V516
	check(compressedData.Num());

	UncompressedData.Reset(Width * Height * NumChannels);
	UncompressedData.AddUninitialized(Width * Height * NumChannels);

	int Flag = TJFLAG_PROGRESSIVE;
	if (tjDecompress2(Decompressor, compressedData.GetData(), compressedData.Num(), UncompressedData.GetData(), Width, 0, Height, TJPixelFormat, Flag) != 0)
	{
		//returns 0 on Success
		return false;
	}

/*	if (deGamma)
		ParallelFor(Width * Height * NumChannels, [&](int32 pixel)
			{
				uint8 pixelValue = UncompressedData[pixel];
				double normalizedValue = (double)pixelValue / MAX_uint8;
				UncompressedData[pixel] = round((pow(normalizedValue, 2.2f) * MAX_uint8));
			});*/

	return true;
}

bool Tex::CopyImageToBuffer(EPixelFormat& PixelFormat, int32& Width, int32& Height, TArray<uint8>& InputBuffer, TArray<uint8>& OutputBuffer)
{
	// there's no need for this, just let ImageWrapper do the loading ; use IImageWrapperModule::DecompressImage or FImageUtils::DecompressImage

	IImageWrapperModule& ImageWrapperModule = FModuleManager::Get().LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));

	EImageFormat ImageFormat = ImageWrapperModule.DetectImageFormat(InputBuffer.GetData(), InputBuffer.GetAllocatedSize());

	if (ImageFormat != EImageFormat::Invalid)
	{
		TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(ImageFormat);

		bool isOperationSuccess = ImageWrapper->SetCompressed((void*)InputBuffer.GetData(), InputBuffer.GetAllocatedSize());

		if (isOperationSuccess)
		{
			PixelFormat = PF_Unknown;
			ERGBFormat RGBFormat = ImageWrapper->GetFormat();

			int32 BitDepth = ImageWrapper->GetBitDepth();

			Width = ImageWrapper->GetWidth();
			Height = ImageWrapper->GetHeight();

			if (BitDepth == 16)
			{
				if (RGBFormat == ERGBFormat::GrayF)
				{
					PixelFormat = PF_R16F;
					RGBFormat = ERGBFormat::GrayF;
				}
				else
				{
					PixelFormat = PF_FloatRGBA;
					RGBFormat = ERGBFormat::RGBAF;
				}
			}
			else if (BitDepth == 8)
			{
				PixelFormat = PF_B8G8R8A8;
				RGBFormat = ERGBFormat::BGRA;

				// In the case of jpeg, make sure we use a accurate uncompressor
				if (ImageFormat == EImageFormat::JPEG)
				{
					UncompressJpeg(RGBFormat, BitDepth, Width, Height, InputBuffer, OutputBuffer);
					return true;
				}
			}
			else if (BitDepth == 32)
			{
				if (RGBFormat == ERGBFormat::GrayF)
				{
					PixelFormat = PF_R32_FLOAT;
					RGBFormat = ERGBFormat::GrayF;
				}
				else
				{
					PixelFormat = PF_A32B32G32R32F;
					RGBFormat = ERGBFormat::RGBAF;
				}
			}
			else
			{
				UE_LOGF(LogTexture, Warning, "Error creating Texture. Bit depth is unsupported. (%d)", BitDepth);
				return false;
			}

			return ImageWrapper->GetRaw(RGBFormat, BitDepth, OutputBuffer);
		}
	}
	return false;
}

//cti::continuable<TArray<uint8>> Tex::LoadImageBuffer(const FString& filename)
//{
//	return 
//}

void Tex::GenerateMips()
{
	if (!Desc.bMipMaps) 
		return;

	/// Currently only supported for render targets
	check(RT);
	Util::OnRenderingThread([this](FRHICommandListImmediate& RHI) mutable
	{
		FRDGBuilder GraphBuilder(RHI);
		TRefCountPtr<IPooledRenderTarget> PooledRenderTarget = CreateRenderTarget(GetRHITexture(), TEXT("MipGeneration"));
		FRDGTextureRef TextureRDG = GraphBuilder.RegisterExternalTexture(PooledRenderTarget);
		// TODO: get FeatureLevel from an outside source.
		ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
		FGenerateMips::Execute(GraphBuilder, FeatureLevel, TextureRDG);
	});
}

RawBufferPtr Tex::Raw(const BufferDescriptor* SrcDesc) const
{
	check(RT || Texture);

	auto TexDesc = Desc.ToBufferDescriptor();
	if (SrcDesc)
		TexDesc = *SrcDesc;

	if (Texture)
		return TextureHelper::RawFromTexture(Texture, TexDesc);

	return TextureHelper::RawFromRT(RT, TexDesc);
}

size_t Tex::GetMemSize() const
{
	if (Texture || RT)
		return Desc.ToBufferDescriptor().Size();
	return 0;
}

void Tex::TransferTextureToRT(FRHICommandListImmediate& RHI, UTexture2D** TextureToTransfer, bool FreeAfterUse)
{
	check(IsInRenderingThread());

	/// Must have a valid RT by now
	check(RT);

	RT->Filter = Texture->Filter;

	const RenderMaterialPtr DefaultMaterial = GetDefaultMaterial();

	DefaultMaterial->SetSourceTexture(*TextureToTransfer);
	DefaultMaterial->BlitTo(RHI, RT);

	if (FreeAfterUse)
	{
		Util::OnGameThread([=]() 
		{
			FreeTexture(TextureToTransfer);
		});
	}
}

void Tex::TransferVirtualTextureToRT(FRHICommandListImmediate& RHI, UTexture2D** TextureToTransfer, bool FreeAfterUse)
{
	check(IsInRenderingThread());
	///// Must have a valid RT by now
	check(RT);
	RT->Filter = Texture->Filter;
	auto VTexture = (*TextureToTransfer);

	check(VTexture);
	check(VTexture->IsCurrentlyVirtualTextured());

	// Fully stream in the texture before drawing it.
	// we need the all tiles with higest resolution to be loaded 
	VTexture->SetForceMipLevelsToBeResident(30.0f);
	VTexture->WaitForStreaming();

	FVirtualTexture2DResource* VTResource = static_cast<FVirtualTexture2DResource*>(VTexture->GetResource());
	FVector2D ScreenSpaceSize = FIntPoint(VTResource->GetSizeX(), VTResource->GetSizeY());
	const FVector2D ViewportPosition = FVector2D(0, 0);
	const FVector2D UV0 = FVector2D(0, 0);
	const FVector2D UV1 = FVector2D(1, 1);

	const ERHIFeatureLevel::Type InFeatureLevel = GMaxRHIFeatureLevel;
	const int32 MipLevel = -1;

	//// AcquireAllocatedVT() must happen on render thread
	IAllocatedVirtualTexture* AllocatedVT = VTResource->AcquireAllocatedVT();

	//Request and Prefetch tiles before rednering
	IRendererModule& RenderModule = GetRendererModule();
	RenderModule.RequestVirtualTextureTiles(ScreenSpaceSize, MipLevel);
	RenderModule.LoadPendingVirtualTextureTiles(RHI, InFeatureLevel);
	

	//Set Shader Parameters
	FSH_SimpleVT::FParameters FSHParams;

	FSHParams.InPhysicalTexture = AllocatedVT->GetPhysicalTextureSRV((uint32)0, VTexture->SRGB);
	FSHParams.InTextureSampler = VTResource->SamplerStateRHI.GetReference();

	FSHParams.InPageTableTexture0 = AllocatedVT->GetPageTableTexture(0u);
	FSHParams.InPageTableTexture1 = AllocatedVT->GetNumPageTableTextures() > 1u ? AllocatedVT->GetPageTableTexture(1u) : GBlackTexture->TextureRHI.GetReference();

	FUintVector4 VTPackedPageTableUniform[2];
	FUintVector4 VTPackedUniform;

	AllocatedVT->GetPackedPageTableUniform(VTPackedPageTableUniform);
	AllocatedVT->GetPackedUniform(&VTPackedUniform, (uint32)0);

	FSHParams.VTPackedPageTableUniform[0] = VTPackedPageTableUniform[0];
	FSHParams.VTPackedPageTableUniform[1] = VTPackedPageTableUniform[1];
	FSHParams.VTPackedUniform = VTPackedUniform;

	std::shared_ptr<Fx_FullScreenCopyVT> Mat = std::make_shared<Fx_FullScreenCopyVT>(FSHParams);
	RenderMaterialPtr DefaultMaterial = std::make_shared<RenderMaterial_FX>(TEXT("Tex::FullScreenCopyVT"), (Mat));

	DefaultMaterial->BlitTo(RHI, RT);

	if (FreeAfterUse)
	{
		Util::OnGameThread([=]()
		{
			FreeTexture(TextureToTransfer);
		});
	}
}

AsyncTiledBlobRef Tex::ToSingleBlob(CHashPtr Hash, bool TransferToRT /* = false */, bool ResolveOnRenderThread /* = false */, bool NoCache /* = false */)
{
	/// TransferToRT has started doing some weird filtering. The mipmap process has changed
	/// anyway. Need to deprecate this option (or support it properly) in the Texture descriptor
	/// Keeping it here for the time being for reference!
#if 0 /// TODO: Read above ^^^
	if (_desc.mipmaps && IsValid(_texture))
		TransferToRT = true;	//Force transfer if desc has mips generation on and we are not RT.
#endif 
#if WITH_EDITOR
	// Early out if commandlet execution is not allowed
	if (TextureGraphEngine::BlockCommandlets())
		return static_cast<AsyncTiledBlobRef>(cti::make_ready_continuable<TiledBlobRef>(TiledBlobRef()));

	if ((TransferToRT && !IsValid(RT)) || !IsValid(Texture) || IsValidVirtualTexture())		// For cases where we want to transfer UTexture2D to UTextureRenderTarget2D
	{
		UTexture2D* saveTexture = Texture; //Save the ref to this Texture to keep it from destroying
		Texture = nullptr;
		InitRT(false);
		Texture = saveTexture;
	}

	check(!Hash || !HashObj || *HashObj == *Hash);

	return Device_FX::Get()->Use()
		.then([=, this]() mutable
		{
			auto& RHI = Device_FX::Get()->RHI();
			if (IsValidVirtualTexture())
			{
				TransferVirtualTextureToRT(RHI, ToRawPtr(MutableView(Texture)), true);
			}
			else if (TransferToRT)
			{
				TransferTextureToRT(RHI, ToRawPtr(MutableView(Texture)), true);
			}
			else if (IsValid(RT))
			{
				//Clear it with default color
				Clear(RHI);
			}

			const bool IsNowRT = RT != nullptr;	// Have we shifted to being an RT now?

			check(RT || Texture);

			/// Transfer the RT over to the device
			const DeviceBufferRef Buffer = IsNowRT ? Device_FX::Get()->CreateFromRT(RT, Desc.ToBufferDescriptor()) : Device_FX::Get()->CreateFromTexture(Texture, Desc.ToBufferDescriptor());
			HashObj = TextureGraphEngine::GetBlobber()->AddGloballyUniqueHash(Buffer->Hash());

			const BlobRef BlobObj = TextureGraphEngine::GetBlobber()->Create(Buffer, NoCache);

			BlobPtrTiles Tiles(1, 1);
			Tiles[0][0] = BlobObj;

			auto TiledBlobObj = TiledBlob::InitFromTiles(Desc.ToBufferDescriptor(), Tiles);
			check(TiledBlobObj);

			CHashPtr BlobHash = TiledBlobObj->Hash();
			TiledBlobRef FinalBlobRef = TiledBlobObj;

			if (!NoCache)
			{
				FinalBlobRef = TextureGraphEngine::GetBlobber()->AddTiledResult(BlobHash, std::move(TiledBlobObj));
			}

			check(FinalBlobRef);
			
			/// We don't own this anymore since its been transferred to the device
			RT = nullptr; 
			Texture = nullptr;

			if (!ResolveOnRenderThread)
			{
				return static_cast<AsyncTiledBlobRef>(PromiseUtil::OnGameThread().then([=]() 
					{ 
						return FinalBlobRef; 
					}));
			}

			return static_cast<AsyncTiledBlobRef>(cti::make_ready_continuable<TiledBlobRef>(std::move(FinalBlobRef)));
		});
#else
	return static_cast<AsyncTiledBlobRef>(cti::make_ready_continuable<TiledBlobRef>(TiledBlobRef()));
#endif
}

AsyncTiledBlobRef Tex::ToBlob(int32 NumRows, int32 NumCols, uint32 Width /* = 0 */, uint32 Height /* = 0 */, bool TransferToRT /* = false */)
{
	check(IsInGameThread());

	/// Check that we have a valid Texture loaded
	check(Texture || RT);
	
	if (!Width)
		Width = GetWidth();
	if (!Height)
		Height = GetHeight();

	Desc.Width = Width;
	Desc.Height = Height;

	int32 CheckSizeX = Width;
	int32 CheckSizeY = Height;

#if WITH_EDITOR
	if (Texture->Source.IsValid())
	{
		CheckSizeX = Texture->Source.GetSizeX();
		CheckSizeY = Texture->Source.GetSizeY();
	} 
#endif 

	if (!TextureHelper::CanSplitToTiles(CheckSizeX, CheckSizeY, NumRows, NumCols))
	{
		/// TOOD: need to properly calculate Hash over here
		return ToSingleBlob(nullptr, TransferToRT);
	}

	bool ForceFloat = false;

	if (Desc.bMipMaps && IsValid(Texture))
		TransferToRT = true;	//Force transfer if desc has mips generation on and we are not RT.

	uint32 WidthTilesOffset = Width % NumCols;
	if (WidthTilesOffset != 0)
	{
		Width += (NumCols - WidthTilesOffset);
		Desc.Width = Width;
		TransferToRT = true;
	} 

	uint32 HeightTilesOffset = Height % NumRows;
	if (HeightTilesOffset != 0)
	{
		Height += (NumRows - HeightTilesOffset);
		Desc.Height = Height;
		TransferToRT = true;
	} 

	uint32 TileWidth = Width / NumRows;
	uint32 TileHeight = Height / NumCols;

	if ((TransferToRT && !IsValid(RT)) || !IsValid(Texture) || IsValidVirtualTexture())		// For cases where we want to transfer UTexture2D to UTextureRenderTarget2D
	{
		UTexture2D* SaveTexture = Texture; //Save the ref to this Texture to keep it from destroying
		Texture = nullptr;
		InitRT(ForceFloat);
		Texture = SaveTexture;
	}

	//BlobUPtr* TilesPtr = new BlobUPtr [NumRows * NumCols];
	T_Tiles<BlobPtr>* TilesPtr = new T_Tiles<BlobPtr>(NumRows, NumCols);
	const BufferDescriptor TileDesc = Desc.ToBufferDescriptor(TileWidth, TileHeight);

	Device_FX::InitTiles_Texture(TilesPtr, TileDesc, false);

	return Device_FX::Get()->Use()
		.then([=, this]() mutable
		{
			auto& RHI = Device_FX::Get()->RHI();

			if (IsValidVirtualTexture())
			{
				TransferVirtualTextureToRT(RHI, ToRawPtr(MutableView(Texture)), true);
			}
			else if (TransferToRT)
			{
				TransferTextureToRT(RHI, ToRawPtr(MutableView(Texture)), true);
			}
			else if (IsValid(RT))
			{
				Clear(RHI);
			}

			bool bIsNowRT = RT != nullptr;	// Have we shifted to being an RT now?

			DeviceBufferRef Buffer = bIsNowRT ? Device_FX::Get()->CreateFromRT(RT, Desc.ToBufferDescriptor()) : Device_FX::Get()->CreateFromTexture(Texture, Desc.ToBufferDescriptor());
			T_Tiles<DeviceBufferRef> TileBuffers(NumRows, NumCols);

			for (int32 RowId = 0; RowId < NumRows; RowId++)
			{
				for (int32 ColId = 0; ColId < NumCols; ColId++)
				{
					BlobPtr& Tile = (*TilesPtr)[RowId][ColId];
					check(Tile);
					TileBuffers[RowId][ColId] = Tile->GetBufferRef();
				}
			}

			// Make sure that a Hash has been calculated for the BlobObj
			HashObj = Buffer->Hash(false);
			check(HashObj && HashObj->IsFinal());
			
			CombineSplitArgs SplitArgs { Buffer, TileBuffers };

			return Device::SplitToTiles_Generic(SplitArgs);
		})
		.then([=, this]() mutable
		{
			BlobPtrTiles ResultTiles(NumRows, NumCols);

			for (int32 RowId = 0; RowId < NumRows; RowId++)
			{
				for (int32 ColId = 0; ColId < NumCols; ColId++)
				{
					BlobPtr& Tile = (*TilesPtr)[RowId][ColId]; // [RowId * NumCols + ColId] ;
					check(Tile);

					//check(Tile->IsFinalised());

					/// make sure that there's a Hash
					CHashPtr TileHash = Tile->Hash();
					check(TileHash && TileHash->IsValid() && TileHash->IsFinal());

					TextureGraphEngine::GetBlobber()->AddGloballyUniqueHash(TileHash);

					/// Add the tiled blobs to blobber for re-using later
					/// We do this here since the SplitToTiles function is where the RawObj Hash is updated and we add to blobber after that.
					ResultTiles[RowId][ColId] = TextureGraphEngine::GetBlobber()->AddResult(TileHash, Tile);
				}
			}

			BufferDescriptor BlobDesc = ResultTiles[0][0]->GetDescriptor();
			BlobDesc.Width = Desc.Width;
			BlobDesc.Height = Desc.Height;
			BlobDesc.Name = Desc.Name;
			
			TiledBlobPtr TiledBlobObj = TiledBlob::InitFromTiles(BlobDesc, ResultTiles);
			TiledBlobObj->FinaliseNow(false, nullptr);

			if (HashObj && HashObj->IsFinal())
			{
				TextureGraphEngine::GetBlobber()->AddResult(HashObj, TiledBlobObj);
			}
			
			auto TiledBlobRef = TextureGraphEngine::GetBlobber()->AddTiledResult(TiledBlobObj);

			delete TilesPtr;

			return TiledBlobRef;
		});
}

bool Tex::IsValidVirtualTexture()
{ 
	return(IsValid(Texture) && Texture->IsCurrentlyVirtualTextured()); 
}

bool Tex::LoadAsset(FSoftObjectPath& SoftPath, const DesiredImageProperties* Props /* = nullptr */)
{
	check(IsInGameThread());

	if (Props)
	{
		if (Props->bIsLinear)
			Desc.bIsSRGB = false;
		if (Props->bForceSRGB)
			Desc.bIsSRGB = true;
		if (!Props->Name.IsEmpty())
			Desc.Name = Props->Name;
		if (Props->bMipMaps)
			Desc.bMipMaps = true;
	}

	UObject* Obj = SoftPath.TryLoad();
	check(Obj);

	Texture = static_cast<UTexture2D*>(Obj);

	//For now in case of virtual texture we are making a duplicate of texture and turning off the VirtualTextureStreaming
	//so that it can be loaded like a normal texture 2D. This is a workaround until we find a way to reliably convert the virtual texture to render target
	if (Texture->IsCurrentlyVirtualTextured())
	{
		Texture = (UTexture2D*)StaticDuplicateObject(Obj, GetTransientPackage(), NAME_None, RF_Transient, UTexture2D::StaticClass());
		Texture->Modify();
		Texture->VirtualTextureStreaming = false;
	}

	Desc = TexDescriptor(Texture);

	// override descriptor based on source properties
#if WITH_EDITOR
	check(Texture->Source.IsValid());
	Desc.Width = Texture->Source.GetSizeX();
	Desc.Height = Texture->Source.GetSizeY();
#else
	Desc.Width = Texture->GetSizeX();
	Desc.Height = Texture->GetSizeY();
#endif

	Desc.bMipMaps = false;

#if WITH_EDITOR
	EPixelFormat OutPixelFormat;
	uint32 OutNumChannels;
	const bool ValidConversion = TextureHelper::GetPixelFormatFromTextureSourceFormat(Texture->Source.GetFormat(0), OutPixelFormat, OutNumChannels);
	check(ValidConversion);
	Desc.NumChannels = OutNumChannels;
	Desc.Format = OutPixelFormat;
#endif
	
	return true;
}

bool Tex::LoadFile(const FString& Filename, const DesiredImageProperties* Props /* = nullptr */)
{
	if (Props)
	{
		if (Props->bIsLinear)
			Desc.bIsSRGB = false;
		if (Props->bForceSRGB)
			Desc.bIsSRGB = true;
		if (!Props->Name.IsEmpty())
			Desc.Name = Props->Name;
		if (Props->bMipMaps)
			Desc.bMipMaps = true;
	}

	TArray<uint8> Buffer;
	UE_LOGF(LogTexture, Log, "Trying to load file %ls", *Filename);
	bool DidLoadBuffer = false;

	if (FFileHelper::LoadFileToArray(Buffer, *Filename))
	{
		EPixelFormat PixelFormat;
		int32 Width;
		int32 Height;
		TArray<uint8> UncompressedData;

		bool Success = CopyImageToBuffer(PixelFormat, Width, Height, Buffer, UncompressedData);

		if (Success)
		{
			DidLoadBuffer = true;
			UE_LOGF(LogTexture, Log, "Loading file %ls Success", *Filename);

			try
			{
				auto Package = Util::GetTexturesPackage();
				UTexture2D* NewTexture = nullptr;

				if (FPaths::GetExtension(Filename) == TEXT("HDR"))
				{
					//NewTexture = InitTextureHDR(Buffer, Package);
				}
				else
				{
					NewTexture = InitTextureDefault(Width, Height, PixelFormat, UncompressedData.GetData(), UncompressedData.Num(), Package);
				}

				TexDescriptor NewDesc(NewTexture);

				NewDesc.ClearColor = Desc.ClearColor;
				NewDesc.bMipMaps = Desc.bMipMaps;
				Desc = NewDesc;

				Texture = NewTexture;
			}
			catch (const std::exception_ptr e)
			{
				UE_LOGF(LogTexture, Error, "Exception while loading filename: %ls", *Filename);
				return false;
			}
		}
		else
		{
			UE_LOGF(LogTexture, Log, "CopyImageToBuffer failed for file: %ls", *Filename);
			return false;
		}
	}
	else
	{
		UE_LOGF(LogTexture, Log, "FFileHelper::LoadFileToArray failed for file: %ls", *Filename);
		return false;
	}

	return true;
}

UTexture* Tex::GetTexture() const
{
	if (Texture)
		return Texture;
	else if (RT)
		return RT;

	return nullptr;
}

//FTextureRHIRef Tex::RHITextureRef() const
//{
//	if (_rt)
//		return _rt->GetRenderTargetResource()->TextureRHI;
//	else if (_texture)
//		return ((FTexture2DResource*)_texture->Resource)->GetTexture2DRHI();
//
//	return nullptr;
//}

FRHITexture* Tex::GetRHITexture() const
{
	if (RT)
		return RT->GetResource()->GetTextureRHI();
	else if (Texture)
		return Texture->GetResource()->GetTexture2DRHI();

	return nullptr;
}

void Tex::Bind(FName Name, std::shared_ptr<RenderMaterial> Material) const
{
	//mat->Bind(Name, this);
	Material->SetTexture(Name, GetTexture());

	//if (_rt)
	//	mat->SetTextureParameterValue(Name, _rt);
	//else if (_texture)
	//	mat->SetTextureParameterValue(Name, _texture);
}

void Tex::Release()
{
	/// Clear withoout deleting
	ReleaseTexture();
	ReleaseRT();
}

void Tex::ReleaseRT()
{
	RT = nullptr;
}

void Tex::ReleaseTexture()
{
	Texture = nullptr;
}

bool Tex::SaveImage(UTextureRenderTarget2D* RT, const FString& Path, const FString& Filename, bool bIsHDR)
{
	bool bSuccess = false;
	const FString TotalFileName = FPaths::Combine(*Path, *Filename);
	FText PathError;
	FPaths::ValidatePath(TotalFileName, &PathError);

	if (!RT)
	{
		UE_LOGF(LogTexture, Warning, "TextureRenderTarget must be non-null.");
	}
	else if (!RT->GetResource())
	{
		UE_LOGF(LogTexture, Warning, "TextureRenderTarget has been released");
	}
	else if (!PathError.IsEmpty())
	{
		UE_LOGF(LogTexture, Warning, "Path is invalid - %ls",*PathError.ToString());
	}
	else if (Filename.IsEmpty())
	{
		UE_LOGF(LogTexture, Warning, "No filename specified. Please provide filename with extension");
	}
	else
	{
		FArchive* Ar = IFileManager::Get().CreateFileWriter(*TotalFileName);

		if (Ar)
		{
			FBufferArchive Buffer;

			
			if (RT->RenderTargetFormat == RTF_RGBA16f)
			{
				// Note == is case insensitive
				if (FPaths::GetExtension(TotalFileName) == TEXT("HDR") || bIsHDR)
				{
					bSuccess = FImageUtils::ExportRenderTarget2DAsHDR(RT, Buffer);
				}
				else
				{
					bSuccess = FImageUtils::ExportRenderTarget2DAsEXR(RT, Buffer);
				}

			}
			else if(RT->GetFormat() == PF_B8G8R8A8)
			{
				bSuccess = FImageUtils::ExportRenderTarget2DAsPNG(RT, Buffer);
			}
			

			if (bSuccess)
			{
				Ar->Serialize(const_cast<uint8*>(Buffer.GetData()), Buffer.Num());
			}

			delete Ar;
		}
		else
		{
			UE_LOGF(LogTexture, Warning, "Failed to access path or file");
		}
	}
	return bSuccess;
}

RenderMaterialPtr Tex::GetDefaultMaterial()
{
	if (CopyMat)
		return CopyMat;

	//CopyMat = Engine::MaterialManager()->CreateMaterialInstance(TEXT("Util/CopyTexture"));
	CopyMat = TextureGraphEngine::GetMaterialManager()->CreateMaterialOfType_FX<Fx_FullScreenCopy>(TEXT("Tex::FullScreenCopy"));
	check(CopyMat);

	return CopyMat;
}

void Tex::Clear(FRHICommandList& RHI)
{
	Clear(RHI, Desc.ClearColor);
}

void Tex::Clear(FRHICommandList& RHI, FLinearColor Color)
{
	verify(RT);
	TextureHelper::ClearRT(RHI, RT, Color);
}

void Tex::Clear()
{
	Device_FX::Get()->Use().then([this]() mutable
	{
		Clear(Device_FX::Get()->RHI());
	});
}
