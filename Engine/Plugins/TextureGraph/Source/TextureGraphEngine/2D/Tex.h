// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Data/Blob.h"
#include "Data/TiledBlob.h"
#include "Engine/Texture.h"
#include "Engine/TextureRenderTarget2D.h"
#include "FxMat/RenderMaterial_FX.h"
#include "PixelFormat.h" 
THIRD_PARTY_INCLUDES_START
#include "continuable/continuable.hpp"
THIRD_PARTY_INCLUDES_END
#include "Helper/Promise.h"
#include "UObject/GCObject.h"

#define UE_API TEXTUREGRAPHENGINE_API

class UTexture2D;
class UTextureRenderTarget2D;
class UMaterial;
struct DesiredImageProperties;
class RenderMaterial;

class Tex;

typedef std::shared_ptr<Tex>		TexPtr;

struct DesiredImageProperties
{
	FString							Name;
	bool							bIsLinear = false;
	bool							bForceSRGB = false;
	bool							bIsGrayscale = false;
	bool							bMipMaps = false;
	bool							bPremultiplyAlpha = false;
	bool							bCalculateDisplacementRange = false;
	int32							Width = -1;
	int32							Height = -1;
	EPixelFormat					PixelFormat = static_cast<EPixelFormat>(-1);
};

//////////////////////////////////////////////////////////////////////////
/// TexDescriptor: The Texture descriptor
//////////////////////////////////////////////////////////////////////////
struct TexDescriptor
{
	FString							Name;						/// [Optional] Name of the Texture
	uint32							Width = 0;					/// Width of the Texture in pixels
	uint32							Height = 0;					/// Height of the Texture in pixels
	EPixelFormat					Format = EPixelFormat::PF_R8G8B8A8;	/// The pixel format
	uint32							NumChannels = 4;				/// Channels per-pixels
	bool							bMipMaps = false;			/// Whether to use mipmaps or not
	bool							bAutoGenerateMipMaps = true;	/// Whether to auto-generate mipmaps or not
	bool							bIsSRGB = false;				/// Whether the Texture is in SRGB space or not
	bool							bCompress = false;			/// Whether to use compression on the Texture
	bool							bUAV = false;				/// Whether the resource can create a UAV
	ETextureCompressionQuality		CompressionQuality = ETextureCompressionQuality::TCQ_High;	/// What quality to use on the compression
	TextureCompressionSettings		CompressionSettings = TextureCompressionSettings::TC_Default;/// Compression setting
	FLinearColor					ClearColor = FLinearColor::Transparent;

									TexDescriptor();
									TexDescriptor(uint32 InWidth, uint32 InHeight, EPixelFormat InFormat);
									TexDescriptor(uint32 InWidth, uint32 InHeight, EPixelFormat InFormat, bool bInSRGB);
									TexDescriptor(uint32 InWidth, uint32 InHeight, EPixelFormat InFormat, bool bInMipMaps, bool bInAutoGenerateMipMaps, bool bInSRGB);
	explicit						TexDescriptor(UTexture2D* TextureObj);
	explicit						TexDescriptor(UTextureRenderTarget2D* RT);
	explicit						TexDescriptor(const BufferDescriptor& BufferDesc);

	BufferDescriptor				ToBufferDescriptor(uint32 NewWidth = 0, uint32 NewHeight = 0) const;
	size_t							GetPitch() const;
	FORCEINLINE size_t				Size() const { return GetPitch() * Height; }
	HashType						HashValue() const;
	HashType						Format_HashValue() const;
};

//////////////////////////////////////////////////////////////////////////

/**
 * This is a high level Texture class that wraps a 2D/1D Texture as well as a RenderTarget.
 * Since we do a lot of texturing operations, we need to have some advanced requirements
 * and that is why this class wraps a 2D Texture as well as a RenderTarget. It only has
 * one state at a time though i.e. its either a UTexture or a RenderTarget. It can't be both.
 * This works closely with the RCManager to efficiently manage the resources available to us
 * within the system.
 */
class Tex : public FGCObject
{
	friend class Device_FX;

	//////////////////////////////////////////////////////////////////////////
	/// FGCObject
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual void					AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual FString					GetReferencerName() const override;

public:
	

protected: 
	TObjectPtr<UTexture2D>			Texture = nullptr;				/// The optional Texture
	TObjectPtr<UTextureRenderTarget2D> RT = nullptr;				/// The render target
	TexDescriptor					Desc;							/// The descriptor for this Texture
	TextureFilter					Filter = TextureFilter::TF_Bilinear;/// The filtering of the Texture/RT
	RenderMaterial_FXPtr			CopyMat;						/// Copy material
	CHashPtr						HashObj;						/// The hash for this Texture. This is totally optional
	bool							bNoCache = false;				/// Is this not supposed to be cached

	UE_API void							InitTexture(const uint8* Pixels, size_t Length);
	UE_API UTexture2D*						CreateTexture(uint32 Width, uint32 Height, EPixelFormat PixelFormat, bool sRGB, UObject* Package);
	UE_API UTexture2D*						InitTextureDefault(int32 Width, int32 Height, EPixelFormat PixelFormat, const uint8* uncompressedData, size_t uncompressedDataSize, UObject* Package);

	UE_API bool							CopyImageToBuffer(EPixelFormat& PixelFormat, int32& Width, int32& Height, TArray<uint8>& inputBuffer, TArray<uint8>& OutputBuffer);

	UE_API UTexture2D*						InitTextureHDR(const TArray<uint8>& Buffer, UPackage* Package);

	
	UE_API void							InvalidateCached();
	UE_API RenderMaterialPtr				GetDefaultMaterial();
	UE_API void							UpdateRaw(const uint8* SrcPixels, size_t Length);
	UE_API bool							IsValidVirtualTexture();
public:
	static UE_API EObjectFlags				Flags;
	
									UE_API Tex(int32 Width, int32 Height, EPixelFormat PixelFormat);
									UE_API explicit Tex(const TexDescriptor& InDesc);
									UE_API explicit Tex(UTexture2D* Texture);
									UE_API explicit Tex(RawBufferPtr RawObj);
									UE_API explicit Tex(UTextureRenderTarget2D* RT);
									UE_API Tex();
	UE_API virtual							~Tex() override;

	//////////////////////////////////////////////////////////////////////////
	/// Others
	//////////////////////////////////////////////////////////////////////////
	static UE_API void						FreeGenericTexture(UTexture** Texture);
	static UE_API void						FreeTexture(UTexture2D** Texture);
	static UE_API void						FreeRT(UTextureRenderTarget2D** RT);
	
	UE_API void							InitRT(bool ForceFloat = false);

	UE_API void							Release();
	UE_API void							ReleaseRT();
	UE_API void							ReleaseTexture();
	UE_API void							TransferTextureToRT(FRHICommandListImmediate& RHI, UTexture2D** PrevTexture, bool FreeAfterUse);
	UE_API void							TransferVirtualTextureToRT(FRHICommandListImmediate& RHI, UTexture2D** TextureToTransfer, bool FreeAfterUse);
	UE_API bool							LoadAsset(FSoftObjectPath& SoftPath, const DesiredImageProperties* Props = nullptr);
	UE_API bool							LoadFile(const FString& Filename, const DesiredImageProperties* Props = nullptr);
	UE_API AsyncActionResultPtr			LoadFlat();
	UE_API AsyncTiledBlobRef				ToBlob(int32 XTiles, int32 YTiles, uint32 Width = 0, uint32 Height = 0, bool TransferToRT = false); ///There is a visual difference between having material transfer to RT or directly copy from Texture
	UE_API AsyncTiledBlobRef				ToSingleBlob(CHashPtr Hash, bool TransferToRT = false, bool ResolveOnRenderThread = false, bool NoCache = false);
	UE_API AsyncActionResultPtr			LoadRaw(RawBufferPtr RawObj);
	UE_API void							UpdateRaw(RawBufferPtr RawObj);

	UE_API void							SetFilter(TextureFilter FilterValue);
	
	UE_API virtual void					Bind(FName Name, std::shared_ptr<RenderMaterial> Material) const;
	UE_API void							Clear();
	UE_API void							Clear(FRHICommandList& RHI);
	UE_API void							Clear(FRHICommandList& RHI, FLinearColor Color);

	UE_API void							Free();
	UE_API size_t							GetMemSize() const;
	UE_API RawBufferPtr					Raw(const BufferDescriptor* SrcDesc = nullptr) const; // Create a RawBuffer from the Texture, generate the descriptor if none is proveided
	UE_API void							GenerateMips();

	virtual bool					IsArray()  { return false; }
	UE_API virtual FRHITexture*			GetRHITexture() const;
	UE_API virtual UTexture*				GetTexture() const;

	UE_API virtual bool					IsNull() const;

	UE_API TArray<FColor>					ReadPixels();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	/**
	 * Saves render target to given path and filename (path, filename).
	 *
	 * @param	RenderTarget Input render target to save
	 * @param	Path		 receives the value of the path where the file needs to be saved
	 * @param	Filename	 filename with extension to save in particular format
	 * @param	bIsHDR		whether to save RTF_16 in HDR or EXR (discarded when trying to save PF_B8G8R8A8)	 
	 */
	static	UE_API bool					SaveImage(UTextureRenderTarget2D* RT, const FString& Path, const FString& Filename, bool bIsHDR = true);

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////

	FORCEINLINE FLinearColor		ClearColor() const { return Desc.ClearColor; }
	FORCEINLINE FLinearColor&		ClearColor() { return Desc.ClearColor; }
	FORCEINLINE FIntPoint			GetSize() const { return FIntPoint(Desc.Width, Desc.Height); }
	FORCEINLINE operator			FRHITexture*() const { return GetRHITexture(); }
	//FORCEINLINE operator			FTextureRHIRef() const { return RHITextureRef(); }
	FORCEINLINE operator			UTexture*() const { return GetTexture(); }
	FORCEINLINE FString				GetName() const { return GetTexture()->GetName(); }
	FORCEINLINE const TexDescriptor& GetDescriptor() const { return Desc; }
	FORCEINLINE TextureFilter		GetFilter() const { return Filter; }
	FORCEINLINE uint32				GetWidth() const { return Desc.Width; }
	FORCEINLINE uint32				GetHeight() const { return Desc.Height; }
	FORCEINLINE bool				IsRenderTarget() const { return !Texture && RT; }
	FORCEINLINE UTexture2D*			GetTexture2D() const { return Texture; }
	FORCEINLINE UTextureRenderTarget2D* GetRenderTarget() const { return RT; }
	FORCEINLINE void				SetHash(CHashPtr Hash) { HashObj = Hash; }
	FORCEINLINE CHashPtr			Hash() const { return HashObj; }
};

//typedef std::shared_ptr<Tex>		TexPtr;

#undef UE_API
