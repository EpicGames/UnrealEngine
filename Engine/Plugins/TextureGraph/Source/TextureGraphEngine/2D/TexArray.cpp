// Copyright Epic Games, Inc. All Rights Reserved.
#include "TexArray.h"
#include "Engine/TextureRenderTarget2DArray.h"

TexArray::TexArray(const TexDescriptor& Desc, int32 XTilesTotal, int32 YTilesTotal) : 
	Tex(Desc),
	XTiles(XTilesTotal),
	YTiles(YTilesTotal)
{
	InitRTArray();
}

TexArray::TexArray()
{
	InitRTArray();
}

TexArray::TexArray(Tex& TexObj, int32 XTilesTotal, int32 YTilesTotal) : 
	Tex(TexObj),
	XTiles(XTilesTotal),
	YTiles(YTilesTotal)
{
	InitRTArray();
}

TexArray::TexArray(const TexDescriptor& Desc, UTextureRenderTarget2DArray* rtArray) : 
	Tex(Desc),
	RTArray(rtArray)
{
}

TexArray::~TexArray()
{
	
}

void TexArray::InitRTArray(bool ForceFloat /*= false*/)
{
	check(IsInGameThread());

	if(RTArray)
		return;
	
	FName Name = *FString::Printf(TEXT("%s [RTArr]"), *Desc.Name);
	auto Package = Util::GetRenderTargetPackage();
	FName UniqueName = MakeUniqueObjectName(Package, UTextureRenderTarget2DArray::StaticClass(), Name);
	RTArray = NewObject<UTextureRenderTarget2DArray>(Package, UniqueName);
	check(RTArray);

	RTArray->ClearColor = Desc.ClearColor;

	RTArray->bForceLinearGamma =  !Desc.bIsSRGB ? true : false;
	RTArray->OverrideFormat = Desc.Format;
	RTArray->LODBias = 0;
	// set required size/format
	RTArray->SizeX = Desc.Width / XTiles;
	RTArray->SizeY = Desc.Height / YTiles;
	RTArray->Slices = XTiles * YTiles;

	UE_LOGF(LogTexture, Log, "Init RT Array : %ls , 0x%llx", *Desc.Name, RTArray.Get());
}

UTexture* TexArray::GetTexture() const
{
	return RTArray ? RTArray : Tex::GetTexture();
}

bool TexArray::IsNull() const
{
	return RTArray ? true : false;
}

void TexArray::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (RTArray)
	{
		Collector.AddReferencedObject(RTArray);
	}
}
