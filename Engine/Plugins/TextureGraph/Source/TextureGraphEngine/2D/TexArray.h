// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Tex.h"
#include "Helper/Promise.h"

#define UE_API TEXTUREGRAPHENGINE_API

class UTexture2D;
class UTextureRenderTarget2DArray;

class TexArray;

typedef std::shared_ptr<TexArray>		TexArrayPtr;

class TexArray : public Tex
{
	friend class Device_FX;

protected:
	TObjectPtr<UTextureRenderTarget2DArray>	RTArray = nullptr;			/// The render target array
	int32							XTiles = 0;
	int32							YTiles = 0;
	int32							Index = 0;
	UE_API void							InitRTArray(bool ForceFloat = false);

public:
									UE_API TexArray();
									UE_API TexArray(Tex& TexObj, int32 XTilesTotal, int32 YTilesTotal);
									UE_API TexArray(const TexDescriptor& Desc, UTextureRenderTarget2DArray* rtArray);
									UE_API TexArray(const TexDescriptor& Desc,  int32 XTilesTotal, int32 YTilesTotal);
	UE_API virtual							~TexArray() override;
	UE_API virtual void					AddReferencedObjects(FReferenceCollector& Collector) override;

	virtual bool					IsArray() override { return true; }
	UE_API virtual UTexture*				GetTexture() const override;
	UE_API virtual bool					IsNull() const override;

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE UTextureRenderTarget2DArray* GetRenderTargetArray() const { return RTArray; }
};

#undef UE_API
