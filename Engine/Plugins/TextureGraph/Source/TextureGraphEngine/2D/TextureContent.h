// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h" 
#include "2D/TextureType.h"

#include "TextureContent.generated.h"

UENUM()
enum class TextureContent : uint8
{
	Asset,
	Diffuse,
	Specular,
	Gloss,
	AO,
	Normal,
	Displacement,
	Preview,
	Json,
	Cavity,
	Roughness,
	Metalness,
	Albedo,
	Opacity,
	Curvature,
	Mask,

	//MASKS Components
	PaintMask = 100,
	SolidMask,
	ImageMask,
	NoiseMask,
	PatternMask,
	NormalMask,
	CurvatureMask,
	PositionGradient,

	//UV Modifiers
	CircularModifier,
	TransformModifier,

	//Value Modifiers
	BrightnessMaskModifier = 200,
	ClampMaskModifier,
	InvertMaskModifier,
	NormalizeMaskModifier,
	GradientRemapMaskModifier,
	PosterizeMaskModifier,
	ScatterMaskModifier,

	CustomSource,

	None,

	MaterialIDMask,
	GroupMaskComponent,
	ProjectionModifier,

	SmartMaterial,
	MaterialID,

	/// THIS HAS TO BE THE LAST ENTRY. ADD MORE ABOVE THIS
	Count,
};

