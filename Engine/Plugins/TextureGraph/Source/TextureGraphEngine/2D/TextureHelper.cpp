// Copyright Epic Games, Inc. All Rights Reserved.
#include "TextureHelper.h"
#include "Async/ParallelFor.h"
#include "ClearQuad.h"
#include "Data/RawBuffer.h"
#include "Engine/Texture.h"
#include "Engine/TextureRenderTarget2D.h"
#include "TextureGraphEngineGameInstance.h"
#include "Modules/ModuleManager.h"
#include "RHICommandList.h"
#include "RHIDefinitions.h"
#include "RenderUtils.h"
#include "RendererInterface.h"
#include "Rendering/Texture2DResource.h"
#include "Tex.h"

#include "Data/TiledBlob.h"
#include "CanvasTypes.h"
#include "Engine/Canvas.h"
#include "Misc/App.h"
#include "TextureResource.h"
#include "SceneUtils.h"
#include "Logging/MessageLog.h"
#include "Engine/TextureRenderTarget2D.h"
#include "ImageUtils.h"
#include "OneColorShader.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "Engine/Texture2D.h"
#include <Serialization/BufferArchive.h>
#include <IImageWrapper.h>
#include <IImageWrapperModule.h>
#include <ImageWriteBlueprintLibrary.h>
#include "TextureSet.h"
#include "Helper/ColorUtil.h"
#include "RHIGPUReadback.h"
#include "Data/Blobber.h"
#include "TextureGraphEngine.h"
#include "Curves/CurveLinearColorAtlas.h"
#include "Engine/TextureLightProfile.h"
#include "Model/Mix/MixSettings.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

TiledBlobPtr TextureHelper::GTransparent;
TiledBlobPtr TextureHelper::GBlack;
TiledBlobPtr TextureHelper::GWhite;
TiledBlobPtr TextureHelper::GGray;
TiledBlobPtr TextureHelper::GRed;
TiledBlobPtr TextureHelper::GGreen;
TiledBlobPtr TextureHelper::GBlue;
TiledBlobPtr TextureHelper::GYellow;
TiledBlobPtr TextureHelper::GMagenta;
TiledBlobPtr TextureHelper::GDefaultNormal;
TiledBlobPtr TextureHelper::GWhiteMask;
TiledBlobPtr TextureHelper::GBlackMask;

const uint32 s_maxSolidTextureSize = 1;

cti::continuable<bool> TextureHelper::InitSolidTexture(TiledBlobPtr* BlobObj, FLinearColor clr, FString Name, TexDescriptor Desc)
{
#if WITH_EDITOR
	if (!GEditor)
		return cti::make_ready_continuable(false);

	Desc.Width = s_maxSolidTextureSize;
	Desc.Height = s_maxSolidTextureSize;
	Desc.ClearColor = clr;
	Desc.Name = Name;

	Tex* TexObj = new Tex(Desc);

	return TexObj->ToSingleBlob(std::make_shared<CHash>(TexObj->GetDescriptor().HashValue(), true), false, true, true)
		.then([TexObj, BlobObj](TiledBlobRef Result)
			{
				check(Result);
				Result->GetTile(0, 0)->GetBufferRef()->SetDeviceTransferChain({ DeviceType::FX }, true);
				*BlobObj = Result;

				Util::OnGameThread([=]()
					{
						delete TexObj;
					});

				return true;
			})
		.fail([TexObj, BlobObj](std::exception_ptr e)
			{
				*BlobObj = nullptr;

				Util::OnGameThread([=]()
					{
						delete TexObj;
					});

				throw e;
			});
#else
	return cti::make_ready_continuable(false);
#endif 
}


void TextureHelper::InitStockTextures()
{
#if WITH_EDITOR
	// No need to run during command-let execution
	if (!GEditor)
	{
		return;
	}

	check(IsInGameThread());

	UE_LOGF(LogTemp, Log, "Init stock textures ...");

	/// If we've already go the stuff, then we don't need to worry about it
	if (GBlack)
	{
		return;
	}

	TextureType Type = TextureHelper::TextureContentToTextureType(TextureContent::Albedo);
	TexDescriptor AlbedoDesc(TextureSet::GDesc[(int32)Type]);

	// TODO: For now, just switching the white texture srgb value to linear.
	// Need better solution later.
	TexDescriptor LinearDesc(TextureSet::GDesc[(int32)Type]);
	LinearDesc.bIsSRGB = false;
	
	std::vector<cti::continuable<bool>> Promises;

	// TODO: Need these values to be linear.
	Promises.push_back(InitSolidTexture(&GTransparent, FLinearColor::Transparent, TEXT("TRANSPARENT"), AlbedoDesc));
	Promises.push_back(InitSolidTexture(&GBlack, FLinearColor::Black, TEXT("BLACK"), AlbedoDesc));
	Promises.push_back(InitSolidTexture(&GWhite, FLinearColor::White, TEXT("WHITE"), LinearDesc));
	Promises.push_back(InitSolidTexture(&GGray, FLinearColor::Gray, TEXT("GRAY"), AlbedoDesc));
	Promises.push_back(InitSolidTexture(&GRed, FLinearColor::Red, TEXT("RED"), AlbedoDesc));
	Promises.push_back(InitSolidTexture(&GGreen, FLinearColor::Green, TEXT("GREEN"), AlbedoDesc));
	Promises.push_back(InitSolidTexture(&GBlue, FLinearColor::Blue, TEXT("BLUE"), AlbedoDesc));
	Promises.push_back(InitSolidTexture(&GYellow, FLinearColor::Yellow, TEXT("YELLOW"), AlbedoDesc));
	Promises.push_back(InitSolidTexture(&GMagenta, FLinearColor(1, 0, 1, 1), TEXT("MAGENTA"), AlbedoDesc));
	Promises.push_back(InitSolidTexture(&GDefaultNormal, ColorUtil::DefaultNormal(), TEXT("DefaultNormal"), AlbedoDesc));

	TexDescriptor MaskDesc(1, 1, PF_G8, false, false, false);
	MaskDesc.NumChannels = 1;

	Promises.push_back(InitSolidTexture(&GWhiteMask, FLinearColor::White, TEXT("WhiteMask"), MaskDesc));
	Promises.push_back(InitSolidTexture(&GBlackMask, FLinearColor::Black, TEXT("BlackMask"), MaskDesc));

	cti::when_all(Promises.begin(), Promises.end()).apply(cti::transforms::wait());

	UE_LOGF(LogTemp, Log, "Stock textures init complete!");
#endif 
}

void TextureHelper::FreeStockTextures()
{
	check(IsInGameThread());

	GTransparent = nullptr;
	GBlack = nullptr;
	GWhite = nullptr;
	GGray = nullptr;
	GRed = nullptr;
	GGreen = nullptr;
	GBlue = nullptr;
	GYellow = nullptr;
	GDefaultNormal = nullptr;
}

uint32 TextureHelper::GetChannelsFromPixelFormat(EPixelFormat InPixelFormat)
{
	switch (InPixelFormat)
	{
		case PF_Unknown:
		case PF_A32B32G32R32F:
		case PF_B8G8R8A8:
		case PF_FloatRGBA:
		case PF_A2B10G10R10:
		case PF_A16B16G16R16:
		case PF_R16G16B16A16_UINT:
		case PF_R16G16B16A16_SINT:
		case PF_R32G32B32A32_UINT:
		case PF_R8G8B8A8_UINT:
		case PF_R8G8B8A8_SNORM:
		case PF_R16G16B16A16_UNORM:
		case PF_R16G16B16A16_SNORM:
		case PF_R8G8B8A8:
		case PF_A8R8G8B8:
			return 4;

		case PF_FloatR11G11B10:
		case PF_FloatRGB:
		case PF_R32G32B32F:
		case PF_ETC2_RGB:
		case PF_ATC_RGB:
		case PF_R5G6B5_UNORM:
			return 3;

		case PF_G16R16:
		case PF_G16R16F:
		case PF_G16R16F_FILTER:
		case PF_G32R32F:
		case PF_DepthStencil:
		case PF_V8U8:
		case PF_R8G8:
		case PF_R32G32_UINT:
		//case PF_BC5:
			return 2;
			
		case PF_G8:
		case PF_G16:
		case PF_ShadowDepth:
		case PF_R32_FLOAT:
		case PF_D24:
		case PF_R16F:
		case PF_R16F_FILTER:
		case PF_A8:
		case PF_R32_UINT:
		case PF_R32_SINT:
		case PF_A1:
		case PF_R16_UINT:
		case PF_R16_SINT:
			return 1;

		default:
			return 4;
	}
}

uint32 TextureHelper::GetBppFromPixelFormat(EPixelFormat InPixelFormat)
{
	switch (InPixelFormat)
	{
		case PF_Unknown:
		case PF_A32B32G32R32F:
			return 4 * sizeof(float) * 8;
		case PF_R32G32B32F:
			return 3 * sizeof(float) * 8;
		case PF_B8G8R8A8:
			return 4 * sizeof(char) * 8;
		case PF_A2B10G10R10:
			return 32;
		case PF_A16B16G16R16:
			return 4 * 16;
		case PF_R16G16B16A16_UINT:
			return 4 * 16;
		case PF_R16G16B16A16_SINT:
			return 4 * 16;
		case PF_R32G32B32A32_UINT:
			return 4 * 32;
		case PF_R8G8B8A8_UINT:
			return 4 * 8;
		case PF_R8G8B8A8_SNORM:
			return 4 * 8;
		case PF_R16G16B16A16_UNORM:
			return 4 * 16;
		case PF_R16G16B16A16_SNORM:
			return 4 * 16;
		case PF_R8G8B8A8:
			return 4 * 8;
		case PF_A8R8G8B8:
			return 4 * 8 ;

		case PF_FloatR11G11B10:
			return 32;
		case PF_FloatRGB:	//Float RBA is 16 bit
			return 3 * 16;
		case PF_FloatRGBA:
			return 4 * 16;

		case PF_G16R16:
			return 2 * 16;
		case PF_G16R16F:
			return 2 * 16;
		case PF_G16R16F_FILTER:
			return 2 * 16;
		case PF_G32R32F:
			return 2 * 32;
		case PF_DepthStencil:
			return 24;
		case PF_V8U8:
			return 2 * 8;
		case PF_R8G8:
			return 2 * 8;
		case PF_R32G32_UINT:
			return 2 * 32;
			
		case PF_R8:
		case PF_G8:
		case PF_A8:
			return 8;
		case PF_G16:
			return 16;
		case PF_ShadowDepth:
			return 16;
		case PF_R32_FLOAT:
			return 32;
		case PF_D24:
			return 24;
		case PF_R16F:
			return 16;
		case PF_R16F_FILTER:
			return 16;
		case PF_R32_UINT:
			return 32;
		case PF_R32_SINT:
			return 32;
		case PF_A1:
			return 1;
		case PF_R16_UINT:
			return 16;
		case PF_R16_SINT:
			return 16;

		default:
			return 0;
	}
}

EPixelFormat TextureHelper::GetPixelFormatFromRenderTargetFormat(ETextureRenderTargetFormat RTFormat)
{
	return ::GetPixelFormatFromRenderTargetFormat(RTFormat);
}

bool TextureHelper::IsFloatRT(UTextureRenderTarget2D* RenderTarget)
{
	check(RenderTarget);
	return 
		RenderTarget->RenderTargetFormat == ETextureRenderTargetFormat::RTF_R32f ||
		RenderTarget->RenderTargetFormat == ETextureRenderTargetFormat::RTF_RG32f ||
		RenderTarget->RenderTargetFormat == ETextureRenderTargetFormat::RTF_RGBA32f;
}

ETextureRenderTargetFormat TextureHelper::GetRenderTargetFormatFromPixelFormat(EPixelFormat InPixelFormat)
{
	switch (InPixelFormat)
	{
	case PF_G8: 
		return ETextureRenderTargetFormat::RTF_R8; 
	case PF_R8G8: 
		return ETextureRenderTargetFormat::RTF_RG8; 
	case PF_B8G8R8A8: 
		return ETextureRenderTargetFormat::RTF_RGBA8; 
	case PF_R16F: 
		return ETextureRenderTargetFormat::RTF_R16f; 
	case PF_G16R16F: 
		return ETextureRenderTargetFormat::RTF_RG16f; 
	case PF_R32_FLOAT: 
		return ETextureRenderTargetFormat::RTF_R32f; 
	case PF_G32R32F: 
		return ETextureRenderTargetFormat::RTF_RG32f; 
	case PF_A32B32G32R32F: 
		return ETextureRenderTargetFormat::RTF_RGBA32f; 
	case PF_A2B10G10R10: 
		return ETextureRenderTargetFormat::RTF_RGB10A2; 
	case PF_R32G32B32F:
		return ETextureRenderTargetFormat::RTF_RGBA32f;
	case PF_A16B16G16R16:
	case PF_FloatRGBA:
		return ETextureRenderTargetFormat::RTF_RGBA16f;
	default:
		break;
	}
	return ETextureRenderTargetFormat::RTF_RGBA8;
}


ETextureSourceFormat TextureHelper::GetSourceFormat(ETG_TextureFormat TGTextureFormat)
{
	switch (TGTextureFormat)
	{

	case ETG_TextureFormat::RGBA16F:
		return ETextureSourceFormat::TSF_RGBA16F;

	case ETG_TextureFormat::BGRA8:
		return ETextureSourceFormat::TSF_BGRA8;

	case ETG_TextureFormat::G8:
		return ETextureSourceFormat::TSF_G8;

	case ETG_TextureFormat::R16F:
		return ETextureSourceFormat::TSF_R16F;

	case ETG_TextureFormat::R32F:
		return ETextureSourceFormat::TSF_R32F;

	case ETG_TextureFormat::RGBA32F:
		return ETextureSourceFormat::TSF_RGBA32F;

	default:
		return ETextureSourceFormat::TSF_BGRA8;
	}
}

bool TextureHelper::GetBufferFormatAndChannelsFromTGTextureFormat(ETG_TextureFormat Format, BufferFormat& OutBufferFormat, uint32& OutBufferChannels)
{
	switch (Format)
	{
	case ETG_TextureFormat::Auto:
		OutBufferFormat = BufferFormat::Auto;
		OutBufferChannels = 0;
		break;
		
	case ETG_TextureFormat::RGBA16F:
		OutBufferFormat = BufferFormat::Half;
		OutBufferChannels = 4;
		break;

	case ETG_TextureFormat::BGRA8:
		OutBufferFormat = BufferFormat::Byte;
		OutBufferChannels = 4;
		break;

	case ETG_TextureFormat::G8:
		OutBufferFormat = BufferFormat::Byte;
		OutBufferChannels = 1;
		break;

	case ETG_TextureFormat::R16F:
		OutBufferFormat = BufferFormat::Half;
		OutBufferChannels = 1;
		break;

	case ETG_TextureFormat::R32F:
		OutBufferFormat = BufferFormat::Float;
		OutBufferChannels = 1;
		break;

	case ETG_TextureFormat::RGBA32F:
		OutBufferFormat = BufferFormat::Float;
		OutBufferChannels = 4;
		break;
	default:
		return false;
	}

	return true;
}

ETG_TextureFormat TextureHelper::GetTGTextureFormatFromChannelsAndFormat(uint32 ItemsPerPoint, BufferFormat Format)
{
	ETG_TextureFormat TextureFormat = ETG_TextureFormat::Auto;
	if(ItemsPerPoint == 0 || ItemsPerPoint == 3 || ItemsPerPoint == 4)
	{
		// not using 3 channels at the moment, falling back to use 4
		switch (Format)
		{
		default:
		case BufferFormat::Auto:
		case BufferFormat::Byte:
			TextureFormat = ETG_TextureFormat::BGRA8;
			break;
		case BufferFormat::Half:
			TextureFormat = ETG_TextureFormat::RGBA16F;
			break;
		case BufferFormat::Float:
			TextureFormat = ETG_TextureFormat::RGBA32F;
			break;
		}
	}
	else if (ItemsPerPoint == 1 || ItemsPerPoint == 2)
	{
		// not using 2 channels for now, falling back to one channel
		switch (Format)
		{
		default:
		case BufferFormat::Auto:
		case BufferFormat::Byte:
			TextureFormat = ETG_TextureFormat::G8;
			break;
		case BufferFormat::Half:
			TextureFormat = ETG_TextureFormat::R16F;
			break;
		case BufferFormat::Float:
			TextureFormat = ETG_TextureFormat::R32F;
			break;
		}
	}

	return TextureFormat;
}

uint32 TextureHelper::GetNumChannelsFromTGTextureFormat(ETG_TextureFormat TextureFormat)
{
	switch(TextureFormat)
	{
	default:
	case ETG_TextureFormat::Auto:
		return 0;

	case ETG_TextureFormat::G8:
	case ETG_TextureFormat::R16F:
	case ETG_TextureFormat::R32F:
		return 1;
	
	case ETG_TextureFormat::BGRA8:
	case ETG_TextureFormat::RGBA16F:
	case ETG_TextureFormat::RGBA32F:
		return 4;
	}
}

FString TextureHelper::GetChannelsTextFromItemsPerPoint(const int32 InItemsPerPoint)
{
	FString Channel = "R";
	switch (InItemsPerPoint)
	{
	case 2:
		Channel = "RG";
		break;
	case 3:
		Channel = "RGB";
		break;
	case 4:
		Channel = "RGBA";
		break;
	default:
		Channel = "R";
		break;
	}

	return Channel;
}

void TextureHelper::ClearRT(FRHICommandList& RHI, UTextureRenderTarget2D* RenderTarget, FLinearColor Color /* = FLinearColor::Transparent */)
{
	UE_LOGF(LogTexture, VeryVerbose, "Clear RT: %ls", *RenderTarget->GetName());

	FTextureRenderTarget2DResource* RTRes = (FTextureRenderTarget2DResource*)RenderTarget->GetRenderTargetResource();

	// Silently ignore this
	if (!RTRes)
		return;
	
	FRHIRenderPassInfo RenderPassInfo(RTRes->GetRenderTargetTexture(), ERenderTargetActions::DontLoad_Store);
	TransitionRenderPassTargets(RHI, RenderPassInfo);
	RHI.BeginRenderPass(RenderPassInfo, TEXT("ClearRT"));
	DrawClearQuad(RHI, Color);
	RHI.EndRenderPass();
}

void TextureHelper::ClearRT(UTextureRenderTarget2D* RenderTarget, FLinearColor Color /* = FLinearColor::Transparent */)
{
	ENQUEUE_RENDER_COMMAND(ClearRTCommand)([RenderTarget, Color](FRHICommandList& RHI)
	{
		TextureHelper::ClearRT(RHI, RenderTarget, Color);
	});
}

const char* TextureHelper::TextureTypeToString(TextureType Type)
{
	switch (Type)
	{
	case TextureType::Diffuse: return "Diffuse";
	case TextureType::Specular: return "Specular";
	case TextureType::Albedo: return "Albedo";
	case TextureType::Metalness: return "Metalness";
	case TextureType::Normal: return "Normal";
	case TextureType::Displacement: return "Displacement";
	case TextureType::Opacity: return "Opacity";
	case TextureType::Roughness: return "Roughness";
	case TextureType::AO: return "AO";
	case TextureType::Curvature: return "Curvature";
	case TextureType::Preview: return "Preview";

	default: break;
	}

	return "";
}

const char* TextureHelper::TextureTypeToMegascansType(TextureType Type)
{
	switch (Type)
	{
	case TextureType::Diffuse: return "diffuse";
	case TextureType::Specular: return "specular";
	case TextureType::Albedo: return "albedo";
	case TextureType::Metalness: return "metalness";
	case TextureType::Normal: return "normal";
	case TextureType::Displacement: return "displacement";
	case TextureType::Opacity: return "opacity";
	case TextureType::Roughness: return "roughness";
	case TextureType::AO: return "ao";
	case TextureType::Curvature: return "curvature";
	case TextureType::Preview: return "preview";

	default: break;
	}

	return "";
}

TextureType TextureHelper::TextureContentToTextureType(TextureContent Content)
{
	switch (Content)
	{
	case TextureContent::Diffuse:
		return TextureType::Diffuse;
	case TextureContent::Specular:
		return TextureType::Specular;
	case TextureContent::AO:
		return TextureType::AO;
	case TextureContent::Normal:
		return TextureType::Normal;
	case TextureContent::Displacement:
		return TextureType::Displacement;
	case TextureContent::Roughness:
		return TextureType::Roughness;
	case TextureContent::Metalness:
		return TextureType::Metalness;
	case TextureContent::Albedo:
		return TextureType::Albedo;
	case TextureContent::Opacity:
		return TextureType::Opacity;
	case TextureContent::Curvature:
		return TextureType::Curvature;
	case TextureContent::Preview:
		return TextureType::Preview;
	}

	return TextureType::Unknown;
}

TextureContent TextureHelper::TextureTypeToTextureContent(TextureType Type)
{
	switch (Type)
	{
	case TextureType::Diffuse:
		return TextureContent::Diffuse;
	case TextureType::Specular:
		return TextureContent::Specular;
	case TextureType::Normal:
		return TextureContent::Normal;
	case TextureType::Displacement:
		return TextureContent::Displacement;
	case TextureType::Roughness:
		return TextureContent::Roughness;
	case TextureType::AO:
		return TextureContent::AO;
	case TextureType::Metalness:
		return TextureContent::Metalness;
	case TextureType::Albedo:
		return TextureContent::Albedo;
	case TextureType::Opacity:
		return TextureContent::Opacity;
	case TextureType::Curvature:
		return TextureContent::Curvature;
	case TextureType::Preview:
		return TextureContent::Preview;
	}

	return TextureContent::None;
}

MaskType TextureHelper::TextureContentToMaskType(TextureContent Content)
{
	switch (Content)
	{
	case TextureContent::PaintMask:
		return MaskType::PaintMask;
	case TextureContent::SolidMask:
		return MaskType::SolidMask;
	case TextureContent::ImageMask:
		return MaskType::ImageMask;
	case TextureContent::NoiseMask:
		return MaskType::NoiseMask;
	case TextureContent::PatternMask:
		return MaskType::PatternMask;
	case TextureContent::NormalMask:
		return MaskType::NormalMask;
	case TextureContent::CurvatureMask:
		return MaskType::CurvatureMask;
	case TextureContent::PositionGradient:
		return MaskType::PositionGradient;
	}

	return MaskType::MasksTypeCount;
}

TextureContent TextureHelper::MaskTypeToTextureContent(MaskType Type)
{
	switch (Type)
	{
	case MaskType::PaintMask:
		return TextureContent::PaintMask;
	case MaskType::SolidMask:
		return TextureContent::SolidMask;
	case MaskType::ImageMask:
		return TextureContent::ImageMask;
	case MaskType::NoiseMask:
		return TextureContent::NoiseMask;
	case MaskType::PatternMask:
		return TextureContent::PatternMask;
	case MaskType::NormalMask:
		return TextureContent::NormalMask;
	case MaskType::CurvatureMask:
		return TextureContent::CurvatureMask;
	case MaskType::PositionGradient:
		return TextureContent::PositionGradient;
	}

	return TextureContent::None;
}

MaskModifierType TextureHelper::TextureContentToMaskModifier(TextureContent Content)
{

	switch (Content)
	{
	case TextureContent::BrightnessMaskModifier:
		return MaskModifierType::BrightnessMaskModifier;
	case TextureContent::ClampMaskModifier:
		return MaskModifierType::ClampMaskModifier;
	case TextureContent::InvertMaskModifier:
		return MaskModifierType::InvertMaskModifier;
	case TextureContent::NormalizeMaskModifier:
		return MaskModifierType::NormalizeMaskModifier;
	case TextureContent::GradientRemapMaskModifier:
		return MaskModifierType::GradientRemapMaskModifier;
	case TextureContent::PosterizeMaskModifier:
		return MaskModifierType::PosterizeMaskModifier;
	case TextureContent::ScatterMaskModifier:
		return MaskModifierType::ScatterMaskModifier;
	}

	return MaskModifierType::MaskModifiersTypeCount;
}

TextureContent TextureHelper::MaskModifierToTextureContent(MaskModifierType Type)
{
	switch (Type)
	{
	case MaskModifierType::BrightnessMaskModifier:
		return TextureContent::BrightnessMaskModifier;
	case MaskModifierType::InvertMaskModifier:
		return TextureContent::InvertMaskModifier;
	case MaskModifierType::NormalizeMaskModifier:
		return TextureContent::NormalizeMaskModifier;
	case MaskModifierType::GradientRemapMaskModifier:
		return TextureContent::GradientRemapMaskModifier;
	case MaskModifierType::PosterizeMaskModifier:
		return TextureContent::PosterizeMaskModifier;
	case MaskModifierType::ScatterMaskModifier:
		return TextureContent::ScatterMaskModifier;
	}

	return TextureContent::None;
}

TextureType TextureHelper::TextureStringToType(const FString& TypeString)
{
	for (int i = 0; i < (int)TextureType::Count; i++)
	{
		TextureType Type = (TextureType)i;
		FString CurrentTypeString = TextureHelper::TextureTypeToString(Type);
		if (CurrentTypeString.Equals(TypeString))
		{
			return Type;
		}
	}
	return TextureType::Count;
}

RawBufferPtr TextureHelper::RawFromRT(UTextureRenderTarget2D* RenderTarget, const BufferDescriptor& Desc)
{
	//check(IsInRenderingThread()); //Can be from any thread
	return RawFromResource(FTextureRHIRef(((FTexture2DResource*)RenderTarget->GetResource())->GetTexture2DRHI()), Desc);
}

RawBufferPtr TextureHelper::RawFromTexture(UTexture2D* Texture, const BufferDescriptor& Desc)
{
	check(IsInRenderingThread());

	uint8* RawData = nullptr;
	size_t DataSize = 0;

	//For editor we are using the source texture until we support the compressed Formats
#if WITH_EDITOR
	if (Texture->Source.IsValid())
	{
		// Possibly the data was stored in TextureSource rather than PlatformData
		// The difference is the source data (residing in Utexture) is editor only and can be saved to disk
		// This is mostly used for thumbnail, where thumbnail data can exist across sessions (e.g UAsset)
		// platform data, as the name suggests, is used during cook to convert to platform specific texture. 
		// This is legal because we might not be cooking our data.

		TArray64<uint8> MipDataSrc;
		Texture->Source.GetMipData(MipDataSrc, 0);
		DataSize = MipDataSrc.Num();
		RawData = new uint8[DataSize];
		FMemory::Memcpy(RawData, MipDataSrc.GetData(), DataSize);
	}
#endif

	if (!RawData)
	{
		const uint8* MipData = static_cast<uint8*>(Texture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_ONLY));
		DataSize = Texture->GetPlatformData()->Mips[0].BulkData.GetBulkDataSize();

		if (DataSize != 0)
		{
			size_t DescSize = Desc.Size();
			check(DataSize == DescSize);
			RawData = new uint8[DataSize];

			// Bulk data was already allocated for the correct size when we called CreateTransient above
			FMemory::Memcpy(RawData, MipData, DataSize);
			Texture->GetPlatformData()->Mips[0].BulkData.Unlock();
		}
	}
	
	return std::make_shared<RawBuffer>(RawData, DataSize, Desc);
}

size_t TextureHelper::RoundUpTo(size_t Size, size_t DesiredRounding)
{
	size_t RoundedSize = Size;
	size_t Remainder = RoundedSize % DesiredRounding;

	if (Remainder != 0)
	{
		RoundedSize = Size + DesiredRounding - Remainder;
	}

	return RoundedSize;
}

RawBufferPtr TextureHelper::RawFromResource(const FTextureRHIRef& ResourceRHI, const BufferDescriptor& Desc)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	try
	{
		RawBufferPtr RawObj;

		const EPixelFormat PixelFormat = Desc.PixelFormat();

		/// These must be the same!
		check(TextureHelper::GetBppFromPixelFormat(ResourceRHI->GetFormat()) == TextureHelper::GetBppFromPixelFormat(PixelFormat));

		const uint32 BitsPerPixel = TextureHelper::GetBppFromPixelFormat(PixelFormat);
		check(BitsPerPixel % 8 == 0);
		const uint32 BytesPerPixel = BitsPerPixel / 8;

		const TUniquePtr<FRHIGPUTextureReadback> TextureReadback = MakeUnique<FRHIGPUTextureReadback>(TEXT("RawFromResourceTextureReadback"));
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

		RHICmdList.Transition(FRHITransitionInfo(ResourceRHI, ERHIAccess::Unknown, ERHIAccess::CopySrc));

		TextureReadback->EnqueueCopy(RHICmdList, ResourceRHI);

		RHICmdList.Transition(FRHITransitionInfo(ResourceRHI, ERHIAccess::CopySrc, ERHIAccess::SRVMask));

		RHICmdList.SubmitAndBlockUntilGPUIdle();

		//check(TextureReadback->IsReady());
		{
			const size_t DstDataLength = Desc.Width * Desc.Height * BytesPerPixel;
			uint8* const DstData = new uint8[DstDataLength];

			int32 OutBufferRowPitchInPixels, OutBufferHeight;
			const uint8* SrcData = static_cast<const uint8*>(TextureReadback->Lock(OutBufferRowPitchInPixels, &OutBufferHeight));
			check(SrcData);
			check(static_cast<uint32>(OutBufferRowPitchInPixels) >= Desc.Width && static_cast<uint32>(OutBufferHeight) >= Desc.Height);

			if (OutBufferRowPitchInPixels == Desc.Width)
			{
				// If pitch and width are the same, we can just copy the entire buffer.

				FMemory::Memcpy(DstData, SrcData, DstDataLength);
			}
			else
			{
				// If pitch and width are NOT the same, we need to copy the valid pixels row by row.

				      uint8* RowDstPtr = DstData;
				const size_t RowDstLength = Desc.Width * BytesPerPixel;
				const uint8* RowSrcPtr = SrcData;
				const size_t RowSrcLength = OutBufferRowPitchInPixels * BytesPerPixel;

				for (uint32 Row = 0; Row < Desc.Height; ++Row)
				{
					FMemory::Memcpy(RowDstPtr, RowSrcPtr, RowDstLength);

					RowDstPtr += RowDstLength;
					RowSrcPtr += RowSrcLength;
				}
			}

			TextureReadback->Unlock();

			RawObj = std::make_shared<RawBuffer>(DstData, DstDataLength, Desc);
		}

		return RawObj;
	}
	catch (const std::exception&)
	{
		check(0);
		
	}
	uint8* DstData = new uint8[1];

	return std::make_shared<RawBuffer>(DstData, 0, Desc, nullptr);
}

void TextureHelper::RawFromRT_Tiled(UTextureRenderTarget2D* RenderTarget, const BufferDescriptor& Desc, size_t TileSizeX, size_t TileSizeY, RawBufferPtrTiles& Tiles)
{
	check(IsInRenderingThread());
	FTextureRHIRef ResourceRHI = ((FTextureRenderTarget2DResource*)RenderTarget->GetResource())->GetTextureRHI();
	return RawFromResource_Tiled(ResourceRHI, Desc, TileSizeX, TileSizeY, Tiles);
}

void TextureHelper::RawFromTexture_Tiled(UTexture2D* Texture, const BufferDescriptor& Desc, size_t TileSizeX, size_t TileSizeY, RawBufferPtrTiles& Tiles)
{
	check(IsInRenderingThread());
	FTextureRHIRef ResourceRHI = ((FTexture2DResource*)Texture->GetResource())->GetTexture2DRHI();
	return RawFromResource_Tiled(ResourceRHI, Desc, TileSizeX, TileSizeY, Tiles);
}

void TextureHelper::RawFromResource_Tiled(FTextureRHIRef ResourceRHI, const BufferDescriptor& Desc, size_t TileSizeX, size_t TileSizeY, RawBufferPtrTiles& Tiles)
{
	/// If any of the dimensions are less than the tile size requested then we don't tile at all
	if (Desc.Width < TileSizeX && Desc.Height < TileSizeY)
	{
		auto raw = RawFromResource(ResourceRHI, Desc);
		Tiles.Resize(1, 1);
		Tiles[0][0] = raw;
		return;
	}

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	uint32 stride;
	const uint8* SrcData = (const uint8*)RHICmdList.LockTexture2D(ResourceRHI, 0, RLM_ReadOnly, stride, false);
	size_t srcDataLength = Desc.Size();

	RawFromMem_Tiled(SrcData, srcDataLength, Desc, TileSizeX, TileSizeY, Tiles);

	RHICmdList.UnlockTexture2D(ResourceRHI, 0, false);
}

BufferFormat TextureHelper::FindOptimalSupportedFormat(BufferFormat SrcFormat)
{
	if (SrcFormat == BufferFormat::Short)
		return BufferFormat::Float;
	return SrcFormat;
}

typedef std::function <void(uint8* Dst, size_t DstLength, const BufferDescriptor& DstDesc, const uint8* Src, size_t SrcLength, const BufferDescriptor& SrcDesc)> DataConverter;

static DataConverter GDataConverters[static_cast<int>(BufferFormat::Count)][static_cast<int>(BufferFormat::Count)] = {};

void DefaultDataConverter(uint8* Dst, size_t DstLength, const BufferDescriptor& DstDesc, const uint8* Src, size_t SrcLength, const BufferDescriptor& SrcDesc)
{
	check(SrcLength == DstLength);
	FMemory::Memcpy(Dst, Src, SrcLength);
}

void ShortToHalf_DataConverter(uint8* Dst, size_t DstLength, const BufferDescriptor& DstDesc, const uint8* Src, size_t SrcLength, const BufferDescriptor& SrcDesc)
{
	check(SrcDesc.Format == BufferFormat::Short);
	check(DstDesc.Format == BufferFormat::Half);

	const size_t DstFormatSize = DstDesc.FormatSize();
	const size_t SrcFormatSize = SrcDesc.FormatSize();
	const size_t DstCount = DstLength / DstFormatSize;
	const size_t SrcCount = SrcLength / SrcFormatSize;
	check(DstCount == SrcCount);

	for (size_t PixelIndex = 0; PixelIndex < DstCount; PixelIndex++)
	{
		FFloat16* DstPixel = reinterpret_cast<FFloat16*>(Dst + PixelIndex * DstFormatSize);
		const uint16_t* SrcPixel = reinterpret_cast<const uint16_t*>(Src + PixelIndex * SrcFormatSize);
		float SrcPixelF = float(*SrcPixel) / 65536.0f;
		*DstPixel = SrcPixelF;
	}
}

void ShortToFloat_DataConverter(uint8* Dst, size_t DstLength, const BufferDescriptor& DstDesc, const uint8* Src, size_t SrcLength, const BufferDescriptor& SrcDesc)
{
	check(SrcDesc.Format == BufferFormat::Short);
	check(DstDesc.Format == BufferFormat::Float);

	const size_t DstFormatSize = DstDesc.FormatSize();
	const size_t SrcFormatSize = SrcDesc.FormatSize();
	const size_t DstCount = DstLength / DstFormatSize;
	const size_t SrcCount = SrcLength / SrcFormatSize;
	check(DstCount == SrcCount);

	for (size_t PixelIndex = 0; PixelIndex < DstCount; PixelIndex++)
	{
		float* DstPixel = reinterpret_cast<float*>(Dst + PixelIndex * DstFormatSize);
		const uint16_t* SrcPixel = reinterpret_cast<const uint16_t*>(Src + PixelIndex * SrcFormatSize);
		*DstPixel = float(*SrcPixel) / 65536.0f;
	}
}

DataConverter GetDataConverter(BufferFormat SrcFormat, BufferFormat DstFormat)
{
	check(static_cast<int32>(SrcFormat) >= 0 && SrcFormat < BufferFormat::Count);
	check(static_cast<int32>(DstFormat) >= 0 && DstFormat < BufferFormat::Count);
	
	/// Initialise the array
	if (!GDataConverters[0][0])
	{
		for (int32 fx = 0; fx < static_cast<int32>(BufferFormat::Count); fx++)
		{
			for (int32 fy = 0; fy < static_cast<int32>(BufferFormat::Count); fy++)
			{
				GDataConverters[fx][fy] = std::bind(&DefaultDataConverter, std::placeholders::_1, std::placeholders::_2,
					std::placeholders::_3, std::placeholders::_4, std::placeholders::_5, std::placeholders::_6);
			}
		}

		GDataConverters[(int)BufferFormat::Short][(int)BufferFormat::Half] = std::bind(&ShortToHalf_DataConverter, std::placeholders::_1, std::placeholders::_2,
					std::placeholders::_3, std::placeholders::_4, std::placeholders::_5, std::placeholders::_6);

		GDataConverters[(int)BufferFormat::Short][(int)BufferFormat::Float] = std::bind(&ShortToFloat_DataConverter, std::placeholders::_1, std::placeholders::_2,
					std::placeholders::_3, std::placeholders::_4, std::placeholders::_5, std::placeholders::_6);
	}

	return GDataConverters[(int)SrcFormat][(int)DstFormat];
}

void TextureHelper::RawFromMem_Tiled(const uint8* SrcData, size_t SrcDataLength, const BufferDescriptor& SrcDesc, size_t TileSizeX, size_t TileSizeY, RawBufferPtrTiles& RawTiles)
{
	int SourceWidth = SrcDesc.Width;
	int SourceHeight = SrcDesc.Height;

	int TileWidth = TileSizeX;
	int TileHeight = TileSizeY;

	BufferDescriptor DstTileDesc = SrcDesc;
	DstTileDesc.Width = TileSizeX;
	DstTileDesc.Height = TileSizeY;
	DstTileDesc.Format = FindOptimalSupportedFormat(SrcDesc.Format);

	size_t NumCols = SrcDesc.Width / TileSizeX;
	size_t NumRows = SrcDesc.Height / TileSizeY;

	T_Tiles<uint8*> Tiles(NumRows, NumCols);
	size_t SrcFormatSize = SrcDesc.FormatSize();
	size_t SrcTilePitch = TileWidth * SrcFormatSize * SrcDesc.ItemsPerPoint;

	check(SrcDataLength % SrcDesc.Height == 0);
	size_t SrcPitch = SrcDataLength / SrcDesc.Height;

	size_t DstFormatSize = DstTileDesc.FormatSize();
	size_t DstTilePitch = TileWidth * DstFormatSize * DstTileDesc.ItemsPerPoint;
	size_t ActualTileSize = DstTilePitch * TileSizeY;
	size_t TileSize = ActualTileSize; // DataUtil::GetOptimalHashingSize(ActualTileSize);

	RawTiles.Resize(NumRows, NumCols);

	for (size_t RowIndex = 0; RowIndex < NumRows; RowIndex++)
	{
		for (size_t ColIndex = 0; ColIndex < NumCols; ColIndex++)
		{
			uint8* DstData = new uint8[TileSize];
			memset(DstData, 0, TileSize);

			Tiles[RowIndex][ColIndex] = DstData;

			BufferDescriptor ThisTileDesc = DstTileDesc;
			ThisTileDesc.Name = TextureHelper::CreateTileName(SrcDesc.Name, ColIndex, RowIndex);
		}
	}
	
	auto Converter = GetDataConverter(SrcDesc.Format, DstTileDesc.Format);
	check(Converter != nullptr);
	
	ParallelFor(SourceHeight, [&](int32 Y)
	{
		int RowIndex = FMath::Abs(Y / TileHeight);
		for (int ColIndex = 0; ColIndex < NumCols; ColIndex++)
		{
			uint8* Tile = Tiles[RowIndex][ColIndex];
			uint8* Dst = Tile + ((Y % TileHeight) * DstTilePitch);
			const uint8* Src = SrcData + (ColIndex * SrcTilePitch) + (Y * SrcPitch);
			Converter(Dst, DstTilePitch, DstTileDesc, Src, SrcTilePitch, SrcDesc);
		}
	});

	for (size_t RowId = 0; RowId < NumRows; RowId++)
	{
		for (size_t ColId = 0; ColId < NumCols; ColId++)
		{
			BufferDescriptor ThisTileDesc = DstTileDesc;
			ThisTileDesc.Name = TextureHelper::CreateTileName(SrcDesc.Name, RowId, ColId);

			/// Only TileSize use it for actual hashing calculation
			HashType TileHashValue = DataUtil::Hash(Tiles[RowId][ColId], TileSize);
			CHashPtr TileHash = std::make_shared<CHash>(TileHashValue, true);
			RawTiles[RowId][ColId] = std::make_shared<RawBuffer>(Tiles[RowId][ColId], ActualTileSize, ThisTileDesc, TileHash);
		}
	}
}

RawBufferPtr TextureHelper::CombineRaw_Tiles(const RawBufferPtrTiles& Tiles, CHashPtr HashValue /* = nullptr */, bool bIsTransient /* = false */)
{
	uint8* Data = nullptr;
	size_t DataLength = 0;

	try
	{
		check(Tiles.Rows() && Tiles.Cols());

		size_t NumRows = Tiles.Rows();
		size_t NumCols = Tiles.Cols();

		RawBufferPtr Tile0 = Tiles[0][0];
		BufferDescriptor TileDesc = Tile0->GetDescriptor();
		BufferDescriptor Desc = Tile0->GetDescriptor();

		Desc.Width = NumRows * TileDesc.Width;
		Desc.Height = NumCols * TileDesc.Height;
		Desc.bIsTransient = bIsTransient;

		size_t TotalTiles = NumRows * NumCols;

		/// This is just cecking that the tiles are the same dimensions
		for (size_t RowIndex = 0; RowIndex < NumRows; RowIndex++)
		{
			for (size_t ColIndex = 0; ColIndex < NumCols; ColIndex++)
			{
				RawBufferPtr Tile = Tiles[RowIndex][ColIndex];

				/// We can currently only combine homogeneous tiles together
				check(Tile->Width() == TileDesc.Width);
				check(Tile->Height() == TileDesc.Height);

				//desc.width += tile->Width();
				//desc.height += tile->Height();
			}
		}

		size_t TilePitch = TileDesc.Pitch();
		size_t DstPitch = Desc.Pitch();

		Data = new uint8 [DstPitch * Desc.Height];
		T_Tiles<CHashPtr> TileHashes(0, 0); 

		if (HashValue == nullptr || HashValue->IsNull())
			TileHashes.Resize(NumRows, NumCols);

		ParallelFor(TotalTiles, [&](int32 TileIndex)
		{
			int32 RowIndex = (int32)(TileIndex / NumRows);
			int32 ColIndex = TileIndex - (RowIndex * NumRows);
			RawBufferPtr Tile = Tiles[RowIndex][ColIndex];
			const uint8* SrcBase = Tile->GetData();
			int32 DstOffset = ColIndex * DstPitch * Tile->Height() + RowIndex * TilePitch;
			uint8* DstBase = Data + DstOffset;

			//UE_LOGF(LogData, Log, "Combining slice: [%d, %d] - [Tile Base: 0x%x] => %d ", xi, yi, srcBase, dstOffset);

			for (size_t TileRow = 0; TileRow < Tile->Height(); TileRow++)
				memcpy(DstBase + (TileRow * DstPitch), SrcBase + (TileRow * TilePitch), TilePitch);

			if (HashValue == nullptr || HashValue->IsNull())
			{
				CHashPtr TileHash = Tile->Hash();
				TileHashes[RowIndex][ColIndex] = TileHash;
			}
		});

		if (HashValue == nullptr || HashValue->IsNull())
			HashValue = CHash::ConstructFromSources(TileHashes.Tiles());

		DataLength = Desc.Size();

		return std::make_shared<RawBuffer>(Data, DataLength, Desc, HashValue);
	}
	catch (const std::exception& Ex)
	{
		delete[] Data;
		Data = nullptr;

		throw Ex;
	}
	catch (...)
	{
		/// Make sure we delete the data
		delete[] Data;
		Data = nullptr;
	}

	return nullptr;
}

void sRGBToLinear(uint8* Value, float BitDepth)
{
	bool IsHalf = BitDepth == 16;
	if (IsHalf)
	{
		uint16_t* HalfVal = (uint16_t*)Value;
		uint16_t LinearVal = std::pow(((*HalfVal + 0.055) / 1.055), 2.4); //enhanced formula taken from http://entropymine.com/imageworsener/srgbformula/
		*Value = LinearVal;
	}
	else
	{
		float* ValFloat = (float*)Value;
		float LinearVal = std::pow(((*ValFloat + 0.055) /1.055), 2.4); //enhanced formula taken from http://entropymine.com/imageworsener/srgbformula/
		*ValFloat = LinearVal;
	}
}

FLinearColor TextureHelper::GetPixelValueFromRaw(RawBufferPtr RawObj, int32 Width, int32 Height, int32 X, int32 Y)
{
	check(RawObj);
	const uint8* Data = RawObj->GetData();
	check(Data);

	auto PixelIndex = X + (Y * Width);
	
	return RawObj->GetAsLinearColor(PixelIndex);
}

AsyncBool TextureHelper::ExportRaw(RawBufferPtr RawObj, const FString& CompletePath)
{
	return cti::make_continuable<bool>([RawObj, CompletePath](auto Promise) mutable 
	{
		check((!TextureGraphEngine::IsDestroying()));
	
		FText PathError;
		FPaths::ValidatePath(CompletePath, &PathError);

		if (CompletePath.IsEmpty())
		{
			UE_LOGF(LogTexture, Warning, "Provide path is empty!");
			Promise.set_value(false);
		}
		else if (!PathError.IsEmpty())
		{
			UE_LOGF(LogTexture, Warning, "Invalid file path provided: %ls", *(PathError.ToString()));
			Promise.set_value(false);
		}

		bool bSuccess = false;

		
		if (RawObj->IsPadded())
		{
			const EPixelFormat PixelFormat = BufferDescriptor::BufferPixelFormat(RawObj->GetDescriptor().Format, RawObj->GetDescriptor().ItemsPerPoint);
			const int32 NumBlocksX = RawObj->Width() / GPixelFormats[PixelFormat].BlockSizeX;
			const int32 NumBlocksY = RawObj->Height() / GPixelFormats[PixelFormat].BlockSizeY;
			const uint32 DestStride = NumBlocksX * GPixelFormats[PixelFormat].BlockBytes;
			const int64 DestSize = DestStride * NumBlocksY;
			
			uint8* DstData = new uint8 [DestSize];
			RawObj->CopyUnpaddedBytes(DstData);

			// overwrite the RawBufferPtr to now use the newly created one without padding
			RawObj = std::make_shared<RawBuffer>(DstData, DestSize, RawObj->GetDescriptor());
		}
		
		const uint8* RawDataPtr = RawObj->GetData();
		if (RawDataPtr == nullptr)
			Promise.set_value(false);

		EPixelFormat PixelFormat = BufferDescriptor::BufferPixelFormat(RawObj->GetDescriptor().Format, RawObj->GetDescriptor().ItemsPerPoint);
		EImageFormat ImageFormat = EImageFormat::EXR;
		
		if (PixelFormat == PF_B8G8R8A8 || PixelFormat == PF_R8G8B8A8 || PixelFormat == PF_G8 || PixelFormat == PF_R8)
			ImageFormat = EImageFormat::PNG;

		bool bIsPng = ImageFormat == EImageFormat::PNG;

		FString OutputPath = CompletePath;
		if (FPaths::GetExtension(OutputPath).IsEmpty())
			OutputPath += bIsPng ? ".png" : ".exr";

		FArchive* FileArchive = IFileManager::Get().CreateFileWriter(*OutputPath);

		if (FileArchive)
		{
			IImageWrapperModule& ImageWrapperModule = FModuleManager::Get().LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
			
			ERGBFormat RGBFormat = ERGBFormat::BGRA;
			bool FlipGreenAndRed = false;
			bool ConvertToRGBA = false;
			bool ExtractOneComponentToGrayF = false;
			int32 ComponentSentToGrey = 0;
			int32 BitDepth = TextureHelper::GetBppFromPixelFormat(PixelFormat) / RawObj->GetDescriptor().ItemsPerPoint;

			// in the case we recognize metadata in the descriptor, enforce the export format
			if (RawObj->GetDescriptor().HasMetadata(RawBufferMetadataDefs::G_LAYER_MASK))
			{
				RGBFormat = ERGBFormat::GrayF;
				ExtractOneComponentToGrayF = true;
				ComponentSentToGrey = 1; // export Green component
			}
			// Else export the raw buffer matching the export file and format as close as possible to the pixel format
			else
			{

				if (PixelFormat == PF_B8G8R8A8)
				{
					const uint8 ClearColor = RawObj->GetDescriptor().DefaultValue.ToFColor(false).A;
					size_t Count = RawObj->GetLength() / 4;
					ParallelFor(Count, [&](size_t pixel)
						{
							uint8* PA = const_cast<uint8*>(RawDataPtr) + (pixel * 4) + 3;
							*PA = ClearColor;
						});
				}
				else if (RawObj->GetDescriptor().ItemsPerPoint == 1)
				{
					FlipGreenAndRed = true;

					if (PixelFormat == PF_R32_FLOAT)
						RGBFormat = ERGBFormat::GrayF;
					else
						RGBFormat = ERGBFormat::Gray;
				}
				else if (RawObj->GetDescriptor().ItemsPerPoint == 2)
				{
					ConvertToRGBA = true;
					RGBFormat = ERGBFormat::RGBAF;
				}
				else
				{
					RGBFormat = ERGBFormat::RGBAF;
				}
			}


			// If texels are sRGB and we are supposed to save in exr format then convert to linear color space
			if (RawObj->GetDescriptor().bIsSRGB && ImageFormat == EImageFormat::EXR)
			{
				// Adjustment for linear images
 				ParallelFor(RawObj->Width() * RawObj->Height(), [&](int32 pixel)
 				{
					int32 ItemsPerPoint = RawObj->GetDescriptor().ItemsPerPoint;
					uint8* Location = const_cast<uint8*>(RawDataPtr) + pixel * (BitDepth / 8) * ItemsPerPoint;
					uint8* Red = Location;

					sRGBToLinear(Red, BitDepth);

					if (ItemsPerPoint >= 2)
					{
						uint8* Green = Location + (1 * BitDepth / 8);
						sRGBToLinear(Green, BitDepth);
					}

					if (ItemsPerPoint >= 3)
					{
						uint8* Blue = Location + (2 * BitDepth / 8);
						sRGBToLinear(Blue, BitDepth);
					}
 				});
			}

			if (ExtractOneComponentToGrayF)
			{
				int32 ItemsPerPoint = RawObj->GetDescriptor().ItemsPerPoint;
				int32 ByteDepth = BitDepth / 8;
				uint64 NumPixels = (RawObj->GetLength() / (ItemsPerPoint * ByteDepth));
				const float* SrcData = (float*)RawDataPtr;
				
				int32 DestPixelByteSize = ByteDepth;
				float* Greys = new float[NumPixels];	//for 4 components
				float AlphaValue = 1.0;

				ParallelFor(NumPixels, [&](size_t PixelNum)
					{
						const float* SrcLocation = SrcData + PixelNum * (ItemsPerPoint);
						float srcV = SrcLocation[ComponentSentToGrey];
						Greys[PixelNum] = srcV;
					});

				RawDataPtr = (uint8*) Greys;
				RawObj = std::make_shared<RawBuffer>(RawDataPtr, NumPixels * ByteDepth, BufferDescriptor{ RawObj->Width(), RawObj->Height(), 1 }, nullptr, false);
			}

			if (ConvertToRGBA)
			{
				int32 ItemsPerPoint = RawObj->GetDescriptor().ItemsPerPoint;
				static constexpr int32 FullColorPoint = 4;
				uint64 DataSize = (RawObj->GetLength() / ItemsPerPoint) * FullColorPoint; //RGBA
				uint8* NewRaw = new uint8[DataSize];
				static constexpr float Zero = 0;
				static constexpr float One = 1;
				
				ParallelFor(RawObj->Width() * RawObj->Height(), [&](int32 Pixel)
 				{
					uint8* SrcLocation = const_cast<uint8*>(RawDataPtr) + Pixel * (BitDepth / 8) * ItemsPerPoint;
					uint8* SrcRed = SrcLocation;

					uint8* DstRed = NewRaw + Pixel * (BitDepth / 8) * FullColorPoint;
					FMemory::Memcpy(DstRed, SrcRed, BitDepth / 8);
						
					uint8* SrcGreen = nullptr;
					uint8* SrcBlue = nullptr;
					uint8* SrcAlpha = nullptr;
					uint8* DstGreen = DstRed + (1 * BitDepth / 8);
					uint8* DstBlue = DstRed + (2 * BitDepth / 8);
					uint8* DstAlpha = DstRed + (3 * BitDepth / 8);

					SrcGreen = SrcLocation + (1 * BitDepth / 8);
					FMemory::Memcpy(DstGreen, SrcGreen, BitDepth / 8);
						
					FMemory::Memcpy(DstBlue, &Zero, BitDepth / 8);
						
					FMemory::Memcpy(DstAlpha, &One, BitDepth / 8);
 				});

				RawObj = std::make_shared<RawBuffer>(NewRaw, DataSize, BufferDescriptor{RawObj->Width(), RawObj->Height(), 4}, nullptr, false );
				RawDataPtr = NewRaw;
			}

			TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(ImageFormat);

			ImageWrapper->SetRaw(RawDataPtr, RawObj->GetLength(), RawObj->Width(), RawObj->Height(), RGBFormat, BitDepth);

			const TArray64<uint8>& CompressedData = ImageWrapper->GetCompressed();
			
			if (FlipGreenAndRed)
			{
				*((char*)CompressedData.GetData() + 28) = 'R';
			}
				
			FileArchive->Serialize((void*)CompressedData.GetData(), CompressedData.GetAllocatedSize());
			
			bSuccess = true;
			delete FileArchive;
			FileArchive = nullptr;
		}

		check((!TextureGraphEngine::IsDestroying()));

		Promise.set_value(bSuccess);
	});
}

bool TextureHelper::CanSupportTexture(UTexture* Tex)
{
	bool CanSupport = false;
	if (Tex != nullptr)
	{
		UTexture2D* Texture2D = Cast<UTexture2D>(Tex);
		UTextureLightProfile* LightProfileTexture = Cast<UTextureLightProfile>(Tex);
		UCurveLinearColorAtlas* CurveAtlas = Cast<UCurveLinearColorAtlas>(Tex);

		if (Texture2D != nullptr && !LightProfileTexture && !CurveAtlas/*&& !Texture2D->VirtualTextureStreaming*/)
		{
			CanSupport = true;
		}
	}

	return CanSupport;
}

bool TextureHelper::CanSplitToTiles(int Width, int Height, int TilesX, int TilesY)
{
	bool bSplitToTiles = true;

	bool bSingleTile = (TilesX == 1 && TilesY == 1);
	//Height and width both must be big enough to support tiling
	bool bSizeNotBigEnough = (Width <= TilesX || Height <= TilesY);

	if (bSingleTile || bSizeNotBigEnough)
	{
		bSplitToTiles = false;
	}

	return bSplitToTiles;
}

bool TextureHelper::GetPixelFormatFromTextureSourceFormat(ETextureSourceFormat SourceFormat, EPixelFormat &OutPixelFormat, uint32 &OutNumChannels)
{
	switch (SourceFormat)
	{
	// Currently supported formats : 
	case ETextureSourceFormat::TSF_RGBA16F:
		OutPixelFormat = PF_FloatRGBA;
		OutNumChannels = 4;
		break;

	case ETextureSourceFormat::TSF_RGBA32F:
	case ETextureSourceFormat::TSF_RGBA16:
	case ETextureSourceFormat::TSF_BGRA8:
	case ETextureSourceFormat::TSF_BGRE8:
	case ETextureSourceFormat::TSF_R32F:
	case ETextureSourceFormat::TSF_R16F:
	case ETextureSourceFormat::TSF_G16:
	case ETextureSourceFormat::TSF_G8:
	// Commenting as 'TSF_RGBA8': Legacy ETextureSourceFormat not supported, 
	// use BGRA8 Please update your code to the new API before upgrading to the next release, otherwise your project will no longer compile.
	//case ETextureSourceFormat::TSF_RGBA8:
	//case ETextureSourceFormat::TSF_RGBE8:
		OutPixelFormat = GTextureSourceFormats[SourceFormat].PixelFormat;
		OutNumChannels = GTextureSourceFormats[SourceFormat].NumComponents;
		break;

	case ETextureSourceFormat::TSF_Invalid:
	default:
		return false;
	}

	return true;
}

ETextureSourceFormat TextureHelper::GetTextureSourceFormat(BufferFormat Format, uint32 ItemsPerPoint)
{
	ETextureSourceFormat SourceFormat = ETextureSourceFormat::TSF_Invalid;

	if (ItemsPerPoint == 1)
	{
		switch (Format)
		{
		case BufferFormat::Byte:
			return ETextureSourceFormat::TSF_G8;
		case BufferFormat::Float:
			return ETextureSourceFormat::TSF_R32F;
		case BufferFormat::Half:
			return ETextureSourceFormat::TSF_R16F;
		}
	}
	else if (ItemsPerPoint == 4)
	{
		switch (Format)
		{
		case BufferFormat::Byte:
			return ETextureSourceFormat::TSF_BGRA8;
		case BufferFormat::Float:
			return ETextureSourceFormat::TSF_RGBA32F;
		case BufferFormat::Half:
			return ETextureSourceFormat::TSF_RGBA16F;
		case BufferFormat::Short:
			return ETextureSourceFormat::TSF_RGBA16;
		}
	}

	return SourceFormat;
}
