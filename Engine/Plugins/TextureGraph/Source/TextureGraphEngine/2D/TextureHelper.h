// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "2D/Mask/MaskEnums.h"
#include "CoreMinimal.h"
#include "Data/RawBuffer.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Helper/Promise.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "PixelFormat.h" 
#include "TextureContent.h"
#include "TextureType.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Tex;
class UTextureRenderTarget2D;
class UTexture2D;
struct BufferDescriptor;
enum class ETG_TextureFormat : uint8;
enum class ETSBufferFormat;
class TiledBlob;
typedef std::shared_ptr<TiledBlob>		TiledBlobPtr;

typedef cti::continuable<bool>			AsyncBool;
typedef cti::continuable<FLinearColor>	AsyncLinearColor;

struct TextureHelper
{
	static UE_API TiledBlobPtr					GTransparent;			/// Transparent RGBA texture
	static UE_API TiledBlobPtr					GBlack;					/// Black RGBA texture
	static UE_API TiledBlobPtr					GWhite;					/// White RGBA texture
	static UE_API TiledBlobPtr					GGray;					/// Gray RGBA texture
	static UE_API TiledBlobPtr					GRed;					/// Red RGBA texture
	static UE_API TiledBlobPtr					GGreen;					/// Green RGBA texture
	static UE_API TiledBlobPtr					GBlue;					/// Blue RGBA texture
	static UE_API TiledBlobPtr					GYellow;				/// Yellow RGBA texture
	static UE_API TiledBlobPtr					GMagenta;				/// Magenta RGBA texture
	static UE_API TiledBlobPtr					GDefaultNormal;			/// Default Normal RGBA texture
	static UE_API TiledBlobPtr					GWhiteMask;				/// Default white mask
	static UE_API TiledBlobPtr					GBlackMask;				/// Default black mask

	static UE_API cti::continuable<bool>		InitSolidTexture(TiledBlobPtr* BlobObj, FLinearColor Color, FString Name, struct TexDescriptor Desc);
	static UE_API void							InitStockTextures();
	static UE_API void							FreeStockTextures();

	static FORCEINLINE TiledBlobPtr		GetTransparent() { check(GTransparent); return GTransparent; }
	static FORCEINLINE TiledBlobPtr		GetBlack() { check(GBlack); return GBlack; }
	static FORCEINLINE TiledBlobPtr		GetWhite() { check(GWhite); return GWhite; }
	static FORCEINLINE TiledBlobPtr		GetGray() { check(GGray);  return GGray; }
	static FORCEINLINE TiledBlobPtr		GetRed() { check(GRed);  return GRed; }
	static FORCEINLINE TiledBlobPtr		GetGreen() { check(GGreen);  return GGreen; }
	static FORCEINLINE TiledBlobPtr		GetBlue() { check(GBlue);  return GBlue; }
	static FORCEINLINE TiledBlobPtr		GetYellow() { check(GYellow);  return GYellow; }
	static FORCEINLINE TiledBlobPtr		GetMagenta() { check(GMagenta);  return GMagenta; }
	static FORCEINLINE TiledBlobPtr		GetErrorBlob() { return GetMagenta(); }
	static FORCEINLINE TiledBlobPtr		GetDefaultNormal() { check(GDefaultNormal); return GDefaultNormal; }
	static FORCEINLINE TiledBlobPtr		GetWhiteMask() { check(GWhiteMask); return GWhiteMask; }
	static FORCEINLINE TiledBlobPtr		GetBlackMask() { check(GBlackMask); return GBlackMask; }

	//////////////////////////////////////////////////////////////////////////
	static UE_API uint32						GetChannelsFromPixelFormat(EPixelFormat InPixelFormat);
	static UE_API uint32						GetBppFromPixelFormat(EPixelFormat InPixelFormat);
	static UE_API ETextureRenderTargetFormat	GetRenderTargetFormatFromPixelFormat(EPixelFormat InPixelFormat);
	static UE_API ETextureSourceFormat			GetSourceFormat(ETG_TextureFormat TGTextureFormat);
	static UE_API bool							GetBufferFormatAndChannelsFromTGTextureFormat(ETG_TextureFormat Format, BufferFormat& OutBufferFormat, uint32& OutBufferChannels);
	static UE_API EPixelFormat					GetPixelFormatFromRenderTargetFormat(ETextureRenderTargetFormat RTFormat);
	static UE_API bool							GetPixelFormatFromTextureSourceFormat(ETextureSourceFormat SourceFormat, EPixelFormat& OutPixelFormat, uint32& OutNumChannels);
	static UE_API ETextureSourceFormat			GetTextureSourceFormat(BufferFormat Format, uint32 ItemsPerPoint);
	static UE_API ETG_TextureFormat			GetTGTextureFormatFromChannelsAndFormat(uint32 ItemsPerPoint, BufferFormat Format);
	static UE_API uint32						GetNumChannelsFromTGTextureFormat(ETG_TextureFormat TextureFormat);
	static UE_API FString						GetChannelsTextFromItemsPerPoint(const int32 InItemsPerPoint);	
	static UE_API void							ClearRT(UTextureRenderTarget2D* RenderTarget, FLinearColor Color = FLinearColor::Transparent);
	static UE_API void							ClearRT(FRHICommandList& RHI, UTextureRenderTarget2D* RenderTarget, FLinearColor Color = FLinearColor::Transparent);

	static UE_API const char*					TextureTypeToString(TextureType Type);
	static UE_API TextureType					TextureStringToType(const FString& TypeString);
	static UE_API const char*					TextureTypeToMegascansType(TextureType Type);

	static UE_API TextureType					TextureContentToTextureType(TextureContent Content);
	static UE_API TextureContent				TextureTypeToTextureContent(TextureType Type);
	static UE_API MaskType						TextureContentToMaskType(TextureContent Content);
	static UE_API TextureContent				MaskTypeToTextureContent(MaskType Type);
	static UE_API MaskModifierType				TextureContentToMaskModifier(TextureContent Content);
	static UE_API TextureContent				MaskModifierToTextureContent(MaskModifierType Type);

	static UE_API RawBufferPtr					RawFromRT(UTextureRenderTarget2D* RenderTarget, const BufferDescriptor& Desc);
	static UE_API RawBufferPtr					RawFromTexture(UTexture2D* Texture, const BufferDescriptor& Desc);
	static UE_API RawBufferPtr					RawFromResource(const FTextureRHIRef& ResourceRHI, const BufferDescriptor& Desc);
	static UE_API BufferFormat					FindOptimalSupportedFormat(BufferFormat SrcFormat);

	static UE_API void							RawFromRT_Tiled(UTextureRenderTarget2D* RenderTarget, const BufferDescriptor& Desc, size_t TileSizeX, size_t TileSizeY, RawBufferPtrTiles& Tiles);
	static UE_API void							RawFromTexture_Tiled(UTexture2D* Texture, const BufferDescriptor& Desc, size_t TileSizeX, size_t TileSizeY, RawBufferPtrTiles& Tiles);
	static UE_API void							RawFromResource_Tiled(FTextureRHIRef ResourceRHI, const BufferDescriptor& Desc, size_t TileSizeX, size_t TileSizeY, RawBufferPtrTiles& Tiles);
	static UE_API void							RawFromMem_Tiled(const uint8* SrcData, size_t SrcDataLength, const BufferDescriptor& SrcDesc, size_t TileSizeX, size_t TileSizeY, RawBufferPtrTiles& Tiles);
	static UE_API RawBufferPtr					CombineRaw_Tiles(const RawBufferPtrTiles& Tiles, CHashPtr HashValue = nullptr, bool bIsTransient = false);

	static UE_API bool							IsFloatRT(UTextureRenderTarget2D* RenderTarget);
	static UE_API FLinearColor					GetPixelValueFromRaw(RawBufferPtr RawObj, int32 Width, int32 Height, int32 X, int32 Y);
	static UE_API AsyncBool					ExportRaw(RawBufferPtr RawObj, const FString& CompletePath);
	static UE_API bool							CanSupportTexture(UTexture* Tex);
	static UE_API bool							CanSplitToTiles(int Width, int Height, int TilesX, int TilesY);
	static UE_API size_t						RoundUpTo(size_t Size, size_t DesiredRounding);

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	static FORCEINLINE FString			CreateTileName(const FString& BaseName, int32 TileX, int32 TileY)
	{
		return FString::Printf(TEXT("%s-%d,%d"), *BaseName, TileX, TileY);
	}

	template <class T>
	static FORCEINLINE bool				IsPowerOf2(T Value)
	{
		return Value > 0 && (Value & (Value - 1)) == 0;
	}

	template <class T>
	static FORCEINLINE T				FloorToPowerOf2(T Value)
	{
		Value = Value | (Value >> 1);
		Value = Value | (Value >> 2);
		Value = Value | (Value >> 4);
		Value = Value | (Value >> 8);
		Value = Value | (Value >> 16);
		return Value - (Value >> 1);;
	}

	template <class T>
	static FORCEINLINE T				CeilToPowerOf2(T Value)
	{
		Value--;

		Value |= Value >> 1;
		Value |= Value >> 2;
		Value |= Value >> 4;
		Value |= Value >> 8;
		Value |= Value >> 16;
		Value++;

		return Value;
	}


	// Calculate the total number of mip levels for a given base size.
	// This is including the level 0 of said base size 
	static FORCEINLINE uint32			CalcNumMipLevels(uint32 BaseSize)
	{
		return (uint32)(ceil(log2(double(BaseSize))) + 1.0);
	}
};

#undef UE_API
