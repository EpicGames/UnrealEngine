// Copyright Epic Games, Inc. All Rights Reserved.
#include "TextureSet.h"
#include "Device/FX/Device_FX.h"
#include "Helper/Util.h"
#include "TextureGraphEngine.h"
#include "Model/Mix/Mix.h"
#include "TextureHelper.h"

#include "2D/Tex.h"
#include "Helper/ColorUtil.h"
#include "Model/Mix/MixUpdateCycle.h"
#include "Misc/ScopeLock.h"

/// Single process-wide lock serializing all access to the texture maps (TexturesMap here plus
/// BoundTexturesMap/Callbacks in TargetTextureSet). Those maps are mutated from the render
/// thread (SetTexture, via T_UpdateTargets) and the game thread (BindTo/FreeAt, from the
/// editor); without serialization a concurrent rehash/remove crashes the editor (UE-376270).
/// Kept as a function-local static rather than a class member so the fix is binary-compatible
/// for hotfix (no class layout change). FCriticalSection is recursive, so the derived overrides
/// may hold it across their bodies while calling the base methods below.
FCriticalSection& GetTextureSetMapCS()
{
	static FCriticalSection MapCS;
	return MapCS;
}

const BufferDescriptor TextureSet::GDesc[GMaxTextures] =
{
	/* Diffuse 		*/ BufferDescriptor(Util::GDefaultWidth, Util::GDefaultHeight, 4, BufferFormat::Byte, ColorUtil::DefaultColor(TextureType::Diffuse), BufferType::Image, false, true),
	/* Specular 	*/ BufferDescriptor(Util::GDefaultWidth, Util::GDefaultHeight, 4, BufferFormat::Byte, ColorUtil::DefaultColor(TextureType::Specular), BufferType::Image, false, true),
	/* Albedo 		*/ BufferDescriptor(Util::GDefaultWidth, Util::GDefaultHeight, 4, BufferFormat::Byte, ColorUtil::DefaultColor(TextureType::Albedo), BufferType::Image, false, true),
	/* Metalness 	*/ BufferDescriptor(Util::GDefaultWidth, Util::GDefaultHeight, 1, BufferFormat::Float, ColorUtil::DefaultColor(TextureType::Metalness), BufferType::Image),
	/* Normals 		*/ BufferDescriptor(Util::GDefaultWidth, Util::GDefaultHeight, 4, BufferFormat::Float, ColorUtil::DefaultColor(TextureType::Normal)),
	/* Displacement */ BufferDescriptor(Util::GDefaultWidth, Util::GDefaultHeight, 1, BufferFormat::Float, ColorUtil::DefaultColor(TextureType::Displacement), BufferType::Image, true), /// Displacement asset is mipmaped
	/* Opacity 		*/ BufferDescriptor(Util::GDefaultWidth, Util::GDefaultHeight, 1, BufferFormat::Float, ColorUtil::DefaultColor(TextureType::Opacity), BufferType::Image),
	/* Roughness 	*/ BufferDescriptor(Util::GDefaultWidth, Util::GDefaultHeight, 1, BufferFormat::Float, ColorUtil::DefaultColor(TextureType::Roughness)),
	/* AO 			*/ BufferDescriptor(Util::GDefaultWidth, Util::GDefaultHeight, 1, BufferFormat::Float, ColorUtil::DefaultColor(TextureType::AO)),
	/* Curvature 	*/ BufferDescriptor(Util::GDefaultWidth, Util::GDefaultHeight, 1, BufferFormat::Float, ColorUtil::DefaultColor(TextureType::Curvature)),
	/* Preview	 	*/ BufferDescriptor(Util::GDefaultWidth, Util::GDefaultHeight, 4, BufferFormat::Byte, ColorUtil::DefaultColor(TextureType::Preview),  BufferType::Image, false, true)
};

TextureSet::TextureSet(int32 InWidth, int32 InHeight) : Width(InWidth), Height(InHeight)
{

	for (int32 ti = 0; ti < GMaxTextures; ti++)
	{
		InitBufferDescriptor(ti);
	}
}

void TextureSet::InitBufferDescriptor(int32 ti)
{
	Desc[ti] = GDesc[ti];
	Desc[ti].Width = Width;
	Desc[ti].Height = Height;
	Desc[ti].DefaultValue = ColorUtil::DefaultColor((TextureType)ti);
	Desc[ti].Name = FString(TextureHelper::TextureTypeToString((TextureType)ti));
}

TSet<FName> TextureSet::GetTextureList() const
{
	FScopeLock Lock(&GetTextureSetMapCS());

	TSet<FName> TextureSet;

	TexturesMap.GetKeys(TextureSet);

	return TextureSet;
}

TextureSet::~TextureSet()
{
	/// Texture sets must be destroyed from the game thread
	check(IsInGameThread() || IsInRenderingThread());
	TextureSet::Free();
}

void TextureSet::Init()
{
	verify(IsInGameThread());

	// This is being used in multiple places in old Texture Graph code base.
	// For now, just removing the code.
	// We will do the refactor later.
	// TODO: We need to remove this function
}

void TextureSet::SetTexture(FName TextureName, TiledBlobRef Texture)
{
	check(Texture);

	/// Must either be in cache or a strong ref
	check(Texture.IsKeepStrong() || TextureGraphEngine::GetBlobber()->Find(Texture->Hash()->Value()));

	FScopeLock Lock(&GetTextureSetMapCS());

	if (TexturesMap.Contains(TextureName))
	{
		TexturesMap[TextureName] = Texture;
	}
	else
	{
		TexturesMap.Add(TextureName, Texture);
	}
}

bool TextureSet::RenameTexture(FName TextureName, FName NewName)
{
	/// Held across FreeAt/SetTexture (which re-lock) so the rename is atomic vs. concurrent access
	FScopeLock Lock(&GetTextureSetMapCS());

	if (TexturesMap.Contains(NewName))
		return false;

	if (TexturesMap.Contains(TextureName) && !TexturesMap.Contains(NewName))
	{
		TiledBlobRef Texture = TexturesMap[TextureName];
		FreeAt(TextureName);
		SetTexture(NewName, Texture);
	}

	return true;
}

void TextureSet::SetTexture(int32 TypeIndex, TiledBlobRef texture)
{
	// This is being used in multiple places in old Texture Graph code base.
	// For now, just removing the code.
	// We will do the refactor later.
	// TODO: We need to remove this function
	check(false);
}

void TextureSet::InitTex(int32 InTypeIndex)
{
	// This is being used in multiple places in old Texture Graph code base.
	// For now, just removing the code.
	// We will do the refactor later.
	// TODO: We need to remove this function
	check(false);
}

void TextureSet::InitTexFromFile(int32 InTypeIndex, const FString& FileName, int32 NumTilesX, int32 NumTilesY)
{
	// This is being used in multiple places in old Texture Graph code base.
	// For now, just removing the code.
	// We will do the refactor later.
	// TODO: We need to remove this function
	check(false);
}

void TextureSet::Invalidate(TextureType InType, bool bRecreate)
{
	check(false);
}

void TextureSet::Invalidate(FName TextureName, bool bRecreate)
{
	check(false);

	OwnedTexturesMap[TextureName] = TiledBlobRef();
	TexturesMap[TextureName] = TiledBlobRef();

	if (bRecreate)
	{
		// InitBufferDescriptor(itype);
		// InitTex(itype);
	}
}

void TextureSet::ChangeFormat(TextureType InType, EPixelFormat InPixelFormat, bool bRecreate)
{
	/// TODO
	check(false);
	// #ifdef TODO
	// 	int itype = (int)type;
	//
	// 	/// If we have a texture already that's in the correct format, then we don't do anything
	// 	if (_textures[itype] && _desc[itype].format == format)
	// 		return;
	//
	// 	_desc[itype].format = format;
	// 	_textures[itype] = TiledBlobPtrW();
	// 	InitTex(itype);
	// #endif 
}

void TextureSet::ChangeResolution(int32 InWidth, int32 InHeight, bool bRecreate)
{
	/// TODO
	check(false);
	// #ifdef TODO
	// 	if (_width == width && _height == height)
	// 		return;
	//
	// 	_width = width;
	// 	_height = height;
	//
	// 	for (int ti = 0; ti < s_maxTextures; ti++)
	// 	{
	// 		_desc[ti].width = width;
	// 		_desc[ti].height = height;
	//
	// 		if (recreate)
	// 			InitTex(ti);
	// 	}
	// #endif
}

void TextureSet::Free()
{
	FScopeLock Lock(&GetTextureSetMapCS());

	for (auto TextureEntry : TexturesMap)
	{
		TexturesMap[TextureEntry.Key] = TiledBlobRef();
	}

	TexturesMap.Empty();
}

void TextureSet::FreeAt(FName TextureName)
{
	FScopeLock Lock(&GetTextureSetMapCS());

	if (TexturesMap.Contains(TextureName))
	{
		TexturesMap.Remove(TextureName);
	}
}

void TextureSet::FreeAt(TextureType type)
{
	check(false);
}

void TextureSet::Update(MixUpdateCyclePtr cycle, int32 targetId)
{
	/// No-op
}
