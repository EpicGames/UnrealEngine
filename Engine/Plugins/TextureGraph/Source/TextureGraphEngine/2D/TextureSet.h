// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "TextureType.h"
#include "Data/RawBuffer.h"
#include "Data/Blobber.h"

#define UE_API TEXTUREGRAPHENGINE_API

class MixUpdateCycle;
typedef std::shared_ptr<MixUpdateCycle> MixUpdateCyclePtr;

class TextureSet
{
public:
	static constexpr int			GMaxTextures = (int)TextureType::Count;
	static UE_API const BufferDescriptor	GDesc[GMaxTextures];		/// The default descriptors for the textures
	static constexpr int32			GDefaultTexSize = 256;		/// Default texture size
	
protected:
	TMap<FName, TiledBlobRef>		OwnedTexturesMap; 	/// The textures within this texture set that we own a reference of

public:
	int32							Width = 0;					/// Width of this target in pixels
	int32							Height = 0;					/// Height of this target in pixels

	TMap<FName,TiledBlobRef>		TexturesMap;				/// The textures within this texture set that we don't own a reference of (could be one of the owned textures)
	BufferDescriptor				Desc[GMaxTextures];			/// Descriptors for all the blobs
	
	UE_API virtual void					InitTex(int32 InTypeIndex);
	UE_API virtual void					InitTexFromFile(int32 InTypeIndex, const FString& FileName, int32 NumTilesX, int32 NumTilesY);
	UE_API void							InitBufferDescriptor(int32 ti);
	UE_API TSet<FName>						GetTextureList() const;

public:
									UE_API TextureSet(int32 InWidth, int32 InHeight);


	UE_API virtual							~TextureSet();

	UE_API virtual void					Init();
	UE_API virtual void					Free();
	UE_API virtual void					FreeAt(FName TextureName);
	UE_API void							FreeAt(TextureType type);

	UE_API virtual void					ChangeFormat(TextureType InType, EPixelFormat InPixelFormat, bool bRecreate);
	UE_API virtual void					ChangeResolution(int32 InWidth, int32 InHeight, bool bRecreate);
	UE_API virtual void					Invalidate(FName TextureName, bool bRecreate);
	UE_API virtual void					Invalidate(TextureType InType, bool bRecreate);

	UE_API virtual void					SetTexture(FName TextureName, TiledBlobRef Texture);
	UE_API virtual bool					RenameTexture(FName TextureName, FName NewName);
	UE_API virtual void					SetTexture(int32 TypeIndex, TiledBlobRef texture);
	
	UE_API virtual void					Update(MixUpdateCyclePtr cycle, int32 targetId);

	
	FORCEINLINE bool				ContainsTexture(FName TextureName) { return TexturesMap.Contains(TextureName); }
	FORCEINLINE TiledBlobRef		GetTexture(FName TextureName) const { return TexturesMap.Contains(TextureName) ? TexturesMap[TextureName] : TiledBlobRef(); }
	FORCEINLINE TiledBlobRef		GetOwnedTexture(FName TextureName) const { return OwnedTexturesMap.Contains(TextureName)? OwnedTexturesMap[TextureName] : TiledBlobRef(); }
	FORCEINLINE TiledBlobRef		GetTexture(TextureType type) const { return TiledBlobRef(); }
	FORCEINLINE TiledBlobRef		GetTexture(int32 TypeIndex) const { return TiledBlobRef(); }
	FORCEINLINE TiledBlobRef		GetOwnedTexture(int32 TypeIndex) const { return TiledBlobRef(); }
	//TextureSet&						operator = (const TextureSet& rhs);
	FORCEINLINE TiledBlobRef		CheckAndGetTexture(FName TextureName) const { check(!TexturesMap.Contains(TextureName)); return TexturesMap[TextureName]; }

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////

	// TODO: Need to remove these getters for PBR specific textures
	// There are multiple references of these getters in the code. To avoid major refactoring.
	// For now just passing the default tiled reference.
    FORCEINLINE TiledBlobRef		GetDiffuse() const { return TiledBlobRef(); }
	FORCEINLINE TiledBlobRef		GetSpecular() const { return TiledBlobRef(); }
	FORCEINLINE TiledBlobRef		GetAlbedo() const { return TiledBlobRef(); }
	FORCEINLINE TiledBlobRef		GetMetalness() const { return TiledBlobRef(); }
	FORCEINLINE TiledBlobRef		GetNormal() const { return TiledBlobRef(); }
	FORCEINLINE TiledBlobRef		GetDisplacement() const { return TiledBlobRef(); }
	FORCEINLINE TiledBlobRef		GetOpacity() const { return TiledBlobRef(); }	
	FORCEINLINE TiledBlobRef		GetRoughness() const { return TiledBlobRef(); }
	FORCEINLINE TiledBlobRef		GetAO() const { return TiledBlobRef(); }

	FORCEINLINE bool				IsValid(FName TextureName) const { return GetTexture(TextureName) != nullptr ? true : false; }
	// FORCEINLINE TiledBlobRef const* Textures() const { return _texturesMap; }

	FORCEINLINE int32				GetWidth() const { return Width; }
	FORCEINLINE int32				GetHeight() const { return Height; }
};

typedef std::unique_ptr<TextureSet>	TextureSetPtr;
typedef std::vector<TextureSet>		TextureSetVec;

#undef UE_API
