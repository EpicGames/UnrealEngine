// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "TextureType.generated.h"

class Tex;

UENUM()
enum class TextureType
{
	Diffuse,
	Specular,
	Albedo,			/// We always use the diffuse texture for albedo (see the duplicate enum below)
	Metalness,
	Normal,
	Displacement,
	Opacity,
	Roughness,
	AO,
	Curvature,
	Preview,		/// We use this for thumbnails as well

	/// Always has to be the last
	Count,

	Unknown = -1
};


UENUM()
enum class LayerBlendMode : uint8
{
	Normal,
	Add,
	Subtract,
	Multiply,
	Divide,
	Difference,
	Max,
	Min,
	Step,
	Overlay,
	Distort,
};