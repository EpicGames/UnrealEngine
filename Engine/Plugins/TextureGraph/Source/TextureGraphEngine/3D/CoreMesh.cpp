// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMesh.h"
#include <memory>
#include "PipelineStateCache.h"

TEXTUREGRAPHENGINE_API TGlobalResource<CoreMeshVertex_Decl> g_coreMeshVertexDecl;

void CoreMeshVertex_Decl::InitRHI(FRHICommandListBase& RHICmdList) 
{
	FVertexDeclarationElementList elements;
	uint16 stride = sizeof(CoreMeshVertex);

	elements.Add(FVertexElement(0, STRUCT_OFFSET(CoreMeshVertex, position), VET_Float3, 0, stride));
	elements.Add(FVertexElement(0, STRUCT_OFFSET(CoreMeshVertex, uv), VET_Float2, 1, stride));
	elements.Add(FVertexElement(0, STRUCT_OFFSET(CoreMeshVertex, normal), VET_Float3, 2, stride));
	elements.Add(FVertexElement(0, STRUCT_OFFSET(CoreMeshVertex, tangent), VET_Float4, 3, stride));
	elements.Add(FVertexElement(0, STRUCT_OFFSET(CoreMeshVertex, color), VET_Color, 4, stride));

	_rhiDecl = PipelineStateCache::GetOrCreateVertexDeclaration(elements);
}

void CoreMeshVertex_Decl::ReleaseRHI() 
{
	_rhiDecl.SafeRelease();
}
