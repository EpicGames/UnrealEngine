// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ProceduralMeshComponent.h" 
#include "Math/Box.h" 
#include "GraphicsDefs.h"
#include <memory>
#include "RenderResource.h"
#include "RHIFwd.h"
#include "RHI.h"
#include "RHIResources.h"
#include "RHIUtilities.h"

//////////////////////////////////////////////////////////////////////////
struct CoreMeshVertex
{
	FVector3f					position;			/// The vertex position
	FVector2f					uv;					/// The texture coordinates
	FVector3f					normal;				/// Per-vertex normal
	FVector4f					tangent;			/// Tangent with W
	FVector4f					color;				/// Per vertex colour
};

struct CoreMesh
{
	TArray<FVector>				vertices;
	TArray<FVector>				normals;
	TArray<FVector2D>			uvs;
	TArray<Tangent>				tangents;
	TArray<FLinearColor>		vertexColors;

	//TArray<CoreMeshVertex>		vertices;			/// The vertices for this mesh. Available as a single array uploadable to the GPU

	TArray<int32>				triangles;			/// The triangles for this mesh
	FBox						bounds;				/// The bounds for this mesh
	FString						name;				/// The name of the mesh
	int32						materialIndex;		/// Material index that is used for UDIM/multi-material meshes
};

//////////////////////////////////////////////////////////////////////////
class CoreMeshVertex_Decl : public FRenderResource
{
	FVertexDeclarationRHIRef	_rhiDecl;			/// Declaration RHI

public:
	virtual						~CoreMeshVertex_Decl() override = default;

	virtual void				InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void				ReleaseRHI() override;

	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE FVertexDeclarationRHIRef RHI_Decl() const { return _rhiDecl; }
};

extern TEXTUREGRAPHENGINE_API TGlobalResource<CoreMeshVertex_Decl> g_coreMeshVertexDecl;

typedef std::shared_ptr<CoreMesh>	CoreMeshPtr;
