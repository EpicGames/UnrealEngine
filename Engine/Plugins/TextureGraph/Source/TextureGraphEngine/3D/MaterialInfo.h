// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include <memory>
/**
 * 
 */
struct MaterialInfo
{
	FString							name;
	int32							id;
};

typedef std::shared_ptr<MaterialInfo>		MaterialInfoPtr;

