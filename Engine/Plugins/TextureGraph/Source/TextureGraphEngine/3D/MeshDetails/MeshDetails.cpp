// Copyright Epic Games, Inc. All Rights Reserved.
#include "MeshDetails.h"
#include "3D/MeshInfo.h"

MeshDetails::MeshDetails(MeshInfo* mesh) : Mesh(mesh)
{
}

MeshDetails::~MeshDetails()
{
}

void MeshDetails::CalculateTri(size_t ti)
{
}

void MeshDetails::CalculateVertex(size_t vi)
{
}

MeshDetailsPAsync MeshDetails::Calculate()
{
	for (size_t ti = 0; ti < Mesh->NumTriangles(); ti++)
		CalculateTri(ti);

	for (size_t vi = 0; vi < Mesh->NumVertices(); vi++)
		CalculateVertex(vi);

	return cti::make_ready_continuable(this);
}

MeshDetailsPAsync MeshDetails::Finalise()
{
	bIsFinalised =true;
	return cti::make_ready_continuable(this);
}

void MeshDetails::RenderDebug()
{
}

void MeshDetails::Release()
{
}
