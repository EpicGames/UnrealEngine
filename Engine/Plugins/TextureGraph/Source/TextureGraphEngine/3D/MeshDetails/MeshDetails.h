// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h" 
#include <memory>
#include "Helper/Promise.h"
#include "Helper/DataUtil.h"
#include <array>

#define UE_API TEXTUREGRAPHENGINE_API

class MeshInfo;
class MeshDetails;
typedef cti::continuable<MeshDetails*>	MeshDetailsPAsync;

class RawBuffer;
typedef std::shared_ptr<RawBuffer>		RawBufferPtr;

//////////////////////////////////////////////////////////////////////////
class MeshDetails
{
protected:
	MeshInfo*					Mesh;					/// The mesh that is containing this detail
	bool						bIsFinalised = false;		/// Whether this detail has been finalised or not
	CHashPtr					HashValue;					/// What is the hash of this detail

	UE_API virtual void				CalculateTri(size_t ti);
	UE_API virtual void				CalculateVertex(size_t vi);

public:
								UE_API MeshDetails(MeshInfo* mesh);
	UE_API virtual						~MeshDetails();

	UE_API virtual MeshDetailsPAsync	Calculate();
	UE_API virtual MeshDetailsPAsync	Finalise();
	UE_API virtual void				RenderDebug();
	UE_API virtual void				Release();
};

typedef std::shared_ptr<MeshDetails> MeshDetailsPtr;

#undef UE_API
