// Copyright Epic Games, Inc. All Rights Reserved.
#include "MeshDetails_Edges.h"
#include "3D/MeshInfo.h"
#include "3D/CoreMesh.h"
#include "Helper/GraphicsUtil.h"
#include "Helper/Util.h"
#include "Data/RawBuffer.h"
#include <cmath>

MeshDetails_Edges::Edge::Edge(int32 i_v0, int32 i_v1, int32 ti)
{
	this->i_v0 = i_v0;
	this->i_v1 = i_v1;
	this->ti = ti;

	if (numCongruent < MeshDetails_Edges::s_maxCongruent)
		triangles[numCongruent++] = ti;
}

//////////////////////////////////////////////////////////////////////////
MeshDetails_Edges::MeshDetails_Edges(MeshInfo* mesh) : MeshDetails(mesh)
{
}

MeshDetails_Edges::~MeshDetails_Edges()
{
}

void MeshDetails_Edges::Release()
{
	_edges.clear();
	_vertexEdges.clear();
	_disconnected.clear();
	_vertexTriangles.clear();

	delete[] _congruent;
	_congruent = nullptr;
}

void MeshDetails_Edges::CalculateTri(size_t ti)
{
}

MeshDetailsPAsync MeshDetails_Edges::Calculate()
{
	return cti::make_ready_continuable(this);
}
