// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MeshDetails.h"
#include "GraphicsDefs.h"
#include <vector>
#include <list>
#include <unordered_map>

#define UE_API TEXTUREGRAPHENGINE_API

class MeshDetails_Edges : public MeshDetails
{
public:
	static const int			s_maxCongruent = 8;

	struct Edge
	{
		int32          			i_v0;                   /// Index of the first vertex
		int32          			i_v1;                   /// Index of the second vertex
		int32          			ti;                     /// Triangle index

		int32    				triangles[MeshDetails_Edges::s_maxCongruent];	/// Triangles that share this edge
		int32					numCongruent = 0;		/// Number of valid values in congruent

								Edge(int32 i_v0, int32 i_v1, int32 ti);
	};


protected:
	typedef std::unordered_map<long, Edge*> EdgeLUT;
	typedef std::vector<std::list<Edge*> >	VecListsEdge;
	typedef std::vector<std::list<int32> >	VecListsTri;
	typedef std::vector<int32>				IntVec;

    EdgeLUT 					_edges;        			/// All the edges within the mesh, one for each vertex
	VecListsEdge  				_vertexEdges;           /// All the edges for a vertex
	VecListsEdge   				_disconnected;          /// Edges that are disconnected 
	VecListsTri   				_vertexTriangles;       /// The triangles that share a particular vertex
    IntVec 						_numCongruent;          /// How many congruent vertices
    int32*        				_congruent = nullptr;   /// Adjacency/Congruency information for disconnected vertices 2D array

    std::list<int32>     		_adjTriangles;          /// Indices of adjacent triangles
    IntVec         				_adjStart;              /// Adjacency start index into the _adjTriangles array (for each triangle)
    IntVec         				_adjCount;              /// Number of adjacent triangles in the _adjTriangles array (for each triangle)

	UE_API virtual void				CalculateTri(size_t ti) override;

public:
								UE_API MeshDetails_Edges(MeshInfo* mesh);
	UE_API virtual						~MeshDetails_Edges();

	UE_API virtual MeshDetailsPAsync	Calculate() override;
	UE_API virtual void				Release() override;
};

#undef UE_API
