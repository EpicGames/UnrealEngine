// Copyright Epic Games, Inc. All Rights Reserved.
#include "MeshDetails_Spatial.h"
#include "3D/MeshInfo.h"
#include "3D/CoreMesh.h"
#include "Helper/GraphicsUtil.h"
#include "Helper/Util.h"
#include "Data/RawBuffer.h"
#include "3D/MeshInfo.h"
#include "MeshDetails_Tri.h"
#include <cmath>

//////////////////////////////////////////////////////////////////////////
MeshDetails_Spatial::ONode::ONode(int parentNodeId)
{
	Level = 0;
	ChildIndex = 0;
	LevelIndex = 0;
	ParentNodeId = parentNodeId;
	NodeId = 0;
	TriStart = 0;
	TriCount = 0;
	HasChildren = 0;

	memset(Children, 0, sizeof(Children));
}

//////////////////////////////////////////////////////////////////////////
void MeshDetails_Spatial::Node::RenderDebug() const
{
	auto clr = FColor::Red;
	auto clrId = _level % ONode::s_maxChildren;

	switch (_childIndex)
	{
	case 0: clr = FColor::Red; break;
	case 1: clr = FColor::Green; break;
	case 2: clr = FColor::Blue; break;
	case 3: clr = FColor::Yellow; break;
	case 4: clr = FColor::Magenta; break;
	case 5: clr = FColor::Cyan; break;
	case 6: clr = FColor::White; break;
	case 7: clr = FColor(128, 128, 128); break;
	default: break;
	}

	if (_level > 1)
		return;

	FBox tmp(ForceInit);

	/// draw the bounds slightly inwards
	tmp.Min = _bounds.Min + Vector3(0.01f, 0.01f, 0.01f);
	tmp.Max = _bounds.Max - Vector3(0.01f, 0.01f, 0.01f);

	/// 
	//DebugExtension.DrawBounds(tmp, clr);

	for (auto child : _children)
	{
		if (child)
			child->RenderDebug();
	}
}

MeshDetails_Spatial::Node* MeshDetails_Spatial::Node::CreateChild(int ci, int li, int ni)
{
	FBox bounds = GraphicsUtil::GetChildBounds_Octree(_bounds, (BoundsCorners)ci);
	Node* child = new Node(_details, this, bounds, ni);
	child->_level = _level + 1;
	child->_triangles = new std::vector<int>();
	child->_childIndex = ci;
	child->_levelIndex = li;
	return child;
}

int MeshDetails_Spatial::Node::FindTriangle(MeshInfo& mesh, const Vector3& pos, Vector2& bcCoord)
{
	if (HasChildren())
	{
		/// Check all the children
		for (int ci = 0; ci < (int)_children.size(); ci++)
		{
			/// Since only one box can contain a point and it can't span across two bounds
			if (_children[ci]->Bounds().IsInside(pos))
				return _children[ci]->FindTriangle(mesh, pos, bcCoord);
		}

		return -1;
	}

	/// OK, check against the triangles
	for (int tii = 0; tii < _triangles->size(); tii++)
	{
		int ti = _triangles->at(tii);

		/// Check whether the triangle bounds actually contain the point
		if (mesh.d_Tri()->Triangles()[ti].bounds.IsInside(pos))
		{
			auto v = mesh.d_Tri()->GetVertices(ti);

			/// If yes, then we do the more detailed check
			auto barycentricCoordinates = GraphicsUtil::CalculateBarycentricCoordinates(v[0], v[1], v[2], pos);

			if (GraphicsUtil::IsPointInTriangle(barycentricCoordinates))
			{
				bcCoord = barycentricCoordinates;
				return ti;
			}
		}
	}

	return -1;
}

bool MeshDetails_Spatial::Node::Divide(MeshDetails_Tri* d_tri, Node* triNodes)
{
	int maxLevel;

	/// This needs to be before the early exit
	{
		FScopeLock lock(&_mutex);
		maxLevel = std::max<int>(_details->_maxLevel, _level);
	}

	/// No need to further sub-divide this node
	if (_triangles->size() < s_maxTriangles || _level >= s_maxLevel)
		return false;

	/// Create all the children for this
	for (int ci = 0; ci < (int)_children.size(); ci++)
	{
		int nodeId;
		{
			FScopeLock lock(&_mutex);
			++_details->_nodeId;
			nodeId = _details->_nodeId;
		}
		_children[ci] = CreateChild(ci, _levelIndex * 8 + ci, nodeId);
	}

	/// TODO: Refactor
	//ParallelFor((int)_triangles->size(), [this, d_tri](int ti) 
	//{ 
	//	const MeshDetails_Tri::Triangle& tri = d_tri->Triangles()[_triangles[ti]];
	//	bool doesContainFully = false;
	//	bool didAdd = false;
	//	auto vertices = d_tri->GetVertices(ti);

	//	for (int ci = 0; ci < (int)_children.size() && !doesContainFully; ci++)
	//	{
	//		auto bounds = _children[ci]->_bounds;

	//		/// Check whether the bounds encapsulates the centre of the triangle
	//		if (bounds.Intersect(tri.bounds))
	//		{
	//			/// TODO: Refactor
	//			//lock(_children[ci]._triangles)
	//			//	_children[ci]._triangles.Add(_triangles[ti]);

	//			didAdd = true;

	//			bool b0 = bounds.IsInside(vertices[0]);
	//			bool b1 = bounds.IsInside(vertices[1]);
	//			bool b2 = bounds.IsInside(vertices[2]);

	//			/// Check whether it contains the triangle fully, if yes then set the flag so that it doesn't 
	//			/// get added to any other triangle
	//			doesContainFully = b0 && b1 && b2;
	//		}
	//	}

	//	if (!didAdd)
	//	{
	//		//UE_LOGF(LogMesh, Log, "Triangle not added to any of the child nodes: %d", ti));
	//	}
	//});

	//List<Task<bool>> divTasks = new List<Task<bool>>();

	///// Eventually, we want to add non-empty children and sub-divide them further
	//Parallel.For(0, _children.Length, ci = >
	//{
	//	/// Only going to add non-empty children
	//	if (_children[ci]._triangles.Count > 0)
	//	{
	//		/// Divide this child
	//		_children[ci].Divide(d_tri, triNodes);
	//	}
	//});

	return true;
}

bool MeshDetails_Spatial::Node::CheckIntersection(const FBox& nodeBounds, FRay ray, float& distance, float length /* = -1 */, FBox* bounds /* = nullptr */)
{
	distance = -1.0f;

	/// Trivially reject
	if (bounds != nullptr && !nodeBounds.Intersect(*bounds))
		return false;

	/// Then do the detailed check
	/// TODO: 
	//if (nodeBounds.IntersectRay(ray, out distance))
	//{
	//	/// A max length was given and the intersection distance was greater than 
	//	if (length > 0.0f && distance > length)
	//		return false;

	//	return true;
	//}

	return false;
}

std::list<MeshDetails_Spatial::Node*> MeshDetails_Spatial::Node::GetIntersectingLeafNodes(std::list<Node> nodes, FRay ray, float* distance, 
	float length /* = -1 */, FBox* bounds /* = nullptr */, std::list<Node *>* ignoreLeafs /* = nullptr */)
{
	/// TODO:
	//if (CheckIntersection(_bounds, ray, out distance, length, bounds))
	//{
	//	if (HasChildren)
	//	{
	//		foreach(auto child in _children)
	//			child.GetIntersectingLeafNodes(nodes, ray, out distance, length, bounds, ignoreLeafs);
	//	}
	//	else
	//	{
	//		/// This is a leaf node
	//		if (ignoreLeafs == null || !ignoreLeafs.Contains(this))
	//			nodes.Add(this);
	//		return nodes;
	//	}
	//}

	//return nodes;

	return std::list<Node*>();
}

//void FindCongruentVertices(MeshInfo mesh, Vector3 v, int i, List<int> congruent)
//{
//	/// If this doesn't contain the vertex, then just return immediately
//	//if (!_bounds.Contains(v))
//	//    return -1;
//
//	/// Non-leaf node
//	if (_children != null && _children.Length > 0)
//	{
//		for (int ci = 0; ci < _children.Length; ci++)
//		{
//			if (_children[ci]._bounds.Contains(v))
//				_children[ci].FindCongruentVertices(mesh, v, i, congruent);
//		}
//
//		return;
//	}
//
//	/// This is a leaf node, we'd like to check out the triangles
//	for (int t = 0; t < _triangles.Count; t++)
//	{
//		int ti = _triangles[t];
//		int[] indices = mesh.d_Tri.GetIndices(ti);
//
//		/// Allow compiler to unroll this loop
//		for (int index = 0; index < 3; index++)
//		{
//			int vertexIndex = indices[index];
//			if (vertexIndex != i)
//			{
//				Vector3 vertex = mesh.Vertices[vertexIndex];
//				float d = (vertex - v).sqrMagnitude;
//				if (d < Mathf.Epsilon && !congruent.Contains(vertexIndex))
//					congruent.Add(vertexIndex);
//			}
//		}
//	}
//}
//
//int FindCongruentVertex(MeshInfo mesh, Vector3 v, int i)
//{
//	/// If this doesn't contain the vertex, then just return immediately
//	//if (!_bounds.Contains(v))
//	//    return -1;
//
//	/// Non-leaf node
//	if (_children != null && _children.Length > 0)
//	{
//		for (int ci = 0; ci < _children.Length; ci++)
//		{
//			if (_children[ci]._bounds.Contains(v))
//			{
//				int i_congruent = _children[ci].FindCongruentVertex(mesh, v, i);
//
//				/// We have found a vertex
//				if (i_congruent >= 0)
//					return i_congruent;
//			}
//		}
//
//		return -1;
//	}
//
//	/// This is a leaf node, we'd like to check out the triangles
//	for (int t = 0; t < _triangles.Count; t++)
//	{
//		int ti = _triangles[t];
//		int[] indices = mesh.d_Tri.GetIndices(ti);
//
//		/// Allow compiler to unroll this loop
//		for (int index = 0; index < 3; index++)
//		{
//			int vertexIndex = indices[index];
//			if (vertexIndex != i)
//			{
//				Vector3 vertex = mesh.Vertices[vertexIndex];
//				float d = (vertex - v).sqrMagnitude;
//				if (d < Mathf.Epsilon)
//					return vertexIndex;
//			}
//		}
//	}
//
//	return -1;
//}
//
//void GetAllNodes(ref ONode[] allNodes, List<int> tris, ref int triIndex)
//{
//	Debug.Assert(allNodes.Length > _nodeId, string.Format("Invalid all nodes size: {0} [ < {1}]", allNodes.Length, _nodeId));
//
//	allNodes[_nodeId] = new ONode()
//	{
//		Bounds = new AABB(_bounds),
//		ParentNodeId = _parent != null ? _parent._nodeId : -1,
//		Level = _level,
//		ChildIndex = _childIndex,
//		LevelIndex = _levelIndex,
//		NodeId = _nodeId,
//		TriStart = 0,
//		TriCount = 0,
//		HasChildren = HasChildren ? 1 : 0,
//	};
//
//	if (HasChildren)
//	{
//		for (int i = 0; i < _children.Length; i++)
//		{
//			switch (i)
//			{
//			case 0: allNodes[_nodeId].Child_0 = _children[0]._nodeId; break;
//			case 1: allNodes[_nodeId].Child_1 = _children[1]._nodeId; break;
//			case 2: allNodes[_nodeId].Child_2 = _children[2]._nodeId; break;
//			case 3: allNodes[_nodeId].Child_3 = _children[3]._nodeId; break;
//			case 4: allNodes[_nodeId].Child_4 = _children[4]._nodeId; break;
//			case 5: allNodes[_nodeId].Child_5 = _children[5]._nodeId; break;
//			case 6: allNodes[_nodeId].Child_6 = _children[6]._nodeId; break;
//			case 7: allNodes[_nodeId].Child_7 = _children[7]._nodeId; break;
//			}
//
//			// allNodes[_nodeId].Children[i] = _children[i]._nodeId;
//			_children[i].GetAllNodes(ref allNodes, tris, ref triIndex);
//		}
//	}
//	else
//	{
//		allNodes[_nodeId].TriStart = tris.Count; ///triIndex;
//		allNodes[_nodeId].TriCount = _triangles.Count;
//
//		for (int t = 0; t < _triangles.Count; t++)
//		{
//			//tris[triIndex++] = _triangles[t];
//			tris.Add(_triangles[t]);
//		}
//	}
//}
//
//List<Node> GetIntersectingLeafNodes(List<Node> nodes, Hemisphere hem)
//{
//	if (hem.CheckIntersection(_bounds))
//	{
//		if (HasChildren)
//		{
//			foreach(auto child in _children)
//				child.GetIntersectingLeafNodes(nodes, hem);
//		}
//		else
//		{
//			/// This is a leaf node
//			nodes.Add(this);
//			return nodes;
//		}
//	}
//
//	return nodes;
//}
//
//List<Node> GetIntersectingLeafNodes(List<Node> nodes, Bounds bounds)
//{
//	if (bounds.Intersects(_bounds))
//	{
//		if (HasChildren)
//		{
//			foreach(auto child in _children)
//				child.GetIntersectingLeafNodes(nodes, bounds);
//		}
//		else
//		{
//			/// This is a leaf node
//			nodes.Add(this);
//			return nodes;
//		}
//	}
//
//	return nodes;
//}
//
//void ClearNonLeafNodes(List<Node> leafs)
//{
//	if (HasChildren)
//	{
//		_triangles.Clear();
//		_triangles = null;
//
//		foreach(auto child in _children)
//			child ? .ClearNonLeafNodes(leafs);
//	}
//	else
//	{
//		/// This is a leaf, we add it to the leaf nodes
//		leafs.Add(this);
//	}
//}
//
//void Release()
//{
//	_triangles ? .Clear();
//
//	if (_children != null)
//	{
//		foreach(auto child in _children)
//			child.Release();
//
//		_children = null;
//	}
//}
//
//void EncodeOctree(OctreeEncode enc)
//{
//	/// Encode leaf information
//	if (_children == null)
//	{
//		int startIndex = enc.AddTris(_triangles, _levelIndex);
//
//		/// See how many triangles does it have
//		enc.Levels[_level].AddInfo(_level, _levelIndex, _childIndex, true, _triangles.Count == 0);
//		return;
//	}
//
//	/// First, encode the information for 'this' node
//	enc.Levels[_level].AddInfo(_level, _levelIndex, _childIndex, false, false);
//
//	/// Now we handle the children
//	foreach(auto child in _children)
//		child.EncodeOctree(enc);
//}
//
//Node(MeshDetails_Spatial details, Node parent, Bounds bounds, int nodeId, List<int> triangles = null)
//{
//	_details = details;
//	_parent = parent;
//	_bounds = bounds;
//	_nodeId = nodeId;
//	_triangles = triangles;
//}

//////////////////////////////////////////////////////////////////////////

MeshDetails_Spatial::MeshDetails_Spatial(MeshInfo* mesh) : MeshDetails(mesh)
{
}

MeshDetails_Spatial::~MeshDetails_Spatial()
{
}

void MeshDetails_Spatial::Release()
{
}

void MeshDetails_Spatial::CalculateTri(size_t ti)
{
}

MeshDetailsPAsync MeshDetails_Spatial::Calculate()
{
	return cti::make_ready_continuable<MeshDetails*>(this);
}
