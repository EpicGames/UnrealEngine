// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MeshDetails.h"
#include "GraphicsDefs.h"
#include "Helper/GraphicsUtil.h"
#include <vector>
#include <list>
#include <array>

#define UE_API TEXTUREGRAPHENGINE_API

class MeshDetails_Tri;

class MeshDetails_Spatial : MeshDetails
{
public:
    struct ONode
    {
        FBox         		    FBox;
	    int					    Level;
	    int					    ChildIndex;
	    int					    LevelIndex;
        int          		    ParentNodeId;
	    int					    NodeId;
	    int					    TriStart;
	    int					    TriCount;
	    int          		    HasChildren;

        /// We need to do stupid shit like this to avoid having int[] Children in here because it's non-blittable
        /// for the compute shader and throws an error.
        static const int        s_maxChildren = 8;
        int                     Children[s_maxChildren];    /// The children for this node

                                ONode(int parentNodeId);
    };

    //////////////////////////////////////////////////////////////////////////
    class Node
    {
    private:
        static const int        s_maxTriangles = 256;
        static const int        s_maxChildren = 8;
        static const int        s_maxLevel = 9;

        FCriticalSection		_mutex;             /// Mutex for parallel compute

        /// The bounds for this node
        FBox          			_bounds;            /// The bounds of this node

        /// The triangles in this node. These are indexed into MeshDetails_Tri
        std::vector<int>*      	_triangles;
		MeshDetails_Spatial*	_details;

		/// The parent of this Octree
		Node*            		_parent = nullptr;

        int             		_level = 0;         /// What level is this node on
        int             		_childIndex = 0;    /// Which child is it (Check out Gemoetry.Util.BoundsCorners in GeometryUtil.cs)
        int             		_levelIndex = 0;    /// What is the index of this node at the depth level that it's at
        int             		_nodeId = 0;        /// Unique node id

        /// The children of this octree
        std::array<Node*, 8>    _children;          /// Children for this node

    public:

        //////////////////////////////////////////////////////////////////////////
        bool             		HasChildren() const {  return _children[0] != nullptr; }
        const std::vector<int>* Triangles() const { return _triangles; }
        const FBox&           	Bounds() const { return _bounds; }
        const std::array<Node*, 8>& Children() const { return _children; }
        const Node*             Parent() const { return _parent; }
        int              		NodeId() const { return _nodeId; }

        void                    RenderDebug() const;
        Node*                   CreateChild(int ci, int li, int ni);
        int                     FindTriangle(MeshInfo& mesh, const Vector3& pos, Vector2& bcCoord);
        bool                    Divide(MeshDetails_Tri* d_tri, Node* triNodes);

        static bool             CheckIntersection(const FBox& nodeBounds, FRay ray, float& distance, float length = -1, FBox* bounds = nullptr);
        std::list<Node*>        GetIntersectingLeafNodes(std::list<Node> nodes, FRay ray, float* distance, float length = -1, FBox* bounds = nullptr, std::list<Node*>* ignoreLeafs = nullptr);
        //void                    FindCongruentVertices(MeshInfo& mesh, const Vector3& v, int i, std::vector<int>& congruent);
        //int                     FindCongruentVertex(MeshInfo& mesh, const Vector3& v, int i);
        //void                    GetAllNodes(ONode** allNodes, const std::vector<int>& tris, int& triIndex);
        //std::list<Node*>        GetIntersectingLeafNodes(const std::list<Node>& nodes, Hemisphere hem)
        //std::list<Node*>        GetIntersectingLeafNodes(const std::list<Node>& nodes, const FBox& bounds);
        //void                    ClearNonLeafNodes(const std::list<Node*>& leafs);
        //void                    Release();

        Node(MeshDetails_Spatial* details, Node* parent, const FBox& bounds, int nodeId, std::vector<int>* triangles = nullptr)
        {
			_details = details;
            _parent = parent;
            _bounds = bounds;
            _nodeId = nodeId;
            _triangles = triangles;
        }
    };

protected:
    friend class Node;

	/// The root node of this Octree
	MeshDetails_Spatial::Node*  _root = nullptr;        /// The root of the Octree

	/// The leaf nodes of this Octree for easy access
	std::vector<MeshDetails_Spatial::Node*> _leafs;     /// The leaf nodes

	int					        _maxLevel = 0;          /// The current max level of the tree

	std::array<Vector3, 8>      _diagNormals;           /// Normals for the diagonals of Level-0 and Level-1 of the octree. These allow us to 
														/// to recursively calculate the AABB of a child node at L(n) given a parent at L(n - 1)


	//private ComputeBuffer       b_octree;
	//private ComputeBuffer       b_octreeTris;

	//public Node[]               TriNodes { get => _triNodes; }

	int					        _nodeId = 0;            /// The current node ID that we're using
	std::vector<ONode*>         _allNodes;              /// ALL the nodes in the Octree
	std::vector<int>            _tris;                  /// The triangles
	int                         _triCount = 0;          /// How many triangles do we have

	UE_API virtual void				CalculateTri(size_t ti) override;

public:
								UE_API MeshDetails_Spatial(MeshInfo* mesh);
	UE_API virtual						~MeshDetails_Spatial();

	UE_API virtual MeshDetailsPAsync	Calculate() override;
	UE_API virtual void				Release() override;
};

#undef UE_API
