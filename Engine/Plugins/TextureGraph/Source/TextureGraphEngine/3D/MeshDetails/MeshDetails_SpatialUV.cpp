// Copyright Epic Games, Inc. All Rights Reserved.
#include "MeshDetails_SpatialUV.h"
#include "3D/MeshInfo.h"
#include "3D/CoreMesh.h"
#include "Helper/GraphicsUtil.h"
#include "Helper/Util.h"
#include "Data/RawBuffer.h"
#include <cmath>

//////////////////////////////////////////////////////////////////////////
MeshDetails_SpatialUV::MeshDetails_SpatialUV(MeshInfo* mesh) : MeshDetails(mesh)
{
}

MeshDetails_SpatialUV::~MeshDetails_SpatialUV()
{
}

void MeshDetails_SpatialUV::Release()
{
}

void MeshDetails_SpatialUV::CalculateTri(size_t ti)
{
}

MeshDetailsPAsync MeshDetails_SpatialUV::Calculate()
{
	return cti::make_ready_continuable(this);
}
