// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MeshDetails.h"
#include "GraphicsDefs.h"
#include <vector>
#include <list>

#define UE_API TEXTUREGRAPHENGINE_API

class MeshDetails_SpatialUV : public MeshDetails
{
protected:
	UE_API virtual void				CalculateTri(size_t ti) override;

public:
								UE_API MeshDetails_SpatialUV(MeshInfo* mesh);
	UE_API virtual						~MeshDetails_SpatialUV();

	UE_API virtual MeshDetailsPAsync	Calculate() override;
	UE_API virtual void				Release() override;
};

#undef UE_API
