// Copyright Epic Games, Inc. All Rights Reserved.
#include "MeshDetails_Tri.h"
#include "3D/MeshInfo.h"
#include "3D/CoreMesh.h"
#include "Helper/GraphicsUtil.h"
#include "Helper/Util.h"
#include "Data/RawBuffer.h"
#include <cmath>
#include "Chaos/ParallelFor.h"

MeshDetails_Tri::Triangle::Triangle(int32 i0, int32 i1, int32 i2) : indices{ i0, i1, i2 }
{
}

bool MeshDetails_Tri::Triangle::HasIndex(int32 i)
{
	return indices[0] == i || indices[1] == i || indices[2] == i;
}

std::array<int32, 2> MeshDetails_Tri::Triangle::GetOtherVertices(int32 vi) const
{
	if (vi == indices[0])
		return { indices[1], indices[2] };
	else if (vi == indices[1])
		return { indices[0], indices[2] };

	return { indices[1], indices[2] };
}

//////////////////////////////////////////////////////////////////////////
const size_t MeshDetails_Tri::s_maxBatch = 1024;

MeshDetails_Tri::MeshDetails_Tri(MeshInfo* mesh) : MeshDetails(mesh)
{
}

MeshDetails_Tri::~MeshDetails_Tri()
{
}

void MeshDetails_Tri::CalculateTri(size_t ti)
{
	check(_triangles != nullptr);

	auto& tris		= Mesh->CMesh()->triangles;
	auto& verts		= Mesh->CMesh()->vertices;
	auto& uvs		= Mesh->CMesh()->uvs;
	auto& normals	= Mesh->CMesh()->normals;

	int32 i0		= tris[ti * 3];
	int32 i1		= tris[ti * 3 + 1];
	int32 i2		= tris[ti * 3 + 2];

	Vector2& uv0	= uvs[i0];
	Vector2& uv1	= uvs[i1];
	Vector2& uv2	= uvs[i2];

	Vector3& v0		= verts[i0];
	Vector3& v1		= verts[i1];
	Vector3& v2		= verts[i2];

	Triangle* tri	= _triangles + ti;

	Vector3 e0		= v1 - v0;
	Vector3 e1		= v2 - v0;
	tri->normal		= Vector3::CrossProduct(e1, e0);

	tri->normal.Normalize();

	/// Check whether the normal is pointing the correct direction
	Vector3 v0Normal = normals[i0];

	/// If the normals don't point in the same direction then we just inverse the normal
	if (Vector3::DotProduct(v0Normal, tri->normal) < 0.0f)
		tri->normal *= -1.0f;

	/// One of the edges is already orthagonal to the normal, so that can serve as the tangent
	tri->tangent	= e0;
	tri->tangent.Normalize();

	tri->biTangent = Vector3::CrossProduct(tri->tangent, tri->normal);

	/// Distance from the origin for the plane represented by this triangle
	/// Ax + By + Cz + D = 0
	/// where <X, Y, Z> is any point on the plane (we can use V0)
	/// while <A, B, C> is normal of the plane
	tri->distance = -(tri->normal.X * v0.X + tri->normal.Y * v0.Y + tri->normal.Z * v0.Z);

	/// Now we calculate the bounds of this triangle
	tri->bounds.Min.X = std::min(std::min(v0.X, v1.X), v2.X);
	tri->bounds.Min.Y = std::min(std::min(v0.Y, v1.Y), v2.Y);
	tri->bounds.Min.Z = std::min(std::min(v0.Z, v1.Z), v2.Z);

	tri->bounds.Max.X = std::max(std::max(v0.X, v1.X), v2.X);
	tri->bounds.Max.Y = std::max(std::max(v0.Y, v1.Y), v2.Y);
	tri->bounds.Max.Z = std::max(std::max(v0.Z, v1.Z), v2.Z);

	tri->uvBounds.Min.X = std::min(std::min(uv0.X, uv1.X), uv2.X);
	tri->uvBounds.Min.Y = std::min(std::min(uv0.Y, uv1.Y), uv2.Y);
	tri->uvBounds.Min.Z = -0.1f;

	tri->uvBounds.Max.X = std::max(std::max(uv0.X, uv1.X), uv2.X);
	tri->uvBounds.Max.Y = std::max(std::max(uv0.Y, uv1.Y), uv2.Y);
	tri->uvBounds.Max.Z = 0.1f;
}

MeshDetailsPAsync MeshDetails_Tri::Calculate()
{
	if (_triangles != nullptr)
		return cti::make_ready_continuable(this);

	auto& triangles = Mesh->CMesh()->triangles;
	int32 numTriangleIndices = triangles.Num();

	auto& vertices = Mesh->CMesh()->vertices;
	auto& normals = Mesh->CMesh()->normals;
	auto& tangents = Mesh->CMesh()->tangents;

	_numTriangles = numTriangleIndices / 3;
	check(_numTriangles > 0);

	HashTypeVec hashes =
	{
		Mesh->Hash()->Value(),
		DataUtil::Hash_GenericString_Name(TEXT("Tri")),
	};

	HashType hashValue = DataUtil::Hash(hashes);
	HashValue = std::make_shared<CHash>(hashValue, true);

	BufferDescriptor desc;
	desc.Name = TEXT("Tris");
	desc.Width = _numTriangles;
	desc.Height = 1;
	desc.ItemsPerPoint = 1;
	desc.Format = BufferFormat::Custom;
	desc.Type = BufferType::MeshDetail;

	size_t dataLength = sizeof(Triangle) * _numTriangles;
	uint8* data = new uint8 [dataLength];

	_raw = std::make_shared<RawBuffer>(data, dataLength, desc, HashValue);

	_triangles = (Triangle*)data;

	size_t numBatches = (size_t)std::ceil((float)_numTriangles / (float)s_maxBatch);

	return cti::make_continuable<MeshDetails*>([this, numBatches](auto&& promise) mutable 
	{
		/// run in another thread
		Util::OnBackgroundThread([this, numBatches, FWD_PROMISE(promise)]() mutable
		{
			ParallelFor(numBatches, [this, numBatches, FWD_PROMISE(promise)](size_t index) mutable
			{
				size_t start = index * s_maxBatch;
				size_t end = std::min(start = s_maxBatch, _numTriangles);

				for (size_t ti = start; ti < end; ti++)
				{
					CalculateTri(ti);
				}

				promise.set_value(this);
			});
		});
	});
}

int32* MeshDetails_Tri::GetIndices(size_t ti) const
{
	check(_triangles);
	check(ti < (size_t)Mesh->NumTriangles());
	return _triangles[ti].indices;
}

std::array<Vector3, 3> MeshDetails_Tri::GetVertices(size_t ti) const
{
	check(_triangles);
	check(ti < (size_t)Mesh->NumTriangles());
	int32* i = _triangles[ti].indices;
	return Mesh->Vertices(i[0], i[1], i[2]);
}

std::array<Vector2, 3> MeshDetails_Tri::GetUVs(size_t ti) const
{
	check(_triangles);
	check(ti < (size_t)Mesh->NumTriangles());
	int32* i = _triangles[ti].indices;
	auto& uvs = Mesh->CMesh()->uvs;
	return { uvs[i[0]], uvs[i[1]], uvs[i[2]] };
}

std::array<Vector3, 3> MeshDetails_Tri::GetNormals(size_t ti) const
{
	check(_triangles);
	check(ti < (size_t)Mesh->NumTriangles());
	int32* i = _triangles[ti].indices;
	auto& nms = Mesh->CMesh()->normals;
	return { nms[i[0]], nms[i[1]], nms[i[2]] };
}

std::array<Tangent, 3> MeshDetails_Tri::GetTangents(size_t ti) const
{
	check(_triangles);
	check(ti < (size_t)Mesh->NumTriangles());
	int32* i = _triangles[ti].indices;
	auto& tans = Mesh->CMesh()->tangents;
	return { tans[i[0]], tans[i[1]], tans[i[2]] };
}
