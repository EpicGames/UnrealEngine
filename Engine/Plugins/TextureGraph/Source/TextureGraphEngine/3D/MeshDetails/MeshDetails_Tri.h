// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MeshDetails.h"
#include "GraphicsDefs.h"

#define UE_API TEXTUREGRAPHENGINE_API

class MeshDetails_Tri : public MeshDetails
{
public:
	struct Triangle
	{
		FBox					bounds;
		FBox					uvBounds;
		Vector3					normal;
		Vector3					tangent;
		Vector3					biTangent;
		float					distance;
		Vector3					centre;

        int32					indices[3];

								Triangle(int32 i0, int32 i1, int32 i2);
		bool					HasIndex(int32 i);
		std::array<int32, 2>	GetOtherVertices(int32 vi) const;
	};

protected:
	static UE_API const size_t			s_maxBatch;
	Triangle*					_triangles = nullptr;			/// The triangles that we have
	size_t						_numTriangles = 0;				/// How many triangles do we have
	RawBufferPtr				_raw;							/// The triangle buffer is indexed into this RawBuffer

	UE_API virtual void				CalculateTri(size_t ti) override;

public:
								UE_API MeshDetails_Tri(MeshInfo* mesh);
	UE_API virtual						~MeshDetails_Tri();

	UE_API virtual MeshDetailsPAsync	Calculate() override;

	UE_API int32*						GetIndices(size_t ti) const;
	UE_API std::array<Vector3, 3>		GetVertices(size_t ti) const;
	UE_API std::array<Vector2, 3>		GetUVs(size_t ti) const;
	UE_API std::array<Vector3, 3>		GetNormals(size_t ti) const;
	UE_API std::array<Tangent, 3>		GetTangents(size_t ti) const;

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE Triangle*		Triangles() const { return _triangles; }
	FORCEINLINE size_t			NumTriangles() const { return _numTriangles; }
};


#undef UE_API
