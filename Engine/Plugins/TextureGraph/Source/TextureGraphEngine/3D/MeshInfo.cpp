// Copyright Epic Games, Inc. All Rights Reserved.
#include "MeshInfo.h"
#include "CoreMesh.h"
#include "3D/MeshDetails/MeshDetails_Tri.h"
#include "Math/Box.h"
#include "Helper/MathUtils.h"

MeshInfo::MeshInfo(CoreMeshPtr cmesh) : _cmesh(cmesh)
{
	HashType hashValue = DataUtil::Hash_GenericString_Name(cmesh->name);
	_hash = std::make_shared<CHash>(hashValue, true);
}

MeshInfo::~MeshInfo()
{
}

size_t MeshInfo::NumVertices() const
{
	return _cmesh->vertices.Num();
}

size_t MeshInfo::NumTriangles() const
{
	return _cmesh->triangles.Num() / 3;
}

std::array<Vector3, 3> MeshInfo::Vertices(int32 i0, int32 i1, int32 i2) const
{
	return { _cmesh->vertices[i0], _cmesh->vertices[i1], _cmesh->vertices[i2] };
}

int32 MeshInfo::GetMaterialIndex()
{
	return _cmesh->materialIndex;
}

void MeshInfo::InitBounds(FVector min, FVector max)
{
	_cmesh->bounds.BuildAABB(FVector::ZeroVector,FVector::ZeroVector);
	_cmesh->bounds.Min = min;
	_cmesh->bounds.Max = max;
}

void MeshInfo::UpdateBounds(const FVector& vert)
{
	check(CMesh())

	MathUtils::UpdateBounds(CMesh()->bounds,vert);
}

//////////////////////////////////////////////////////////////////////////
MeshDetails_Tri* MeshInfo::d_Tri() 
{ 
	return GetAttribute<MeshDetails_Tri>("d_Tri"); 
}
