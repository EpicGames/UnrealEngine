// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "GraphicsDefs.h"
#include "Helper/DataUtil.h"
#include <memory>
#include <array>
#include <unordered_map>
#include <limits>

#define UE_API TEXTUREGRAPHENGINE_API

struct CoreMesh;
typedef std::shared_ptr<CoreMesh>	CoreMeshPtr;

class MeshDetails;
typedef std::shared_ptr<MeshDetails> MeshDetailsPtr;

class MeshDetails_Tri;

class MeshInfo
{
protected:
    typedef std::unordered_map<const char*, MeshDetailsPtr> MeshDetailsPtrLookup;

	CoreMeshPtr					_cmesh;				/// The core mesh data structure that should've been loaded by now
	CHashPtr					_hash;				/// The hash for the mesh
	mutable FCriticalSection	_detailsMutex;		/// Mutex for details
    MeshDetailsPtrLookup        _details;           /// Details lookup

public:
								UE_API MeshInfo(CoreMeshPtr cmesh);
								UE_API ~MeshInfo();

	UE_API size_t						NumVertices() const;
	UE_API size_t						NumTriangles() const;
	UE_API std::array<Vector3, 3>		Vertices(int32 i0, int32 i1, int32 i2) const;
	UE_API int32		                GetMaterialIndex();
    UE_API void                        InitBounds(FVector min, FVector max);
    UE_API void                        UpdateBounds(const FVector& vert);

	//////////////////////////////////////////////////////////////////////////
    UE_API MeshDetails_Tri*            d_Tri();

    template <typename T_Details>
    T_Details* GetAttribute(const char* name, bool create = true) 
    {
		FScopeLock lock(&_detailsMutex);
        {
            auto iter = _details.find(name);

            /// If the details already exists
            if (iter != _details.end())
                return static_cast<T_Details*>(iter->second.get());

            if (!create)
                return nullptr;

            auto detail = std::make_shared<T_Details>(this);
            _details[name] = std::static_pointer_cast<MeshDetails>(detail);

            return detail.get();
        }
    }

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE CoreMeshPtr		CMesh() const { return _cmesh; } 
	FORCEINLINE CHashPtr		Hash() const { return _hash; }
};

typedef std::shared_ptr<MeshInfo> MeshInfoPtr;

#undef UE_API
