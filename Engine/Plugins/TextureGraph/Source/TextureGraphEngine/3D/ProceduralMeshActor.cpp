// Copyright Epic Games, Inc. All Rights Reserved.

//Procedural Mesh

#include "ProceduralMeshActor.h" 
#include "2D/Tex.h"
#include "TextureGraphEngineGameInstance.h"
#include "UObject/ConstructorHelpers.h"
#include <Components/SceneCaptureComponent2D.h>
#if WITH_EDITOR
#include "Editor.h"
#endif
// Sets default values

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProceduralMeshActor)
AProceduralMeshActor::AProceduralMeshActor()
{
 	// Set this actor to call Tick() every frame.  You can turn this off to improve performance if you don't need it.
	PrimaryActorTick.bCanEverTick = false;
	InitMeshObj();
	SetActorLocation(FVector(0, 0, 1));
	//InitSceneCaptureComponent();
}

AProceduralMeshActor::~AProceduralMeshActor()
{
	UE_LOGF(LogTemp, Log, "Actor being destructed");
}

void AProceduralMeshActor::InitSceneCaptureComponent()
{
	// add scene capture component
	_sceneCaptureComp = CreateDefaultSubobject<USceneCaptureComponent2D>(TEXT("SceneCaptureComponent"));
	_sceneCaptureComp->ProjectionType = ECameraProjectionMode::Orthographic;
	_sceneCaptureComp->OrthoWidth = 1000.0f;
	_sceneCaptureComp->SetRelativeRotation(FQuat::MakeFromEuler(FVector(-90.0f, -90.0f, 180.0f)));
	_sceneCaptureComp->SetRelativeLocation(FVector(0.0f, 0.0f, 500.0f));
	_sceneCaptureComp->bCaptureEveryFrame = false;
	_sceneCaptureComp->bCaptureOnMovement = false;
	_sceneCaptureComp->PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList;
	_sceneCaptureComp->ShowOnlyActors.Add(this);

	ConstructorHelpers::FObjectFinder<UTextureRenderTarget2D> renderTargetAsset(TEXT("/Game/Textures/SceneCapture/RT_SceneCapture.RT_SceneCapture"));
	_sceneCaptureComp->TextureTarget = renderTargetAsset.Object;
}

void AProceduralMeshActor::InitMeshObj()
{
	_meshObj = CreateDefaultSubobject<UProceduralMeshComponent>(TEXT("GeneratedMesh"));
	RootComponent = _meshObj;
	// New in UE 4.17, multi-threaded PhysX cooking.
	_meshObj->bUseAsyncCooking = false;
	//For displacement map rendering pass

	//Previously this was set to 20 to avoid flickering when changing height, however that doesnt seem
	//to be a problem anymore. If the case exists again then please spcecify maximum achievable height as bound scale
	//for this mesh component
	_meshObj->SetBoundsScale(1.0);
	
}

// Called when the game starts or when spawned
void AProceduralMeshActor::BeginPlay()
{
	Super::BeginPlay();
	UE_LOGF(LogTemp, Warning, "Actor lifespan %f", GetLifeSpan());
}

// Called every frame
void AProceduralMeshActor::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);
	//if (_sceneCaptureComp->IsActive())
	//{
	//	_sceneCaptureComp->SetActive(false);
	//	_sceneCaptureComp->TextureTarget = nullptr;
	//}
}


void AProceduralMeshActor::ToggleDebug(int32 debugType)
{
#if WITH_EDITOR
	DrawDebugLines(_debugPoints[debugType], GetActorRelativeScale3D());
#endif
}

void AProceduralMeshActor::SetMeshData(CoreMeshPtr mesh)
{
	_meshObj->CreateMeshSection_LinearColor(0, mesh->vertices, mesh->triangles, mesh->normals, mesh->uvs, mesh->vertexColors, mesh->tangents, false);

#if WITH_EDITOR

	GEditor->SelectNone(true, true, true);
	GEditor->MoveViewportCamerasToActor(*this, true);
	GEditor->SelectActor(this, true, true);
	_debugPoints.Add(mesh->vertices);

#endif
}

void AProceduralMeshActor::DrawDebugLines(TArray<FVector>& positionArray, FVector length)
{
	
	FVector offset = GetActorLocation();
	for (FVector& lineStart : positionArray)
	{
		DrawDebugLine(GetWorld(), offset + lineStart, offset + lineStart + (lineStart*length), FColor::Blue, true, -1.F, 0, 2.0f);
	}
}

void AProceduralMeshActor::SetMaterial(UMaterialInterface* mat)
{
	check (_meshObj)
	_meshObj->SetMaterial(0, mat);
}

UMaterialInterface* AProceduralMeshActor::GetMaterial()
{
	check (_meshObj)
	return _meshObj->GetMaterial(0);
}

void AProceduralMeshActor::BlitInternal(UMaterialInterface* mat)
{
	
	UMaterialInterface* orignalMat = _meshObj->OverrideMaterials[0];
	_meshObj->SetMaterial(0, mat);

	_sceneCaptureComp->CaptureScene();
	
	_meshObj->SetMaterial(0, orignalMat);

}

void AProceduralMeshActor::BlitTo(UTextureRenderTarget2D* rt, UMaterialInterface* mat)
{
	auto existingRT = _sceneCaptureComp->TextureTarget;
	_sceneCaptureComp->TextureTarget = rt;

	BlitInternal(mat);

	_sceneCaptureComp->TextureTarget = existingRT;
}

void AProceduralMeshActor::BeginDestroy()
{
#if WITH_EDITOR
	if (GEditor)
		GEditor->SelectNone(true, true, true);
#endif
	Super::BeginDestroy();
}
