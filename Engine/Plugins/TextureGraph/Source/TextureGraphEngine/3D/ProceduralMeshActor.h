// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
//Set material
#include "Materials/MaterialInterface.h" 
#include "Materials/MaterialInstanceDynamic.h" 
//Load texture from file
#include "Kismet/KismetRenderingLibrary.h" 
//FCoreMesh
#include "CoreMesh.h"
#include "ProceduralMeshComponent.h"
//Debug
#include "DrawDebugHelpers.h" 
//For World normals
#include "Data/TiledBlob.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include "ProceduralMeshActor.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

class USceneCaptureComponent2D;

class Tex;
typedef std::shared_ptr<Tex>		TexPtr;
UCLASS(MinimalAPI)
class AProceduralMeshActor : public AActor
{
	GENERATED_BODY()
private:
	TArray<TArray<FVector>>				_debugPoints;						
	bool								_isDebug = false;
	FString								_meshName = FString("");

	UE_API void								BlitInternal(UMaterialInterface* mat);

protected:
	// Called when the game starts or when spawned
	UE_API virtual void						BeginPlay() override;
	// Called every frame
	UE_API virtual void						Tick(float DeltaTime) override;

	
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Hierarchy")
		TObjectPtr<UProceduralMeshComponent>		_meshObj;

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Hierarchy")
		TObjectPtr<USceneCaptureComponent2D>		_sceneCaptureComp;
public:	
	// Sets default values for this actor's properties
										UE_API AProceduralMeshActor();
										UE_API ~AProceduralMeshActor();

										UE_API void InitSceneCaptureComponent();

										UE_API void InitMeshObj();

	UE_API void								SetMeshData(CoreMeshPtr mesh);
	UE_API void 								DrawDebugLines(TArray<FVector>& positionArray, FVector length);
	UE_API void 								SetMaterial(UMaterialInterface* mat);
	UE_API UMaterialInterface*  				GetMaterial();
	
	UE_API void								BlitTo(UTextureRenderTarget2D* rt, UMaterialInterface* mat);
	UE_API virtual void						BeginDestroy();

	UFUNCTION(BlueprintCallable, Category = "Debug")
		UE_API void							ToggleDebug(int32 debugType);

	UFUNCTION(BlueprintCallable,Category = "Info")
		const FString &					GetMeshName() { return _meshName; }

	UFUNCTION(BlueprintCallable, Category = "Info")
		void							SetMeshName(const FString& name) { _meshName = name; }

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
