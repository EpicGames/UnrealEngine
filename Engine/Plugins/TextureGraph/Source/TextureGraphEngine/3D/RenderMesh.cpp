// Copyright Epic Games, Inc. All Rights Reserved.
#include "RenderMesh.h"
#include "TextureGraphEngineGameInstance.h"
#include "3D/ProceduralMeshActor.h"
#include "TextureGraphEngine.h"
#include "Device/FX/Device_FX.h"
#include "Model/Mix/Mix.h"
#include "3D/MaterialInfo.h"
#include "MeshInfo.h"
#include "ProceduralMeshActor.h"
#include "Transform/Mesh/T_MeshEncodedAsTexture.h"
#include "Transform/Mesh/T_MeshDilateUVs.h"
#include "Helper/MathUtils.h"
#include "RHIResourceUtils.h"

#if WITH_EDITOR

#include "DrawDebugHelpers.h" 

#endif
#include "RenderMesh_Procedural.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RenderMesh)
THIRD_PARTY_INCLUDES_START
#include "continuable/continuable-base.hpp"
THIRD_PARTY_INCLUDES_END
#include "Transform/Utility/T_SplitToTiles.h"
#include <Engine/World.h>

RenderMesh::RenderMesh(const MeshLoadInfo loadInfo)
{
}

RenderMesh::RenderMesh(RenderMesh* parent, TArray<MeshInfoPtr> meshes, MaterialInfoPtr matInfo)
{
	verify(!_meshes.Num());

	_parentMesh = parent;

	_meshes.Append(meshes);
	_currentMaterials.Add(matInfo);

	FBox bounds = FBox(Vector3::ZeroVector, Vector3::ZeroVector);
	bounds.Min = MathUtils::MaxFVector();
	bounds.Max = MathUtils::MinFVector();

	for (int meshIndex = 0; meshIndex < meshes.Num(); meshIndex++)
	{
		MeshInfoPtr meshInfo = meshes[meshIndex];

		MathUtils::EncapsulateBound(bounds, meshInfo->CMesh()->bounds);
	}
	_originalBounds = bounds;

	if (_parentMesh)
	{
		_originalScale = _parentMesh->OriginalScale();
		_viewScale = _parentMesh->ViewScale();
	}
}

RenderMesh::~RenderMesh()
{
	
}

void RenderMesh::PrepareForRendering(UWorld* world,FVector scale)
{
	if (_parentMesh)
		_parentMesh->SetViewScale(scale);

	_viewScale = scale;

	SpawnActors(world);
	UpdateMeshTransforms();
}

void RenderMesh::Clear()
{
	_meshes.Empty();	

	_originalBounds.Max = MathUtils::MinFVector();
	_originalBounds.Min = MathUtils::MaxFVector();

	RemoveActors();
}

void RenderMesh::UpdateMeshTransforms()
{
	UpdateBounds();

	for (AActor* actor : _meshActors)
	{
		actor->SetActorScale3D(_viewScale);
	}
}

void RenderMesh::UpdateBounds()
{
	_viewBounds = FBox::BuildAABB(_originalBounds.GetCenter(), _originalBounds.GetExtent() * ViewScale());

	UE_LOGF(LogMesh, Log, "Current Bounds: %ls", *(_viewBounds.GetSize().ToString()));
}

void RenderMesh::SetMaterial(UMaterialInterface* mat)
{
	_currentMat = mat;
	
	try
	{
		for (int i = 0; i < _meshActors.Num();i++)
		{
			AProceduralMeshActor* actor = Cast<AProceduralMeshActor>(_meshActors[i]);
			
			if (IsValid(actor))
			{
				actor->SetMaterial(_currentMat);
			}
			else
			{
				break;
			}
		
		}
	}
	catch (const std::exception& e)
	{
		UE_LOGF(LogTemp, Warning, "Exception %s", e.what());
	}	
}

void RenderMesh::SpawnActors(UWorld* world)
{
	UE_LOGF(LogMesh, Log, "Spawning actors");

	_meshActors.Empty();

	if (ensure(world))
	{
		for (MeshInfoPtr meshInfo : _meshes)
		{
			CoreMeshPtr mesh = meshInfo->CMesh();

			UE_LOGF(LogMesh, Log, "Adding mesh section");
			check(IsInGameThread());
			AProceduralMeshActor* meshActor = world->SpawnActorDeferred<AProceduralMeshActor>(AProceduralMeshActor::StaticClass(), FTransform::Identity);
			
			if (ensure(meshActor))
			{
				meshActor->SetMeshName(*(mesh->name));
				meshActor->SetMeshData(mesh);
				_meshActors.Add(meshActor);
			}

		}
	}
}

void RenderMesh::RemoveActors()
{
	auto world = Util::GetGameWorld();
	for (AActor* actor : _meshActors)
	{
		UE_LOGF(LogMesh, Log, "Removing mesh section");

		actor->Destroy();
	}

	_meshActors.Empty();
}


FString RenderMesh::Name() const
{
	return TEXT("PLANE");
}


FString RenderMesh::GetMaterialName()
{
	if (!_parentMesh)
	{
		return FString("Texture Set 1");
	}

	check(_currentMaterials.Num() > 0)

	return _currentMaterials[0]->name;
}

TiledBlobPtr RenderMesh::WorldPosTexture(MixUpdateCyclePtr cycle, int32 targetId, bool singleBlob)
{
	if (!WorldPosition())
	{
		auto preDilatedPos = T_MeshEncodedAsTexture::Create_WorldPos(cycle, targetId);
		auto dilatedWorldPos = T_MeshDilateUVs::Create(cycle, targetId, preDilatedPos);
		_worldMapsSingleBlob[WorldTextures::Position] = dilatedWorldPos;
		_worldMaps[WorldTextures::Position] = T_SplitToTiles::Create(cycle, targetId, dilatedWorldPos);
	}
	
	return (singleBlob ? WorldPositionSingleBlob() : WorldPosition());
}

TiledBlobPtr RenderMesh::WorldNormalsTexture(MixUpdateCyclePtr cycle, int32 targetId, bool singleBlob)
{
	if (!WorldNormals())
	{
		auto preDilatedNormals = T_MeshEncodedAsTexture::Create_WorldNormals(cycle, targetId);
		auto dilatedNormals = T_MeshDilateUVs::Create(cycle, targetId, preDilatedNormals);
		_worldMapsSingleBlob[WorldTextures::Normals] = dilatedNormals;
		_worldMaps[WorldTextures::Normals] = T_SplitToTiles::Create(cycle, targetId, dilatedNormals);
	}
	
	return (singleBlob ? WorldNormalsSingleBlob() : WorldNormals());
}

TiledBlobPtr RenderMesh::WorldTangentsTexture(MixUpdateCyclePtr cycle, int32 targetId, bool singleBlob)
{
	if (!WorldTangents())
	{
		auto preDilatedTangents = T_MeshEncodedAsTexture::Create_WorldTangents(cycle, targetId);
		auto dilatedTangents = T_MeshDilateUVs::Create(cycle, targetId, preDilatedTangents);
		_worldMapsSingleBlob[WorldTextures::Tangents] = dilatedTangents;
		_worldMaps[WorldTextures::Tangents] = T_SplitToTiles::Create(cycle, targetId, dilatedTangents);
	}

	return (singleBlob ? WorldTangentsSingleBlob() : WorldTangents());
}

TiledBlobPtr RenderMesh::WorldUVMaskTexture(MixUpdateCyclePtr cycle, int32 targetId, bool singleBlob)
{
	if (!WorldUVMask())
	{
		auto uvMask = T_MeshEncodedAsTexture::Create_WorldUVMask(cycle, targetId);
		_worldMapsSingleBlob[WorldTextures::UVMask] = uvMask;
		_worldMaps[WorldTextures::UVMask] = T_SplitToTiles::Create(cycle, targetId, uvMask);
	}

	return (singleBlob ? WorldUVMaskSingleBlob() : WorldUVMask());
}

CHashPtr RenderMesh::Hash() const
{
	if (_hash) 
		return _hash;

	check(IsInGameThread());

	check(_meshes.Num() > 0);

	CHashPtrVec hashes(_meshes.Num());

	for (size_t mi = 0; mi < _meshes.Num(); mi++)
		hashes[mi] = _meshes[mi]->Hash();

	_hash = CHash::ConstructFromSources(hashes);
	return _hash;
}

FMatrix	RenderMesh::LocalToWorldMatrix() const 
{ 
	return _meshActors[0]->ActorToWorld().ToMatrixWithScale(); 
}

void RenderMesh::Init_PSO(FGraphicsPipelineStateInitializer& pso) const
{
	pso.BoundShaderState.VertexDeclarationRHI = g_coreMeshVertexDecl.RHI_Decl();
	pso.PrimitiveType = PT_TriangleList;
}

void RenderMesh::Render_Now(FRHICommandList& RHICmdList, int32 targetId) const
{
	//check(targetId >= 0 && targetId < (int32)_meshes.Num());

	/// This needs to be optimised, but that's less important right now
	for (int32 mi = 0; mi < _meshes.Num(); mi++)
	{
		CoreMeshPtr cmesh = _meshes[mi]->CMesh();

		size_t numVertices = cmesh->vertices.Num();
		size_t numIndices = cmesh->triangles.Num();
		check(numIndices % 3 == 0);
		size_t numTris = numIndices / 3;

		/// 1 FVector for position + 1 FVector for normal + 1 FVector2D for UV
		size_t vertexSize = sizeof(CoreMeshVertex);

		static const size_t maxVertices = 16 * 1024;
		size_t maxChunks = numVertices / maxVertices;

		TArray<CoreMeshVertex> vertices;
		vertices.SetNumUninitialized(numVertices);

		for (size_t vi = 0; vi < numVertices; vi++)
		{
			CoreMeshVertex& vertex = vertices[vi];
			vertex.position = FVector3f(cmesh->vertices[vi].X, cmesh->vertices[vi].Y, cmesh->vertices[vi].Z);
			vertex.uv = FVector2f(cmesh->uvs[vi].X, cmesh->uvs[vi].Y);
			vertex.normal = FVector3f(cmesh->normals[vi].X, cmesh->normals[vi].Y, cmesh->normals[vi].Z);

			const auto& t = cmesh->tangents[vi];
			vertex.tangent = FVector4f(t.TangentX.X, t.TangentX.Y, t.TangentX.Z, t.bFlipTangentY ? -1.0f : 1.0f);

			FColor color;

			if (cmesh->vertexColors.Num())
			{
				color = cmesh->vertexColors[vi].ToFColor(false);
			}
			else
			{
				color = FColor::Black;
			}

			vertex.color = FVector4f(color.R, color.G, color.B, color.A);
		}

		TArray<uint32> indices;
		indices.SetNumUninitialized(numIndices);

		for (size_t ii = 0; ii < numIndices; ii++)
		{
			indices[ii] = cmesh->triangles[ii];
		}

		FBufferRHIRef vertexBuffer = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("RenderMesh_VB"), EBufferUsageFlags::Static, MakeConstArrayView(vertices));
		FBufferRHIRef indexBuffer = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("RenderMesh_IB"), EBufferUsageFlags::Static, MakeConstArrayView(indices));

		RHICmdList.SetStreamSource(0, vertexBuffer, 0);
		RHICmdList.DrawIndexedPrimitive(indexBuffer, 0, 0, numVertices, 0, numTris, 1);

		vertexBuffer.SafeRelease();
		indexBuffer.SafeRelease();
	}
}

/// <summary>
/// This method is used to add material info from external sources, 
/// </summary>
/// <param name="id">id is expected to be unique and represents a material in _originalMaterials at index id</param>
/// <param name="matName"></param>
void RenderMesh::AddMaterialInfo(int32 id, FString& matName)
{
	// throw an exception if material id already exists in stored materials
	if (id < _originalMaterials.Num())
		throw std::runtime_error("Material ID already added");

	MaterialInfoPtr matInfo = std::make_shared<MaterialInfo>();
	matInfo->id = id;
	matInfo->name = matName;

	_originalMaterials.Add(matInfo);
}

void RenderMesh::DrawBounds(UWorld* world)
{
#if WITH_EDITOR

	UE_LOGF(LogMesh, Log, "MeshName : %ls : MeshCenter : %ls : Mesh Extents : %ls : Min : %ls : Max : %ls", *GetMaterialName(), *_originalBounds.GetCenter().ToString(),*_originalBounds.GetExtent().ToString(), *_originalBounds.Min.ToString(), *_originalBounds.Max.ToString());

	FColor color = _parentMesh != nullptr ? FColor::Red : FColor::Green;
	DrawDebugBox(world, _originalBounds.GetCenter(), _originalBounds.GetExtent(),FQuat::Identity,color,false,0.5f, 0, 2);	
#endif
}

RenderMeshPtr RenderMesh::Create(const MeshLoadInfo _loadInfo)
{
	MeshType meshType = _loadInfo.meshType;
	if (meshType==MeshType::Plane)
	{
		return std::make_shared<RenderMesh_Procedural>(_loadInfo);
	}
	/*else if (meshType == MeshType::CustomMesh || meshType == MeshType::ShaderBall)
	{
		return std::make_shared<RenderMesh_Custom>(_loadInfo);
	}*/
	else
	{
		UE_LOGF(LogMesh, Log, "MeshAsset : Unkown mesh type %i", int(meshType));
		throw std::make_exception_ptr(std::runtime_error("MeshAsset : MeshType is not handled yet"));
	}

}
