// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Math/Box.h"
#include "FxMat/RenderMaterial_BP.h"
#include "Data/TiledBlob.h"
#include "Chaos/Matrix.h"
#include "Helper/Promise.h"
#include "MaterialInfo.h"

#include "RenderMesh.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

UENUM()
enum class MeshType : uint8
{
	Plane				UMETA(DisplayName = "Plane"),
	LibraryAsset		UMETA(DisplayName = "Library Asset"),
	ShaderBall			UMETA(DisplayName = "Shader Ball"),
	CustomMesh			UMETA(DisplayName = "Custom 3D Mesh"),
	Editor				UMETA(DisplayName = "Editor"),
};

UENUM()
enum class MeshSplitType: uint8
{
	Material = 0,
	UDIM = 1,
	Single = 2
};

UENUM()
enum class BoundsType : uint8
{
	Target				UMETA(DisplayName = "Align to texture sets individually"),
	CombinedTarget		UMETA(DisplayName = "Align to texture sets combined"),
	Full				UMETA(DisplayName = "Align to model")
};

UENUM()
enum WorldTextures : int
{
	Normals				UMETA(DisplayName = "WorldNormals"),
	Tangents			UMETA(DisplayName = "WorldTangents"),
	Position			UMETA(DisplayName = "WorldPosition"),
	UVMask				UMETA(DisplayName = "WorldUVMask"),
	WorldTextureCount	UMETA(DisplayName = "Total World Textures")
};

class Tex;
class UWorld;
typedef std::shared_ptr<Tex>		TexPtr;

class MeshInfo;
typedef std::shared_ptr<MeshInfo>	MeshInfoPtr;

struct MaterialInfo;
typedef std::shared_ptr<MaterialInfo> MaterialInfoPtr;

class MixUpdateCycle;
typedef std::shared_ptr<MixUpdateCycle>	MixUpdateCyclePtr;

struct MeshLoadInfo
{
	FString							filename;
	FVector							scale = FVector::OneVector;
	MeshSplitType					meshSplitType = MeshSplitType::Single;
	MeshType						meshType = MeshType::Plane;
	FVector2D						dimension = FVector2D::UnitVector;
	int32							tesselation = 32;
	UMixInterface*					mix = nullptr;
};

typedef std::shared_ptr<RenderMesh>				RenderMeshPtr;

//////////////////////////////////////////////////////////////////////////
DEFINE_LOG_CATEGORY_STATIC(LogMesh, Log, All);
class RenderMesh 
{

protected:
	static const int				s_maxTex = 512;

	bool							_isPlane = false;
	TArray<MeshInfoPtr>				_meshes;					/// All the sub-meshens within this mesh
	TArray<MaterialInfoPtr>			_originalMaterials;			/// Original Mesh materials.
	TArray<MaterialInfoPtr>			_currentMaterials;			/// Materials can be generated for mesh materials or based on UDIM data.
	TArray<AActor*>					_meshActors;				/// All the actors this mesh spawned
	FBox							_originalBounds;			/// These are in cm units.
																/// What are the original bounds (as loaded) for the mesh
																/// this can be something other than OneVector in case of Megascans Assets
																/// (e.g. OBJ loading in cms, where-as JSON size suggests otherwise)
	FBox							_viewBounds;				/// These are in cm units.
																/// corresponding current bounds (e.g changed via UI) (_meshBoundsSizeToUse in Unity version)

	FVector							_originalScale	= FVector::OneVector;	/// What is the default scale (transform) of the mesh (as loaded) (_defaultMeshSize in Unity version)
	FVector							_viewScale	= FVector::OneVector;	/// Current mesh scale (e.g changed via size UI) (meshScaleToUse in Unity version)

	UMaterialInterface*				_currentMat;				/// What is the currently active material. Mesh can only have UMaterial applied on it

	RenderMesh*						_parentMesh = nullptr;		/// What is the parent mesh of this mesh

	MeshSplitType					_meshSplitType = MeshSplitType::Material;	/// How is the mesh being split (by submeshes, UDIMs or materials etc.)
	mutable CHashPtr				_hash;						/// What is the hash of this mesh. Mutable because
																/// we want the Hash() function to be const but we also

	TiledBlobPtr					_worldMaps[WorldTextures::WorldTextureCount];
	TiledBlobPtr					_worldMapsSingleBlob[WorldTextures::WorldTextureCount];
	
	UE_API void							AddMaterialInfo(int32 id, FString& matName);
	UE_API virtual void					SpawnActors(UWorld* world);
	UE_API void							UpdateMeshTransforms();
	UE_API void							UpdateBounds();
protected:
	virtual void					LoadInternal() = 0;
public:
									RenderMesh() = default;
									UE_API RenderMesh(const MeshLoadInfo loadInfo);
									UE_API RenderMesh(RenderMesh* parent, TArray<MeshInfoPtr> mesh, MaterialInfoPtr matInfo);

	UE_API virtual							~RenderMesh();

	UE_API virtual void					PrepareForRendering(UWorld* world,FVector scale);
	
	UE_API void							DrawBounds(UWorld* world);
	UE_API virtual void					SetMaterial(UMaterialInterface* material);	
	virtual AsyncActionResultPtr	Load() = 0;	
	UE_API FString							GetMaterialName();

	UE_API virtual CHashPtr				Hash() const;
	UE_API virtual FMatrix					LocalToWorldMatrix() const;
	UE_API virtual void					RemoveActors();
	UE_API virtual void					Render_Now(FRHICommandList& rhi, int32 targetId) const;
	UE_API virtual void					Init_PSO(FGraphicsPipelineStateInitializer& pso) const;

	UE_API virtual TiledBlobPtr			WorldPosTexture(MixUpdateCyclePtr cycle, int32 targetId, bool singleBLob = false);
	UE_API virtual TiledBlobPtr			WorldNormalsTexture(MixUpdateCyclePtr cycle, int32 targetId, bool singleBLob = false);
	UE_API virtual TiledBlobPtr			WorldTangentsTexture(MixUpdateCyclePtr cycle, int32 targetId, bool singleBLob = false);
	UE_API virtual TiledBlobPtr			WorldUVMaskTexture(MixUpdateCyclePtr cycle, int32 targetId, bool singleBLob = false);

	UE_API virtual FString					Name() const;
	UE_API virtual void					Clear();

	bool							ContainsWorldTexture(WorldTextures texture) { return _worldMaps[texture] != nullptr; }
	bool							ContainsWorldNormals()		{ return _worldMaps[WorldTextures::Normals] != nullptr;}
	bool							ContainsWorldTangents()		{ return _worldMaps[WorldTextures::Tangents] != nullptr;}
	bool							ContainsWorldPosition()		{ return _worldMaps[WorldTextures::Position] != nullptr;}
	bool							ContainsWorldUVMask()		{ return _worldMaps[WorldTextures::UVMask] != nullptr;}

	TiledBlobPtr					WorldTexture(WorldTextures texture)	{ return _worldMaps[texture];}
	TiledBlobPtr					WorldNormals()				{ return _worldMaps[WorldTextures::Normals];}
	TiledBlobPtr					WorldTangents()				{ return _worldMaps[WorldTextures::Tangents];}
	TiledBlobPtr					WorldPosition()				{ return _worldMaps[WorldTextures::Position];}
	TiledBlobPtr					WorldUVMask()				{ return _worldMaps[WorldTextures::UVMask];}

	TiledBlobPtr					WorldTextureSingleBlob(WorldTextures texture) { return _worldMapsSingleBlob[texture]; }
	TiledBlobPtr					WorldNormalsSingleBlob() { return _worldMapsSingleBlob[WorldTextures::Normals]; }
	TiledBlobPtr					WorldTangentsSingleBlob() { return _worldMapsSingleBlob[WorldTextures::Tangents]; }
	TiledBlobPtr					WorldPositionSingleBlob() { return _worldMapsSingleBlob[WorldTextures::Position]; }
	TiledBlobPtr					WorldUVMaskSingleBlob() { return _worldMapsSingleBlob[WorldTextures::UVMask]; }

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE const FVector&		ViewScale()					{ return (_parentMesh == nullptr) ? _viewScale : _parentMesh->ViewScale(); }
	FORCEINLINE const FVector&		OriginalScale()				{ return (_parentMesh == nullptr) ? _originalScale : _parentMesh->OriginalScale(); }
	FORCEINLINE const FVector		InvOriginalBoundsDiameter()	{ return FVector(1.0f / _originalBounds.GetSize().X, 1.0f / _originalBounds.GetSize().Y, 1.0f / _originalBounds.GetSize().Z); }
	FORCEINLINE const FBox&			OriginalBounds()			{ return _originalBounds; }
	FORCEINLINE	void				SetViewScale(FVector scale) { _viewScale = scale; }
	FORCEINLINE const MeshSplitType GetMeshSplitType()			{ return _meshSplitType; }

	FORCEINLINE TArray<MeshInfoPtr>& Meshes()					{ return _meshes; }
	FORCEINLINE const TArray<MeshInfoPtr>& Meshes() const		{ return _meshes; }
	// Original mesh materials. 								
	FORCEINLINE const TArray<MaterialInfoPtr>& OriginalMaterials() const{ return _originalMaterials; }
	// Materials can be generated for mesh materials or based on UDIM data.
	FORCEINLINE const TArray<MaterialInfoPtr>& CurrentMaterials() const{ return _currentMaterials; }
	
	FORCEINLINE TArray<AActor*>&	Actors()					{ return _meshActors; }
	FORCEINLINE const TArray<AActor*>& Actors() const			{ return _meshActors; }
	FORCEINLINE bool				IsPlane()					{ return _isPlane; }

	//////////////////////////////////////////////////////////////////////////
	//// Static Methods
	//////////////////////////////////////////////////////////////////////////
	static UE_API RenderMeshPtr			Create(const MeshLoadInfo _loadInfo);
};

#undef UE_API
