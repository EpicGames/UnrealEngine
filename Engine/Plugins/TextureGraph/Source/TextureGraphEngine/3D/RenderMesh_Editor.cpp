// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_EDITOR
#include "RenderMesh_Editor.h"
#include "CoreMesh.h"
#include "MeshInfo.h"
#include "KismetProceduralMeshLibrary.h"
#include "../Helper/MathUtils.h"
#include <Engine/StaticMesh.h>
#include <StaticMeshResources.h>

RenderMesh_Editor::RenderMesh_Editor()
{
	_isPlane = false;
}

RenderMesh_Editor::RenderMesh_Editor(RenderMesh* parent, TArray<MeshInfoPtr> meshes, MaterialInfoPtr matInfo) : RenderMesh(parent, meshes, matInfo)
{

}

RenderMesh_Editor::RenderMesh_Editor(UStaticMeshComponent* staticMeshComponent, UWorld* world)
{
	World = world;
	if (staticMeshComponent)
		MeshComponents.Add(staticMeshComponent);
}

void RenderMesh_Editor::PrepareForRendering(UWorld* world, FVector scale)
{
	if (_parentMesh)
		_parentMesh->SetViewScale(scale);

	_viewScale = scale;
}

AsyncActionResultPtr RenderMesh_Editor::Load()
{
	// Need to figure out how we are going to support MTS and UDIM meshes.
	_meshSplitType = MeshSplitType::Single;

	//_meshActor->GetComponents<UStaticMeshComponent>(MeshComponents);
	if (MeshComponents.Num() <= 0)
		return cti::make_ready_continuable(std::make_shared<ActionResult>());

	return cti::make_continuable<ActionResultPtr>([this](auto&& promise)
		{
			AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, promise = std::forward<decltype(promise)>(promise)]() mutable
				{
					try
					{
						LoadInternal();
						promise.set_value(std::make_shared<ActionResult>());
					}
					catch (cti::exception_t ex)
					{
						promise.set_exception(ex);
					}
				});
		});
}

void RenderMesh_Editor::SetMaterial(UMaterialInterface* material)
{
	for (int MeshIndex = 0; MeshIndex < MeshComponents.Num(); MeshIndex++)
	{
		UStaticMeshComponent* MeshComponent = MeshComponents[MeshIndex];
		if (MeshComponent->IsValidLowLevel())
		{
			MeshInfoPtr MeshInfo = _meshes[MeshIndex];
			MeshComponent->SetMaterial(MeshInfo->GetMaterialIndex(), material);
		}
	}
}

void RenderMesh_Editor::LoadInternal()
{
	for (UStaticMeshComponent* MeshComponent : MeshComponents)
	{
		LoadSingleMeshComponent(*MeshComponent);
	}
}
void RenderMesh_Editor::LoadSingleMeshComponent(const UStaticMeshComponent& MeshComponent)
{
	UStaticMesh* StaticMesh = MeshComponent.GetStaticMesh();

	//Set bAllowCPUAccess to true as it is required to use  GetSectionFromStaticMesh
	StaticMesh->bAllowCPUAccess = true;

	const int32 LODindex = 0;
	int32 numSections = StaticMesh->GetNumSections(LODindex);

	// Return if RenderData is invalid
	if (!StaticMesh->GetRenderData())
		return;

	// No valid mesh data on lod 0 (shouldn't happen)
	if (!StaticMesh->GetRenderData()->LODResources.IsValidIndex(LODindex))
		return;

	// load materials
	//const TArray<UMaterialInterface*> materialInterfaces = MeshComponent.GetMaterials();
	for (int32 materialIndex = 0; materialIndex < MeshComponent.GetNumMaterials(); ++materialIndex)
	{
		UMaterialInterface* MaterialInterface = MeshComponent.GetMaterial(materialIndex);
		if (MaterialInterface)
		{
			FString MatName;
			MaterialInterface->GetName(MatName);
			AddMaterialInfo(materialIndex, MatName);
		}
	}
	_currentMaterials.Empty();
	_currentMaterials.Append(_originalMaterials);

	TArray<FVector> Vertices;
	TArray<int32> Triangles;
	TArray<FVector> Normals;
	TArray<FVector2D> UVs;
	TArray<FProcMeshTangent> Tangents;

	_originalBounds.Init();

	for (int32 sectionIndex = 0; sectionIndex < numSections; sectionIndex++)
	{
		CoreMesh* cmesh = new CoreMesh();
		CoreMeshPtr cmeshPtr = CoreMeshPtr(cmesh);
		MeshInfoPtr MeshInfo = std::make_shared<::MeshInfo>(cmeshPtr);

		cmesh->bounds = StaticMesh->GetBoundingBox();
		MeshComponent.GetName(cmesh->name);

		FMeshSectionInfo sectionInfo = StaticMesh->GetSectionInfoMap().Get(LODindex, sectionIndex);
		UKismetProceduralMeshLibrary::GetSectionFromStaticMesh(StaticMesh, LODindex, sectionIndex, cmesh->vertices, cmesh->triangles, cmesh->normals, cmesh->uvs, cmesh->tangents);
		cmesh->materialIndex = sectionInfo.MaterialIndex;

		MathUtils::EncapsulateBound(_originalBounds, cmeshPtr->bounds);

		_meshes.Add(MeshInfo);
	}
}
#endif // WITH_EDITOR