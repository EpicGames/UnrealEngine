// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#if WITH_EDITOR
#include "CoreMinimal.h"
#include "RenderMesh.h"
#include "Components/StaticMeshComponent.h"

struct CoreMesh;

class RenderMesh_Editor: public RenderMesh
{

protected:
	TArray<UStaticMeshComponent*>			MeshComponents;
	UWorld*									World = nullptr;
	TEXTUREGRAPHENGINE_API virtual void							LoadInternal() override;
	TEXTUREGRAPHENGINE_API void									LoadSingleMeshComponent(const UStaticMeshComponent& mesh);

public:
											TEXTUREGRAPHENGINE_API RenderMesh_Editor();
											TEXTUREGRAPHENGINE_API RenderMesh_Editor(RenderMesh* parent, TArray<MeshInfoPtr> meshes, MaterialInfoPtr matInfo);
											TEXTUREGRAPHENGINE_API RenderMesh_Editor(UStaticMeshComponent* staticMeshComponent, UWorld* world);

	FORCEINLINE								TArray<UStaticMeshComponent*> GetMeshComponents() { return MeshComponents; }
	TEXTUREGRAPHENGINE_API virtual AsyncActionResultPtr			Load() override;
	virtual FMatrix							LocalToWorldMatrix() const override { return MeshComponents[0]->GetComponentTransform().ToMatrixWithScale(); }
	TEXTUREGRAPHENGINE_API virtual void							PrepareForRendering(UWorld* world, FVector scale) override;
	TEXTUREGRAPHENGINE_API virtual void							SetMaterial(UMaterialInterface* material) override;
};

typedef std::shared_ptr<RenderMesh_Editor> RenderMesh_EditorPtr;
#endif
