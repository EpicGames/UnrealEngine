// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_EDITOR
#include "RenderMesh_EditorScene.h"
#include "CoreMesh.h"
#include "MeshInfo.h"
#include "KismetProceduralMeshLibrary.h"

RenderMesh_EditorScene::RenderMesh_EditorScene()
{
	_isPlane = false;
}

RenderMesh_EditorScene::RenderMesh_EditorScene(RenderMesh* parent, TArray<MeshInfoPtr> meshes, MaterialInfoPtr matInfo) : RenderMesh_Editor(parent, meshes, matInfo)
{

}

RenderMesh_EditorScene::RenderMesh_EditorScene(AActor* actor) :
	MeshActor(actor)
{
	MeshActor->GetComponents<UStaticMeshComponent>(MeshComponents);
}

void RenderMesh_EditorScene::PrepareForRendering(UWorld* world, FVector scale)
{
	if (_parentMesh)
		_parentMesh->SetViewScale(scale);

	_viewScale = scale;
	//SpawnActors(_world);
	UpdateMeshTransforms();
}
AsyncActionResultPtr RenderMesh_EditorScene::Load()
{
	// Need to figure out how we are going to support MTS and UDIM meshes.
	_meshSplitType = MeshSplitType::Single;

	if (MeshComponents.Num() == 0)
	{
		MeshActor->GetComponents<UStaticMeshComponent>(MeshComponents);
	}

	if (GetMeshComponents().Num() <= 0)
		return cti::make_exceptional_continuable<ActionResultPtr>(std::make_exception_ptr(std::runtime_error("No mesh components found on actor")));


	return cti::make_continuable<ActionResultPtr>([this](auto&& promise)
		{
			Util::OnGameThread([this, promise = std::forward<decltype(promise)>(promise)]() mutable
			{
				try
				{
					LoadInternal();
					promise.set_value(ActionResultPtr(new ActionResult(nullptr)));
					_meshActors.Add(MeshActor);
				}
				catch (std::exception_ptr ex)
				{
					promise.set_exception(ex);
				}
			});
		});


}

void RenderMesh_EditorScene::RemoveActors()
{
	//Do nothing since we are not the owner of the actors here
}
#endif // WITH_EDITOR