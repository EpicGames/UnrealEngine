// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#if WITH_EDITOR
#include "CoreMinimal.h"
#include "RenderMesh_Editor.h"

struct CoreMesh;

class RenderMesh_EditorScene: public RenderMesh_Editor
{
private:
	AActor*									MeshActor;				/// Actor that this rendermesh is associated to

public:
											TEXTUREGRAPHENGINE_API RenderMesh_EditorScene();
											TEXTUREGRAPHENGINE_API RenderMesh_EditorScene(AActor* actor);
											TEXTUREGRAPHENGINE_API RenderMesh_EditorScene(RenderMesh* parent, TArray<MeshInfoPtr> meshes, MaterialInfoPtr matInfo);

	TEXTUREGRAPHENGINE_API AsyncActionResultPtr					Load();
	TEXTUREGRAPHENGINE_API void									PrepareForRendering(UWorld* world, FVector scale) override;
	TEXTUREGRAPHENGINE_API void									RemoveActors() override;

};

typedef std::shared_ptr<RenderMesh_EditorScene> RenderMesh_EditorScenePtr;
#endif
