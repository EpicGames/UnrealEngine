// Copyright Epic Games, Inc. All Rights Reserved.
#include "RenderMesh_Procedural.h"
#include "CoreMesh.h"
#include "MeshInfo.h"
#include "Helper/MathUtils.h"

RenderMesh_Procedural::RenderMesh_Procedural(const MeshLoadInfo loadInfo)
{
	_isPlane = true;
	_meshSplitType = MeshSplitType::Single;
	_tesselation = loadInfo.tesselation;
	_dimension = loadInfo.dimension;
	_offSet = FVector::ZeroVector;
}

AsyncActionResultPtr RenderMesh_Procedural::Load()
{
	
	return cti::make_continuable<ActionResultPtr>([this](auto&& promise)
		{
			AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, promise = std::forward<decltype(promise)>(promise)]() mutable
			{
				try
				{
					LoadInternal();
					promise.set_value(ActionResultPtr(new ActionResult(nullptr)));
				}
				catch (std::exception_ptr ex)
				{
					promise.set_exception(ex);
				}
			});
		});


}

void RenderMesh_Procedural::LoadInternal()
{
	CoreMeshPtr cmeshPtr = std::make_shared<CoreMesh>();
	MeshInfoPtr meshInfo = std::make_shared<MeshInfo>(cmeshPtr);

	GenerateProcedural(_tesselation, _dimension, cmeshPtr);
	_meshes.Add(meshInfo);
}

void RenderMesh_Procedural::GenerateProcedural(int tesselation, FVector2D dimension, CoreMeshPtr cmesh)
{
	
	const float bottomLeft = -MathUtils::GMeterToCm;
	const float topRight = MathUtils::GMeterToCm;

	float minX = bottomLeft * (dimension.X/2.0F);
	float maxX = topRight * (_dimension.X/2.0F);

	float minY = bottomLeft * (dimension.Y / 2.0F);
	float maxY = topRight * (_dimension.Y / 2.0F);

	cmesh->bounds = FBox::BuildAABB(FVector::ZeroVector, FVector::ZeroVector);
	cmesh->bounds.Max = FVector(FVector2D(maxX, maxY), 0);
	cmesh->bounds.Min = FVector(FVector2D(minX, minY), 0);

	TArray<FVector> vertices;
	TArray<FVector> normals;
	TArray<FVector2D> uvs;
	TArray<int32> triangles;
	TArray<FProcMeshTangent> tangents;
	int rowSize = FMath::RoundToInt(FMath::Pow(2.0f, FMath::Clamp<float>(tesselation, 0.0f, 10.0f)));
	float divisionX = (maxX*2.0f) / (float)rowSize;
	float divisionY = (maxY*2.0f) / (float)rowSize;
	//-128,128 = 0,0
	//128,128 = 1,0
	//-128,-128 =0,1
	//128,-128 = 1,1
	int width = maxX * 2;
	int height = maxY * 2;

	FProcMeshTangent tangent(1.f, 0.f, 0.f);
	for (float y = minY; y <= maxY; y+=divisionY) {
		for (float x = minX; x <= maxX; x+=divisionX) {
			FVector vertex(float(x), float(y), 0.f);

			vertices.Add(vertex);
			normals.Add(FVector (0, 0, 1));
			float uvX = (float(x) + minX) / -width;
			float uvY = (float(y) - minY) / height;

			uvs.Add(FVector2D(uvY,uvX));
			//uvs.Add(FVector2D((float(x) - min) / rowSize,(float(y) + min) / -rowSize ));
			//UE_LOGF(LogTemp, Warning, "%d x, %d y has u.v %f u %f v", x, y, (float(x) - min) / rowSize, (float(y) - min) / rowSize);
			tangents.Add(tangent);
		}
	}

	for (int vertexIndex = 0, y = 0; y < rowSize; y++, vertexIndex++) 
	{
		for (int x = 0; x < rowSize; x++, vertexIndex++) 
		{
			triangles.Add(vertexIndex);
			triangles.Add(vertexIndex + rowSize + 1);
			triangles.Add(vertexIndex + 1);
			triangles.Add(vertexIndex + 1);
			triangles.Add(vertexIndex + rowSize + 1);
			triangles.Add(vertexIndex + rowSize + 2);
		}
	}

	cmesh->vertices = vertices;
	cmesh->normals = normals;
	cmesh->tangents = tangents;
	cmesh->triangles = triangles;
	cmesh->uvs = uvs;

	_originalBounds = cmesh->bounds;
}
