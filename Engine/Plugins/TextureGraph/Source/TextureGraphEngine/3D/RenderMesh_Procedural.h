// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "3D/RenderMesh.h"
#include "Helper/Promise.h"

#define UE_API TEXTUREGRAPHENGINE_API

struct CoreMesh;
typedef std::shared_ptr<CoreMesh>	CoreMeshPtr;


class RenderMesh;

class RenderMesh_Procedural: public RenderMesh
{
private:
	int										_tesselation = 32;
	FVector2D								_dimension= FVector2D::UnitVector;
	FVector									_offSet;
	
public:
	UE_API explicit								RenderMesh_Procedural(const MeshLoadInfo loadInfo);
	UE_API virtual AsyncActionResultPtr			Load() override;
	UE_API virtual void							LoadInternal() override;
	UE_API void									GenerateProcedural(int tesselation, FVector2D dimension, CoreMeshPtr cmesh);

	//////////////////////////////////////////////////////////////////////////
	////Inline Functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE	int&						Tesselation() { return _tesselation; }
};

typedef std::shared_ptr<RenderMesh_Procedural> RenderMesh_ProceduralPtr;

#undef UE_API
