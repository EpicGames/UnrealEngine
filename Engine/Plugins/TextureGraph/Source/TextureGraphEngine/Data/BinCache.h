// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
//#include "Containers/LruCache.h"
#include "Data/LruCache.h"

//////////////////////////////////////////////////////////////////////////
template <typename T_Key, typename T_Value>
class BinCache
{
	typedef LruCache<T_Key, T_Value> LRUCache;

	mutable FCriticalSection		CacheLock;				/// Mutex for the actions (mutable because FScopeLock needs non-const object)
	LRUCache						Cache;					/// The buffer cache that we have

public:
	BinCache(int32 MaxElements = 100000) : Cache(MaxElements)
	{
	}

	~BinCache() = default;

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE const T_Value* Find(T_Key Key, bool bTouch)
	{
		const T_Value* Value = nullptr;

		if (bTouch)
			Value = Cache.FindAndTouch(Key);
		else
			Value = Cache.Find(Key);

		return Value;
	}

	FORCEINLINE void UpdateValue(T_Key Key, T_Value NewValue)
	{
		Cache.Add(Key, NewValue);
	}

	/// Thread-Safe touch
	FORCEINLINE void TouchThreadSafe(T_Key Key)
	{
		FScopeLock Lock(&CacheLock);
		Touch(Key);
	}

	FORCEINLINE void Touch(T_Key Key)
	{
		Cache.FindAndTouch(Key);
	}

	FORCEINLINE T_Value RemoveThreadSafe(T_Key Key)
	{
		FScopeLock Lock(&CacheLock);
		return Remove(Key);
	}

	FORCEINLINE T_Value Remove(T_Key Key)
	{
		T_Value* ValuePtr = Cache.Find(Key);
		T_Value Value;

		if (ValuePtr)
			Value = *ValuePtr;

		Cache.Remove(Key);
		return Value;
	}

	FORCEINLINE void InsertThreadSafe(T_Key Key, T_Value Value)
	{
		FScopeLock Lock(&CacheLock);
		Insert(Key, Value);
	}

	FORCEINLINE void Insert(T_Key Key, T_Value Value)
	{
		Cache.Add(Key, Value);
	}

	FORCEINLINE void Empty()
	{
		Cache.Empty(Cache.Max());
	}

	FORCEINLINE const LRUCache&		GetCache() const { return Cache; }
	FORCEINLINE LRUCache&			GetCache() { return Cache; }
	FORCEINLINE FCriticalSection*	GetMutex() { return &CacheLock; }
};
