// Copyright Epic Games, Inc. All Rights Reserved.
#include "Blob.h"
#include "Device/DeviceBuffer.h"
#include "TextureGraphEngine.h"
#include "Device/Mem/Device_Mem.h"
#include "Transform/BlobTransform.h"
#include "Device/DeviceManager.h"
#include "TextureGraphEngine.h"
#include "Device/Null/Device_Null.h"
#include "Model/Mix/Mix.h"
#include "Blobber.h"

#include "Job/JobBatch.h"

const char* Blob::LODTransformName = "LOD";

#if DEBUG_BLOB_REF_KEEPING == 1
DebugBlobLock::DebugBlobLock()
{
	if (!TextureGraphEngine::Instance() || !TextureGraphEngine::Blobber())
		return;

	TextureGraphEngine::Blobber()->GetDebugBlobMutex()->Lock();
}

DebugBlobLock::~DebugBlobLock()
{
	if (!TextureGraphEngine::Instance() || !TextureGraphEngine::Blobber())
		return;

	TextureGraphEngine::Blobber()->GetDebugBlobMutex()->Unlock();
}
#endif 

//////////////////////////////////////////////////////////////////////////
Blob::Blob() : Buffer(Device_Null::Get()->Create(BufferDescriptor(), nullptr))
{
}

Blob::Blob(DeviceBufferRef InBuffer) 
	: Buffer(InBuffer)
{
	if (InBuffer && InBuffer->IsValid() && !InBuffer->IsNull())
	{
		bIsFinalised = true;
		FinaliseTS = FDateTime::Now();
	}
}

/// Allocate a NULL device buffer by default
Blob::Blob(const BufferDescriptor& InDesc, CHashPtr InHash) : Buffer(Device_Null::Get()->Create(InDesc, InHash))
{
}

Blob::~Blob()
{
#if DEBUG_BLOB_REF_KEEPING == 1

	//if (!TextureGraphEngine::IsTestMode() && !TextureGraphEngine::IsDestroying())
	//{
	//	DebugBlobLock lock;

	//	if (!_owners.empty())
	//	{
	//		for (size_t i = 0; i < _owners.size(); i++)
	//		{
	//			BlobPtrW tblobW = _owners[i];

	//			if (!tblobW.expired())
	//			{
	//				std::shared_ptr<TiledBlob> tblob = std::static_pointer_cast<TiledBlob>(tblobW.lock());
	//				check(!tblob->HasBlobAsTile(this));
	//			}
	//		}
	//	}
	//}
	//if (TextureGraphEngine::Blobber())
	//{
	//	check(!TextureGraphEngine::Blobber()->IsBlobReferenced(this));
	//}
#endif 
}

#if DEBUG_BLOB_REF_KEEPING == 1
void Blob::AddTiledOwner(BlobPtrW owner)
{
	if (HasOwner(owner))
		return;

	_owners.push_back(owner);
}

void Blob::AddOwner(BlobPtrW owner)
{
	if (HasOwner(owner))
		return;

	_owners.push_back(owner);
}

void Blob::RemoveOwner(BlobPtrW owner)
{
	auto iter = FindOwner(owner);
	if (iter == _owners.end())
		return;

	_owners.erase(iter);
}

void Blob::RemoveOwner(Blob* owner)
{
	auto iter = FindOwner(owner);
	if (iter == _owners.end())
		return;

	_owners.erase(iter);
}

Blob::OwnerList::iterator Blob::FindOwner(BlobPtrW owner) 
{
	return std::find_if(_owners.begin(), _owners.end(), [&owner](const BlobPtrW& rhs) 
	{ 
		return owner.lock() == rhs.lock(); 
	});
}

Blob::OwnerList::iterator Blob::FindOwner(Blob* owner) 
{
	return std::find_if(_owners.begin(), _owners.end(), [&owner](const BlobPtrW& rhs) 
	{ 
		return owner == rhs.lock().get();
	});
}

bool Blob::HasOwner(Blob* owner)
{
	return FindOwner(owner) != _owners.end();
}

bool Blob::HasOwner(BlobPtrW owner) 
{
	return FindOwner(owner) != _owners.end();
}
#endif 

FString Blob::DisplayName() const
{
	if (!Buffer)
		return TEXT("");
	return Buffer->GetName();
}

void Blob::Touch(uint64 BatchId)
{
	// Just update the access info for the time being
	if (Buffer)
	{
		Buffer->UpdateAccessInfo(BatchId);

		// Don't touch the buffer here. This will be done in the blobber idle loop 
		// Buffer->Touch(BatchId);
		TextureGraphEngine::GetBlobber()->Touch(this);
	}
}

void Blob::UpdateAccessInfo(uint64 batchId)
{
	if (Buffer)
		Buffer->UpdateAccessInfo(batchId);
}

DeviceBufferRef Blob::GetBufferRef() const
{ 
	return Buffer; 
}

void Blob::SetHash(CHashPtr Hash)
{
	if (Buffer)
		Buffer->SetHash(Hash);
}

AsyncDeviceBufferRef Blob::TransferTo(Device* TargetDevice)
{
	/// Must have a valid buffer here
	check(Buffer && Buffer->IsValid());

	DeviceBufferRef ExistingBuffer = Buffer;
	return TargetDevice->Transfer(Buffer).then([this, TargetDevice, ExistingBuffer](DeviceBufferRef NewBuffer) mutable 
	{
		check(NewBuffer);
		check(NewBuffer != ExistingBuffer);

		/// At this point, the old buffer can deallocate
		Buffer = NewBuffer;

		return Buffer;
	});
}

AsyncBufferResultPtr Blob::Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo)
{
	/// Ok, we must have a RawObj buffer over here to transfer over to the device buffer
	//check(_buffer);

	Device* Dev = BindInfo.Dev;
	if (!Dev)
		Dev = Buffer->GetOwnerDevice();

	check(Dev);

	/// Touch this buffer if we're not tiled ... TiledBlob must touch its own
	/// buffers because it involves touching all the tiles as well ... which may 
	/// be un-necessary in a lot of cases
	if (!IsTiled())
		Touch(BindInfo.BatchId);

	/// If the buffer isn't compatible then we transfer it over to the other new device
	if (!Buffer->IsCompatible(Dev))
	{
		return Dev->Transfer(Buffer).then([this, Transform, BindInfo](DeviceBufferRef Result)
		{
			Buffer = Result;
			return Buffer->Bind(Transform, BindInfo);
		});
	}

	/// Ok now we can actually bind this
	return Buffer->Bind(Transform, BindInfo);
}

AsyncBufferResultPtr Blob::Unbind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo)
{
	return Buffer->Unbind(Transform, BindInfo);
}

bool Blob::IsValid() const
{
	return Buffer->IsValid();
}

AsyncRawBufferPtr Blob::Raw()
{
	/// Could possibly be going to another thread
	/// Save the current temp Hash
	CHashPtr PrevHash = Buffer->Hash(false);

	return Buffer->Raw().then([this, PrevHash](RawBufferPtr RawObj)
	{
		const BufferDescriptor& BufferDesc = Buffer->Descriptor();
		CHashPtr Hash = Buffer->Hash(false);

		if (!BufferDesc.bIsTransient && Hash->IsFinal() && (!PrevHash || !PrevHash->IsFinal()))
			Buffer = Buffer->GetOwnerDevice()->AddInternal(Buffer);
		else
		{
			HashType PrevHashValue = PrevHash ? PrevHash->Value() : DataUtil::GNullHash;
			UE_LOGF(LogDevice, VeryVerbose, "DeviceBuffer has a new Hash without owning reference. Unless this is manually cached by the device, this buffer will be deleted which is undesirable. Name: %ls, Hash: %llu [Prev Hash: %llu, Size: %dx%d]",
				*BufferDesc.Name, PrevHashValue, Hash->Value(), BufferDesc.Width, BufferDesc.Height);
		}

		return Buffer->Raw_Now();
	});
}

AscynCHashPtr Blob::CalcHash()
{
	/// If we already have a have a Hash for the buffer then don't bother
	if (Buffer)
	{
		CHashPtr BufferHash = Buffer->Hash(false);
		if (BufferHash && BufferHash->IsFinal())
			return cti::make_ready_continuable(BufferHash);
	}

	return Raw().then([this] 
	{ 
		return Buffer->Hash(false);
	});
}

bool Blob::IsNull() const
{
	return Buffer->IsNull();
}

CHashPtr Blob::Hash() const
{
	/// Use own address as Hash. This will still ensure that we detect object re-use
	/// even if they're late bound or haven't been calculated yet
	return Buffer->Hash(false);
}

void Blob::OnFinaliseInternal(BlobReadyCallback Callback) const
{
	Callback(this);
}

AsyncBlobResultPtr Blob::OnFinalise() const
{
	/// If already finalised then just return a fulfilled promise
	if (IsFinalised())
		return cti::make_ready_continuable(this);

	return cti::make_continuable<const Blob*>([this](auto&& Promise) 
	{
		OnFinaliseInternal([this, FWD_PROMISE(Promise)](const Blob* BlobObj) mutable
		{
			Promise.set_value(BlobObj);
		});
	});
}

AsyncBufferResultPtr Blob::Flush(const ResourceBindInfo& BindInfo)
{
	/// We've got nothing to 
	check(Buffer);
	return Buffer->Flush(BindInfo);
}

void Blob::ResetBuffer()
{
	if (Buffer)
	{
		/// Release the native buffer information
		Buffer->ReleaseNative();

		Buffer = Device_Null::Get()->Create(Buffer->Descriptor(), nullptr);
		UE_LOGF(LogData, Log, "Resetting buffer: %ls", *DisplayName());
	}
	else
		Buffer = Device_Null::Get()->Create(BufferDescriptor(), nullptr);
}

AsyncPrepareResult Blob::PrepareForWrite(const ResourceBindInfo& BindInfo)
{
	check(!bIsFinalised || BindInfo.bIsCombined);
	check(Buffer);

	if(BindInfo.bIsCombined)
		Buffer->Desc = GetDescriptor();

	return Buffer->PrepareForWrite(BindInfo);
}

bool Blob::HasMinMax() const
{
	return (MinValue && MaxValue) || (MinMax != nullptr);
}

float Blob::GetMinValue() const
{
	check(MinValue);
	return *MinValue;
}

float Blob::GetMaxValue() const
{
	check(MaxValue);
	return *MaxValue;
}

BlobPtr Blob::GetMinMaxBlob()
{
	check(MinMax);
	return MinMax;
}

void Blob::SetMinMax(BlobPtr InMinMax)
{
	MinMax = InMinMax;

	InMinMax->OnFinalise()
		.then([this](const Blob* FinalisedBlob)
		{
			check(FinalisedBlob);
			return MinMax->Raw();
		})
		.then([this](RawBufferPtr Raw)
		{
			check(Raw);
			const float* data = reinterpret_cast<const float*>(Raw->GetData());

			check(data);
			check(Raw->GetDescriptor().Width == 1 && Raw->GetDescriptor().Height == 1);

			MinValue = std::make_shared<float>(data[0]);
			MaxValue = std::make_shared<float>(data[1]);

			/// We don't need to keep this anymore
			MinMax = nullptr;
		});
}

int32 Blob::NumLODLevels() const
{
	return (int32)LODLevels.size();
}

bool Blob::HasLODLevels() const
{
	return LODLevels.size() > 0;
}

bool Blob::HasLODLevel(int32 Index) const
{
	return Index >= 0 && Index <= (int32)LODLevels.size() && !LODLevels[Index - 1].expired();
}

BlobPtrW Blob::GetLODLevel(int32 Level)
{
	check(Level > 0 && Level <= (int32)LODLevels.size());
	return LODLevels[Level - 1];
}

void Blob::SetLODLevel(int32 Level, BlobPtr LODBlob, BlobPtrW LODParentBlob, BlobPtrW LODSourceBlob, bool bAddToBlobber)
{
	check(LODBlob);
	check(!LODParentBlob.expired());
	check(!LODSourceBlob.expired());

	LODParent = LODParentBlob;
	LODSource = LODSourceBlob;

	LODBlob->bIsLODLevel = true;

	if (LODLevels.empty())
	{
		/// -1 because we don't wanna keep a weak pointer of Mip Level 0. 'this' is supposed to 
		/// the the mip Level 0, so there's no point keeping it in the mip chain
		int32 numLevels = TextureHelper::CalcNumMipLevels(std::max(GetWidth(), GetHeight())) - 1;

		if (numLevels > 0)
			LODLevels.resize(numLevels);
	}

	check(Level > 0 && Level <= (int32)LODLevels.size());
	check(LODBlob->GetWidth() < GetWidth() && LODBlob->GetHeight() < GetHeight());

	/// Make sure we're not replacing an existing lod Level
	//check(_lodLevels[Level - 1].expired() || (!_lodLevels[Level - 1].expired() && _lodLevels[Level - 1].lock() == blob));

	LODLevels[Level - 1] = LODBlob;

	BlobPtr ThisBlob = shared_from_this();

	/// Add hashes to the blobber
	if (bAddToBlobber)
	{
#if 0 /// TODO
		CHashPtr lodHash = Blob::CalculateMipHash(Hash(), Level);
		TextureGraphEngine::Blobber()->AddResult(lodHash, blob);

		CHashPtr lodHash_0 = Blob::CalculateMipHash(Hash(), 0);
		TextureGraphEngine::Blobber()->AddResult(lodHash_0, thisBlob);
#endif /// 

		//CHashPtrVec thisLodHashes = Blob::CalculateMipHashes(lodSource.lock()->Hash(), lodParent.lock()->Hash(), 0);
		//for (CHashPtr& lodHash : thisLodHashes)
		//{
		//	TextureGraphEngine::Blobber()->AddResult(lodHash, thisBlob);
		//}
	}

	for (int32 li = 0; li < Level - 1; li++)
	{
		BlobPtr lodLevel = LODLevels[li].lock();

		if (lodLevel)
		{
			lodLevel->SetLODLevel(Level - li - 1, LODBlob, ThisBlob, LODSourceBlob, bAddToBlobber);
		}
	}
}

void Blob::UpdateLinkedBlobs(bool bDoFinalise)
{
	check(IsInGameThread());

	for (BlobPtrW LinkedBlobW : LinkedBlobs)
	{
		BlobPtr LinkedBlob = LinkedBlobW.lock();

		if (LinkedBlob && bDoFinalise)
		{
			LinkedBlob->FinaliseFrom(this);
		}
	}
}

BlobPtr	Blob::GetHistogram() const
{ 
	return Histogram ? Histogram : std::static_pointer_cast<Blob>(TextureHelper::GetBlack()); 
}

void Blob::AddLinkedBlob(BlobPtr LinkedBlob)
{
	if (IsFinalised())
	{
		LinkedBlob->FinaliseFrom(this);
		return;
	}
	LinkedBlobs.push_back(LinkedBlob);
}

void Blob::SyncWith(Blob* RHS)
{
	/// Synchronise the histograms
	if (!Histogram && RHS->Histogram)
		Histogram = RHS->Histogram;
	else if (Histogram && !RHS->Histogram)
		RHS->Histogram = Histogram;
	else if (Histogram && RHS->Histogram)
	{
		/// Sync based on whatever has been finalised
		if (RHS->Histogram->IsFinalised() && !Histogram->IsFinalised())
			Histogram->FinaliseFrom(RHS->Histogram.get());
		else if (Histogram->IsFinalised() && !RHS->Histogram->IsFinalised())
			RHS->Histogram->FinaliseFrom(Histogram.get());

		/// If both have been finalised then there's nothing we should do.
		/// We have two copies within the system and in due course, one
		/// of these will get deleted
	}
}

void Blob::FinaliseFrom(Blob* RHS)
{
	// We are only saving the buffer transient state when buffer is valid.
	// This buffer will be empty in case of non single blob.
	const bool HadValidBuffer = Buffer && Buffer->IsValid();
	const bool bIsTransient = Buffer && Buffer->IsValid() && Buffer->IsTransient();

	// We need to save the transient state before copying the RHS buffer.
	Buffer = RHS->Buffer;

	// Only try to update the buffer transient state when we found a valid buffer from RHS.
	if(HadValidBuffer && Buffer && Buffer->IsValid())
	{
		Buffer->Desc.bIsTransient &= bIsTransient;		
	}
	
	LODLevels = RHS->LODLevels;
	MinMax = RHS->MinMax;
	MinValue = RHS->MinValue;
	MaxValue = RHS->MaxValue;

	LODParent = RHS->LODParent;
	LODSource = RHS->LODSource;
	bIsLODLevel = RHS->bIsLODLevel;

	SyncWith(RHS);
	FinaliseNow(true, nullptr);
}

void Blob::FinaliseNow(bool bNoCalcHash, CHashPtr FixedHash)
{
	/// If already finalised then nothing to do over here
	if (bIsFinalised)
	{
		return;
	}

	check(IsInGameThread());

	bIsFinalised = true;
	FinaliseTS = FDateTime::Now();

	UpdateLinkedBlobs(true);
}

AsyncBufferResultPtr Blob::Finalise(bool bNoCalcHash, CHashPtr FixedHash)
{
	FinaliseNow(bNoCalcHash, FixedHash);
	return cti::make_ready_continuable(std::make_shared<BufferResult>());
}

CHashPtr Blob::CalculateMipHash(CHashPtr MainHash, int32 Level)
{
	CHashPtrVec Sources =
	{
		MainHash,
		std::make_shared<CHash>(DataUtil::Hash_Int32(Level), true),
		std::make_shared<CHash>(DataUtil::Hash_GenericString_Name(FString(LODTransformName)), true)
	};

	return CHash::ConstructFromSources(Sources);
}

CHashPtrVec Blob::CalculateMipHashes(CHashPtr MainHash, CHashPtr ParentHash, int32 Level)
{
	CHashPtr MainHashAtLOD = CalculateMipHash(MainHash, Level);
	CHashPtr ParentHashAtLOD = CalculateMipHash(ParentHash, Level);

	/// Just return one unique Hash
	if (MainHashAtLOD->Value() != ParentHashAtLOD->Value())
		return { MainHashAtLOD, ParentHashAtLOD };

	return { MainHashAtLOD };
}

//////////////////////////////////////////////////////////////////////////
