// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RawBuffer.h"
#include "Device/DeviceBuffer.h"
#include "Helper/DataUtil.h"
#include "2D/TextureHelper.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Blob;
class Device;
class RawBuffer;
class BlobTransform;
class Blobber;
struct ResourceBindInfo;

typedef TUniqueFunction<void(const Blob*)> BlobReadyCallback;
typedef cti::continuable<const Blob*> AsyncBlobResultPtr;

typedef std::shared_ptr<Blob>	BlobPtr;
typedef std::unique_ptr<Blob>	BlobUPtr;
typedef std::weak_ptr<Blob>		BlobPtrW;
typedef std::vector<BlobPtr>	BlobPtrVec;
typedef std::vector<BlobPtrW>	BlobPtrWVec;

#if defined(DEBUG_BLOB_REF_KEEPING) && DEBUG_BLOB_REF_KEEPING == 1
struct DebugBlobLock
{
public:
	DebugBlobLock();
	~DebugBlobLock();
};
#endif 

class Blob : public std::enable_shared_from_this<Blob>
{
	friend class Blobber;
	friend class TiledBlob;
	friend class TiledBlob_Promise;

public:
	UE_API static const char*				LODTransformName;	
	typedef std::vector<BlobPtrW>	OwnerList;
	typedef std::vector<std::weak_ptr<Blob>> LinkedBlobsVec;

protected:
	DeviceBufferRef					Buffer;							/// Device buffer that is attached to this blob
	bool							bIsFinalised = false;			/// Whether the promise has been finalised or not
	int32							ReplayCount = 0;				/// For debug purpose, Incremented when replaying a job and regenerating the content of the blob.
	FDateTime						FinaliseTS = FDateTime::Now();	/// Time when the blob was finalised

	BlobPtrWVec						LODLevels;						/// The mip chain for this blob (for textures). Can also be Mesh data with fewer triangles etc. Doesn't matter to us

	BlobPtr							MinMax;							/// The blob that's used for calculating min/max
	std::shared_ptr<float>			MinValue;						/// Extracted min value (this is not always present)
	std::shared_ptr<float>			MaxValue;						/// Extracted min value (this is not always present)

	BlobPtr							Histogram;						/// The blob that's used for storing histogram data. not always have a histogram only created when required.

	BlobPtrW						LODParent;						/// The parent of this blob. Essentially the blob that was used to generate THIS lod Level
	BlobPtrW						LODSource;						/// The original source of this blob, the mip Level 0

	bool							bIsLODLevel = false;			/// Whether this blob already represents an LOD'd blob
	LinkedBlobsVec					LinkedBlobs;					/// Blobs that are linked to this tiled blob. These are objects that are 
																	/// uniquely created within the system but eventually resolve to the same
																	/// tiled blob. We need to ensure that these are resolved correctly
																	/// so that if some object is holding a pointer to a particular TiledBlob_Promise
																	/// it gets the same view upon the completion.
																	/// This is usually very uncommon with Blobs but more of a possibility with TiledBlob_Promise
																	/// but we need to ensure that it works correctly for all blobs

#if defined(DEBUG_BLOB_REF_KEEPING) && DEBUG_BLOB_REF_KEEPING == 1
	OwnerList							_owners;					/// The TiledBlobs that contain this blob as a tile
	UE_API virtual void					AddTiledOwner(BlobPtrW owner);
#endif 

	UE_API 								Blob();
	UE_API virtual void					OnFinaliseInternal(BlobReadyCallback callback) const;
	UE_API virtual void					ResetBuffer();
	UE_API virtual void					Touch(uint64 BatchId);
	UE_API virtual void					UpdateAccessInfo(uint64 batchId);
	UE_API virtual void					SetHash(CHashPtr Hash);

	UE_API virtual void					UpdateLinkedBlobs(bool bDoFinalise);
	UE_API virtual void					AddLinkedBlob(BlobPtr LinkedBlob);

	UE_API virtual void					FinaliseFrom(Blob* RHS);
	UE_API virtual void					SyncWith(Blob* RHS);

public:
	UE_API 								Blob(DeviceBufferRef InBuffer);
	UE_API 								Blob(const BufferDescriptor& InDesc, CHashPtr InHash);
		 								Blob(const Blob& InBlob) = delete; /// non-copyable

	UE_API virtual						~Blob();

	UE_API virtual AsyncBufferResultPtr	Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo);
	UE_API virtual AsyncBufferResultPtr	Unbind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo);
	UE_API virtual bool					IsValid() const;
	UE_API virtual bool					IsNull() const;
	UE_API virtual CHashPtr				Hash() const;
	//UE_API virtual void				SetHash(CHashPtr hash);
	UE_API virtual AsyncBufferResultPtr	Flush(const ResourceBindInfo& BindInfo);
	UE_API virtual AsyncPrepareResult	PrepareForWrite(const ResourceBindInfo& BindInfo);

#if 0 /// TODO /// Do we really need this?
	UE_API virtual AsyncBufferResultPtr	PrepareForWrite(const ResourceBindInfo& BindInfo);
#endif 

	UE_API virtual FString				DisplayName() const;
	virtual bool						IsFinalised() const { return bIsFinalised; }
	virtual bool						IsTiled() const { return false; }
	virtual bool						IsPromise() const { return false; }
	virtual bool						CanCalculateHash() const { return IsFinalised(); }

	UE_API virtual void					FinaliseNow(bool bNoCalcHash, CHashPtr FixedHash);
	UE_API virtual AsyncBufferResultPtr	Finalise(bool bNoCalcHash, CHashPtr FixedHash);

	UE_API virtual AsyncRawBufferPtr	Raw();			/// SLOW: Read doc for DeviceBuffer::Raw() in DeviceBuffer.h
	UE_API virtual AscynCHashPtr		CalcHash();		/// SLOW: Because calls Raw() underneath. Don't use directly

	virtual FString&					Name() { check(Buffer); return Buffer->GetName(); }
	virtual const FString&				Name() const { check(Buffer); return Buffer->GetName(); }
	virtual const BufferDescriptor&		GetDescriptor() const { /*check(_buffer);*/ return Buffer->Descriptor(); }
	
	UE_API virtual AsyncBlobResultPtr	OnFinalise() const;

	UE_API virtual AsyncDeviceBufferRef	TransferTo(Device* TargetDevice);
	UE_API virtual DeviceBufferRef		GetBufferRef() const;

	//////////////////////////////////////////////////////////////////////////
	/// Min/Max related
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual BlobPtr				GetMinMaxBlob();
	UE_API virtual void					SetMinMax(BlobPtr InMinMax);
	UE_API virtual float				GetMinValue() const;
	UE_API virtual float				GetMaxValue() const;
	UE_API virtual bool					HasMinMax() const;
	//////////////////////////////////////////////////////////////////////////
	/// LOD/MipMaps related
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual BlobPtrW				GetLODLevel(int32 Level);
	UE_API virtual bool					HasLODLevels() const;
	UE_API virtual bool					HasLODLevel(int32 Index) const;
	UE_API virtual int32				NumLODLevels() const;
	UE_API virtual void					SetLODLevel(int32 Level, BlobPtr LODBlob, BlobPtrW LODParentBlob, BlobPtrW LODSourceBlob, bool bAddToBlobber); 
	UE_API BlobPtr						GetHistogram() const; 

	//////////////////////////////////////////////////////////////////////////
	/// Ownership related
	//////////////////////////////////////////////////////////////////////////
#if defined(DEBUG_BLOB_REF_KEEPING) && DEBUG_BLOB_REF_KEEPING == 1
	UE_API virtual void					AddOwner(BlobPtrW owner);
	UE_API virtual void					RemoveOwner(BlobPtrW owner);
	UE_API void							RemoveOwner(Blob* owner);
	UE_API virtual bool					HasOwner(BlobPtrW owner);
	UE_API bool							HasOwner(Blob* owner);
	UE_API virtual OwnerList::iterator	FindOwner(BlobPtrW owner);
	UE_API OwnerList::iterator			FindOwner(Blob* owner);
	virtual bool						HasBlobAsTile(Blob* blob) const { return false; };
#endif

	//////////////////////////////////////////////////////////////////////////
	/// Inline function
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE uint32				GetWidth() const { return GetDescriptor().Width; }
	FORCEINLINE uint32				GetHeight() const { return GetDescriptor().Height; }
	FORCEINLINE bool				IsWellDefined() { return !IsLateBound() && GetWidth() > 0 && GetHeight() > 0; }
	//FORCEINLINE const DeviceBufferRef& Buffer() const { return _buffer; }

	FORCEINLINE	DeviceType			GetDeviceType() const { return Buffer.GetDeviceType(); }

	FORCEINLINE int32				GetReplayCount() const { return ReplayCount; } /// For debug purpose, report the replay number
	FORCEINLINE const FDateTime&	FinaliseTimestamp() const { return FinaliseTS; }
	FORCEINLINE bool				IsLateBound() const { return GetDescriptor().Format == BufferFormat::LateBound; }
	FORCEINLINE BlobPtrW			GetLODParent() const { return LODParent; }
	FORCEINLINE BlobPtrW			GetLODSource() const { return LODSource; }
	FORCEINLINE bool				IsLODLevel() const { return bIsLODLevel; }
	FORCEINLINE bool				IsTransient() const 
	{ 
		bool TransientBuffer = false;
		if (Buffer && !Buffer.IsValid())
			TransientBuffer = Buffer->IsTransient();
		return TransientBuffer || GetDescriptor().bIsTransient; 
	}

	//Sets the histogram of the blob 
	FORCEINLINE void				SetHistogram(BlobPtr InHistogram) { check(!Histogram); Histogram = InHistogram; }
	//Check if blob already has a histogram
	FORCEINLINE bool				HasHistogram() const { return Histogram != nullptr; }

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////

	UE_API static CHashPtr			CalculateMipHash(CHashPtr MainHash, int32 Level);
	UE_API static CHashPtrVec		CalculateMipHashes(CHashPtr MainHash, CHashPtr ParentHash, int32 Level);
};


typedef T_Tiles<BlobPtr>			BlobPtrTiles;

#undef UE_API