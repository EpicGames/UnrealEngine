// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "TextureGraphEngine.h"
#include "Blob.h"
#include "Device/DeviceBuffer.h"
#include <memory>

//typename std::enable_if<std::is_base_of<Blob, BlobType>::value, void>::type
#define BLOB_REF_SIMPLE 1
#define DEBUG_BLOB_REF_KEEPING 0

template <class BlobType>
class T_BlobRef 
{
private:
	static_assert(std::is_base_of<Blob, BlobType>::value, "BlobType must be derived from Blob or any of its derived classes.");

	typedef std::shared_ptr<BlobType>	BlobTypePtr;
	typedef std::weak_ptr<BlobType>		BlobTypePtrW;
	typedef T_BlobRef<BlobType>			BlobTypeRef;

#if BLOB_REF_SIMPLE == 0
	mutable BlobTypePtrW			PtrWeak;			/// The weak pointer of the blob type
	mutable CHashPtr				Hash;				/// The actual hash value that was saved at initialisation stage
	mutable bool					KeepStrong = false;	/// Whether we should keep a strong ref or not
	mutable bool					CheckCached = false;
#endif 

	mutable BlobTypePtr				PtrStrong;			/// The strong pointer of the blob type. This is only kept in certain cases when the 
														/// blob doesn't have a well defined hash. Once the hash is evaluated, then the strong
														/// ref can be released

	BlobTypePtr						CheckReleaseStrongRef() const
	{
#if BLOB_REF_SIMPLE == 0
		if (KeepStrong)
			return PtrStrong;

		auto SPtr = PtrStrong;

		/// Get the latest hash
		Hash = PtrStrong->Hash();

		/// If we finally have a hash, then release the strong ref
		if (Hash)
			PtrStrong = nullptr;

		return SPtr;
#else
		return PtrStrong;
#endif 
	}

	void							CheckShouldKeepStrongRef();
	
public:
	T_BlobRef()
	{
	}

	~T_BlobRef()
	{
#if DEBUG_BLOB_REF_KEEPING == 1
		if (PtrStrong && PtrStrong.use_count() == 1)
			check(!TextureGraphEngine::Blobber()->IsBlobReferenced(PtrStrong.get()));
#endif 
	}

#if BLOB_REF_SIMPLE == 0
	T_BlobRef(BlobTypePtrW Ptr, bool CheckBlobber = true) : PtrWeak(Ptr), CheckCached(CheckBlobber)
	{
		if (PtrWeak.expired())
			return;

		auto SPtr = PtrWeak.lock();

		/// Get the hash and save it and it must be a valid hash
		Hash = SPtr->Hash();

		if (CheckBlobber || SPtr->IsTransient())
			CheckShouldKeepStrongRef();

		/// If there is no hash then hold a strong ref
		if (!Hash || KeepStrong)
			PtrStrong = SPtr;
	}

	T_BlobRef(BlobTypePtr Ptr, bool CheckBlobber = true) : PtrWeak(Ptr), CheckCached(CheckBlobber)
	{
		if (!Ptr)
			return;

		Hash = Ptr->Hash();

		if (CheckBlobber || Ptr->IsTransient())
			CheckShouldKeepStrongRef();

		/// If this blob doesn't have a hash, then keep the strong ref
		if (!Hash || KeepStrong)
			PtrStrong = Ptr;
	}

	T_BlobRef(BlobTypePtr Ptr, bool KeepStrong_, bool CheckBlobber) : KeepStrong(KeepStrong_), CheckCached(CheckBlobber)
	{
		check(Ptr);

		if ((!KeepStrong && CheckBlobber) || Ptr->IsTransient())
			CheckShouldKeepStrongRef();

		if (KeepStrong)
			PtrStrong = Ptr;

		PtrWeak = Ptr;
		Hash = Ptr->Hash();

		/// Must have a hash here
		check(Hash);
	}

#else
	T_BlobRef(BlobTypePtr Ptr, bool KeepStrong, bool CheckBlobber = false) : PtrStrong(Ptr)
	{
	}
	T_BlobRef(const BlobTypePtr& Ptr) : PtrStrong(Ptr)
	{
	}

	T_BlobRef(const T_BlobRef<BlobType>& RHS) : PtrStrong(RHS.PtrStrong)
	{
	}
#endif 

	std::shared_ptr<BlobType>		get() const;
	BlobTypePtr						lock() const
	{
		return get();
	}

	bool							expired() const
	{
		return !get();
	}

	BlobType*						operator -> () const
	{ 
		return get().get();
	}

	bool							operator == (const BlobTypeRef& RHS) const { return get() == RHS.get(); }
	bool							operator != (const DeviceBufferRef& RHS) const { return get() != RHS.get(); }
	bool							operator != (const BlobType* RHS) const { return get().get() != RHS; }

	bool							operator == (const BlobType* RHS) const {
		return get().get() == RHS ||
			*GetHash() == RHS->Hash() ||
			*GetHash() == *RHS->Hash();
	}

	BlobTypeRef& operator = (const BlobTypeRef& RHS) const
	{
#if BLOB_REF_SIMPLE == 0
		PtrWeak = RHS.PtrWeak;
		Hash = RHS.Hash;

#if DEBUG_BLOB_REF_KEEPING == 1
		if (PtrStrong)
		{
			if (PtrStrong.use_count() == 1)
				check(!TextureGraphEngine::Blobber()->IsBlobReferenced(PtrStrong.get()));
		}
#endif 
			
		PtrStrong = RHS.PtrStrong;
		KeepStrong = RHS.KeepStrong;
		CheckCached = RHS.CheckCached;

		if (RHS)
		{
			check(!PtrWeak.expired());

			if (!Hash)
				PtrStrong = PtrWeak.lock();
		}
#else
		PtrStrong = RHS.PtrStrong;
#endif
		
		return const_cast<BlobTypeRef&>(*this);
	}

	operator						bool() const { return !!get(); }
	operator						std::shared_ptr<BlobType>() const { return get(); }

	void							Finalise()
	{
#if BLOB_REF_SIMPLE == 0
		get();
		check(!PtrStrong);
		check(Hash);
#endif 
	}

#if BLOB_REF_SIMPLE == 0
	bool							ShouldCheckCached() const { return CheckCached; }
	bool							IsKeepStrong() const { return KeepStrong; }
	FORCEINLINE CHashPtr			GetHash() const { check(get()); return Hash; }
#else
	bool							IsKeepStrong() const { return true; }
	FORCEINLINE CHashPtr			GetHash() const { check(get()); return get()->Hash(); }
#endif 

	bool							IsNull() const { return !get(); }
};

typedef T_BlobRef<Blob>				BlobRef;