// Copyright Epic Games, Inc. All Rights Reserved.
#include "Blobber.h"
#include "Device/Device.h"
#include "TextureGraphEngine.h"
#include "TextureGraphEngineGameInstance.h"
#include "Device/DeviceManager.h"
#include "Job/TempHashService.h"
#include "Job/BlobHasherService.h"
#include "Job/Scheduler.h"
#include "Blob.h"
#include "TiledBlob.h"
#include "FxMat/RenderMaterial_Thumbnail.h"

DEFINE_LOG_CATEGORY(LogBlob);
DECLARE_CYCLE_STAT(TEXT("Blobber_Update"), STAT_Blobber_Update, STATGROUP_TextureGraphEngine);
DECLARE_CYCLE_STAT(TEXT("Blobber_Find"), STAT_Blobber_Find, STATGROUP_TextureGraphEngine);

Blobber::Blobber()
{
	ObserverSource = std::make_shared<BlobberObserverSource>();
	ShouldPrintStats = false;
}

Blobber::~Blobber()
{
	TransientBlobs.clear();
	BlobCache.Empty();
}

Blobber::BlobCacheEntryPtr Blobber::FindInternal(HashType Hash)
{
	if (!bEnableCache)
	{
		UE_LOGF(LogBlob, VeryVerbose, "CACHING DISABLED!");
		return nullptr;
	}

	if (TextureGraphEngine::IsDestroying())
		return nullptr;

	SCOPE_CYCLE_COUNTER(STAT_Blobber_Find);

	UE_LOGF(LogBlob, VeryVerbose, "Blobber Find_Internal: %#016llx", Hash);

	FScopeLock CacheLock(&BlobLookupLock);
	HashType OrgHash = Hash;
	static const int32 MaxIter = 4;

	int32 NumIter = 0;

	while (Hash != DataUtil::GNullHash && NumIter++ < MaxIter)
	{
		check(Hash != DataUtil::GNullHash);
		const BlobCacheEntryPtr* BlobEntry = BlobCache.Find(Hash, false);

		/// if we already have something then we just return that
		if (BlobEntry != nullptr)
		{
			UE_LOGF(LogBlob, VeryVerbose, "Blobber Find_Internal: %#016llu => %ls", Hash, *(*BlobEntry)->BlobObj->Name());
			return *BlobEntry;
		}

		/// Otherwise see if its in the mappings
		{
			FScopeLock HashLock(&HashMutex);

			auto HashIter = HashMappings.find(Hash);

			/// if no mapping was found 
			/// then just ignore it ...
			if (HashIter == HashMappings.end())
			{
				UE_LOGF(LogBlob, VeryVerbose, "Blobber Find_Internal: %#016llx => NONE", Hash);
				return nullptr;
			}

			/// there's a cyclical Hash link, then ignore it as well
			if (HashIter->second->Value() == Hash)
			{
				UE_LOGF(LogBlob, VeryVerbose, "Blobber Find_Internal: %#016llx => NONE", Hash);
				return nullptr;
			}

			/// Otherwise, we need to use the RHS of the mapping
			CHashPtr MappedHash = HashIter->second;
			UE_LOGF(LogBlob, VeryVerbose, "Blobber Find_Internal Remapped: %#016llu => %#016llu", Hash, MappedHash->Value());

			if (Hash == *MappedHash)
			{
				HashMappings.erase(HashIter);
				return nullptr;
			}

			check(Hash != *MappedHash);
			Hash = MappedHash->Value();

			/// This is a cyclical link
			check(Hash != OrgHash);
		}
	}

	return nullptr;
}

bool Blobber::IsBlobCached(HashType Hash) 
{
	BlobCacheEntryPtr Entry = FindInternal(Hash);
	return Entry != nullptr;
}

BlobRef Blobber::FindSingle(HashType Hash)
{
	UE_LOGF(LogBlob, VeryVerbose, "Blobber Find_Single: %#016llx", Hash);

	BlobCacheEntryPtr Entry = FindInternal(Hash);

	if (Entry)
	{
		if (Entry->BlobObj->IsTiled())
		{
			/// This has to be either a 1x1 tiled BlobObj OR 
			TiledBlobPtr TiledResult = Entry->AsTiledBlob();

			if (TiledResult->Rows() == 1 && TiledResult->Cols() == 1)
			{
				BlobPtr Tile = TiledResult->GetTile(0, 0);
				check(Tile);
				return Tile;
			}

			/// A multi tiled BlobObj with exactly the same
			/// image in all the Tiles
			/// In this case we return null as a single BlobObj must be created
			return BlobRef(std::static_pointer_cast<Blob>(TiledResult), false);
		}

		return BlobRef(Entry->BlobObj, false);
	}

	return BlobRef();
}

TiledBlobRef Blobber::FindTiled(HashType Hash)
{
	UE_LOGF(LogBlob, VeryVerbose, "Blobber Find_Tiled: %#016llx", Hash);

	BlobCacheEntryPtr Entry = FindInternal(Hash);

	if (Entry)
	{
		if (!Entry->BlobObj->IsTiled())
		{
			BlobPtrTiles Tiles(1, 1);
			Tiles[0][0] = BlobRef(Entry->BlobObj);

			/// Make the simple BlobObj into a 1x1 tiled BlobObj
			return TiledBlobRef(std::make_shared<TiledBlob>(Entry->BlobObj->GetDescriptor(), Tiles), true, false);
		}

		return TiledBlobRef(Entry->AsTiledBlob(), false);
	}

	return TiledBlobRef();
}

void Blobber::UpdateTransient()
{
	auto Iter = TransientBlobs.begin();
	while (Iter != TransientBlobs.end())
	{
		BlobCacheEntryPtr& Entry = *Iter;
		
		// If this is the only entry then we need to remove it
		if (Entry->BlobObj.use_count() == 1)
		{
			// remove this blob
			auto Hash = Entry->BlobObj->Hash()->Value();
			UE_LOGF(LogBlob, VeryVerbose, "Removing bIsTransient blob %ls (Hash: %llu)", *Entry->BlobObj->DisplayName(), Hash);
			BlobCache.Remove(Hash);
			Iter = TransientBlobs.erase(Iter);
		}
		else
		{
			++Iter;
		}
	}
}

void Blobber::UpdateTouchedHashes()
{
	check(IsInGameThread());

	for (CHashPtr Hash : TouchedHashes)
	{
		auto HashValue = Hash->Value();
		const BlobCacheEntryPtr* Entry = BlobCache.Find(HashValue, true);
		
		if (Entry && *Entry)
		{
			BlobCache.Touch(HashValue);
			(*Entry)->AccessTimestamp = Util::Time();

			auto BlobObj = (*Entry)->BlobObj;
			if (BlobObj && BlobObj->GetBufferRef())
				BlobObj->GetBufferRef()->GetOwnerDevice()->Touch(HashValue);
		}
	}

	TouchedHashes.clear();
}

Blobber::BlobCacheEntry::~BlobCacheEntry()
{
	UE_LOGF(LogBlob, VeryVerbose, "Deleting blob cache entry ...");
}

void Blobber::UpdateBlobCache()
{
#if UE_BUILD_DEBUG
	if (!EnableGC)
	{
		UE_LOGF(LogBlob, Warning, "GC DISABLED!");
		return;
	}
#endif /// UE_BUILD_DEBUG
	check(IsInGameThread());
	
	FScopeLock Lock(&BlobLookupLock);
	auto Iter = BlobCache.GetCache().begin();
	bool bStop = false;
	HashTypeVec HashesToRemove;
	int32 IterCount = 0;
	double PrevTimestamp = -1;

	while (Iter != BlobCache.GetCache().end() && !bStop)
	{
		BlobCacheEntryPtr& Entry = *Iter;
		check(Entry && Entry->BlobObj);

		// if the blob has a single ref count left then we can safely de-cache it
		if (Entry->BlobObj.use_count() == 1)
		{
			UE_LOGF(LogBlob, VeryVerbose, "Removing permanent blob %ls (Hash: %llu)", *Entry->BlobObj->DisplayName(), Entry->BlobObj->Hash()->Value());
			HashesToRemove.push_back(Iter.Key());

			HashTypeVec IntermediateHashes = Entry->BlobObj->Hash()->GetIntermediateHashes();
			if (!IntermediateHashes.empty())
				HashesToRemove.insert(HashesToRemove.end(), IntermediateHashes.begin(), IntermediateHashes.end());

			if (Entry->BlobObj->Hash()->Value() != Iter.Key())
				HashesToRemove.push_back(Entry->BlobObj->Hash()->Value());
		}
		else
		{
			// If the last blob had more than one ref count then any other blobs will certainly be more
			// recently used. So we can safely stop iteration at this point
			//bStop = true;
		}

		double CurrTimestamp = Entry->AccessTimestamp;
		check(PrevTimestamp < 0 || PrevTimestamp <= CurrTimestamp);

		PrevTimestamp = CurrTimestamp;

		IterCount++;
		++Iter;
	}

	if (!HashesToRemove.empty())
	{
		UE_LOGF(LogBlob, VeryVerbose, "Removing num items from the cache: %llu", HashesToRemove.size());
		for (size_t RemoveIndex = 0; RemoveIndex < HashesToRemove.size(); RemoveIndex++)
		{
			HashType HashToRemove = HashesToRemove[RemoveIndex];
			BlobCache.Remove(HashToRemove);
			
			auto MappingIter = HashMappings.find(HashToRemove);
			if (MappingIter != HashMappings.end())
				HashMappings.erase(MappingIter);
		}
	}
}

AsyncJobResultPtr Blobber::UpdateIdle()
{
	if (LastIdleUpdateTimestamp > InvalidateTimestamp)
		return cti::make_ready_continuable<JobResultPtr>(std::make_shared<JobResult>());
	
	// First we update the bIsTransient blobs. This will remove any unnecessary blobs from the cache
	UpdateTransient();

	// Make sure we are updating any touched hashes
	UpdateTouchedHashes();

	// Finally we're going to update the blob cache and see what we can remove
	UpdateBlobCache();

	// Print stats on Idle updates (Temporary and eventually cleaned up, but I want everyone to see the blobber cache for the time being)
	PrintStats();
	
	LastIdleUpdateTimestamp = Util::Time();
	return cti::make_ready_continuable<JobResultPtr>(std::make_shared<JobResult>());
}

void Blobber::Update(float DT)
{
	SCOPE_CYCLE_COUNTER(STAT_Blobber_Update);

	//if (ShouldPrintStats)
	//{
	//	float Delta = Util::TimeDelta(TimeStatsPrinted);
	//	if (Delta > Device::PrintStatsInterval)
	//	{
	//		PrintStats();
	//	}
	//}

	if (!TextureGraphEngine::IsTestMode())
		ObserverSource->Broadcast();
}

void Blobber::PrintStats()
{
	TimeStatsPrinted = Util::Time();
	
	FScopeLock Lock(&BlobLookupLock);
	size_t NumObjects = BlobCache.GetCache().Num();
	
	/// we don't print stats for devices that have no objects at all
	if (!NumObjects)
		return;

	size_t NumTransient = TransientBlobs.size();
	size_t NumThumbnails = 0;
	size_t NumLargeImages = 0;
	float ThumbnailMemUsage = 0;
	float LargeImageMemUsage = 0;
	size_t NumBuffers = 0;
	size_t NumTiled = 0;

	auto Iter = BlobCache.GetCache().begin();

	while (Iter != BlobCache.GetCache().end())
	{
		BlobCacheEntryPtr& Entry = *Iter;
		BlobPtr BlobObj = Entry->BlobObj;
		const BufferDescriptor& Desc = BlobObj->GetDescriptor();
		float* MemUsage = &LargeImageMemUsage;

		if (BlobObj->GetWidth() <= RenderMaterial_Thumbnail::GThumbWidth || BlobObj->GetHeight() <= RenderMaterial_Thumbnail::GThumbHeight)
		{
			NumThumbnails++;
			MemUsage = &ThumbnailMemUsage;
		}
		else
			NumLargeImages++;

		if (BlobObj->IsTiled())
			NumTiled++;

		auto Buffer = BlobObj->GetBufferRef();

		if (Buffer)
		{
			*MemUsage += (float)Buffer->MemSize();
			NumBuffers++;
		}

		++Iter;
	}

	static constexpr float MBConv = 1024.0f * 1024.0f;
	ThumbnailMemUsage = ThumbnailMemUsage / MBConv;
	LargeImageMemUsage = LargeImageMemUsage / MBConv;
	
	UE_LOGF(LogBlob, VeryVerbose, "===== BEGIN Blobber STATS =====");
	
	UE_LOGF(LogBlob, VeryVerbose, "Total Objects    : %llu (Transient: %llu, Tiled: %llu)", NumObjects, NumTransient, NumTiled);
	UE_LOGF(LogBlob, VeryVerbose, "Num Large Images	: %llu (Thumbnails: %llu)", NumLargeImages, NumThumbnails);
	UE_LOGF(LogBlob, VeryVerbose, "Large Image Mem  : %0.2f MB (Thumbnail Mem: %0.2f MB)", LargeImageMemUsage, ThumbnailMemUsage);
	
	UE_LOGF(LogBlob, VeryVerbose, "===== END Blobber STATS   =====");
}

BlobRef Blobber::Create(Device* Dev, RawBufferPtr Raw)
{
	check(Dev);
	check(Raw);
	check(!Raw->Hash()->IsTemp());

	DeviceBufferRef DevBuffer = Dev->Create(Raw);

	return Create(DevBuffer);
}

BlobRef Blobber::Create(DeviceBufferRef DevBuffer, bool NoCache /* = false */)
{
	check(!DevBuffer->Hash()->IsTemp());
	BlobPtr BlobObj = std::make_shared<Blob>(DevBuffer);
	return !NoCache ? AddInternal(BlobObj, BlobCacheOptions()) : BlobRef(BlobObj);
}

BlobRef Blobber::Create(const BufferDescriptor& Desc)
{
	Device* NullDevice = TextureGraphEngine::GetDeviceManager()->GetDevice(DeviceType::Null);
	RawBufferPtr Raw = std::make_shared<RawBuffer>(Desc);
	DeviceBufferRef DevBuffer = NullDevice->Create(Raw);

	BlobPtr BlobObj = std::make_shared<Blob>(DevBuffer);

	return AddInternal(BlobObj, BlobCacheOptions());
}

void Blobber::UpdateGloballyUniqueHashInternal(HashType OldValue)
{
	/// not thread-safe
	auto Iter = GlobalHashes.find(OldValue);

	if (Iter != GlobalHashes.end())
	{
		CHashPtr Hash = Iter->second;

		if (Hash->Value() != OldValue)
		{
			/// Remove the pointer from the old index and make it point to the new value
			GlobalHashes.erase(Iter);
			GlobalHashes[Hash->Value()] = Hash;
		}
	}
}

void Blobber::ClearCache()
{
	check(IsInGameThread());
	check(TextureGraphEngine::IsTestMode()); /// Only allowed in test mode

	FScopeLock Lock(&BlobLookupLock);

#if DEBUG_BLOB_REF_KEEPING == 1
	FScopeLock RefLock(&ReferencedBlobsLock);
	ReferencedBlobs.clear();
#endif 

	TransientBlobs.clear();
	BlobCache.Empty();
	HashMappings.clear();
	GlobalHashes.clear();

	for (size_t di = 0; di < TextureGraphEngine::GetDeviceManager()->GetNumDevices(); di++)
	{
		if (TextureGraphEngine::GetDeviceManager()->GetDevice(di))
			TextureGraphEngine::GetDeviceManager()->GetDevice(di)->ClearCache();
	}
}

#if DEBUG_BLOB_REF_KEEPING == 1
void Blobber::AddReferencedBlob(Blob* BlobObj, JobArg* Arg)
{
	FScopeLock Lock(&ReferencedBlobsLock);

	auto Iter = ReferencedBlobs.find(BlobObj);

	if (Iter == ReferencedBlobs.end())
	{
		DebugRefBlob Ref;
		Ref.RefCount = 1;

		if (Arg)
			Ref.JobArgs.push_back(Arg);

		ReferencedBlobs[BlobObj] = Ref;
	}
	else
	{
		if (Arg)
			Iter->second.JobArgs.push_back(Arg);

		Iter->second.RefCount++;
	}
}

void Blobber::RemoveReferencedBlob(Blob* BlobObj, JobArg* Arg)
{
	FScopeLock Lock(&ReferencedBlobsLock);

	auto Iter = ReferencedBlobs.find(BlobObj);
	check(Iter != ReferencedBlobs.end());

	Iter->second.RefCount--;
	if (Arg)
	{
		auto ArgIter = std::find(Iter->second.JobArgs.begin(), Iter->second.JobArgs.end(), Arg);

		if (ArgIter != Iter->second.JobArgs.end())
			Iter->second.JobArgs.erase(ArgIter);
	}

	if (Iter->second.RefCount <= 0)
		ReferencedBlobs.erase(Iter);
}

bool Blobber::IsBlobReferenced(Blob* BlobObj)
{
	FScopeLock Lock(&ReferencedBlobsLock);

	auto Iter = ReferencedBlobs.find(BlobObj);

	//if (Iter != ReferencedBlobs.end())
	//{
	//	/// Ok, check whether this is in the cache as another object
	//	BlobCacheEntry* Entry = Find_Internal(Hash);

	//	/// No entry found, then this is the last reference of this hash being deleted
	//	if (!Entry)
	//		return true;

	//	return Entry->BlobObj.get() == BlobObj ? true : false;
	//}

	//return false;

	return Iter != ReferencedBlobs.end();
}
#endif 

void Blobber::UpdateMappingInternal(HashType OldValue, CHashPtr NewValue)
{
	check(IsInGameThread());

	/// not thread-safe
	auto Iter = HashMappings.find(OldValue);

	if (Iter != HashMappings.end())
	{
		CHashPtr RHS = Iter->second;

		/// If RHS is the same as the new value, then we don't need to do anything else
		/// as the mapping already exists
		if (*RHS == *NewValue)
			return;

		/// Add_Internal this as a mapping as well only if the underlying Hash has changed
		/// and has been finalised
		if (RHS->Value() != OldValue)
		{
			UE_LOGF(LogBlob, VeryVerbose, "Hash Remapping: %llu => %llu => %llu", OldValue, RHS->Value(), NewValue->Value());
			AddHashMapping(RHS->Value(), NewValue);
		}
	}

	UE_LOGF(LogBlob, VeryVerbose, "Hash Remapping: %llu => %llu", OldValue, NewValue->Value());

	/// Put new has mapping for the old value
	AddHashMapping(OldValue, NewValue);
}

void Blobber::UpdateBlobLookupInternal(HashType OldValue, BlobPtr BlobObj)
{
	CHashPtr NewValue = BlobObj->Hash();

	FScopeLock Lock(&BlobLookupLock);

	const BlobCacheEntryPtr* OldEntry = BlobCache.Find(OldValue, false);

	/// Remove the old BlobObj from the system
	if (OldEntry)
	{
		check(*OldEntry);
		
		if (BlobObj != (*OldEntry)->BlobObj || OldValue != NewValue->Value())
		{
			/// Copy the Options over before Remove call
			auto Options = (*OldEntry)->Options;
			BlobCache.Remove(OldValue);

			/// Clear this out since this is certainly deleted now
			OldEntry = nullptr;
			AddBlobEntry(NewValue->Value(), BlobObj, Options);
		}
	}
}

void Blobber::UpdateBlobHash(HashType OldHash, BlobRef InBlob)
{
	check(IsInGameThread());

	BlobPtr BlobObj = InBlob.lock();
	check(BlobObj);

	CHashPtr Hash = BlobObj->Hash();

	// TODO: I had to comment this check otherwise we would crash when using Insight and trying to represetn the BlobObj images
	//	check(Hash->IsFinal());


	if (OldHash == Hash->Value())
	{
		UE_LOGF(LogBlob, VeryVerbose, "Unchanged Hash passed for update: %llu. Ignoring ...", OldHash);
		return;
	}

	{
		FScopeLock Lock(&HashMutex);

		/// Here we check to see if there's an existing mapping for the final blob hash
		if (Hash->IsFinal())
		{
			auto BlobHashIter = HashMappings.find(Hash->Value());
			if (BlobHashIter != HashMappings.end())
				HashMappings.erase(BlobHashIter);
		}

		UpdateGloballyUniqueHashInternal(OldHash);
		UpdateMappingInternal(OldHash, Hash);
	}

	UpdateBlobLookupInternal(OldHash, BlobObj);

	if (!TextureGraphEngine::IsTestMode())
		ObserverSource->RemapHash(OldHash, Hash->Value());
}

void Blobber::UpdateHash(HashType OldHash, CHashPtr Hash)
{
	check(IsInGameThread());

	/// Both hashes have to be valid
	check(OldHash != DataUtil::GNullHash && Hash->IsValid());

	if (OldHash == Hash->Value())
	{
		UE_LOGF(LogBlob, VeryVerbose, "Unchanged Hash passed for update: %llu. Ignoring ...", OldHash);
		return;
	}

	BlobCacheEntryPtr Entry = FindInternal(OldHash);

	if (!Entry)
	{
		AddHashMapping(OldHash, Hash);
		return;
	}

	BlobPtr BlobObj = Entry->BlobObj;

	if (!BlobObj->IsTiled())
	{
		CHashPtr ExistingHash = BlobObj->Hash();
		
		/// If the Existing Hash of the buffer is already final, then that means that the 
		/// BlobObj Hasher service has gotten to it before this. In this case we just add a 
		/// new mapping, as the buffer's Hash has already been finalised
		if (ExistingHash->IsFinal())
		{
			AddHashMapping(Hash->Value(), ExistingHash);
			AddHashMapping(OldHash, ExistingHash);
		}
		else
			BlobObj->GetBufferRef()->SetHash(Hash);
	}
	else
	{
		TiledBlobPtr TiledBlobObj = std::static_pointer_cast<TiledBlob>(BlobObj);
		TiledBlobObj->HashValue = Hash;
	}

	UpdateBlobHash(OldHash, BlobObj);
}

CHashPtr Blobber::AddGloballyUniqueHashInternal(CHashPtr Hash)
{
	check(Hash);
	/// For internal use only [not thread-safe]
	auto Iter = GlobalHashes.find(Hash->Value());

	if (Iter == GlobalHashes.end())
	{
		GlobalHashes[Hash->Value()] = Hash;

		return Hash;
	}

	return Iter->second;
}

void Blobber::Touch(const Blob* BlobObj)
{
	check(IsInGameThread());
	
	CHashPtr Hash = BlobObj->Hash();
	TouchedHashes.push_back(Hash);
	
	InvalidateTimestamp = Util::Time();

	// /// Don't wanna do anything if its not the final Hash 
	// if (!Hash || !Hash->IsFinal())
	// 	return;

	/// We also don't touch tiled buffers
#if 0
	if (BlobObj->IsTiled())
		return;
#endif /// Deprecate

#if 0
	size_t numDevices = Engine::DVM()->NumDevices();
	HashType blobHash = Hash->Value();

	for (size_t dvi = 0; dvi < numDevices; dvi++)
	{
		Device* device = Engine::DVM()->GetDevice(dvi);

		if (device)
			device->Touch(blobHash);
	}
#endif
}

void Blobber::AddHashMapping(HashType LHS, CHashPtr RHS)
{
	/// Prevent duplicate from being added
	if (LHS == RHS->Value())
		return;

	UE_LOGF(LogBlob, VeryVerbose, "Blobber AddHashMapping: %#016llx => %#016llx", LHS, RHS->Value());

	auto ExistingMapping = HashMappings.find(RHS->Value());

	/// Check that no cyclical value can exist
	
	if(!(ExistingMapping == HashMappings.end() || ExistingMapping->second->Value() != LHS))
	{
		return;
	}

	/// TODO: Check for cyclical references as well
	HashMappings[LHS] = RHS;
}

TiledBlobRef Blobber::AddTiledResult(TiledBlobPtr Result, BlobCacheOptions Options /* = {} */)
{
	check(Result);
	CHashPtr LHash = Result->Hash();
	return AddTiledResult(LHash, Result, Options);
}

TiledBlobRef Blobber::AddTiledResult(CHashPtr LHash, TiledBlobPtr Result, BlobCacheOptions Options /* = {} */)
{
	BlobRef Ret = AddResult(LHash, std::static_pointer_cast<Blob>(Result), Options);
	check(Ret->IsTiled());

	TiledBlobPtr CachedPtr = std::static_pointer_cast<TiledBlob>(Ret.get());
	check(CachedPtr->IsTiled());

	if (CachedPtr && Result != CachedPtr)
	{
		CachedPtr->AddLinkedBlob(Result);
	}

	/// If we got a different pointer back from the blobber then that means that this result was already cached into 
	/// the system. We need to copy it over to the original pointer if it's a finalised blob
	/// so that anyone keeping a copy of Result has the exact same view as the original result 
	/// and we don't have do any awkward pointer adjustment shenanigans to make things work
	//if (Result && CachedPtr->IsFinalised() && Result != CachedPtr)
	//{
	//	/// Copy the contents over
	//	*Result = *CachedPtr;
	//}

	return TiledBlobRef(std::static_pointer_cast<TiledBlob>(CachedPtr), Ret.IsKeepStrong(), false);
}

BlobRef Blobber::AddResult(CHashPtr LHash, BlobRef Result, BlobCacheOptions Options)
{
	CHashPtr RHash = Result->Hash();
	check(RHash && RHash->IsValid());

	if (*RHash != *LHash)
	{
		FScopeLock Lock(&HashMutex);

		auto Iter = HashMappings.find(LHash->Value());

		/// if this didn't exist exist before then we add it
		if (Iter == HashMappings.end())
		{
			/// Add_Internal the mapping
			AddHashMapping(LHash->Value(), RHash);

			if (!TextureGraphEngine::IsTestMode())
				ObserverSource->AddHash(LHash->Value());
		}
		else
		{
			CHashPtr ExistingRHash = Iter->second;

			/// Make sure that the BlobObj has the same RHash
			if (ExistingRHash != RHash && *ExistingRHash != *RHash)
			{
				/// There shouldn't be an Existing mapping for RHash
				//check(HashMappings.find(RHash->Value()) == HashMappings.end() || *ExistingRHash == *RHash);

				/// Ok here's another mapping that we need to add
				AddHashMapping(RHash->Value(), ExistingRHash);

				/// We only need to set this for single blobs
				if (!Result->IsTiled())
				{
					/// replace with the pointer with the oldest one that we've had
					Result->GetBufferRef()->SetHash(ExistingRHash);
				}
			}
		}
	}

	return Result;
}

BlobRef Blobber::AddResult(CHashPtr LHash, BlobPtr InBlob, BlobCacheOptions Options /* = {} */)
{
	UE_LOGF(LogBlob, VeryVerbose, "AddResult: %#016llx => %ls", LHash->Value(), *InBlob->Name());

	check(LHash && LHash->IsValid());
	check(InBlob);

	BlobRef Result = AddInternal(InBlob, Options);
	check(Result.get());

	return AddResult(LHash, Result, Options);
}

BlobPtr Blobber::Remove(HashType Hash)
{
	FScopeLock Lock(&BlobLookupLock);
	BlobCacheEntryPtr Entry = BlobCache.Remove(Hash);
	auto BlobObj = Entry->BlobObj;
	return BlobObj;
}

void Blobber::AddBlobEntryThreadSafe(HashType Hash, BlobPtr BlobObj, BlobCacheOptions Options)
{
	if (!bEnableCache)
		return;

	FScopeLock Lock(&BlobLookupLock);
	AddBlobEntry(Hash, BlobObj, Options);
}

void Blobber::AddBlobEntry(HashType Hash, BlobPtr BlobObj, BlobCacheOptions Options)
{
	if (!bEnableCache)
		return;

#if DEBUG_BLOB_REF_KEEPING == 1
	AddReferencedBlob(BlobObj.get(), nullptr);
#endif 

	/// Sanity check that we're not adding this multiple times
	const BlobCacheEntryPtr* ExistingEntry = BlobCache.Find(Hash, false);

	if (ExistingEntry && (*ExistingEntry)->BlobObj == BlobObj)
		return;

	check(!ExistingEntry || (*ExistingEntry)->BlobObj != BlobObj);
	
	BlobCacheEntryPtr Entry = std::make_shared<BlobCacheEntry>();

	Entry->BlobObj = BlobObj;
	Entry->Options = Options;
	Entry->CreateTimestamp = Util::Time();
	Entry->AccessTimestamp = Entry->CreateTimestamp;

	if (Options.Discard	 || Options.NoCacheBatch || BlobObj->IsTransient())
		TransientBlobs.push_back(Entry);

	BlobCache.Insert(Hash, Entry);

	InvalidateTimestamp = Util::Time();
}

void Blobber::RemoveHashMapping(HashType Hash)
{
	UE_LOGF(LogBlob, Warning, "Removing incorrect hash mapping: %#016llx", Hash);

	FScopeLock HashLock(&HashMutex);

	auto HashIter = HashMappings.find(Hash);

	/// if no mapping was found 
	/// then just ignore it ...
	if (HashIter == HashMappings.end())
		return;

	/// there's a cyclical Hash link, then ignore it as well
	if (HashIter->second->Value() == Hash)
		return;

	/// Otherwise, we need to use the RHS of the mapping
	CHashPtr MappedHash = HashIter->second;

	HashMappings.erase(HashIter);
	Hash = MappedHash->Value();
}

BlobRef Blobber::AddInternal(BlobPtr BlobObj, BlobCacheOptions Options)
{
	CHashPtr Hash = BlobObj->Hash();
	check(Hash && !Hash->IsNull());

	UE_LOGF(LogBlob, VeryVerbose, "Blobber Add_Internal BlobObj: %#016llx => %ls", Hash->Value(), *BlobObj->Name());

	BlobRef Existing = Find(Hash->Value());

	/// if we already have something then we just return that
	if (Existing)
	{
		/// Make sure whether tiled and un-tiled information should match up
		if (BlobObj->IsTiled() != Existing->IsTiled())
		{
			UE_LOGF(LogBlob, VeryVerbose, "Tiled and non-tiled Result mix-up: %ls", *BlobObj->Name());

			/// Two scenarios to consider here. 

			/// 1. If the incoming BlobObj is a TiledBlob but the Existing one isn't
			if (BlobObj->IsTiled() && !Existing->IsTiled())
			{
				UE_LOGF(LogBlob, VeryVerbose, "Incoming BlobObj is tiled and Existing isn't: %ls", *BlobObj->Name());

				TiledBlobPtr TiledBlobObj = std::static_pointer_cast<TiledBlob>(BlobObj);

				/// Two further scenarios here:
				/// 1. Ok, if the incoming BlobObj is a 1x1 tiled BlobObj then we can do the simple handling 
				/// of replacing the simple BlobObj with a tiled BlobObj.
				if (TiledBlobObj->Rows() == 1 && TiledBlobObj->Cols() == 1)
				{
					TiledBlobObj->SetTile(0, 0, Existing);
					return BlobRef(std::static_pointer_cast<Blob>(TiledBlobObj), true, false);
				}

				/// We need to remove this hash mapping
				RemoveHashMapping(Hash->Value());
				Existing = BlobRef();
			}

			/// 2. If the incoming BlobObj is un-tiled but the Existing one IS tiled
			else if (!BlobObj->IsTiled() && Existing->IsTiled())
			{
				UE_LOGF(LogBlob, VeryVerbose, "Incoming BlobObj is non-tiled and Existing is: %ls", *BlobObj->Name());

				TiledBlobPtr ExistingTiled = std::static_pointer_cast<TiledBlob>(Existing.lock());

				/// The hashes can only match up if the Existing (tiled) BlobObj is 1x1
				if (ExistingTiled->Rows() == 1 && ExistingTiled->Cols() == 1)
				{
					//check(ExistingTiled->Rows() == 1 && ExistingTiled->Cols() == 1);
					//check(*ExistingTiled->GetTile(0, 0)->Hash() == *Hash);
					return ExistingTiled->GetTile(0, 0).lock();
				}

				/// We need to remove this hash mapping
				RemoveHashMapping(Hash->Value());
				Existing = BlobRef();
			}
		}

		if (Existing)
		{
			check(BlobObj->IsTiled() == Existing->IsTiled());
			return Existing;
		}
	}

	/// Don't do this with temp hashes of blobs because they could be the same as 
	/// the job hashes and may Result in the two being linked perpetually
	if (Hash->IsFinal())
	{
		CHashPtr UniqueHash = AddGloballyUniqueHash(Hash);
		if (UniqueHash != Hash)
			BlobObj->SetHash(UniqueHash);
	}

	/// Add it to the lookup
	AddBlobEntryThreadSafe(Hash->Value(), BlobObj, BlobCacheOptions());

	if (!BlobObj->IsTiled())
	{
		bool bIsTransient = BlobObj->GetBufferRef() ? BlobObj->GetBufferRef()->Descriptor().bIsTransient : false;

		if (!bIsTransient && !Options.Discard && !Options.NoCacheBatch && !BlobObj->IsTiled())
		{
			if (BlobObj->GetBufferRef() && !Hash->IsFinal())
			{
				BlobHasherServicePtr BlobHasher = TextureGraphEngine::GetScheduler()->GetBlobHasherService().lock();
				if (BlobHasher)
				{
					BlobHasher->Add(BlobRef(BlobObj, true, false));
				}
			}

			UE_LOGF(LogBlob, Verbose, "Added BlobObj with Hash: %llu [Name: %ls]", Hash->Value(), *BlobObj->Name());
		}
	}

	return BlobObj;
}

// BlobRef Blobber::Add_Internal(BlobUPtr Blob_, BlobCacheOptions Options)
// {
// 	BlobPtr BlobObj = std::move(Blob_);
// 	return Add_Internal(BlobObj, Options);
// }

void Blobber::RegisterObserverSource(const BlobberObserverSourcePtr& observerSource)
{
	if (TextureGraphEngine::IsTestMode())
		return;

	if (observerSource)
	{
		ObserverSource = observerSource;
	}
	else
	{
		ObserverSource = std::make_shared<BlobberObserverSource>();
	}
}

void BlobberObserverSource::AddHash(HashType Hash)
{
	FScopeLock observerLock(&ObserverLock);
	AddedHashStack.emplace_back(Hash);
}

void BlobberObserverSource::RemapHash(HashType OldHash, HashType NewHash)
{
	FScopeLock observerLock(&ObserverLock);
	RemappedHashStack.emplace_back(OldHash);
	RemappedHashStack.emplace_back(NewHash);
}

void BlobberObserverSource::Broadcast()
{
	HashArray addedHashes;
	HashArray remappedHashes;

	{ /// Just here, record the removed Hash value for observers
		FScopeLock observerLock(&ObserverLock);
		addedHashes = std::move(AddedHashStack);
		remappedHashes = std::move(RemappedHashStack);
		AddedHashStack.clear();
		RemappedHashStack.clear();
	}

	if (addedHashes.size() || remappedHashes.size())
	{
		BlobberUpdated(std::move(addedHashes), std::move(remappedHashes));
		Version++;
	}
}
