// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BinCache.h"
#include "CoreMinimal.h"
#include "Device/DeviceBuffer.h"
#include "Helper/DataUtil.h"
#include "RawBuffer.h"
#include "TiledBlob.h"
#include <unordered_map>

#include "Helper/Util.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Device;
struct JobResult;
typedef std::shared_ptr<JobResult>			JobResultPtr;
typedef cti::continuable<JobResultPtr>		AsyncJobResultPtr;
typedef T_BlobRef<Blob>						BlobRef;
typedef T_BlobRef<TiledBlob>				TiledBlobRef;

DECLARE_LOG_CATEGORY_EXTERN(LogBlob, Log, All);

typedef std::vector<HashType>   HashTypes;
class JobArg;

class BlobberObserverSource
{
public:
	using HashArray = std::vector<HashType>;

	UE_API void AddHash(HashType Hash);
	UE_API void RemapHash(HashType OldHash, HashType NewHash);

protected:
	/// Protected interface of emitters called by the scheduler to notify the observers
	friend class Blobber;

	FCriticalSection		ObserverLock;

	HashArray				AddedHashStack;
	HashArray				RemappedHashStack;

	uint32					Version = 1;

	/// Trigger the broadcast of the changes, this call is issued from the Blobber::Update function
	UE_API void Broadcast();

	/// The customisable handler method triggered from Broadcast if any buffers were added or removed
	virtual void BlobberUpdated(HashArray&& InAddedHashes, HashArray&& InRemappedHashes) {}

public:
	BlobberObserverSource() = default;
	virtual						~BlobberObserverSource() = default;

	uint32						GetVersion() const { return Version; } /// The version of the current state of the Observer. Incremented when Broadcast trigger
};

typedef std::shared_ptr<BlobberObserverSource> BlobberObserverSourcePtr;

struct BlobCacheOptions
{
	bool						Discard = false;			/// Whether the BlobObj is part of a "discard" cycle (user tweaking usually)
	bool						NoCacheBatch = false;		/// Whether the batch is marked for no caching
};

class Blobber
{
	friend class Blob;
	friend class DeviceTransferService;
	friend class Device;
	friend class DeviceBuffer;

public:
	struct BlobCacheEntry
	{
		BlobPtr						BlobObj;						/// The actual BlobObj object within the cache entry
		BlobCacheOptions			Options;						/// The options associated with this BlobObj entry
		double						CreateTimestamp = Util::Time(); /// The timestamp when this entry was created
		double						AccessTimestamp = Util::Time();	/// The timestamp when this entry was last accessed

		~BlobCacheEntry();

		FORCEINLINE TiledBlobPtr	AsTiledBlob() const { return std::static_pointer_cast<TiledBlob>(BlobObj); }
	};
	using BlobCacheEntryPtr = std::shared_ptr<BlobCacheEntry>;
	typedef BinCache<HashType, BlobCacheEntryPtr> BlobLRUCache;

private:
	using BlobLUT					= std::unordered_map<HashType, BlobCacheEntry>;
	using HashMappingTable			= std::unordered_map<HashType, CHashPtr>;
	using BlobCacheEntryPtrVec		= std::vector<BlobCacheEntryPtr>;
	using CHashPtrVec				= std::vector<CHashPtr>;

	//typedef std::map<HashType, std::vector<HashType>> TempHashTable;
	mutable FCriticalSection		BlobLookupLock;				/// Mutex for the actions (mutable because FScopeLock needs non-const object)
	BlobLUT							BlobLookup;					/// The buffer cache that we have
	BlobCacheEntryPtrVec			TransientBlobs;				/// List of transient blobs
	BlobLRUCache					BlobCache;					/// Blob cache used for lookups and saving
	CHashPtrVec						TouchedHashes;				/// Hashes that have been touched since the last idle update
	
	double							LastIdleUpdateTimestamp = 0;/// The last time the idle loop was run
	double							InvalidateTimestamp = 0;	/// The last time the blobber cache was invalidated
																		
	size_t							MemUsed = 0;				/// How much memory are we using
	size_t							MemUsedNative = 0;			/// Native memory used for all the blobs within the system

	double							TimeStatsPrinted = 0;		/// The last time when stats were printed
	bool							ShouldPrintStats = false;	/// Whether to print stats all all

	FCriticalSection				HashMutex;					/// Mutex for the hashing tables

	HashMappingTable				GlobalHashes;				/// All the globally unique hashes within the system. While the Hash object itself
																/// can be created as and when needed, these are hashes that are used to detect
																/// duplicate data and jobs. These can evolve from temp to final hashes and can	
																/// change many times during the course of the mix and application. In order to ease
																/// the burden of updating too many Hash tables within the system. We keep a globally
																/// unique mapping from the Hash value (HashType) to a Hash pointer. This Hash pointer
																/// is unique within the system and can only be one for one of such hashes. 
																/// The benefit that we get is that once a temp Hash is updated internally within the 
																/// CHash object. We just update the value from the old mapping to the new mapping
																/// and we won't have to modify any other mappings within the Blobber structure since
																/// all the RHS values in the Hash table are CHashPtr types.

	HashMappingTable				HashMappings;				/// These are mappings from one Hash to another. The LHS is the Hash of the job
																/// and the RHS tells what (if any) that results in. Its entirely possible
																/// that we don't have that mapped BlobObj in the cache or within the system
																/// as it may have garbage collected

	bool							bEnableCache = true;		/// Enable and disable caching within the TextureGraph system

#if UE_BUILD_DEBUG
	bool							EnableGC = true;			/// This is for debugging purposes only. Never turn it off otherwise
public:
	FORCEINLINE void				SetEnableGC(bool InEnableGC) { EnableGC = InEnableGC; }
	FORCEINLINE bool				IsGCEnabled() const { return EnableGC; }

#endif /// UE_BUILD_DEBUG

#if DEBUG_BLOB_REF_KEEPING == 1
	struct DebugRefBlob
	{
		int32						RefCount = 0;
		std::vector<JobArg*>		JobArgs;
	};

	typedef std::unordered_map<Blob*, DebugRefBlob> RefBlobMap;

	mutable FCriticalSection		ReferencedBlobsLock;		/// Mutex for the referenced blob LUT
	RefBlobMap						ReferencedBlobs;			/// Blobs that are referenced (Debug only!)

public:
	UE_API void							AddReferencedBlob(Blob* BlobObj, JobArg* Arg);
	UE_API void							RemoveReferencedBlob(Blob* BlobObj, JobArg* Arg);
	UE_API bool							IsBlobReferenced(Blob* BlobObj);
private:
#endif 
	
	BlobberObserverSourcePtr		ObserverSource;				/// Observer where the lifecycle of the blobs and hashes is recorded

#if DEBUG_BLOB_REF_KEEPING == 1
	FCriticalSection				DebugBlobMutex;				/// The TiledBlobs that contain this blob as a tile
#endif 

	UE_API void							AddHashMapping(HashType LHS, CHashPtr RHS);
	UE_API void							AddBlobEntry(HashType Hash, BlobPtr BlobObj, BlobCacheOptions Options);
	UE_API void							AddBlobEntryThreadSafe(HashType Hash, BlobPtr BlobObj, BlobCacheOptions Options);
	UE_API void							RemoveHashMapping(HashType Hash);

	UE_API void							PrintStats();
	UE_API BlobRef							AddInternal(BlobPtr BlobObj, BlobCacheOptions Options);
	UE_API BlobPtr							Remove(HashType Hash);

	UE_API CHashPtr						AddGloballyUniqueHashInternal(CHashPtr Hash);
	UE_API void							UpdateGloballyUniqueHashInternal(HashType OldValue);
	UE_API void							UpdateMappingInternal(HashType OldValue, CHashPtr NewValue);
	UE_API void							UpdateBlobLookupInternal(HashType OldValue, BlobPtr BlobObj);
	UE_API void							Touch(const Blob* BlobObj);
	UE_API BlobCacheEntryPtr				FindInternal(HashType Hash);
	UE_API BlobRef							AddResult(CHashPtr LHash, BlobRef Result, BlobCacheOptions Options);
	UE_API void							UpdateTransient();
	UE_API void							UpdateTouchedHashes();
	UE_API void							UpdateBlobCache();

public:
									UE_API Blobber();
									UE_API ~Blobber();

	UE_API BlobRef							FindSingle(HashType Hash);
	UE_API TiledBlobRef					FindTiled(HashType Hash);

	UE_API BlobRef							Create(const BufferDescriptor& Desc);
	UE_API BlobRef							Create(Device* Dev, RawBufferPtr Raw);
	UE_API BlobRef							Create(DeviceBufferRef DevBuffer, bool NoCache = false);

	UE_API BlobRef							AddResult(CHashPtr LHash, BlobPtr InBlob, BlobCacheOptions Options = {});
	UE_API void							ClearCache();

	/// In order to minimise the changes brought upon by weak reference work we've made this function.
	/// If we were to force this to take a unique_ptr<TiledBlob> it would've resulted in lots of changes
	/// around asset manager and other bits of code (Jobs etc), that we can do without.
	/// Eventually, this should either go away, or be converted to accept a unique pointer to force stop
	/// shared_ptr being passed around
	UE_API TiledBlobRef					AddTiledResult(CHashPtr LHash, TiledBlobPtr Result, BlobCacheOptions Options = {});
	UE_API TiledBlobRef					AddTiledResult(TiledBlobPtr Result, BlobCacheOptions Options = {});

	UE_API void							UpdateBlobHash(HashType OldHash, BlobRef InBlob);
	UE_API void							UpdateHash(HashType OldHash, CHashPtr NewHash);
	UE_API void							Update(float DT);
	UE_API AsyncJobResultPtr				UpdateIdle();
	UE_API bool							IsBlobCached(HashType Hash);

	UE_API void							RegisterObserverSource(const BlobberObserverSourcePtr& observerSource); ///

	template <class BlobType>
	T_BlobRef<BlobType>				Find(HashType Hash)
	{
		BlobCacheEntryPtr Entry = FindInternal(Hash);
		if (Entry == nullptr)
			return nullptr;
		return T_BlobRef<BlobType>(std::static_pointer_cast<BlobType>(Entry->BlobObj));
	}

	template <>
	BlobRef							Find<Blob>(HashType Hash)
	{
		return FindSingle(Hash);
	}

	template <>
	TiledBlobRef					Find<TiledBlob>(HashType Hash)
	{
		return FindTiled(Hash);
	}

	BlobRef							Find(HashType Hash)
	{
		return Find<Blob>(Hash);
	}

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE CHashPtr			AddGloballyUniqueHash(CHashPtr Hash)
	{
		FScopeLock lock(&HashMutex);
		return AddGloballyUniqueHashInternal(Hash);
	}

	FORCEINLINE void				SetEnableCache(bool InEnableCache) { bEnableCache = InEnableCache; }
	FORCEINLINE bool				IsCacheEnabled() const { return bEnableCache; }

	FORCEINLINE BlobberObserverSourcePtr GetObserverSource() const { return ObserverSource; }

#if DEBUG_BLOB_REF_KEEPING == 1
	FCriticalSection*				GetDebugBlobMutex() { return &DebugBlobMutex; }
#endif 

};

typedef std::unique_ptr<Blobber>	BlobberPtr;

////////////////////////////////////////////////////////////////////////////////////////////
template <class BlobType>
void T_BlobRef<BlobType>::CheckShouldKeepStrongRef()
{
#if BLOB_REF_SIMPLE == 0
	/// If its already set then don't care
	if (KeepStrong || !Hash)
		return;

	/// If its a transient blob then keep the reference
	if (PtrWeak.lock()->IsTransient())
	{
		KeepStrong = true;
		return;
	}

	/// Try to find this blob in the cache
	T_BlobRef<BlobType> CachedBlob = TextureGraphEngine::Blobber()->Find<BlobType>(Hash->Value());

	/// If we can't find it in the cache
	if (!CachedBlob)
		KeepStrong = true;
#endif
}

template <class BlobType>
std::shared_ptr<BlobType> T_BlobRef<BlobType>::get() const
{
#if BLOB_REF_SIMPLE == 0
	if (PtrStrong)
		return CheckReleaseStrongRef();

	/// If the pointer hasn't expired then we get the 
	if (!PtrWeak.expired())
	{
		auto SPtr = PtrWeak.lock();

		/// Must have a valid hash pointer at the very least at this point
		check(Hash);

		/// always try to update the hash to the latest if it's not final
		if (!Hash->IsFinal())
			Hash = SPtr->Hash();

		return SPtr;
	}

	/// If there's no hash, then just return
	if (!Hash)
		return nullptr;

	/// This can happen when the engine is being destroyed 
	if (!TextureGraphEngine::Blobber())
		return nullptr;

	/// If we don't have this object anymore, then we need
	/// to access it from the blobber
	T_BlobRef<BlobType> CachedBlob = TextureGraphEngine::Blobber()->Find<BlobType>(Hash->Value());
	check(CachedBlob);

	/// Update the pointer and the hash
	*this = CachedBlob;

	return std::static_pointer_cast<BlobType>(PtrWeak.lock());
#else
	return PtrStrong;
#endif 
}

#undef UE_API
