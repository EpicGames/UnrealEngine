// Copyright Epic Games, Inc. All Rights Reserved.
#include "G_Collectible.h"
#include "Helper/Util.h"
#include "TextureGraphEngine.h"
#include "Job/JobBatch.h"

void G_Collectible::UpdateAccessInfo(uint64 BatchId /* = 0 */)
{
	/// This should never be called from anything other than the game thread
	check(IsInGameThread());

	if (BatchId == 0)
		BatchId = JobBatch::CurrentBatchId();

	AccessDetails.Timestamp = Util::Time();
	AccessDetails.Count++;
	AccessDetails.BatchId = BatchId;
	AccessDetails.FrameId = TextureGraphEngine::GetFrameId();
}
