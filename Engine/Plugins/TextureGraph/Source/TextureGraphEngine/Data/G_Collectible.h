// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include <atomic>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
struct AccessInfo
{
	std::atomic<double>				Timestamp = 0;		/// The last time was accessed. We use timestamp to determine modification 
														/// between two time intervals. Hence atomic.

	/// If any of these need to be accessed in a thread-safe manner
	/// then these can be turned into atomics as well
	uint64							FrameId = 0;		/// When as the last frame that we were bound
	uint64							BatchId = 0;		/// The batch ID when this was last bound
	uint64							Count = 0;			/// The number of times this was bound and accessed

	AccessInfo()
	{
	}

	AccessInfo(const AccessInfo& RHS)
	{
		*this						= RHS;
	}

	FORCEINLINE void				operator = (const AccessInfo& RHS)
	{
		Timestamp					= RHS.Timestamp.load();
		FrameId						= RHS.FrameId;
		BatchId						= RHS.BatchId;
		Count						= RHS.Count;
	}
};

//////////////////////////////////////////////////////////////////////////
/// Garbage collectible data structure
class G_Collectible
{
protected:
	AccessInfo						AccessDetails;				/// Information about the last time this DeviceBuffer was accessed

public:
	UE_API virtual void					UpdateAccessInfo(uint64 BatchId = 0);
	virtual							~G_Collectible() = default;

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE const AccessInfo&	GetAccessInfo() const { return AccessDetails; }
	FORCEINLINE bool				IsSame(const std::atomic<double>& Timestamp) const { return AccessDetails.Timestamp == Timestamp; }
};

#undef UE_API
