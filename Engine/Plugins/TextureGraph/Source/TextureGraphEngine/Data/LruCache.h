// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/LruCache.h"
#include "Containers/Set.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"

/// Forked from TLruCache ["Core/Public/Containers/LruCache.h"]

template<typename KeyType, typename ValueType, typename KeyComp = DefaultKeyComparer<KeyType> >
class LruCache
{
	/** An entry in the LRU cache. */
	struct FCacheEntry
	{
		/** The entry's lookup key. */
		KeyType Key;

		/** The less recent entry in the linked list. */
		FCacheEntry* LessRecent;

		/** The more recent entry in the linked list. */
		FCacheEntry* MoreRecent;

		/** The entry's value. */
		ValueType Value;

		/**
		 * Create and initialize a new instance.
		 *
		 * @param InKey The entry's key.
		 * @param InValue The entry's value.
		 */
		FCacheEntry(const KeyType& InKey, const ValueType& InValue)
			: Key(InKey)
			, LessRecent(nullptr)
			, MoreRecent(nullptr)
			, Value(InValue)
		{ }

		/** Add this entry before the given one. */
		FORCEINLINE void LinkBefore(FCacheEntry* Other)
		{
			LessRecent = Other;

			if (Other != nullptr)
			{
				Other->MoreRecent = this;
			}
		}

		/** Remove this entry from the list. */
		FORCEINLINE void Unlink()
		{
			if (LessRecent != nullptr)
			{
				LessRecent->MoreRecent = MoreRecent;
			}

			if (MoreRecent != nullptr)
			{
				MoreRecent->LessRecent = LessRecent;
			}

			LessRecent = nullptr;
			MoreRecent = nullptr;
		}
	};

	/** Lookup set key functions. */
	struct FKeyFuncs : public BaseKeyFuncs<FCacheEntry*, KeyType>
	{
		FORCEINLINE static const KeyType& GetSetKey(const FCacheEntry* Entry)
		{
			return Entry->Key;
		}

		FORCEINLINE static bool Matches(KeyType A, KeyType B)
		{
			return KeyComp::Matches(A, B);
		}

		FORCEINLINE static uint32 GetKeyHash(KeyType Key)
		{
			return KeyComp::GetKeyHash(Key);
		}
	};

public:

	/** Default constructor (empty cache that cannot hold any values). */
	LruCache()
		: LeastRecent(nullptr)
		, MostRecent(nullptr)
		, MaxNumElements(0)
	{ }

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InMaxNumElements The maximum number of elements this cache can hold.
	 */
	LruCache(int32 InMaxNumElements)
		: LeastRecent(nullptr)
		, MostRecent(nullptr)
		, MaxNumElements(InMaxNumElements)
	{
		Empty(InMaxNumElements);
	}

	/** Destructor. */
	~LruCache()
	{
		Empty();
	}

public:

	/**
	 * Add an entry to the cache.
	 *
	 * If an entry with the specified key already exists in the cache,
	 * the value of the existing entry will be updated. The added or
	 * updated entry will be marked as the most recently used one.
	 *
	 * @param Key The entry's lookup key.
	 * @param Value The entry's value.
	 * @see Empty, Find, GetKeys, Remove
	 */
	void Add(const KeyType& Key, const ValueType& Value)
	{
		check(MaxNumElements > 0 && "Cannot add values to zero size LruCache");

		FCacheEntry** EntryPtr = LookupSet.Find(Key);

		if (EntryPtr != nullptr)
		{
			// update existing entry
			FCacheEntry* Entry = *EntryPtr;
			checkSlow(Entry->Key == Key);

			Entry->Value = Value;
			MarkAsRecent(*Entry);
		}
		else
		{
			// add new entry
			if (LookupSet.Num() == MaxNumElements)
			{
				Remove(LeastRecent);
			}

			FCacheEntry* NewEntry = new FCacheEntry(Key, Value);
			NewEntry->LinkBefore(MostRecent);
			MostRecent = NewEntry;

			if (LeastRecent == nullptr)
			{
				LeastRecent = NewEntry;
			}

			LookupSet.Add(NewEntry);
		}
	}

	/**
	 * Check whether an entry with the specified key is in the cache.
	 *
	 * @param Key The key of the entry to check.
	 * @return true if the entry is in the cache, false otherwise.
	 * @see Add, ContainsByPredicate, Empty, FilterByPredicate, Find, GetKeys, Remove
	 */
	FORCEINLINE bool Contains(const KeyType& Key) const
	{
		return LookupSet.Contains(Key);
	}

	/**
	 * Check whether an entry for which a predicate returns true is in the cache.
	 *
	 * @param Pred The predicate functor to apply to each entry.
	 * @return true if at least one matching entry is in the cache, false otherwise.
	 * @see Contains, FilterByPredicate, FindByPredicate, RemoveByPredicate
	 */
	template<typename Predicate>
	FORCEINLINE bool ContainsByPredicate(Predicate Pred) const
	{
		for (const FCacheEntry* Entry : LookupSet)
		{
			if (Pred(Entry->Key, Entry->Value))
			{
				return true;
			}
		}

		return false;
	}

	/**
	 * Empty the cache.
	 *
	 * @param InMaxNumElements The maximum number of elements this cache can hold (default = 0).
	 * @see Add, Find, GetKeys, Max, Num, Remove
	 */
	void Empty(int32 InMaxNumElements = 0)
	{
		check(InMaxNumElements >= 0);

		for (FCacheEntry* Entry : LookupSet)
		{
			delete Entry;
		}

		MaxNumElements = InMaxNumElements;
		LookupSet.Empty(MaxNumElements);

		MostRecent = nullptr;
		LeastRecent = nullptr;
	}

	/**
	 * Filter the entries in the cache using a predicate.
	 *
	 * @param Pred The predicate functor to apply to each entry.
	 * @return Collection of values for which the predicate returned true.
	 * @see ContainsByPredicate, FindByPredicate, Find, RemoveByPredicate
	 */
	template<typename Predicate>
	TArray<ValueType> FilterByPredicate(Predicate Pred) const
	{
		TArray<ValueType> Result;

		for (const FCacheEntry* Entry : LookupSet)
		{
			if (Pred(Entry->Key, Entry->Value))
			{
				Result.Add(Entry->Value);
			}
		}

		return Result;
	}

	/**
	 * Find the value of the entry with the specified key.
	 *
	 * @param Key The key of the entry to get.
	 * @return Pointer to the value, or nullptr if not found.
	 * @see Add, Contains, Empty, FindAndTouch, GetKeys, Remove
	 */
	FORCEINLINE const ValueType* Find(const KeyType& Key) const
	{
		FCacheEntry*const * EntryPtr = LookupSet.Find(Key);

		if (EntryPtr != nullptr)
		{
			return &(*EntryPtr)->Value;
		}

		return nullptr;
	}

	FORCEINLINE ValueType* Find(const KeyType& Key)
	{
		FCacheEntry* const* EntryPtr = LookupSet.Find(Key);

		if (EntryPtr != nullptr)
		{
			return &(*EntryPtr)->Value;
		}

		return nullptr;
	}

	/**
	 * Find the value of the entry with the specified key.
	 *
	 * @param Key The key of the entry to get.
	 * @return Reference to the value, or triggers an assertion if the key does not exist.
	 */
	FORCEINLINE const ValueType& FindChecked(const KeyType& Key) const
	{
		FCacheEntry*const * EntryPtr = LookupSet.Find(Key);
		
		check(EntryPtr);

		return (*EntryPtr)->Value;
	}

	/**
	 * Find the value of the entry with the specified key.
	 *
	 * @param Key The key of the entry to get.
	 * @return Copy of the value, or the default value for the ValueType if the key does not exist.
	 */
	FORCEINLINE ValueType FindRef(const KeyType& Key) const
	{
		FCacheEntry*const * EntryPtr = LookupSet.Find(Key);

		if (EntryPtr != nullptr)
		{
			return (*EntryPtr)->Value;
		}

		return ValueType();
	}

	/**
	 * Find the value of the entry with the specified key and mark it as the most recently used.
	 *
	 * @param Key The key of the entry to get.
	 * @return Pointer to the value, or nullptr if not found.
	 * @see Add, Contains, Empty, Find, GetKeys, Remove
	 */
	const ValueType* FindAndTouch(const KeyType& Key)
	{
		FCacheEntry** EntryPtr = LookupSet.Find(Key);

		if (EntryPtr == nullptr)
		{
			return nullptr;
		}

		MarkAsRecent(**EntryPtr);

		return &(*EntryPtr)->Value;
	}

	/**
	 * Find the value of the entry with the specified key and mark it as the most recently used.
	 *
	 * @param Key The key of the entry to get.
	 * @return Pointer to the value, or triggers an assertion if the key does not exist.
	 */
	const ValueType& FindAndTouchChecked(const KeyType& Key)
	{
		FCacheEntry** EntryPtr = LookupSet.Find(Key);
		
		check(EntryPtr);

		MarkAsRecent(**EntryPtr);

		return (*EntryPtr)->Value;
	}

	/**
	 * Find the value of the entry with the specified key and mark it as the most recently used.
	 *
	 * @param Key The key of the entry to get.
	 * @return Copy of the value, or the default value for the ValueType if the key does not exist.
	 */
	ValueType FindAndTouchRef(const KeyType& Key)
	{
		FCacheEntry** EntryPtr = LookupSet.Find(Key);

		if (EntryPtr == nullptr)
		{
			return ValueType();
		}

		MarkAsRecent(**EntryPtr);

		return (*EntryPtr)->Value;
	}

	/**
	 * Find the value of an entry using a predicate.
	 *
	 * @param Pred The predicate functor to apply to each entry.
	 * @return A value for which the predicate returned true, or nullptr if not found.
	 * @see ContainsByPredicate, FilterByPredicate, RemoveByPredicate
	 */
	template<typename Predicate>
	const ValueType* FindByPredicate(Predicate Pred) const
	{
		for (const FCacheEntry* Entry : LookupSet)
		{
			if (Pred(Entry->Key, Entry->Value))
			{
				return Entry->Value;
			}
		}

		return nullptr;
	}

	/**
	 * Find the keys of all cached entries.
	 *
	 * @param OutKeys Will contain the collection of keys.
	 * @see Add, Empty, Find
	 */
	void GetKeys(TArray<KeyType>& OutKeys) const
	{
		for (const FCacheEntry* Entry : LookupSet)
		{
			OutKeys.Add(Entry->Key);
		}
	}

	/**
	 * Get the maximum number of entries in the cache.
	 *
	 * @return Maximum number of entries.
	 * @see Empty, Num
	 */
	FORCEINLINE int32 Max() const
	{
		return MaxNumElements;
	}

	/**
	 * Get the number of entries in the cache.
	 *
	 * @return Number of entries.
	 * @see Empty, Max
	 */
	FORCEINLINE int32 Num() const
	{
		return LookupSet.Num();
	}

	/**
	 * Remove all entries with the specified key from the cache.
	 *
	 * @param Key The key of the entries to remove.
	 * @see Add, Empty, Find, RemoveByPredicate
	 */
	ValueType Remove(const KeyType& Key)
	{
		FCacheEntry** EntryPtr = LookupSet.Find(Key);

		if (EntryPtr != nullptr)
		{
			return Remove(*EntryPtr);
		}

		return ValueType();
	}

	/**
	 * Remove all entries using a predicate.
	 *
	 * @param Pred The predicate function to apply to each entry.
	 * @return Number of removed entries.
	 * @see ContainsByPredicate, FilterByPredicate, FindByPredicate, Remove
	 */
	template<typename Predicate>
	int32 RemoveByPredicate(Predicate Pred)
	{
		int32 NumRemoved = 0;

		for (const FCacheEntry* Entry : LookupSet)
		{
			if (Pred(Entry->Key, Entry->Value))
			{
				Remove(const_cast<FCacheEntry*>(Entry));
				++NumRemoved;
			}
		}

		return NumRemoved;
	}

	/**
	 * Remove and return the least recent element from the cache.
	 *
	 * @return Copy of removed value.
	 */
	FORCEINLINE ValueType RemoveLeastRecent()
	{
		check(LeastRecent);
		ValueType LeastRecentElement = MoveTemp(LeastRecent->Value);
		Remove(LeastRecent);
		return LeastRecentElement;
	}

public:

	/**
	 * Base class for cache iterators.
	 *
	 * Iteration begins at the most recent entry.
	 */
	template<bool Const>
	class TBaseIterator
	{
	public:

		FORCEINLINE TBaseIterator()
			: CurrentEntry(nullptr)
		{ }

		FORCEINLINE TBaseIterator(const LruCache& Cache)
			: CurrentEntry(Cache.LeastRecent)
		{ }

	public:

		FORCEINLINE TBaseIterator& operator++()
		{
			Increment();
			return *this;
		}

		FORCEINLINE friend bool operator==(const TBaseIterator& Lhs, const TBaseIterator& Rhs)
		{
			return Lhs.CurrentEntry == Rhs.CurrentEntry;
		}

		FORCEINLINE friend bool operator!=(const TBaseIterator& Lhs, const TBaseIterator& Rhs)
		{
			return Lhs.CurrentEntry != Rhs.CurrentEntry;
		}

		ValueType& operator->() const
		{
			check(CurrentEntry != nullptr);
			return CurrentEntry->Value;
		}

		ValueType& operator*() const
		{
			check(CurrentEntry != nullptr);
			return CurrentEntry->Value;
		}

		FORCEINLINE explicit operator bool() const
		{
			return (CurrentEntry != nullptr);
		}

		FORCEINLINE bool operator!() const
		{
			return !(bool)*this;
		}

	public:

		FORCEINLINE KeyType& Key() const
		{
			check(CurrentEntry != nullptr);
			return CurrentEntry->Key;
		}

		FORCEINLINE ValueType& Value() const
		{
			check(CurrentEntry != nullptr);
			return CurrentEntry->Value;
		}

	protected:

		FCacheEntry* GetCurrentEntry()
		{
			return CurrentEntry;
		}

		void Increment()
		{
			check(CurrentEntry != nullptr);
			CurrentEntry = CurrentEntry->MoreRecent;
		}

	private:

		FCacheEntry* CurrentEntry;
	};


	/**
	 * Cache iterator (const).
	 */
	class TConstIterator
		: public TBaseIterator<true>
	{
	public:

		FORCEINLINE TConstIterator()
			: TBaseIterator<true>()
		{ }

		FORCEINLINE TConstIterator(const LruCache& Cache)
			: TBaseIterator<true>(Cache)
		{ }
	};

	
	/**
	 * Cache iterator.
	 */
	class TIterator
		: public TBaseIterator<false>
	{
	public:

		FORCEINLINE TIterator()
			: TBaseIterator<false>()
			, Cache(nullptr)
		{ }

		FORCEINLINE TIterator(LruCache& InCache)
			: TBaseIterator<false>(InCache)
			, Cache(&InCache)
		{ }

		/** Removes the current element from the cache and increments the iterator. */
		FORCEINLINE void RemoveCurrentAndIncrement()
		{
			check(Cache != nullptr);

			FCacheEntry* MoreRecentEntry = this->GetCurrentEntry();
			this->Increment();
			Cache->Remove(MoreRecentEntry);
		}

	private:
		
		LruCache* Cache;
	};

protected:

	/**
	 * Mark the given entry as recently used.
	 *
	 * @param Entry The entry to mark.
	 */
	FORCEINLINE void MarkAsRecent(FCacheEntry& Entry)
	{
		check(LeastRecent != nullptr);
		check(MostRecent != nullptr);

		// if entry is least recent and not the only item in the list, make it not least recent
		if ((&Entry == LeastRecent) && (LeastRecent->MoreRecent != nullptr))
		{
			LeastRecent = LeastRecent->MoreRecent;
		}

		// relink if not already the most recent item
		if (&Entry != MostRecent)
		{
			Entry.Unlink();
			Entry.LinkBefore(MostRecent);
			MostRecent = &Entry;
		}
	}

	/**
	 * Remove the specified entry from the cache.
	 *
	 * @param Entry The entry to remove.
	 */
	FORCEINLINE ValueType Remove(FCacheEntry* Entry)
	{
		if (Entry == nullptr)
		{
			return ValueType();
		}

		LookupSet.Remove(Entry->Key);

		if (Entry == LeastRecent)
		{
			LeastRecent = Entry->MoreRecent;
		}

		if (Entry == MostRecent)
		{
			MostRecent = Entry->LessRecent;
		}
		
		ValueType ReturnVal = Entry->Value;

		Entry->Unlink();
		delete Entry;

		return ReturnVal;
	}

public:

	TIterator begin() { return TIterator(*this); }
	TConstIterator begin() const { return TConstIterator(*this); }
	TIterator end() { return TIterator(); }
	TConstIterator end() const { return TConstIterator(); }

private:

	/** Set of entries for fast lookup. */
	TSet<FCacheEntry*, FKeyFuncs> LookupSet;

	/** Least recent item in the cache. */
	FCacheEntry* LeastRecent;

	/** Most recent item in the cache. */
	FCacheEntry* MostRecent;

	/** Maximum number of elements in the cache. */
	int32 MaxNumElements;
};
