// Copyright Epic Games, Inc. All Rights Reserved.
#include "RawBuffer.h"

#include <IImageWrapperModule.h>
#include <IImageWrapper.h>
#include "Helper/Util.h"
#include "Misc/Compression.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Async/ParallelFor.h"
#include "Async/Async.h"

const FString RawBufferMetadataDefs::G_FX_UAV = "FX:UAV";
const FString RawBufferMetadataDefs::G_LAYER_MASK = "LAYER_MASK";

const char* BufferDescriptor::FormatToString(BufferFormat InFormat)
{
	switch (InFormat)
	{
	case BufferFormat::Float: 
		return "Float";
	case BufferFormat::Byte: 
		return "Byte";
	case BufferFormat::Int:
		return "Int";
	case BufferFormat::Short:
		return "Short";
	case BufferFormat::Half:
		return "Half";
	default:
		break;
	}

	return "";
}

BufferDescriptor::BufferDescriptor(uint32 InWidth, uint32 InHeight, uint32 InItemsPerPoint, BufferFormat InFormat /* = BufferFormat::Float */, FLinearColor InDefaultValue /*= FLinearColor::Black*/,
	BufferType InType /* = BufferType::Image */, bool bInMipMaps /* = false*/, bool bInSRGB/* = false*/)
	: Width(InWidth)
	, Height(InHeight)
	, ItemsPerPoint(InItemsPerPoint)
	, Format(InFormat)
	, Type(InType)
	, DefaultValue(InDefaultValue)
	, bIsSRGB(bInSRGB)
	, bMipMaps(bInMipMaps)
{
}

bool BufferDescriptor::IsValid() const
{
	return Width > 0 && Height > 0 && !IsLateBound() && !IsAuto() && ItemsPerPoint > 0;
}

HashType BufferDescriptor::HashValue() const
{
	std::vector<HashType> Hashes = 
	{
		FormatHashValue(),
		DataUtil::Hash_Simple(DefaultValue),
		!Name.IsEmpty() ? DataUtil::Hash_Simple(Name) : 0,
	};		

	return DataUtil::Hash(Hashes);
}

bool BufferDescriptor::operator!=(const BufferDescriptor& RHS) const
{
	return !(*this == RHS);
}

bool BufferDescriptor::operator==(const BufferDescriptor& rhs) const
{
	bool bIsSame = true;
	
	if (Width > 0 && rhs.Width > 0)
		bIsSame &= Width == rhs.Width;

	if (bIsSame && Height > 0 && rhs.Height > 0)
		bIsSame &= Height == rhs.Height;
	
	if (bIsSame && ItemsPerPoint > 0 && rhs.ItemsPerPoint > 0)
		bIsSame &= ItemsPerPoint == rhs.ItemsPerPoint;
	
	if (bIsSame && Format != BufferFormat::Auto && rhs.Format != BufferFormat::Auto)
		bIsSame &= Format == rhs.Format;
	
	bIsSame &= bIsSRGB == rhs.bIsSRGB;

	return bIsSame;
}

HashType BufferDescriptor::FormatHashValue() const
{
	std::vector<HashType> Hashes =
	{
		MX_HASH_VAL_DEF(Width),
		MX_HASH_VAL_DEF(Height),
		MX_HASH_VAL_DEF(ItemsPerPoint),
		MX_HASH_VAL_DEF(Format),
		MX_HASH_VAL_DEF(Type),
		MX_HASH_VAL_DEF(bMipMaps),
		MX_HASH_VAL_DEF(bIsSRGB),
	};

	std::vector<HashType> MetaHashes;

	if (Metadata.size())
	{
		MetaHashes.reserve(Metadata.size());

		for (const FString& meta : Metadata)
			MetaHashes.push_back(DataUtil::Hash_GenericString_Name(meta));

		Hashes.insert(Hashes.end(), MetaHashes.begin(), MetaHashes.end());
	}

	return DataUtil::Hash(Hashes);
}

size_t BufferDescriptor::BufferFormatSize(BufferFormat InFormat)
{
	switch (InFormat)
	{
	case BufferFormat::Byte:
		return sizeof(char);
	case BufferFormat::Float:
		return sizeof(float);
	case BufferFormat::Half:
		return sizeof(float) >> 1;
	case BufferFormat::Int:
		return sizeof(int);
	case BufferFormat::Short:
		return sizeof(short);
	}

	return 0;
}

BufferFormat BufferDescriptor::BufferFormatFromPixelFormat(EPixelFormat PixelFormat)
{
	switch (PixelFormat)
	{
	case PF_R8:
	case PF_G8:
	case PF_R8G8:
	case PF_B8G8R8A8:
	case PF_A8R8G8B8:
	case PF_R8G8B8A8:
		return BufferFormat::Byte;
	case PF_R16F:
	case PF_G16R16F:
	case PF_FloatRGB:
	case PF_FloatRGBA:
	case PF_A16B16G16R16:
		return BufferFormat::Half;
	case PF_G16:
	case PF_R16G16B16A16_UINT:
		return BufferFormat::Short;
	case PF_R32_FLOAT:
	case PF_G32R32F:
	case PF_R32G32B32F:
	case PF_A32B32G32R32F:
		return BufferFormat::Float;
	default:
		break;
	}

	/// Defult to float ...
	return BufferFormat::Float;
}

EPixelFormat BufferDescriptor::BufferPixelFormat(BufferFormat InFormat, uint32 InItemsPerPoint)
{
	if (InItemsPerPoint == 1)
	{
		switch (InFormat)
		{
		case BufferFormat::Byte:
			return EPixelFormat::PF_G8;
		case BufferFormat::Float:
			return EPixelFormat::PF_R32_FLOAT;
		case BufferFormat::Half:
			return EPixelFormat::PF_R16F;
		case BufferFormat::Int:
			return EPixelFormat::PF_R32_SINT;
		case BufferFormat::Short:
			return EPixelFormat::PF_R16_SINT;
		}
	}
	else if (InItemsPerPoint == 2)
	{
		switch (InFormat)
		{
		case BufferFormat::Byte:
			return EPixelFormat::PF_R8G8;
		case BufferFormat::Float:
			return EPixelFormat::PF_G32R32F;
		case BufferFormat::Half:
			return EPixelFormat::PF_G16R16F;
		case BufferFormat::Int:
			return EPixelFormat::PF_R32G32_UINT;
		case BufferFormat::Short:
			return EPixelFormat::PF_R16G16_UINT;
		}
	}
	if (InItemsPerPoint == 3)
	{
		switch (InFormat)
		{
		case BufferFormat::Byte:
			return EPixelFormat::PF_B8G8R8A8;
		case BufferFormat::Float:
			return EPixelFormat::PF_R32G32B32F;
		case BufferFormat::Half:
			return EPixelFormat::PF_FloatRGB;
		}
	}
	if (InItemsPerPoint == 4)
	{
		switch (InFormat)
		{
		case BufferFormat::Byte:
			return EPixelFormat::PF_B8G8R8A8;
		case BufferFormat::Float:
			return EPixelFormat::PF_A32B32G32R32F;
		case BufferFormat::Half:
			return EPixelFormat::PF_FloatRGBA;
		case BufferFormat::Int:
			return EPixelFormat::PF_R32G32B32A32_UINT;
		case BufferFormat::Short:
			return EPixelFormat::PF_R16G16B16A16_UINT;
		}
	}

	return EPixelFormat::PF_Unknown;
}

bool BufferDescriptor::IsFinal() const
{
	return
		Width > 0 &&
		Height > 0 &&
		ItemsPerPoint > 0 &&
		Format != BufferFormat::Auto;
}

BufferDescriptor BufferDescriptor::Combine(const BufferDescriptor& Desc1, const BufferDescriptor& Desc2)
{
	BufferDescriptor CombinedDesc;

	CombinedDesc.Width = std::max(Desc1.Width, Desc2.Width);
	CombinedDesc.Height = std::max(Desc1.Height, Desc2.Height);
	CombinedDesc.Format = std::max(Desc1.Format, Desc2.Format);
	CombinedDesc.ItemsPerPoint = std::max(Desc1.ItemsPerPoint, Desc2.ItemsPerPoint);
	CombinedDesc.DefaultValue = Desc1.DefaultValue;
	CombinedDesc.bIsSRGB = Desc1.bIsSRGB || Desc2.bIsSRGB;
	CombinedDesc.bMipMaps = Desc1.bMipMaps || Desc2.bMipMaps;
	
	/// If either of the formats are late bound then they take precedence over everything else
	if (Desc1.Format == BufferFormat::LateBound || Desc2.Format == BufferFormat::LateBound)
	{
		/// If we're doing late bound then we may as well reset the width and height and let them be
		/// recalculated later on anyway
		CombinedDesc.Width = CombinedDesc.Height = CombinedDesc.ItemsPerPoint = 0;
		CombinedDesc.Format = BufferFormat::LateBound;
	}

	return CombinedDesc;
}

BufferDescriptor BufferDescriptor::CombineWithPreference(const BufferDescriptor* BaseDesc, const BufferDescriptor* OverrideDesc, const BufferDescriptor* RefDesc)
{
	BufferDescriptor CombinedDesc = *BaseDesc;

	if (OverrideDesc)
	{
		if (OverrideDesc->Width)
		{
			CombinedDesc.Width = OverrideDesc->Width;
		}

		if (OverrideDesc->Height)
		{
			CombinedDesc.Height = OverrideDesc->Height;
		}

		/// 
		if (!OverrideDesc->IsAuto() && !OverrideDesc->IsLateBound())
			CombinedDesc.Format = OverrideDesc->Format;

		if (OverrideDesc->ItemsPerPoint > 0)
			CombinedDesc.ItemsPerPoint = OverrideDesc->ItemsPerPoint;

		CombinedDesc.bMipMaps |= OverrideDesc->bMipMaps;
		CombinedDesc.bIsSRGB |= OverrideDesc->bIsSRGB;
	}

	if (RefDesc)
	{
		if (CombinedDesc.Width <= 0 && RefDesc->Width > 0)
		{
			CombinedDesc.Width = RefDesc->Width;
		}

		if (CombinedDesc.Height <= 0 && RefDesc->Height > 0)
		{
			CombinedDesc.Height = RefDesc->Height;
		}

		if (CombinedDesc.IsAuto() && !RefDesc->IsAuto())
			CombinedDesc.Format = RefDesc->Format;

		if (CombinedDesc.ItemsPerPoint <= 0)
			CombinedDesc.ItemsPerPoint = RefDesc->ItemsPerPoint;

		CombinedDesc.bMipMaps |= RefDesc->bMipMaps;
		CombinedDesc.bIsSRGB |= RefDesc->bIsSRGB;
	}
	
	return CombinedDesc;
}

ETextureSourceFormat BufferDescriptor::TextureSourceFormat(BufferFormat InFormat, uint32 InItemsPerPoint)
{
	if (InFormat == BufferFormat::Byte)
	{
		if (InItemsPerPoint == 1)
			return TSF_G8;
		else if (InItemsPerPoint == 4)
			return TSF_BGRA8;
	}
	else if (InFormat == BufferFormat::Short)
	{
		if (InItemsPerPoint == 1)
			return TSF_G16;
		else if (InItemsPerPoint == 4)
			return TSF_RGBA16;
	}
	else if (InFormat == BufferFormat::Half)
	{
		if (InItemsPerPoint == 4)
			return TSF_RGBA16F;
	}

	return TSF_Invalid;
}

ETextureSourceFormat BufferDescriptor::TextureSourceFormat() const
{
	return TextureSourceFormat(Format, ItemsPerPoint);
}

//////////////////////////////////////////////////////////////////////////
const RawBufferCompressionType RawBuffer::GDefaultCompression = RawBufferCompressionType::LZ4;
const uint64 RawBuffer::GMinCompress = 16 * 1024;

//////////////////////////////////////////////////////////////////////////

RawBuffer::RawBuffer(const uint8* InData, size_t InLength, const BufferDescriptor& InDesc, CHashPtr InHashValue /* = nullptr */, bool bInIsMemoryAutoManaged ) 
	: Data(InData)
	, Length(InLength)
	, Desc(InDesc)
	, bIsMemoryAutoManaged(bInIsMemoryAutoManaged)
{
	/// calculate hash if one isn't provided
	if (InLength <= 0 && !InHashValue) //calculate from desc only when final hash is not in place and length isnt provided
		HashValue =	std::make_shared<CHash>(InDesc.HashValue(), false);
	else if (InHashValue)
	{
		HashValue = InHashValue;
	}
	else
	{
		HashType FinalHash = DataUtil::Hash(InData, InLength);
		HashValue = std::make_shared<CHash>(FinalHash, true);
	}
}

RawBuffer::RawBuffer(const RawBufferPtrTiles& InTiles)
{
}

RawBuffer::RawBuffer(const BufferDescriptor& InDesc) : Desc(InDesc)
{
	HashValue = std::make_shared<CHash>(InDesc.HashValue(), false);
}

RawBuffer::~RawBuffer()
{
	UE_LOGF(LogData, VeryVerbose, "RawBuffer DELETING: %llu [Name: %ls, Size: %dx%d]", HashValue->Value(), *Desc.Name, Width(), Height());
	
	if (!bIsMemoryAutoManaged)
	{
		FreeDisk();
		FreeCompressed();
		FreeUncompressed();
	}
}

void RawBuffer::FreeDisk()
{
	if (!FileName.IsEmpty() && FPaths::ValidatePath(FileName) && FPaths::FileExists(FileName))
	{
		IFileManager::Get().Delete(*FileName);
		FileName = TEXT("");
	}
}

void RawBuffer::FreeUncompressed()
{
	delete[] Data;
	Data = nullptr;
}

void RawBuffer::FreeCompressed()
{
	if (CompressedData)
		free((void*)CompressedData);

	CompressedData = nullptr;
	CompressedLength = 0;
}

RawBufferCompressionType RawBuffer::ChooseBestCompressionFormat() const
{
	if (Desc.Type != BufferType::Image)
		return GDefaultCompression;

	ETextureSourceFormat srcFormat = Desc.TextureSourceFormat();
	if (srcFormat != TSF_Invalid)
		return RawBufferCompressionType::PNG;

	return GDefaultCompression;
}

AsyncRawBufferP RawBuffer::LoadRawBuffer(bool bInDoUncompress, bool bFreeMemory /* = true */)
{
	check(IsInGameThread());

	/// If uncompressed buffer is requested ...
	if (bInDoUncompress)
	{
		/// 1. If already uncompressed then just wrap up
		if (Data)
			return cti::make_ready_continuable(this);
		/// 2. If we already have compressed then just load it
		else if (CompressedData)
			return Uncompress(bFreeMemory);
		/// 3. Otherwise the data may be on disk, then we load it from there
		else if (!FileName.IsEmpty())
			return ReadFromFile(bInDoUncompress);

		/// This shouldn't be reached
		FString errorStr = FString::Printf(TEXT("Invalid raw buffer state or invalid flags passed. Hash: %llu"), HashValue->Value());
		UE_LOGF(LogData, Error, "%ls", *errorStr);

		return cti::make_exceptional_continuable<RawBuffer*>(std::make_exception_ptr(std::runtime_error(TCHAR_TO_UTF8(*errorStr))));
	}

	/// Otherwise compressed buffer is requested
	/// We don't implicitly uncompress the buffer so if either buffers are present
	/// we just early out. The caller must ensure after this function whether to
	/// compress (using one of the compression functions) or not
	if (Data || CompressedData)
		return cti::make_ready_continuable(this);

	/// Otherwise, it must be on the disk ... load it
	else if (!FileName.IsEmpty())
		return ReadFromFile(false);

	/// This shouldn't be reached
	FString errorStr = FString::Printf(TEXT("Invalid raw buffer state or invalid flags passed. Hash: %llu"), HashValue->Value());
	UE_LOGF(LogData, Error, "%ls", *errorStr);

	return cti::make_exceptional_continuable<RawBuffer*>(std::make_exception_ptr(std::runtime_error(TCHAR_TO_UTF8(*errorStr))));
}

AsyncRawBufferP RawBuffer::WriteToFile(const FString& InFileName, bool bFreeMemory /* = true */)
{
	check(IsInGameThread());

	if (!FileName.IsEmpty() && FPaths::FileExists(FileName))
		return cti::make_ready_continuable(this);

	check(!InFileName.IsEmpty());

	if (!FPaths::ValidatePath(InFileName))
	{
		/// This shouldn't be reached
		FString errorStr = FString::Printf(TEXT("Invalid path specified: %s"), *InFileName);
		UE_LOGF(LogData, Error, "%ls", *errorStr);

		return cti::make_exceptional_continuable<RawBuffer*>(std::make_exception_ptr(std::runtime_error(TCHAR_TO_UTF8(*errorStr))));
	}

	FileName = InFileName;

	/// Must be compressed first ...
	return Compress()
		.then([this, bFreeMemory]()
		{
			check(CompressedData);
			return cti::make_continuable<RawBuffer*>([this, bFreeMemory](auto&& promise)
			{
				AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, bFreeMemory, promise = std::forward<decltype(promise)>(promise)]() mutable
				{
					TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*FileName, 0));

					if (!Ar)
					{
						Util::OnGameThread([this, promise = std::forward<decltype(promise)>(promise)]() mutable
						{
							FString errorStr = FString::Printf(TEXT("Error creating writer for file: %s"), *FileName);
							UE_LOGF(LogData, Error, "%ls", *errorStr);
							promise.set_exception(std::make_exception_ptr(std::runtime_error(TCHAR_TO_UTF8(*errorStr))));
						});
						return;
					}

					RawBuffer::FileHeader Header = 
					{
						(uint64)sizeof(FileHeader),
						(uint64)CompressionType,
						(uint64)CompressedLength
					};

					Ar->Serialize(&Header, sizeof(RawBuffer::FileHeader));
					Ar->Serialize((void*)CompressedData, CompressedLength);

					// Always explicitly close to catch errors from flush/close
					Ar->Close();

					if (!Ar->IsError() && !Ar->IsCriticalError())
					{
						if (bFreeMemory)
						{
							FreeCompressed();
						}

						Util::OnGameThread([this, promise = std::forward<decltype(promise)>(promise)]() mutable
						{
							promise.set_value(this);
						});
						return;
					}

					Util::OnGameThread([this, Promise = std::forward<decltype(promise)>(promise)]() mutable
					{
						FString errorStr = FString::Printf(TEXT("Critical error occured while writing file: %s"), *FileName);
						UE_LOGF(LogData, Error, "%ls", *errorStr);
						Promise.set_exception(std::make_exception_ptr(std::runtime_error(TCHAR_TO_UTF8(*errorStr))));
					});
				});
			});
		});
}

AsyncRawBufferP RawBuffer::ReadFromFile(bool bDoUncompress /* = false */)
{
	check(!FileName.IsEmpty() && FPaths::ValidatePath(FileName));

	return cti::make_continuable<RawBuffer*>([this](auto&& promise)
	{
		AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, promise = std::forward<decltype(promise)>(promise)]() mutable
		{
			TUniquePtr<FArchive> Ar = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*FileName, 0));

			if (!Ar)
			{
				Util::OnGameThread([this, Promise = std::forward<decltype(promise)>(promise)]() mutable
				{
					FString errorStr = FString::Printf(TEXT("Error creating reader for file: %s"), *FileName);
					UE_LOGF(LogData, Error, "%ls", *errorStr);
					Promise.set_exception(std::make_exception_ptr(std::runtime_error(TCHAR_TO_UTF8(*errorStr))));
				});
				return;
			}

			RawBuffer::FileHeader Header = 
			{
				(uint64)sizeof(FileHeader),
				(uint64)CompressionType,
				(uint64)CompressedLength
			};

			Ar->Serialize(&Header, sizeof(RawBuffer::FileHeader));

			CompressionType = (RawBufferCompressionType)Header.CompressionType;
			CompressedLength = Header.CompressedLength;

			check(CompressedLength > 0);
			CompressedData = (uint8*)malloc(CompressedLength);

			Ar->Serialize((void*)CompressedData, CompressedLength);

			// Always explicitly close to catch errors from flush/close
			Ar->Close();

			if (!Ar->IsError() && !Ar->IsCriticalError())
			{
				Util::OnGameThread([this, Promise = std::forward<decltype(promise)>(promise)]() mutable
				{
					Promise.set_value(this);
				});
				return;
			}

			Util::OnGameThread([this, Promise = std::forward<decltype(promise)>(promise)]() mutable
			{
				FString errorStr = FString::Printf(TEXT("Critical error occured while reading from file: %s"), *FileName);
				UE_LOGF(LogData, Error, "%ls", *errorStr);
				Promise.set_exception(std::make_exception_ptr(std::runtime_error(TCHAR_TO_UTF8(*errorStr))));
			});
		});
	})
	.then([this, bDoUncompress]()
	{
		if (!bDoUncompress)
			return (AsyncRawBufferP)cti::make_ready_continuable(this);

		return Uncompress(true);
	});
}

AsyncRawBufferP RawBuffer::Compress(RawBufferCompressionType InCompressionType /* = RawBufferCompressionType::Auto */, bool bFreeMemory /* = true */)
{ 
	check(IsInGameThread());

	/// already compressed
	if (CompressionType != RawBufferCompressionType::None && CompressedData)
		return cti::make_ready_continuable(this);

	if (Length <= (uint64)GMinCompress)
		return cti::make_ready_continuable(this);

	return cti::make_continuable<RawBuffer*>([this, InCompressionType, bFreeMemory](auto&& Promise)
	{
		AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, InCompressionType, bFreeMemory, promise = std::forward<decltype(Promise)>(Promise)]() mutable
		{
			if (InCompressionType == RawBufferCompressionType::Auto)
				InCompressionType = ChooseBestCompressionFormat();

			/// Couldn't find a suitable compression InFormat
			check(InCompressionType != RawBufferCompressionType::None);

			bool bDidCompress = false;

			if (InCompressionType == RawBufferCompressionType::ZLib)
				bDidCompress = CompressGeneric(NAME_Zlib, InCompressionType);
			else if (InCompressionType == RawBufferCompressionType::GZip)
				bDidCompress = CompressGeneric(NAME_Gzip, InCompressionType);
			else if (InCompressionType == RawBufferCompressionType::PNG)
				bDidCompress = CompressPNG();
			else if (InCompressionType == RawBufferCompressionType::LZ4)
				bDidCompress = CompressGeneric(NAME_LZ4, InCompressionType);

			if (bDidCompress)
			{
				if (bFreeMemory)
				{
					/// We free up the _data but keep the _length
					FreeUncompressed();
				}
			}

			Util::OnGameThread([this, Promise = std::forward<decltype(promise)>(promise)]() mutable
			{
				Promise.set_value(this);
			});
		});
	});
}

AsyncRawBufferP RawBuffer::Uncompress(bool bFreeMemory /* = true */)
{
	check(IsInGameThread());

	/// Not a compressed buffer
	if (CompressionType == RawBufferCompressionType::None)
		return cti::make_ready_continuable(this);

	return cti::make_continuable<RawBuffer*>([this, bFreeMemory](auto&& promise)
	{
		AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, bFreeMemory, promise = std::forward<decltype(promise)>(promise)]() mutable
		{
			bool bDidUncompress = false;

			if (CompressionType == RawBufferCompressionType::ZLib)
				bDidUncompress = UncompressGeneric(NAME_Zlib);
			else if (CompressionType == RawBufferCompressionType::GZip)
				bDidUncompress = UncompressGeneric(NAME_Gzip);
			else if (CompressionType == RawBufferCompressionType::PNG)
				bDidUncompress = UncompressPNG();
			else if (CompressionType == RawBufferCompressionType::LZ4)
				bDidUncompress = UncompressGeneric(NAME_LZ4);

			if (bDidUncompress)
			{
				if (bFreeMemory)
				{
					/// We free up the compressed data
					FreeCompressed();
				}
			}

			Util::OnGameThread([this, Promise = std::forward<decltype(promise)>(promise)]() mutable
			{
				Promise.set_value(this);
			});
		});
	});
}

bool RawBuffer::CompressPNG()
{
	/// Taken from: Engine/Private/Texture.cpp [Compress() function]
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);

	// TODO: TSF_BGRA8 is stored as RGBA, so the R and B channels are swapped in the internal png. Should we fix this?
	auto SrcFormat = Desc.TextureSourceFormat();
	if (SrcFormat == TSF_Invalid)
		return false;

	ERGBFormat RawFormat = (SrcFormat == TSF_G8 || SrcFormat == TSF_G16) ? ERGBFormat::Gray : ERGBFormat::RGBA;
	if (ImageWrapper.IsValid() && ImageWrapper->SetRaw(Data, Length, Desc.Width, Desc.Height, RawFormat, (SrcFormat == TSF_G16 || SrcFormat == TSF_RGBA16) ? 16 : 8))
	{
		const TArray64<uint8>& ImageCompressedData = ImageWrapper->GetCompressed();
		if (ImageCompressedData.Num() > 0)
		{
			CompressedLength = ImageCompressedData.Num();
			CompressedData = (uint8*) malloc(CompressedLength);

			FMemory::Memcpy((void*)CompressedData, ImageCompressedData.GetData(), CompressedLength);

			CompressionType = RawBufferCompressionType::PNG;

			return true;
		}
	}

	return false;
}

bool RawBuffer::UncompressPNG()
{
	/// Taken from: Engine/Private/Texture.cpp [Compress() function]
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);

	// TODO: TSF_BGRA8 is stored as RGBA, so the R and B channels are swapped in the internal png. Should we fix this?
	auto SrcFormat = Desc.TextureSourceFormat();
	if (SrcFormat == TSF_Invalid)
		return false;

	if (ImageWrapper.IsValid() && ImageWrapper->SetCompressed((const void*)CompressedData, CompressedLength))
	{
		ERGBFormat RawFormat = (SrcFormat == TSF_G8 || SrcFormat == TSF_G16) ? ERGBFormat::Gray : ERGBFormat::RGBA;
		int32 BitDepth = (SrcFormat == TSF_G16 || SrcFormat == TSF_RGBA16) ? 16 : 8;
		TArray64<uint8> UncompressedData;

		if (ImageWrapper->GetRaw(RawFormat, BitDepth, UncompressedData))
		{
			if (UncompressedData.Num() > 0)
			{
				Length = UncompressedData.Num();
				Data = new uint8 [Length];

				FMemory::Memcpy((void*)Data, UncompressedData.GetData(), Length);

				return true;
			}
		}
	}

	return false;
}

bool RawBuffer::CompressGeneric(FName InName, RawBufferCompressionType InType)
{
	int32 CurrentCompressedLength = Length;
	uint8* NewCompressedData = (uint8*)malloc(CurrentCompressedLength);

	bool bDidCompress = FCompression::CompressMemory(InName, (void*)NewCompressedData, CurrentCompressedLength, (const void*)Data, Length);

	if (bDidCompress)
	{
		/// reallocate to proper size
		if (CurrentCompressedLength < Length)
			CompressedData = (uint8*) realloc((void*)NewCompressedData, CurrentCompressedLength);

		CompressedLength = CurrentCompressedLength;
		CompressionType = InType;
	}
	else
		free((void*)NewCompressedData);

	return bDidCompress;
}

bool RawBuffer::UncompressGeneric(FName InName)
{
	/// This must have been assigned before the compression
	check(Length > 0);

	Data = new uint8 [Length];

	return FCompression::UncompressMemory(InName, (void*)Data, (int32)Length, (const void*)CompressedData, (int32)CompressedLength);
}

void RawBuffer::GetAsLinearColor(TArray<FLinearColor>& Pixels) const
{
	Pixels.AddDefaulted(Desc.Width * Desc.Height);

	ParallelFor(Pixels.Num(), [&](int32 Pixel)
	{
		Pixels[Pixel] = GetAsLinearColor(Pixel);
	});
	/*if(Desc.ItemsPerPoint == 4)
	{
		if(Desc.Format == BufferFormat::Byte)
		{
			ParallelFor(Pixels.Num(), [&](int32 Pixel)
			{
				auto Color = *(FColor*)(Data + (Pixel * sizeof(FColor)));
				Pixels[Pixel] = Color.ReinterpretAsLinear();
			});	
		}
		else if(Desc.Format == BufferFormat::Half)
		{
			ParallelFor(Pixels.Num(), [&](int32 Pixel)
			{
				auto Color = *(FFloat16Color*)(Data + (Pixel * sizeof(FFloat16Color)));
				Pixels[Pixel] = Color.GetFloats();
			});	
		}
		else if(Desc.Format == BufferFormat::Float)
		{
			ParallelFor(Pixels.Num(), [&](int32 Pixel)
			{
				auto Color = *(FLinearColor*)(Data + (Pixel * sizeof(FLinearColor)));
				Pixels[Pixel] = Color;
			});	
		}	
	}
	else if(Desc.ItemsPerPoint == 2)
	{
		if(Desc.Format == BufferFormat::Byte)
		{
			ParallelFor(Pixels.Num(), [&](int32 Pixel)
			{
				auto RColor = *(uint8*)(Data + (Pixel * Desc.ItemsPerPoint + 0) * sizeof(uint8));
				auto GColor = *(uint8*)(Data + (Pixel * Desc.ItemsPerPoint + 1) * sizeof(uint8));

				Pixels[Pixel].R = RColor/255.f;
				Pixels[Pixel].G = GColor/255.f;
				Pixels[Pixel].A = 1;
			});	
		}
		else if(Desc.Format == BufferFormat::Half)
		{
			ParallelFor(Pixels.Num(), [&](int32 Pixel)
			{
				auto RColor = *(FFloat16*)(Data + ((Pixel * Desc.ItemsPerPoint + 0) * sizeof(FFloat16)));
				auto GColor = *(FFloat16*)(Data + ((Pixel * Desc.ItemsPerPoint + 1) * sizeof(FFloat16)));

				Pixels[Pixel].R = RColor.GetFloat();
				Pixels[Pixel].G = GColor.GetFloat();
				Pixels[Pixel].A = 1;
			});	
		}
		else if(Desc.Format == BufferFormat::Float)
		{
			ParallelFor(Pixels.Num(), [&](int32 Pixel)
			{
				auto RColor = *(float*)(Data + ((Pixel * Desc.ItemsPerPoint + 0) * sizeof(float)));
				auto GColor = *(float*)(Data + ((Pixel * Desc.ItemsPerPoint + 1) * sizeof(float)));

				Pixels[Pixel].R = RColor;
				Pixels[Pixel].G = GColor;
				Pixels[Pixel].A = 1;
			});	
		}
	}
	else if(Desc.ItemsPerPoint == 1)
	{
		if(Desc.Format == BufferFormat::Byte)
		{
			ParallelFor(Pixels.Num(), [&](int32 Pixel)
			{
				auto RColor = *(uint8*)(Data + (Pixel * sizeof(uint8)));
				Pixels[Pixel].R = RColor/255.f;
				Pixels[Pixel].A = 1;

			});	
		}
		else if(Desc.Format == BufferFormat::Half)
		{
			ParallelFor(Pixels.Num(), [&](int32 Pixel)
			{
				auto RColor = *(FFloat16*)(Data + (Pixel * sizeof(FFloat16)));
				Pixels[Pixel].R = RColor.GetFloat();
				Pixels[Pixel].A = 1;
			});	
		}
		else if(Desc.Format == BufferFormat::Float)
		{
			ParallelFor(Pixels.Num(), [&](int32 Pixel)
			{
				auto RColor = *(float*)(Data + (Pixel * sizeof(float)));
				Pixels[Pixel].R = RColor;
				Pixels[Pixel].A = 1;
			});	
		}
	}*/
}

FLinearColor RawBuffer::GetAsLinearColor(int PixelIndex) const
{
	FLinearColor LinearColor = FLinearColor::Black;

	if (Desc.ItemsPerPoint == 4)
	{
		if (Desc.Format == BufferFormat::Byte)
		{
			auto Color = *(FColor*)(Data + (PixelIndex * sizeof(FColor)));
			LinearColor = Color.ReinterpretAsLinear();
		}
		else if (Desc.Format == BufferFormat::Half)
		{
			auto Color = *(FFloat16Color*)(Data + (PixelIndex * sizeof(FFloat16Color)));
			LinearColor = Color.GetFloats();
		}
		else if (Desc.Format == BufferFormat::Float)
		{
			auto Color = *(FLinearColor*)(Data + (PixelIndex * sizeof(FLinearColor)));
			LinearColor = Color;
		}
	}
	else if (Desc.ItemsPerPoint == 2)
	{
		if (Desc.Format == BufferFormat::Byte)
		{
			auto RColor = *(uint8*)(Data + (PixelIndex * Desc.ItemsPerPoint + 0) * sizeof(uint8));
			auto GColor = *(uint8*)(Data + (PixelIndex * Desc.ItemsPerPoint + 1) * sizeof(uint8));

			LinearColor.R = RColor / 255.f;
			LinearColor.G = GColor / 255.f;
			LinearColor.A = 1;
		}
		else if (Desc.Format == BufferFormat::Half)
		{
			auto RColor = *(FFloat16*)(Data + ((PixelIndex * Desc.ItemsPerPoint + 0) * sizeof(FFloat16)));
			auto GColor = *(FFloat16*)(Data + ((PixelIndex * Desc.ItemsPerPoint + 1) * sizeof(FFloat16)));

			LinearColor.R = RColor.GetFloat();
			LinearColor.G = GColor.GetFloat();
			LinearColor.A = 1;
		}
		else if (Desc.Format == BufferFormat::Float)
		{
			auto RColor = *(float*)(Data + ((PixelIndex * Desc.ItemsPerPoint + 0) * sizeof(float)));
			auto GColor = *(float*)(Data + ((PixelIndex * Desc.ItemsPerPoint + 1) * sizeof(float)));

			LinearColor.R = RColor;
			LinearColor.G = GColor;
			LinearColor.A = 1;
		}
	}
	else if (Desc.ItemsPerPoint == 1)
	{
		if (Desc.Format == BufferFormat::Byte)
		{
			auto RColor = *(uint8*)(Data + (PixelIndex * sizeof(uint8)));
			LinearColor.R = RColor / 255.f;
			LinearColor.A = 1;
		}
		else if (Desc.Format == BufferFormat::Half)
		{
			auto RColor = *(FFloat16*)(Data + (PixelIndex * sizeof(FFloat16)));
			LinearColor.R = RColor.GetFloat();
			LinearColor.A = 1;
		}
		else if (Desc.Format == BufferFormat::Float)
		{
			auto RColor = *(float*)(Data + (PixelIndex * sizeof(float)));
			LinearColor.R = RColor;
			LinearColor.A = 1;
		}
	}
	return LinearColor;
}

bool RawBuffer::IsPadded() const
{
	const EPixelFormat PixelFormat = BufferDescriptor::BufferPixelFormat(GetDescriptor().Format, GetDescriptor().ItemsPerPoint);
	const uint32 PaddedWidth = GetLength() / (Height() * GPixelFormats[PixelFormat].BlockBytes);
	return (PaddedWidth != Width());
}
size_t RawBuffer::GetUnpaddedSize()
{
	const EPixelFormat PixelFormat = BufferDescriptor::BufferPixelFormat(GetDescriptor().Format, GetDescriptor().ItemsPerPoint);
	const int32 NumBlocksX = Width() / GPixelFormats[PixelFormat].BlockSizeX;
	const int32 NumBlocksY = Height() / GPixelFormats[PixelFormat].BlockSizeY;
	const uint32 DestStride = NumBlocksX * GPixelFormats[PixelFormat].BlockBytes;
	return DestStride * NumBlocksY;
}
/// Caller is expected to allocate/manage the memory of DestData themselves.
void RawBuffer::CopyUnpaddedBytes(uint8* DestData)
{
	// expect this method to only be called if there is padding
	check (IsPadded());
	
	// Validate that OutDestPtr is not nullptr before updating
	check (DestData != nullptr);
	
	const EPixelFormat PixelFormat = BufferDescriptor::BufferPixelFormat(GetDescriptor().Format, GetDescriptor().ItemsPerPoint);
	const uint32 PaddedWidth = GetLength() / (Height() * GPixelFormats[PixelFormat].BlockBytes);
	
	// Stride in Source (RawBuffer)
	const uint32 SrcStride = (PaddedWidth / GPixelFormats[PixelFormat].BlockSizeX) * GPixelFormats[PixelFormat].BlockBytes;

	// Destination sizes
	const int32 NumBlocksX = Width() / GPixelFormats[PixelFormat].BlockSizeX;
	const int32 NumBlocksY = Height() / GPixelFormats[PixelFormat].BlockSizeY;
	const uint32 DestStride = NumBlocksX * GPixelFormats[PixelFormat].BlockBytes;
	const int64 DestSize = DestStride * NumBlocksY;

	const uint8* RawDataPtr = GetData();
	const uint8* SrcPtr = RawDataPtr;
	uint8* DestPtr = DestData;
	
	// Validate that the destination buffer is within bounds
	checkf (DestData <= RawDataPtr ||
			DestData + DestSize >= RawDataPtr + GetLength(), TEXT("OutDestPtr is out of bounds of the RawDataPtr buffer."));
	
	// Loop each row and copy from SrcPtr to DestPtr without the padding
	for (uint32 Row = 0; Row < Height(); Row++)
	{
		// Use memcpy to copy data while updating the destination pointer
		FMemory::Memcpy(DestPtr, SrcPtr, DestStride);
		DestPtr += DestStride;
		SrcPtr += SrcStride;
	}

}

