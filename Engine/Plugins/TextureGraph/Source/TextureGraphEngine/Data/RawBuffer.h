// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Helper/DataUtil.h"
THIRD_PARTY_INCLUDES_START
#include "continuable/continuable.hpp"
THIRD_PARTY_INCLUDES_END

#include <set>
#include <PixelFormat.h>
#include <Engine/Texture.h>

#define UE_API TEXTUREGRAPHENGINE_API

enum class BufferFormat 
{
	Byte,
	Half,
	Float,
	Short,
	Int,

	Count,

	Auto = -1,						/// Automatically deduce
	Custom = -2,
	
	LateBound = -3,					/// The Format is unknown at the time and will 
									/// be known after some async operation has finished
									/// Please don't change this value
};

enum class BufferType
{
	Generic,
	Image,
	Mesh,
	MeshDetail
};

//////////////////////////////////////////////////////////////////////////
/// BufferDescriptor Metadata Defs
/// Some standardised metadata defs that are used very often. This is not
/// meant to be a comprehensive repository. Other systems can make 
/// assumptions and have their own names without having them defined over here.
//////////////////////////////////////////////////////////////////////////
struct RawBufferMetadataDefs
{
	//////////////////////////////////////////////////////////////////////////
	/// FX related metadata
	//////////////////////////////////////////////////////////////////////////
	static const FString			G_FX_UAV;

	//////////////////////////////////////////////////////////////////////////
	/// CPU related metadata
	//////////////////////////////////////////////////////////////////////////

	//////////////////////////////////////////////////////////////////////////
	/// Semantic metadata
	//////////////////////////////////////////////////////////////////////////
	static const FString			G_LAYER_MASK;
};

//////////////////////////////////////////////////////////////////////////
struct BufferDescriptor
{
	typedef std::set<FString>		BufferMetadata;

	FString							Name;					/// The name of the buffer
	uint32							Width = 0;				/// Width of the buffer in terms of number of points
	uint32							Height = 0;				/// Height of the buffer in terms of number of points
	uint32							ItemsPerPoint = 0;		/// How many items of Type BufferFormat per point
	BufferFormat					Format = BufferFormat::Auto;		/// What is the Type of each item in the buffer
	BufferType						Type = BufferType::Image;			/// Buffer Type. This is mostly used for debugging
	FLinearColor					DefaultValue = FLinearColor::Black; /// The default value of the blob values

	bool							bIsSRGB = false;
	bool							bIsTransient = false;	/// Whether this is a transient buffer
	bool							bMipMaps = false;		/// Enable mip maps for this buffer

	BufferMetadata					Metadata;				/// The metadata associated with this buffer descriptor

	UE_API HashType						HashValue() const;
	UE_API HashType						FormatHashValue() const;
	static UE_API const char*				FormatToString(BufferFormat InFormat);

									BufferDescriptor() {}
									UE_API BufferDescriptor(uint32 InWidth, uint32 InHeight, uint32 InItemsPerPoint, 
										BufferFormat InFormat = BufferFormat::Float, FLinearColor InDefaultValue = FLinearColor::Black, BufferType InType = BufferType::Image, bool bInMipMaps = false, bool bInSRGB = false);

	static UE_API EPixelFormat				BufferPixelFormat(BufferFormat InFormat, uint32 InItemsPerPoint);
	static UE_API BufferFormat				BufferFormatFromPixelFormat(EPixelFormat PixelFormat);
	static UE_API size_t					BufferFormatSize(BufferFormat InFormat);
	static UE_API ETextureSourceFormat		TextureSourceFormat(BufferFormat InFormat, uint32 InItemsPerPoint);
	static UE_API BufferDescriptor			Combine(const BufferDescriptor& Desc1, const BufferDescriptor& Desc2);
	static UE_API BufferDescriptor			CombineWithPreference(const BufferDescriptor* BaseDesc, const BufferDescriptor* OverrideDesc, const BufferDescriptor* RefDesc);

	UE_API ETextureSourceFormat			TextureSourceFormat() const;
	UE_API bool							IsValid() const;
	UE_API bool							IsFinal() const;

	FORCEINLINE size_t				FormatSize() const { return BufferDescriptor::BufferFormatSize(Format); }
	FORCEINLINE size_t				Pitch() const { return Width * FormatSize() * ItemsPerPoint; }
	FORCEINLINE size_t				Size() const { return Pitch() * Height; }
	FORCEINLINE EPixelFormat		PixelFormat() const { return BufferDescriptor::BufferPixelFormat(Format, ItemsPerPoint); }

	FORCEINLINE bool				HasMetadata(const FString& meta) const { return Metadata.find(meta) != Metadata.end(); }
	FORCEINLINE bool				HasMetadata(FString&& meta) const { return Metadata.find(meta) != Metadata.end(); }
	FORCEINLINE bool				IsLateBound() const { return Format == BufferFormat::LateBound; }
	FORCEINLINE bool				IsAuto() const { return Format == BufferFormat::Auto; }
	FORCEINLINE bool				IsAutoWidth() const { return Width <= 0; }
	FORCEINLINE bool				IsAutoHeight() const { return Height <= 0; }
	FORCEINLINE bool				IsAutoSize() const { return IsAutoWidth() || IsAutoHeight(); }

	void							AddMetadata(const FString& meta) { Metadata.insert(meta); };
	void							AddMetadata(FString&& meta) { Metadata.insert(meta); };

	UE_API bool							operator == (const BufferDescriptor& rhs) const;
	UE_API bool							operator != (const BufferDescriptor& RHS) const;
	
	FORCEINLINE void				AllowUAV() { AddMetadata(RawBufferMetadataDefs::G_FX_UAV); }
	FORCEINLINE bool				RequiresUAV() const { return HasMetadata(RawBufferMetadataDefs::G_FX_UAV); }
};

typedef std::unique_ptr<BufferDescriptor>	BufferDescriptorUPtr;
typedef std::shared_ptr<BufferDescriptor>	BufferDescriptorPtr;

class RawBuffer;
typedef std::shared_ptr<RawBuffer>			RawBufferPtr;
typedef T_Tiles<std::shared_ptr<RawBuffer>> RawBufferPtrTiles;
typedef cti::continuable<RawBufferPtr>		AsyncRawBufferPtr;
typedef cti::continuable<RawBuffer*>		AsyncRawBufferP;

enum class RawBufferCompressionType
{
	None,							/// No compression
	Auto,							/// Automatically choose a compression Format
	PNG,							/// PNG compression
	ZLib,							/// ZLib compression
	GZip,							/// GZip compression
	LZ4,							/// LZ4 compression
};

class RawBuffer
{
public:
	static UE_API const uint64				GMinCompress;				/// Must have at least this much data to consider compression. 
																/// Otherwise, its not worth it

private:
	/// Must be fixed size (ideally every member should be 64-bit aligned)
	struct FileHeader
	{
		uint64						Version = sizeof(FileHeader);
		uint64						CompressionType;
		uint64						CompressedLength = 0;

		/// For backwards compatibility Anything that 
	};

	static UE_API const RawBufferCompressionType GDefaultCompression;

	friend class					Device_Mem;

	//////////////////////////////////////////////////////////////////////////
	/// UnCompressed data
	//////////////////////////////////////////////////////////////////////////
	const uint8*					Data = nullptr;				/// The actual raw buffer
	uint64							Length = 0;					/// The length of the data
	BufferDescriptor				Desc;						/// The buffer descriptor
	mutable CHashPtr				HashValue;						/// The hash for this buffer
	bool							bIsMemoryAutoManaged = false; // There can be instances where the owner of _data isnt RawBuffer, this check ensures those cases

	//////////////////////////////////////////////////////////////////////////
	/// Compressed data
	//////////////////////////////////////////////////////////////////////////
	const uint8*					CompressedData = nullptr;	/// The compressed data
	uint64							CompressedLength = 0;		/// Length of the compressed buffer
	RawBufferCompressionType		CompressionType = RawBufferCompressionType::None; /// What is the Type of compression used

	//////////////////////////////////////////////////////////////////////////
	/// Disk data (compressed)
	////////////////////////////////////////////////////////////////////////// 
	FString							FileName;		/// The path where the file data is kept

	UE_API RawBufferCompressionType		ChooseBestCompressionFormat() const;

	UE_API void							FreeUncompressed();
	UE_API void							FreeCompressed();
	UE_API void							FreeDisk();

	UE_API bool							UncompressPNG();
	UE_API bool							UncompressGeneric(FName InName);

	UE_API bool							CompressPNG();
	UE_API bool							CompressGeneric(FName InName, RawBufferCompressionType InType);

	UE_API AsyncRawBufferP					ReadFromFile(bool bDoUncompress = false);

public:
									UE_API RawBuffer(const uint8* InData, size_t InLength, const BufferDescriptor& InDesc, CHashPtr InHashValue = nullptr, bool bInIsMemoryAutoManaged = false);
	UE_API explicit						RawBuffer(const BufferDescriptor& InDesc);
	UE_API explicit						RawBuffer(const RawBufferPtrTiles& InTiles);
									UE_API ~RawBuffer();

	UE_API AsyncRawBufferP					Compress(RawBufferCompressionType InCompressionType = RawBufferCompressionType::Auto, bool bFreeMemory = true);
	UE_API AsyncRawBufferP					Uncompress(bool bFreeMemory = true);

	UE_API AsyncRawBufferP					WriteToFile(const FString& InFileName, bool bFreeMemory = true);

	UE_API AsyncRawBufferP					LoadRawBuffer(bool bInDoUncompress, bool bFreeMemory = true);

	UE_API void							GetAsLinearColor(TArray<FLinearColor>& Pixels) const;
	UE_API FLinearColor					GetAsLinearColor(int PixelIndex) const;

	UE_API bool							IsPadded() const;
	UE_API size_t							GetUnpaddedSize();
	UE_API void							CopyUnpaddedBytes(uint8* DestData);
	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE const uint8*		GetData() const { return Data; }
	FORCEINLINE size_t				GetLength() const { return Length; }
	FORCEINLINE CHashPtr			Hash() const { return HashValue; }
	FORCEINLINE const FString&		GetName() const { return Desc.Name; }
	FORCEINLINE const BufferDescriptor& GetDescriptor() const { return Desc; }
	FORCEINLINE uint32				Width() const { return Desc.Width; }
	FORCEINLINE uint32				Height() const { return Desc.Height; }
	FORCEINLINE bool				IsTransient() const { return Desc.bIsTransient; }
	FORCEINLINE bool				IsCompressed() const { return CompressedData != nullptr; }
	FORCEINLINE bool				HasData() const { check(IsInGameThread()); return Data != nullptr; }
};

#undef UE_API
