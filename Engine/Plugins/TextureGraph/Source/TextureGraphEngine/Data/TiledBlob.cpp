// Copyright Epic Games, Inc. All Rights Reserved.
#include "TiledBlob.h"
#include "Device/DeviceBuffer.h"
#include "Transform/BlobTransform.h"
#include "Device/DeviceManager.h"
#include "Device/FX/Device_FX.h"
#include "Model/Mix/Mix.h"
#include "Job/Job.h"

#include "Profiling/StatGroup.h"
#include "TextureGraphEngineGameInstance.h"
#include "Data/Blobber.h"

DECLARE_CYCLE_STAT(TEXT("TiledBlob_TileBuffer"), STAT_TiledBlob_TileBuffer, STATGROUP_TextureGraphEngine);


//////////////////////////////////////////////////////////////////////////
TiledBlob::TiledBlob(const BufferDescriptor& InDesc, const BlobPtrTiles& Tiles)  
	: Blob(InDesc, nullptr)
	, Tiles(Tiles)
	, Desc(InDesc)
	, bTiledTarget(Tiles.Rows() > 1 || Tiles.Cols() > 1)
{
	if (Desc.Format != BufferFormat::LateBound)
	{
		Desc.Width = std::max(Desc.Width, static_cast<uint32>(Tiles.Rows()));
		Desc.Height = std::max(Desc.Height, static_cast<uint32>(Tiles.Cols()));
	}
	
	CalcHashNow();
	Buffer.reset();

	bIsFinalised = true;

	for (size_t RowId = 0; RowId < Tiles.Rows() && bIsFinalised; RowId++)
	{
		for (size_t ColId = 0; ColId < Tiles.Cols() && bIsFinalised; ColId++)
		{
			check(Tiles[RowId][ColId]);
			BlobPtr Tile = Tiles[RowId][ColId];

			/// Cannot have tiled BlobObj as part of another tiled BlobObj
			check(!Tile->IsTiled());

			bIsFinalised &= Tile->bIsFinalised;
		}
	}

	if (bIsFinalised)
		FinaliseTS = FDateTime::Now();
}

std::shared_ptr<TiledBlob> TiledBlob::InitFromTiles(const BufferDescriptor& InDesc, BlobPtrTiles& Tiles)
{
	BufferDescriptor Desc = InDesc;
	
	Desc.Width = std::max(Desc.Width, static_cast<uint32>(Tiles.Rows()));
	Desc.Height = std::max(Desc.Height, static_cast<uint32>(Tiles.Cols()));
	
	TiledBlobPtr TBlob = std::make_shared<TiledBlob>(Desc, Tiles);

	for (size_t RowId = 0; RowId < Tiles.Rows(); RowId++)
	{
		for (size_t ColId = 0; ColId < Tiles.Cols(); ColId++)
		{
			check(Tiles[RowId][ColId]);
			BlobPtr Tile = Tiles[RowId][ColId];

			/// Cannot have tiled BlobObj as part of another tiled BlobObj
			check(!Tile->IsTiled());
		}
	}

	/// Mark as finalised
	TBlob->bIsFinalised = true;

	return TBlob;
}

TiledBlob::TiledBlob(BlobRef BlobObj)
	: TiledBlob(BlobObj->GetDescriptor(), BlobPtrTiles({BlobObj}, 1, 1))
{
	/// Cannot have tiled BlobObj inside another
	check(BlobObj && !BlobObj->IsTiled());
}

TiledBlob::TiledBlob(DeviceBufferRef Buffer) 
	: Blob(Buffer)
	, Tiles(0, 0)
	, Desc(Buffer->Descriptor())
{
	/// no hash to calculate so far
}

TiledBlob::TiledBlob(const BufferDescriptor& InDesc, size_t NumRows, size_t NumCols, CHashPtr InHashValue)
	: Blob(InDesc, InHashValue)
	, Tiles(NumRows, NumCols)
	, HashValue(InHashValue)
	, Desc(InDesc)
{
	check(NumRows > 0 && NumCols > 0);
	
	if (Desc.Format != BufferFormat::LateBound)
	{
		Desc.Width = std::max(Desc.Width, static_cast<uint32>(Tiles.Rows()));
		Desc.Height = std::max(Desc.Height, static_cast<uint32>(Tiles.Cols()));
	}
	
	check(HashValue);
}

TiledBlob::~TiledBlob()
{
	UE_LOGF(LogData, VeryVerbose, "Deleting TiledBlob: %ls [%dx%d]", *Desc.Name, Desc.Width, Desc.Height);
	Tiles.Clear();
}

TiledBlobPtr TiledBlob::AsTiledBlob(BlobPtr BlobObj)
{
	if (BlobObj->IsTiled())
		return std::static_pointer_cast<TiledBlob>(BlobObj);
	return std::make_shared<TiledBlob>(BlobObj);
}

void TiledBlob::FinaliseFrom(Blob* RHS)
{
	check(RHS->IsTiled());
	TiledBlob* RHSTiled = static_cast<TiledBlob*>(RHS);

	Tiles = RHSTiled->Tiles;
	SetHash(RHSTiled->HashValue);

	bool bIsTransient = Desc.bIsTransient;
	Desc = RHSTiled->Desc;

	/// Retain the transient information as cached tiled blob could be transient and this one isn't or vice-versa
	Desc.bIsTransient = bIsTransient; 

	bReady = RHSTiled->bReady;
	bTiledTarget = RHSTiled->bTiledTarget;
	SingleBlob = RHSTiled->SingleBlob;

	Blob::FinaliseFrom(RHS);
}

void TiledBlob::SetTransient()
{
	Desc.bIsTransient = true;
}

#if DEBUG_BLOB_REF_KEEPING == 1
bool TiledBlob::HasBlobAsTile(Blob* BlobObj) const
{
	check(IsInGameThread());
	check(BlobObj);

	for (size_t RowId = 0; RowId < Tiles.Rows(); RowId++)
	{
		for (size_t ColId = 0; ColId < Tiles.Cols(); ColId++)
		{
			if (Tiles[RowId][ColId] == BlobObj)
				return true;
		}
	}

	return false;
}
#endif 

void TiledBlob::SetTiles(const BlobPtrTiles& InTiles)
{
	Tiles = InTiles;
	CalcHashNow();
}

void TiledBlob::SetTile(int32 X, int32 Y, BlobRef Tile)
{
#if DEBUG_BLOB_REF_KEEPING == 1
	if (Tiles[RowId][ColId])
		Tiles[RowId][ColId]->RemoveOwner(this);
#endif

	auto Self = shared_from_this();
	check(Tile);

#if DEBUG_BLOB_REF_KEEPING == 1
	Tile.lock()->AddOwner(Self);
#endif 

	Tiles[X][Y] = BlobRef(Tile);
}

TiledBlob& TiledBlob::operator = (const TiledBlob& RHS)
{
	Buffer = RHS.Buffer;
	bIsFinalised = RHS.bIsFinalised;
	FinaliseTS = RHS.FinaliseTS;
	ReplayCount = RHS.ReplayCount;

	Tiles = RHS.Tiles;
	SetHash(RHS.HashValue);

	/// Ensure that 1x1 tiled BlobObj hashes are calculated correctly
	check((Tiles.Rows() > 1 && Tiles.Cols() > 1) ||
		((Tiles.Rows() == 1 && Tiles.Cols() == 1) && (HashValue->IsFinal() || (HashValue->IsTemp() && HashValue->NumSources() == 2))));

	Desc = RHS.Desc;
	bReady = RHS.bReady;
	bTiledTarget = RHS.bTiledTarget;

	MinMax = RHS.MinMax;
	MinValue = RHS.MinValue;
	MaxValue = RHS.MaxValue;

	LODParent = RHS.LODParent;
	LODSource = RHS.LODSource;
	LODLevels = RHS.LODLevels;
	bIsLODLevel = RHS.bIsLODLevel;

	return *this;
}

void TiledBlob::ResolveLateBound(const BufferDescriptor& InDesc, bool bOverrideExisting /* = false */)
{
	auto ExistingDesc = Desc;

	Desc = InDesc;

	/// If there is size information in the existing descriptor then that will
	/// override the size coming from the Ref. This is for things like mip-mapping
	/// and other transforms that wanna produce fixed sized images

	if (!bOverrideExisting)
	{
		if (ExistingDesc.Width > 0)
			Desc.Width = ExistingDesc.Width;

		if (ExistingDesc.Height > 0)
			Desc.Height = ExistingDesc.Height;

		if (ExistingDesc.ItemsPerPoint > 0)
			Desc.ItemsPerPoint = ExistingDesc.ItemsPerPoint;

		Desc.Metadata = ExistingDesc.Metadata;
		Desc.Name = ExistingDesc.Name;
	}

	Desc.Width = std::max(Desc.Width, static_cast<uint32>(Tiles.Rows()));
	Desc.Height = std::max(Desc.Height, static_cast<uint32>(Tiles.Cols()));

	/// Make sure the descriptor is still not late bound!
	check(Desc.Format != BufferFormat::LateBound);
	check(Desc.Width > 0 && Desc.Height > 0);
	check(Desc.ItemsPerPoint > 0);
}

void TiledBlob::ResolveLateBound(BlobPtr Ref)
{
	check(Ref);

	/// Copy the descriptor over and release the Ref
	auto RefDesc = Ref->GetDescriptor();

	ResolveLateBound(RefDesc, false);
}

void TiledBlob::CopyResolveLateBound(BlobPtr RHS_)
{
	check(RHS_ != nullptr);

	if (RHS_->IsTiled())
		*this = *static_cast<const TiledBlob*>(RHS_.get());
	else
	{
		/// Single BlobObj type
		Tiles.Resize(1, 1);
		SetTile(0, 0, RHS_);

		Desc = RHS_->GetDescriptor();

		CalcHashNow();

		bReady = true;
		bTiledTarget = false;
	}

	bIsFinalised = true;
	FinaliseTS = FDateTime::Now();
}

DeviceBufferRef TiledBlob::GetBufferRef() const
{
	if (bTiledTarget)
		return Buffer;

	check(Tiles.Rows() == 1 && Tiles.Cols() == 1);
	check(Tiles[0][0]);

	return Tiles[0][0]->GetBufferRef();
}

void TiledBlob::MakeSingleBlob_Internal()
{ 
	/// Turn this into a single BlobObj
	check(IsInGameThread());
	check(HashValue && HashValue->IsValid());

	if (Rows() > 1 || Cols() > 1)
	{
		check(Buffer && Buffer->IsValid());

		SingleBlob = std::make_shared<Blob>(Buffer);
		Tiles.Clear();

		Tiles.Resize(1, 1);
		Tiles[0][0] = SingleBlob;
	}

	Buffer = DeviceBufferRef();

	UpdateLinkedBlobs(false);
}

AsyncBufferResultPtr TiledBlob::MakeSingleBlob()
{
	/// Turn this into a single BlobObj
	check(IsInGameThread());
	check(HashValue && HashValue->IsValid());

	/// If we already have a Buffer
	if ((Buffer && bReady) || !bTiledTarget) //If its not a tiledTarget then all rendering is done on main Buffer 
	{
		MakeSingleBlob_Internal();
		return cti::make_ready_continuable(std::make_shared<BufferResult>());
	}

	return CombineTiles(false, false).then([this]()
	{
		MakeSingleBlob_Internal();
		return std::make_shared<BufferResult>();
	});
}

AsyncBufferResultPtr TiledBlob::CombineTiles(bool bTouch, bool bIsArray, uint64 BatchId /* = 0 */)
{
	/// If we already have a buffer then we don't need anything else
	if (bReady)
	{
		if (Buffer && Buffer.IsValid())
			return cti::make_ready_continuable(std::make_shared<BufferResult>());
		else if (SingleBlob)
			return cti::make_ready_continuable(std::make_shared<BufferResult>());
	}

	check(Tiles[0][0]);
	const BlobPtr Tile0 = Tiles[0][0];
	Device* Dev = Tile0->GetBufferRef()->GetOwnerDevice();

	if (Rows() == 1 && Cols() == 1)
	{
		check(Tile0->GetBufferRef() && Tile0->GetBufferRef().IsValid());
		return cti::make_ready_continuable(std::make_shared<BufferResult>());
	}

	T_Tiles<DeviceBufferRef> TileBuffers(Rows(), Cols());

	for (int32 RowId = 0; RowId < Rows(); RowId++)
	{
		for (int32 ColId = 0; ColId < Cols(); ColId++)
		{
			check(Tiles[RowId][ColId]);

			const BlobPtr Tile = GetTile(RowId, ColId);
			check(Tile->GetBufferRef());

			TileBuffers[RowId][ColId] = Tile->GetBufferRef();

			/// Touch the Tiles here
			if (bTouch)
				Tile->Touch(BatchId);
		}
	}

	/// If we don't have a hash then calculate it now
	if (!HashValue)
		CalcHashNow();

	/// We've already calculated the hash from all the child blobs
	UE_LOGF(LogData, VeryVerbose, "*** Combine TiledBlob: %ls [Hash: %llu] [T0: %ls] ***", *Desc.Name, HashValue->Value(), *Tile0->Name());

	const CombineSplitArgs CombineArgs 
	{ 
		.Buffer = Buffer, 
		.Tiles = TileBuffers, 
		.IsArray = bIsArray,
		.BufferHash = HashValue
	};

	return Dev->CombineFromTiles(CombineArgs).then([this](DeviceBufferRef CombinedBuffer)
	{
		check(CombinedBuffer->Hash(false) && CombinedBuffer->Hash(false)->IsValid());
		Buffer = CombinedBuffer;
		bReady = true;
		return cti::make_ready_continuable(std::make_shared<BufferResult>());
	});
}

void TiledBlob::TouchTiles(uint64 BatchId)
{
	for (size_t RowId = 0; RowId < Tiles.Rows(); RowId++)
	{
		for (size_t ColId = 0; ColId < Tiles.Cols(); ColId++)
		{
			const BlobPtr& Tile = Tiles[RowId][ColId];
			Tile->Touch(BatchId);
		}
	}
}

void TiledBlob::Touch(uint64 BatchId)
{
	Blob::Touch(BatchId);
	
	/// IMPORTANT: Do not touch the Tiles here ... they must be separately
	/// at Bind locations, strategically to minimise looping overhead
	UpdateAccessInfo(BatchId);
}

AscynCHashPtr TiledBlob::CalcHash()
{
	if (HashValue && HashValue->IsFinal())
		return cti::make_ready_continuable(HashValue);

	UE_LOGF(LogBlob, VeryVerbose, "Finalising TiledBlob hash: %ls", *Desc.Name);

	std::vector<std::decay_t<AscynCHashPtr>, std::allocator<std::decay_t<AscynCHashPtr>>> Promises;
	std::unordered_map<HashType, bool> handled;

	size_t NumRows = Tiles.Rows();
	size_t NumCols = Tiles.Cols();

	for (size_t RowId = 0; RowId < NumRows; RowId++)
	{
		for (size_t ColId = 0; ColId < NumCols; ColId++)
		{
			BlobPtr Tile = Tiles[RowId][ColId];
			CHashPtr TileHash = Tile->Hash();

			if (Tile && 
				(!TileHash || !TileHash->IsFinal()) && 
				handled.find(TileHash->Value()) == handled.end())
			{
				Promises.push_back(Tile->CalcHash());
				handled[TileHash->Value()] = true;
			}
		}
	}

	if (!Promises.empty())
	{
		return cti::when_all(Promises.begin(), Promises.end()).then([this](std::vector<CHashPtr>) mutable
		{
			CalcHashNow();
			check(HashValue->IsFinal());
			return HashValue;
		});
	}

	CalcHashNow();

	return cti::make_ready_continuable(HashValue);
}

void TiledBlob::SetHash(CHashPtr InHashValue)
{
	check(InHashValue);
	check(IsInGameThread() || InHashValue->IsFinal());

	if (HashValue && *HashValue == *InHashValue)
		return;

	/// Update the hash in the blobber
	if (HashValue)
		HashValue = CHash::UpdateHash(InHashValue, HashValue);
	else
		HashValue = InHashValue;
}

void TiledBlob::CalcHashNow() const
{
	if (!Tiles.Rows() || !Tiles.Cols() || !Tiles[0][0])
		return;

	/// Common scenario of 1x1 sized flat textures
	if (Tiles.Rows() == 1 && Tiles.Cols() == 1)
	{
		if (!Tiles[0][0] || !Tiles[0][0]->IsValid())
			return;

		Desc = Tiles[0][0]->GetDescriptor();
		const_cast<TiledBlob*>(this)->SetHash(Tiles[0][0]->Hash());

		return;
	}

	HashValue = nullptr;

	const size_t NumRows = Tiles.Rows();
	const size_t NumCols = Tiles.Cols();
	bool CalculateDim = false;

	if (!Desc.Width || !Desc.Height)
	{
		Desc = Tiles[0][0]->GetDescriptor();
		Desc.Width = 0;
		Desc.Height = 0;
		CalculateDim = true;
	}

	T_Tiles<CHashPtr> TileHashes(NumRows, NumCols);

	for (size_t RowId = 0; RowId < NumRows; RowId++)
	{
		if (CalculateDim && RowId == 0)
			Desc.Width += Tiles[RowId][0]->GetWidth();

		for (size_t ColId = 0; ColId < NumCols; ColId++)
		{
			BlobPtr Tile = Tiles[RowId][ColId];

			if (!Tile)
				return;

			CHashPtr TileHash = Tile->Hash();

			if (!TileHash)
				return;

			TileHashes[RowId][ColId] = TileHash;

			if (CalculateDim && ColId == 0)
				Desc.Width += Tiles[ColId][0]->GetWidth();
		}
	}

	const_cast<TiledBlob*>(this)->SetHash(CHash::ConstructFromSources(TileHashes.Tiles()));
}

CHashPtr TiledBlob::Hash() const
{
	if (!HashValue)
		CalcHashNow();

	return HashValue;
}

AsyncDeviceBufferRef TiledBlob::TransferTo(Device* Target)
{
	auto TargetName = Device::DeviceType_Names(Target->GetType());

	/// Go through my Tile blobs and make sure they are compatible with the Target
	std::vector<AsyncDeviceBufferRef> Promises;

	for (size_t RowId = 0; RowId < Tiles.Rows(); RowId++)
	{
		for (size_t ColId = 0; ColId < Tiles.Cols(); ColId++)
		{
			BlobPtr Tile = Tiles[RowId][ColId];
			check(Tile);

			if (!Tile->GetBufferRef()->IsCompatible(Target))
			{
				UE_LOGF(LogData, Log, "TransferTo %ls TiledBlob: %ls Tile [%zu, %zu]", *TargetName, *Desc.Name, RowId, ColId);
				Promises.emplace_back(Tile->TransferTo(Target));
			}
		}
	}

	if (!Promises.empty())
	{
		return cti::when_all(Promises.begin(), Promises.end()).then([this, TargetName](auto) mutable
			{
				/// Return this tiledBlob Buffer which is empty but this is just to fit the signature
				UE_LOGF(LogData, Log, "TransferTo %ls TiledBlob: %ls DONE", *TargetName, *Desc.Name);
				return Buffer;
			});
	}

	return cti::make_ready_continuable(Buffer);
}

#define TILED_BLOB_DEVICE_OPTIMISED_PATH

AsyncBufferResultPtr TiledBlob::Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo)
{
	check(IsInGameThread());

	/// Manually touch this at the beginning and don't worry about it
	Touch(BindInfo.BatchId);

	/// If we have any Tiles then 
	if (Tiles.Rows() && Tiles.Cols())
	{
		/// Use the optimal Dev implementation for combining the Tiles into a single BlobObj
		if (Tiles.Rows() > 1 || Tiles.Cols() > 1)
		{
#ifdef TILED_BLOB_DEVICE_OPTIMISED_PATH
			if (bReady)
			{
				/// Touch the Tiles since we won't be looping over these
				TouchTiles(BindInfo.BatchId);
				return Blob::Bind(Transform, BindInfo);
			}

			//check(_buffer);

			/// try to salvage in production builds [if it ever gets to it]
			if (!Buffer)
				Buffer = BindInfo.Dev->Create(Desc, HashValue);
				//return cti::make_ready_continuable(std::make_shared<BufferResult>());

			/// Buffer is already there ... just return
			//if (_buffer)
			//	return cti::make_ready_continuable(std::make_shared<BufferResult>());

			return CombineTiles(true, BindInfo.bIsCombined, BindInfo.BatchId).then([this, Transform, BindInfo]
			{
				TouchTiles(BindInfo.BatchId);
				Buffer->GetName() = BindInfo.Target;
				return Blob::Bind(Transform, BindInfo);
			});

			//T_Tiles<DeviceBufferRef> Tiles(Tiles.Rows(), Tiles.Cols());

			//for (int32 RowId = 0; RowId < Tiles.Rows(); RowId++)
			//{
			//	for (int32 ColId = 0; ColId < Tiles.Cols(); ColId++)
			//	{
			//		check(Tiles[RowId][ColId]->Buffer());
			//		Tiles[RowId][ColId] = Tiles[RowId][ColId]->Buffer();

			//		/// Touch the Tiles here
			//		Tiles[RowId][ColId]->Touch();
			//	}
			//}

			///// If we don't have a hash then calculate it now
			//if (!_hash)
			//	CalcHashNow();
			//
			///// We've already calculated the hash from all the child blobs
			//UE_LOGF(LogData, Log, "*** Combine TiledBlob: %ls [Hash: %llu] [T0: %ls] ***", *_desc.name, _hash->Value(), *Tiles[0][0]->Name());
			//return BindInfo.Dev->CombineFromTiles(_buffer, Tiles).then([this, Transform, BindInfo](DeviceBufferRef Buffer)
			//{
			//	_buffer = Buffer;
			//	_buffer->Name() = BindInfo.Target;
			//	_ready = true;
			//	return Blob::Bind(Transform, BindInfo);
			//});
#else
			/// TODO: right now we don't have a good enough way method of rendering render targets as 
			/// Tiles onto a large textures. However, that should be fairly simple to implement. Until
			/// we implement that, we'll just use the dirty way of using Raw data to combine all the 
			/// textures into one!
			//_buffer = BindInfo.Dev->Find(_hash, true);

			/// If we don't have a Buffer, then try to create one
			if (!_ready)
			{
				RawBufferPtrTiles Tiles(Tiles.Rows(), Tiles.Cols());

				for (size_t RowId = 0; RowId < Tiles.Rows(); RowId++)
				{
					for (size_t ColId = 0; ColId < Tiles.Cols(); ColId++)
					{
						RawBufferPtr TileRaw = Tiles[RowId][ColId]->Raw();
						check(TileRaw);
						Tiles[RowId][ColId] = TileRaw;
					}
				}

				/// We've already calculated the hash from all the child blobs

				UE_LOGF(LogData, VeryVerbose, "*** BEGIN Combine TiledBlob: %ls [Hash: %llu] [T0: %ls] ***", *_desc.name, _hash, *Tiles[0][0]->Name());
				double startTime = Util::Time();
				RawBufferPtr Raw = TextureHelper::CombineRaw_Tiles(Tiles, _hash, true);
				UE_LOGF(LogData, VeryVerbose, "*** END Combine TiledBlob: %ls [Hash: %llu] [T0: %ls] ***", *_desc.name, _hash, *Tiles[0][0]->Name());

				Raw->Name() = BindInfo.Target;
				return _buffer->UpdateRaw(Raw, Raw->Hash(), 0).then([this, &BindInfo]()
				{
					return Blob::Bind(Transform, BindInfo);
				});
				//_buffer = BindInfo.Dev->Create(Raw);

				_ready = true;
			}

			return Blob::Bind(Transform, BindInfo);
#endif /// TILED_BLOB_DEVICE_OPTIMISED_PATH
		}

		check(Tiles[0][0]);
		return Tiles[0][0]->Bind(Transform, BindInfo);
	}

	check(Buffer);
	return Blob::Bind(Transform, BindInfo);
}

void TiledBlob::ResetBuffer()
{
	//Resetting the Buffer means deallocation of UObject associated with Render Target. We dont need that
	//Instead this would be garbage collected on Dev cache and reused when required
	//Blob::ResetBuffer();

	bReady = false;
}

AsyncPrepareResult TiledBlob::PrepareForWrite(const ResourceBindInfo& BindInfo_)
{
	bool bCreateTexArray = false;

	if (BindInfo_.bIsCombined)
	{
		bCreateTexArray = true;
	}

	if ((Buffer || bReady) && !bCreateTexArray)
	{
		bReady = true;
		return cti::make_ready_continuable(0);
	}

	if ((Tiles.Rows() > 1 && Tiles.Cols() > 1) || bCreateTexArray)
	{
		if (!Buffer || bCreateTexArray)
		{
			ResourceBindInfo BindInfo = BindInfo_;

#ifndef TILED_BLOB_DEVICE_OPTIMISED_PATH
			BindInfo.staticBuffer = true;
#endif 

			check(BindInfo.Dev);
			if(!Buffer)
				Buffer = BindInfo.Dev->Create(Desc, HashValue);

			return Blob::PrepareForWrite(BindInfo);
		}
	}

	check(Tiles[0][0])

	return Tiles[0][0]->PrepareForWrite(BindInfo_).then([this]()
	{
		Buffer = Tiles[0][0]->GetBufferRef();
		return 0;
	});
}

AsyncBufferResultPtr TiledBlob::Unbind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo)
{
	check(IsInGameThread());

	if (Tiles.Rows() > 1 || Tiles.Cols() > 1)
	{
		/// If this is not a tiled Target and it was a write Target, then the combined Buffer is already ready
		if (BindInfo.bWriteTarget) {
			bReady = true;
			check(Buffer); /// Buffer must be valid for write targets
		}

		/// Don't try to Unbind a buffer that isn't ready yet. The buffer might not even be created yet.
		if (!bReady || !Buffer)
			return cti::make_ready_continuable<BufferResultPtr>(std::make_shared<BufferResult>());

		//_ready = false;
		return Blob::Unbind(Transform, BindInfo).then([this](BufferResultPtr Result)
		{
			/// We don't need the Raw Buffer here anymore [tiled buffers are large and expensive!]
			/// though we still want to keep the native Dev buffers
			/// However, before we do that we need to ensure that the Buffer has been transferred
			/// to the Tiles, if it was being rendered to in non-tiled mode

			/// If its already a tiled Target, then we don't need to do anything at all
			if (bTiledTarget)
			{
				ResetBuffer();
				return Result;
			}

			/// Otherwise, we must Tile it now
			return Result;
		});
	}
	else
		return Tiles[0][0]->Unbind(Transform, BindInfo);
}

AsyncBufferResultPtr TiledBlob::TileBuffer(DeviceBufferRef Buffer, BlobPtrTiles& Tiles)
{
	SCOPE_CYCLE_COUNTER(STAT_TiledBlob_TileBuffer)
	const size_t NumRows = Tiles.Rows();
	const size_t NumCols = Tiles.Cols();
	T_Tiles<DeviceBufferRef>* TilesBuffers = new T_Tiles<DeviceBufferRef>(NumRows, NumCols);
		
	int totalTiles = Tiles.Rows() * Tiles.Cols();

	/// Incase the Buffer is present in Tiles, it would be copied over. This is to avoid allocation of RenderTarget
	/// This allocation is being done on prepare resources due to optimization (e.g lazy evaluation)
	for (size_t RowId = 0; RowId < NumRows; RowId++)
	{
		for (size_t ColId = 0; ColId < NumCols; ColId++)
		{
			check(Tiles[RowId][ColId]);
			(*TilesBuffers)(RowId, ColId) = Tiles[RowId][ColId]->GetBufferRef();
		}
	}

	check(Buffer->GetOwnerDevice());

	CombineSplitArgs splitArgs { Buffer, *TilesBuffers};

	return Buffer->GetOwnerDevice()->SplitToTiles(splitArgs).then([Buffer, &Tiles, TilesBuffers](DeviceBufferRef) mutable
	{
		for (size_t RowId = 0; RowId < Tiles.Rows(); RowId++)
		{
			for (size_t ColId = 0; ColId < Tiles.Cols(); ColId++)
			{
				DeviceBufferRef TileBuffer = (*TilesBuffers)[RowId][ColId];
				BlobPtr Tile = std::make_shared<Blob>(TileBuffer);
				Tiles[RowId][ColId] = BlobRef(Tile);
			}
		}

		/// Clear the pointer
		TilesBuffers->Clear();
		delete TilesBuffers;

		return std::make_shared<BufferResult>();
	});
}

AsyncBufferResultPtr TiledBlob::Flush(const ResourceBindInfo& BindInfo)
{
	if (!Tiles.Rows() || !Tiles.Cols())
	{
		int32 NumRows = BindInfo.NumTilesX;
		int32 NumCols = BindInfo.NumTilesY;

		check(NumRows > 0 && NumCols > 0);
		check(Buffer);

		Tiles.Resize(NumRows, NumCols);
	}

	bReady = false;

	return Blob::Flush(BindInfo)
		.then([this](BufferResultPtr Result)
		{
			return TiledBlob::TileBuffer(Buffer, Tiles);
		})
		.then([this](BufferResultPtr Result)
		{
			/// Clear the Buffer here. We've now read back from this now
			Buffer.reset();
			ResetBuffer();
			return Result;
		});
}

bool TiledBlob::IsValid() const
{
	/// If we have a Buffer, then we just check for the validity of that Buffer
	if (Buffer)
		return Buffer->IsValid();

	/// Otherwise, iterate over the Tiles and check for the validity of each of the Tiles
	bool Valid = true;
	
	for (size_t RowId = 0; RowId < Rows() && Valid; RowId++)
	{
		for (size_t ColId = 0; ColId < Cols() && Valid; ColId++)
		{
			Valid &= Tiles[RowId][ColId] && Tiles[RowId][ColId]->IsValid();
		}
	}

	return Valid;
}

bool TiledBlob::IsNull() const
{
	return !IsValid();
}

FString TiledBlob::DisplayName() const
{
	return Desc.Name;
}

void TiledBlob::AddLinkedBlob(BlobPtr LinkedBlob)
{
	check(IsInGameThread());
	check(LinkedBlob->IsTiled());
	Blob::AddLinkedBlob(LinkedBlob);
}

bool TiledBlob::CanCalculateHash() const
{
	check(IsInGameThread());

	for (size_t RowId = 0; RowId < Tiles.Rows(); RowId++)
	{
		for (size_t ColId = 0; ColId < Tiles.Cols(); ColId++)
		{
			check(Tiles[RowId][ColId]);

			BlobPtr TileBlob = Tiles[RowId][ColId];
			DeviceBufferRef TileBuffer = TileBlob->GetBufferRef();
			CHashPtr TileHash = TileBuffer->Hash();

			if (!TileHash || !TileHash->IsFinal() || !TileBlob->CanCalculateHash())
				return false;
		}
	}

	return true;
}

void TiledBlob::SetMinMax(BlobPtr MinMax_)
{
	/// TODO
	check(MinMax_->IsTiled());

	MinMax = MinMax_;
}

void TiledBlob::SetLODLevel(int32 Level, BlobPtr Blob_, BlobPtrW LODParent_, BlobPtrW LODSource_, bool AddToBlobber)
{
	check(Blob_->IsTiled());
	check(!LODParent_.expired());
	check(!LODSource_.expired());
	check(IsInGameThread());

	LODParent = LODParent_;
	LODSource = LODSource_;

	TiledBlobPtr BlobObj = std::static_pointer_cast<TiledBlob>(Blob_);

	/// Make sure everything matches up
	check(Rows() == BlobObj->Rows());
	check(Cols() == BlobObj->Cols());

	BlobObj->bIsLODLevel = true;

	std::vector<std::decay_t<AsyncBlobResultPtr>, std::allocator<std::decay_t<AsyncBlobResultPtr>>> Promises;
	
	Promises.emplace_back(OnFinalise());
	Promises.emplace_back(BlobObj->OnFinalise());

	if (LODLevels.empty())
	{
		/// -1 because we don't wanna keep a weak pointer of Mip Level 0. 'this' is supposed to 
		/// the the mip Level 0, so there's no point keeping it in the mip chain
		const int32 NumLevels = TextureHelper::CalcNumMipLevels(std::max(GetWidth(), GetHeight())) - 1;

		if (NumLevels > 0)
			LODLevels.resize(NumLevels);
	}

	check(Level > 0 && Level <= static_cast<int32>(LODLevels.size()));

	LODLevels[Level - 1] = BlobObj;
	BlobPtr SelfPtr = shared_from_this();
	TiledBlobPtr Self = std::static_pointer_cast<TiledBlob>(SelfPtr);

	for (int32 li = 0; li < Level - 1; li++)
	{
		BlobPtr lodLevel = LODLevels[li].lock();

		if (lodLevel)
		{
			lodLevel->SetLODLevel(Level - li - 1, BlobObj, Self, LODSource_, AddToBlobber);
		}
	}

	cti::when_all(Promises.begin(), Promises.end()).then([this, BlobObj, Self, Level, LODParent_, LODSource_, AddToBlobber]() mutable
	{
		Util::OnGameThread([this, BlobObj, Self, Level, LODParent_, LODSource_, AddToBlobber]()
		{
			for (size_t RowId = 0; RowId < Rows(); RowId++)
			{
				for (size_t ColId = 0; ColId < Cols(); ColId++)
				{
					BlobRef mipBlobXY = BlobObj->GetTile(RowId, ColId);
					BlobRef baseBlobXY = Self->GetTile(RowId, ColId);

					baseBlobXY->SetLODLevel(Level, mipBlobXY.get(), LODParent_, LODSource_, AddToBlobber);
				}
			}
		});
	});
}

void TiledBlob::FinaliseNow(bool bNoCalcHash, CHashPtr FixedHash)
{
	check(IsInGameThread());

	UE_LOGF(LogJob, Verbose, "Finalised tiled BlobObj: %ls", *Desc.Name);

	/// Debug: check that all Tiles are valid
	for (int32 RowId = 0; RowId < Tiles.Rows(); RowId++)
	{
		for (int32 ColId = 0; ColId < Tiles.Cols(); ColId++)
		{
			check(Tiles[RowId][ColId]);
			BlobPtr Tile = Tiles[RowId][ColId];
			Tile->FinaliseNow(bNoCalcHash, nullptr);
		}
	}

	if (SingleBlob)
		SingleBlob->FinaliseNow(bNoCalcHash, nullptr);

	bIsFinalised = true;
	FinaliseTS = FDateTime::Now();
}

//////////////////////////////////////////////////////////////////////////
TiledBlob_Promise::TiledBlob_Promise(TiledBlobPtr Source) : TiledBlob(Source->GetDescriptor(), Source->GetTiles())
{
}

TiledBlob_Promise::TiledBlob_Promise(DeviceBufferRef Buffer) : TiledBlob(Buffer)
{
}

TiledBlob_Promise::TiledBlob_Promise(const BufferDescriptor& Desc, size_t NumRows, size_t NumCols, CHashPtr Hash) : TiledBlob(Desc, NumRows, NumCols, Hash)
{
}

TiledBlob_Promise::~TiledBlob_Promise()
{
}

AsyncBufferResultPtr TiledBlob_Promise::Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo)
{
	check(IsInGameThread());
	return TiledBlob::Bind(Transform, BindInfo);
}

AsyncBufferResultPtr TiledBlob_Promise::Unbind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo)
{
	check(IsInGameThread());

	/// Cannot bind a promised BlobObj until it has been finalised
	check(!bTiledTarget || IsValid());
	return TiledBlob::Unbind(Transform, BindInfo);
}

void TiledBlob_Promise::AddLinkedBlob(BlobPtr LinkedBlob)
{
	check(IsInGameThread());
	check(LinkedBlob->IsTiled());

	TiledBlobPtr TiledLinkedBlob = std::static_pointer_cast<TiledBlob>(LinkedBlob);

	/// If the number of rows and columns don't match up then we don't do anything
	if (TiledLinkedBlob->Rows() != Rows() || TiledLinkedBlob->Cols() != Cols())
		return;

	/// If we got a different pointer back from the blobber then that means that this result was already cached into 
	/// the system. We need to copy it over to the original pointer if it's a finalised blob
	/// so that anyone keeping a copy of Result has the exact same view as the original result 
	/// and we don't have do any awkward pointer adjustment shenanigans to make things work
	if (IsFinalised())
	{
		TiledLinkedBlob->FinaliseFrom(this);
	}

	if (TiledLinkedBlob->IsPromise())
	{
		TiledBlob_PromisePtr TiledPromiseLinkedBlob = std::static_pointer_cast<TiledBlob_Promise>(LinkedBlob);
		TiledPromiseLinkedBlob->CachedBlob = std::static_pointer_cast<TiledBlob_Promise>(shared_from_this());
	}

	TiledBlob::AddLinkedBlob(LinkedBlob);
}

void TiledBlob_Promise::FinaliseNow(bool bNoCalcHash, CHashPtr FixedHash)
{
	check(IsInGameThread());

	UE_LOGF(LogJob, Verbose, "Finalised promised BlobObj: %ls", *Desc.Name);
	TiledBlob::FinaliseNow(bNoCalcHash, FixedHash);

	/// Trigger the callbacks - Should always be the last step of finalize
	NotifyCallbacks();

	UpdateLinkedBlobs(true);
}

void TiledBlob_Promise::FinaliseFrom(TiledBlobPtr RHS)
{
	FinaliseFrom(std::static_pointer_cast<Blob>(RHS).get());
}

void TiledBlob_Promise::FinaliseFrom(Blob* RHS)
{
	check(RHS->IsTiled());
	TiledBlob* RHSTiled = static_cast<TiledBlob*>(RHS);

	if (RHSTiled->IsPromise())
	{
		TiledBlob_Promise* RHSTiledPromise = static_cast<TiledBlob_Promise*>(RHS);
		bMakeSingleBlob = RHSTiledPromise->bMakeSingleBlob;
	}

	TiledBlob::FinaliseFrom(RHS);
}

AsyncBufferResultPtr TiledBlob_Promise::Finalise(bool bNoCalcHash, CHashPtr FixedHash)
{
	check(IsInGameThread());

	if (bIsFinalised)
		return cti::make_ready_continuable<BufferResultPtr>(std::make_shared<BufferResult>());

	/// cannot re-finalise something that's already been finalised
	check(!bIsFinalised);
	bIsFinalised = true;
	FinaliseTS = FDateTime::Now();

	/// If it is a tiled Target then finalise now
	if (bTiledTarget || bMakeSingleBlob)
	{
		/// If this is not due to be turned into a single BlobObj then just return
		if (!bMakeSingleBlob)
		{
			FinaliseNow(bNoCalcHash, FixedHash);
			return cti::make_ready_continuable(std::make_shared<BufferResult>());
		}

		/// Otherwise we turn into a single BlobObj
		return MakeSingleBlob()
			.then([this, bNoCalcHash, FixedHash](BufferResultPtr SingleBuffer) {
				if (bMakeSingleBlob)
					FinaliseNow(bNoCalcHash, FixedHash);
				
				return SingleBuffer;
			});
	}

	/// We cannot make a surface that's already non-tiled 
	check(!bMakeSingleBlob && Desc.Width >= Tiles.Rows() && Desc.Height >= Tiles.Cols());

	/// Otherwise we need to Tile the Buffer first
	return TiledBlob::TileBuffer(Buffer, Tiles)
		.then([this, bNoCalcHash, FixedHash](BufferResultPtr Result) mutable
		{
			T_Tiles<CHashPtr> TileHashes(Tiles.Rows(), Tiles.Cols());

			for (int32 RowId = 0; RowId < Tiles.Rows(); RowId++)
			{
				for (int32 ColId = 0; ColId < Tiles.Cols(); ColId++)
				{
					check(Tiles[RowId][ColId]);

					BlobPtr Tile = Tiles[RowId][ColId];
					if (!bNoCalcHash)
						Tile->CalcHash();
					else
					{
						CHashPtr TileHash = Tile->Hash();
						check(TileHash);

						TileHashes[RowId][ColId] = TileHash;
					}
				}
			}

			CHashPtr FullHash = CHash::ConstructFromSources(TileHashes.Tiles());
			
			/// Clear the Buffer here. We don't need it anymore
			ResetBuffer();

			/// Reset this back
			bTiledTarget = true;

			/// Tiled blobs always recalculate their has to match those of the children
			FinaliseNow(true, FullHash);

			return Result;
		});

}

void TiledBlob_Promise::OnFinaliseInternal(BlobReadyCallback Callback) const
{
	check(IsInGameThread());

	/// If we're already finalised then we can just return
	if (bIsFinalised)
	{
		Callback(this);
		return;
	}

	/// Otherwise we just queue the callbacks
	Callbacks.push_back(std::move(Callback));
}

void TiledBlob_Promise::SetTile(int32 RowId, int32 ColId, BlobRef InTile)
{
	check(IsValidTileIndex(RowId, ColId));
	check(!InTile.expired());

	BlobPtr Tile = InTile.lock();

	/// Cannot have tiled BlobObj as part of another tiled BlobObj
	check(!Tile->IsTiled());

	TiledBlob::SetTile(RowId, ColId, InTile);

	if (bMakeSingleBlob && Rows() == 1 && Cols() == 1)
		Buffer = Tile->GetBufferRef();

	/// Reset the hash
	HashValue = nullptr;
}

AsyncBufferResultPtr TiledBlob_Promise::MakeSingleBlob()
{
	if (!CachedBlob.expired())
	{
		return CachedBlob.lock()->MakeSingleBlob();
	}

	/// If already finalised then return
	if (bIsFinalised)
		return TiledBlob::MakeSingleBlob();

	/// Otherwise set a flag
	bMakeSingleBlob = true;

	/// Set the size of the Tiles
	Tiles.Clear();
	Tiles.Resize(1, 1);

	SingleBlob = std::make_shared<Blob>(Buffer);
	TiledBlob::SetTile(0, 0, SingleBlob);

	return cti::make_ready_continuable(std::make_shared<BufferResult>());
}

AsyncBufferResultPtr TiledBlob_Promise::Flush(const ResourceBindInfo& BindInfo)
{
	if (((Tiles.Rows() && Tiles.Cols()) || !Buffer) && bTiledTarget)
		return cti::make_ready_continuable(std::make_shared<BufferResult>());

	return TiledBlob::Flush(BindInfo);
}

TiledBlob_Promise& TiledBlob_Promise::operator = (const TiledBlob& RHS)
{
	TiledBlob::operator = (RHS);
	return *this;
}

TiledBlob_Promise& TiledBlob_Promise::operator = (const TiledBlob_Promise& RHS)
{
	TiledBlob::operator = (RHS);
	bMakeSingleBlob = RHS.bMakeSingleBlob;
	return *this;
}

void TiledBlob_Promise::CopyResolveLateBound(BlobPtr RHS_)
{
	check(RHS_ != nullptr);

	if (RHS_->IsTiled() && RHS_->IsPromise())
		*this = *static_cast<TiledBlob_Promise*>(RHS_.get());
	else if (RHS_->IsTiled())
		TiledBlob::CopyResolveLateBound(RHS_);
	else
	{
		TiledBlob::CopyResolveLateBound(RHS_);
		bMakeSingleBlob = true;
	}

	NotifyCallbacks();
}

void TiledBlob_Promise::NotifyCallbacks()
{
	check(IsInGameThread());

	/// Trigger the callbacks - Should always be the last step of finalize
	for (BlobReadyCallback& Callback : Callbacks)
		Callback(this);

	Callbacks.clear();
}

void TiledBlob_Promise::ResetForReplay()
{
	bIsFinalised = false;
	++ReplayCount;
}
