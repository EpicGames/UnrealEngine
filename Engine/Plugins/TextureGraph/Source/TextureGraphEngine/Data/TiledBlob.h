// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "BlobRef.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Job;
typedef std::weak_ptr<Job>			JobPtrW;
typedef T_BlobRef<Blob>				BlobRef;
typedef T_Tiles<BlobPtr>			BlobPtrTiles;

//////////////////////////////////////////////////////////////////////////
/// TiledBlob: Represents a BlobObj that is made up of many Tiles. These blobs
/// are always late bound i.e. they will not allocate any buffers 
/// unless they absolutely need to. 
//////////////////////////////////////////////////////////////////////////
class TiledBlob : public Blob
{
	friend class Blobber;
	friend class TiledBlob_Promise;

protected:
	mutable BlobPtrTiles			Tiles;					/// The Tiles that make up the larger BlobObj
	mutable CHashPtr				HashValue;				/// The HashValue of tiled BlobObj
	mutable BufferDescriptor		Desc;					/// We keep the combined descriptor separately
	bool							bReady = false;			/// Whether the Buffer is already ready
	bool							bTiledTarget = true;	/// Whether we're rendering to this as a tile by tile Target or not
	JobPtrW							JobObj;					/// The job that is potentially generating this tiled BlobObj
	BlobPtr							SingleBlob;			/// Single BlobObj pointer that we keep to prevent going out of Ref

	UE_API void							CalcHashNow() const;
	UE_API virtual void					ResetBuffer() override;

	UE_API virtual void					TouchTiles(uint64 BatchId);
	UE_API virtual void					Touch(uint64 BatchId) override;
	UE_API virtual void					MakeSingleBlob_Internal();
	UE_API TiledBlob&					operator = (const TiledBlob& RHS);
	UE_API virtual void					SetHash(CHashPtr Hash) override;

	UE_API virtual void					AddLinkedBlob(BlobPtr LinkedBlob) override;
	UE_API virtual void					FinaliseFrom(Blob* RHS) override;

public:
	UE_API static AsyncBufferResultPtr	TileBuffer(DeviceBufferRef Buffer, BlobPtrTiles& Tiles);
	UE_API AsyncBufferResultPtr			CombineTiles(bool bTouch, bool bIsArray, uint64 BatchId = 0);

	UE_API 								TiledBlob(const BufferDescriptor& Desc, const BlobPtrTiles& Tiles);

public:
	UE_API 								TiledBlob(const BufferDescriptor& Desc, size_t NumTilesX, size_t NumTilesY, CHashPtr HashValue);
	UE_API 								TiledBlob(DeviceBufferRef Buffer);
	UE_API 								TiledBlob(BlobRef BlobObj);
	UE_API virtual						~TiledBlob() override;

	UE_API static std::shared_ptr<TiledBlob> InitFromTiles(const BufferDescriptor& Desc, BlobPtrTiles& Tiles);

	UE_API virtual AsyncBufferResultPtr	Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) override;
	UE_API virtual AsyncBufferResultPtr	Unbind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) override;

	UE_API virtual bool					IsValid() const override;
	UE_API virtual bool					IsNull() const override;
	UE_API virtual AscynCHashPtr		CalcHash() override;	/// SLOW: Because calls Raw() underneath. Don't use directly

	UE_API virtual CHashPtr				Hash() const override;
	UE_API virtual AsyncBufferResultPtr	Flush(const ResourceBindInfo& BindInfo) override;
	UE_API virtual AsyncPrepareResult	PrepareForWrite(const ResourceBindInfo& BindInfo) override;
	UE_API virtual FString				DisplayName() const override;
	virtual bool						IsTiled() const override { return true; }
	UE_API virtual bool					CanCalculateHash() const override;
	UE_API virtual void					SetTile(int32 X, int32 Y, BlobRef Tile);
	UE_API virtual void					SetTiles(const BlobPtrTiles& InTiles);

	virtual FString&					Name() override { return Desc.Name; }
	virtual const FString&				Name() const override { return Desc.Name; }
	virtual const BufferDescriptor&		GetDescriptor() const override { return Desc; }

	UE_API virtual AsyncBufferResultPtr	MakeSingleBlob();

	UE_API virtual AsyncDeviceBufferRef	TransferTo(Device* Target) override;
	UE_API virtual DeviceBufferRef		GetBufferRef() const override;

	UE_API virtual void					ResolveLateBound(BlobPtr Ref);
	UE_API virtual void					ResolveLateBound(const BufferDescriptor& Desc, bool bOverrideExisting = false);
	UE_API virtual void					CopyResolveLateBound(BlobPtr RHS);
	UE_API virtual void					SetTransient();
	UE_API virtual void					FinaliseNow(bool bNoCalcHash, CHashPtr FixedHash) override;

	//////////////////////////////////////////////////////////////////////////
	/// Min/Max related
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual void					SetMinMax(BlobPtr MinMax_) override;

	//////////////////////////////////////////////////////////////////////////
	/// LOD/MipMaps related
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual void					SetLODLevel(int32 Level_, BlobPtr BlobObj, BlobPtrW LODParent_, BlobPtrW LODSource_, bool AddToBlobber) override;

	//////////////////////////////////////////////////////////////////////////
	/// Ownership related
	//////////////////////////////////////////////////////////////////////////
#if DEBUG_BLOB_REF_KEEPING == 1
	UE_API virtual bool					HasBlobAsTile(Blob* BlobObj) const override;
#endif 

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////

	/// Takes a BlobObj and returns a TiledBlob. If the passed BlobObj is already tiled, then
	/// it does nothing. Otherwise it creates a new TiledBlob with 1x1 tile that contains
	/// the passed BlobObj.
	/// Just a simple helper function, used in places that always require a TiledBlob.
	UE_API static std::shared_ptr<TiledBlob>	AsTiledBlob(BlobPtr BlobObj);

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE size_t				Rows() const { return Tiles.Rows(); }
	FORCEINLINE size_t				Cols() const { return Tiles.Cols(); }
	FORCEINLINE const BlobPtrTiles&	GetTiles() const { return Tiles; }
	FORCEINLINE uint32				GetWidth() const { return Desc.Width; }
	FORCEINLINE uint32				GetHeight() const { return Desc.Height; }
	FORCEINLINE bool				TiledTarget() const { return bTiledTarget; }
	FORCEINLINE bool&				TiledTarget() { return bTiledTarget; }
	FORCEINLINE bool				IsWellDefined() { return !IsLateBound() && GetWidth() > 0 && GetHeight() > 0; }
	FORCEINLINE bool				IsValidTileIndex(int32 RowId, int32 ColId) const
	{
		return RowId < static_cast<int32>(Tiles.Rows()) && ColId < static_cast<int32>(Tiles.Cols());
	}

	FORCEINLINE BlobRef				GetTile(int32 RowId, int32 ColId) const 
	{
		if (Tiles.Rows() == 1 && Tiles.Cols() == 1)
			return BlobRef(Tiles[0][0], false);

		check(IsValidTileIndex(RowId, ColId));
		return BlobRef(Tiles[RowId][ColId], false);
	}
	FORCEINLINE BlobPtr				GetSingleBlob() const { return SingleBlob; }

	JobPtrW							Job() const { return JobObj; }
	JobPtrW&						Job() { return JobObj; }
};

//////////////////////////////////////////////////////////////////////////

typedef std::shared_ptr<TiledBlob>	TiledBlobPtr;
typedef std::unique_ptr<TiledBlob>	TiledBlobUPtr;
typedef std::weak_ptr<TiledBlob>	TiledBlobPtrW;

//////////////////////////////////////////////////////////////////////////

//////////////////////////////////////////////////////////////////////////
/// This is a promised TiledBlob that will be the result of the job.
/// These are lazy evaluated.
//////////////////////////////////////////////////////////////////////////
class TiledBlob_Promise : public TiledBlob
{
protected:
	mutable std::vector<BlobReadyCallback> Callbacks;		/// The callbacks to call when the promised tiled BlobObj is ready
	bool							bMakeSingleBlob = false;/// Make single BlobObj on finalise
	std::weak_ptr<TiledBlob_Promise> CachedBlob;			/// The cached blob that this is derived off

	UE_API virtual void					OnFinaliseInternal(BlobReadyCallback Callback) const override;
	UE_API TiledBlob_Promise&			operator = (const TiledBlob_Promise& RHS);
	UE_API TiledBlob_Promise&			operator = (const TiledBlob& RHS);
	UE_API virtual void					NotifyCallbacks();

	UE_API virtual void					AddLinkedBlob(BlobPtr LinkedBlob) override;
	UE_API virtual void					FinaliseFrom(Blob* RHS) override;

public:
	UE_API 								TiledBlob_Promise(TiledBlobPtr Source);
	UE_API 								TiledBlob_Promise(const BufferDescriptor& Desc, size_t NumTilesX, size_t NumTilesY, CHashPtr HashValue);
	UE_API 								TiledBlob_Promise(DeviceBufferRef Buffer);
	UE_API virtual						~TiledBlob_Promise();

	UE_API virtual AsyncBufferResultPtr	Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) override;
	UE_API virtual AsyncBufferResultPtr	Unbind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) override;
	UE_API virtual AsyncBufferResultPtr	Flush(const ResourceBindInfo& BindInfo) override;
	virtual bool						IsFinalised() const override { return bIsFinalised; }
	UE_API virtual void					SetTile(int32 RowId, int32 ColId, BlobRef Tile) override;
	UE_API virtual AsyncBufferResultPtr	MakeSingleBlob() override;
	UE_API virtual void					CopyResolveLateBound(BlobPtr RHS) override;
	virtual bool						IsPromise() const override { return true; }

	UE_API virtual void					FinaliseNow(bool bNoCalcHash, CHashPtr FixedHash) override;
	UE_API virtual AsyncBufferResultPtr	Finalise(bool bNoCalcHash, CHashPtr FixedHash) override;
	UE_API void							FinaliseFrom(TiledBlobPtr RHS);

	UE_API void							ResetForReplay(); /// For debug purpose, reset the state of the tile as a promise to NOT finalised, increment the replayCount

	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE bool				IsSingleBlob() const { return bMakeSingleBlob; }
};

typedef std::shared_ptr<TiledBlob_Promise>	TiledBlob_PromisePtr;
typedef std::weak_ptr<TiledBlob_Promise>	TiledBlob_PromisePtrW;
typedef T_BlobRef<TiledBlob>				TiledBlobRef;
typedef cti::continuable<TiledBlobRef>		AsyncTiledBlobRef;

#undef UE_API