// Copyright Epic Games, Inc. All Rights Reserved.
#include "Device_CL.h"

Device_CL::Device_CL()
{
}

Device_CL::~Device_CL()
{
}
