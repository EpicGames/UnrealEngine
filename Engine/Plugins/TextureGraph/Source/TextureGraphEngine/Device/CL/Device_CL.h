// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#define UE_API TEXTUREGRAPHENGINE_API

/**
 * 
 */
class Device_CL
{
public:
									UE_API Device_CL();
	UE_API virtual							~Device_CL();

	virtual const char*				Name() const { return "Device_CL"; }

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API Device_CL*				Get();
};

#undef UE_API
