// Copyright Epic Games, Inc. All Rights Reserved.
#include "Device.h"
#include "DeviceBuffer.h"
#include "Helper/Util.h"
#include "2D/TextureHelper.h"

#include "TextureGraphEngine.h"
#include "TextureGraphEngineGameInstance.h"
#include "DeviceNativeTask.h"
#include "Device/Null/Device_Null.h"
#include "Data/Blobber.h"
#include "Job/JobBatch.h"
#include "Job/Scheduler.h"
#include "Job/DeviceTransferService.h"
#include "Job/JobCommon.h"
#include "DeviceObserverSource.h"

DEFINE_LOG_CATEGORY(LogDevice);

const double Device::PrintStatsInterval = 5000.0;
const double Device::DefaultBufferTransferInternal = 1000.0;
const double Device::DefaultMinLastUsageDuration = 10000.0;

const float Device::DefaultMaxCacheConsumptionBeforeCollection = 0.95f;
const float Device::DefaultMinCacheConsumptionBeforeCollection = 0.60f;
const uint64 Device::DefaultMaxBatchDeltaDefault = 5;

bool DeviceNativeTask_CompareStrong::operator()(const std::shared_ptr<DeviceNativeTask>& LHS, const std::shared_ptr<DeviceNativeTask>& RHS)
{
	if (!LHS || !RHS)
		return false;

	return !LHS->IsHigherPriorityThan(*RHS.get());
}

bool DeviceNativeTask_CompareWeak::operator()(const std::weak_ptr<DeviceNativeTask>& InLHS, const std::weak_ptr<DeviceNativeTask>& InRHS)
{
	auto LHS = InLHS.lock();
	auto RHS = InRHS.lock();

	if (!LHS || !RHS)
		return false;

	return DeviceNativeTask_CompareStrong()(LHS, RHS);
}

FString Device::DeviceType_Names(DeviceType DevType)
{
	static const FString names[] = {
		"Device_FX",
		"Device_OpenCL",
		"Device_Mem",
		"Device_MemCompressed",
		"Device_Disk",
		"Device_Remote",

		"Device_Null"
	};
	return names[(uint32)DevType];
}

AsyncBool Device::WaitForQueuedTasks(ENamedThreads::Type ReturnThread /* = ENamedThreads::UnusedAnchor */)
{
	if (NativeTasks.size() == 0)
		return cti::make_ready_continuable(true);

	auto Tasks = NativeTasks.to_vector();

	std::vector<cti::continuable<int32>> Promises;

	for (auto Task : NativeTasks.to_vector())
	{
		// Push back all Tasks, can be any thread
		Promises.push_back(Task->WaitAsync());
	}
	
	return cti::when_all(Promises.begin(), Promises.end())
		.then([=](std::vector<int32> results) 
		{
			return PromiseUtil::OnGameThread();
		})
		.then([=](int32) 
		{
			return true;
		});
}

Device::Device(DeviceType InType, DeviceBuffer* InBufferFactory) 
	: Type(InType)
	, BufferFactory(InBufferFactory)
{
	ObserverSource = std::make_shared<DeviceObserverSource>();

	/// Reserve a good size to begin with
	GCTargets.reserve(1024);
	TimeStatsPrinted = Util::Time();
}

Device::~Device()
{
	Device::UpdateAwaitingCollection();

	delete BufferFactory;
	BufferFactory = nullptr;
}

void Device::CallDeviceUpdate()
{
	E_ResourceUpdate().Broadcast();
}

bool Device::CanFree() const
{
	return NativePreExecWait.empty() && NativePreExec.empty();
}

void Device::Free()
{
	const bool bEngineShutdown = TextureGraphEngine::IsDestroying();

	// Stop threads first (wakes + joins)
	Terminate();

	// In normal operation enforce emptiness; during shutdown, drain instead of crashing
	if (!bEngineShutdown)
	{
		check(NativePreExecWait.empty());
		check(NativePreExec.empty());
		check(NativeExec.empty());
		check(NativePostExec.empty());
	}
	else
	{
		auto Drain = [this](auto& Q)
			{
				auto v = Q.to_vector_and_clear();
				for (auto& w : v)
					if (auto t = w.lock())
						MarkTaskFinished(t, /*bRemoveFromOwnerQueue=*/true);
				return v.size();
			};

		size_t Drained = 0;
		Drained += Drain(NativePreExecWait);
		Drained += Drain(NativePreExec);
		Drained += Drain(NativeExec);
		Drained += Drain(NativePostExec);
		Drained += Drain(NonAsyncPostExecQueue);
		NativeTasks.clear();

		UE_LOGF(LogDevice, Display,
				"[%ls] Shutdown drained %llu native tasks (expected on fast exit)",
				*Name(), (unsigned long long)Drained);
	}

	{
		FScopeLock Lock(Cache.GetMutex());
		Cache.GetCache().Empty();
	}
	GarbageCollect();

	BufferFactory = nullptr;
	E_ResourceUpdate().Broadcast();
}

void Device::Terminate()
{
	ShouldTerminate = true;

	// If we were waiting on a dependency, nudge it to cancel quickly.
	if (CurrentWaitTask)
		CurrentWaitTask->Terminate();

	// Wake all waiters so join() won’t block on condition variables.
	NativePreExecWait.add(DeviceNativeTaskPtrW());
	NativeExec.add(DeviceNativeTaskPtrW());
	NativePostExec.add(DeviceNativeTaskPtrW());

	auto JoinAndReset = [](std::unique_ptr<std::thread>& T)
		{
			if (T && T->joinable())
				T->join();
			T.reset();
		};

	JoinAndReset(PreExecThread);
	JoinAndReset(ExecThread);
	JoinAndReset(PostExecThread);
}

DeviceBufferRef Device::CreateCompatible_Copy(DeviceBufferRef Source)
{
	check(Source);

	/// Sometimes we can have a duplicate Buffer in multiple devices. 
	/// We're gonna check here whether we already have this
	CHashPtr SourceHash = Source->Hash();
	
	if (SourceHash && SourceHash->IsFinal())
	{
		DeviceBufferRef ExistingBuffer = Find(SourceHash->Value(), true);
		if (ExistingBuffer)
			return ExistingBuffer;
	}

	RawBufferPtr Raw = Source->Raw_Now();
	check(Raw);

	/// Create a new native Buffer
	DeviceBufferRef NativeBuffer = Create(Raw);
	return NativeBuffer;
}

AsyncDeviceBufferRef Device::Transfer(DeviceBufferRef Source)
{
	check(Source && Source->IsValid());

	CHashPtr SourceHash = Source->Hash();

	if (SourceHash && SourceHash->IsFinal())
	{
		DeviceBufferRef ExistingBuffer = Find(SourceHash->Value(), true);
		if (ExistingBuffer)
			return cti::make_ready_continuable(ExistingBuffer);
	}

	const BufferDescriptor& sourceDesc = Source->Descriptor();

	/// Create a new native Buffer
	DeviceBufferRef NativeBuffer = Create(sourceDesc, SourceHash);

	/// Copy basic information
	*NativeBuffer.get() = *Source.get();

	/// Also set the owner Dev correctly, which would've been set incorrectly after the assignment operator above
	NativeBuffer->OwnerDevice = this;

	return NativeBuffer->TransferFrom(Source)
		.then([this, NativeBuffer](BufferResultPtr) mutable
		{
			/// Ok we have a successful transfer ... add it to the cache

			/// Over here we just Touch the Buffer so that it updates its 
			/// access information, to prevent immediate collection
			NativeBuffer->UpdateAccessInfo();

			return AddInternal(NativeBuffer);
		})
		.fail([this, NativeBuffer](std::exception_ptr)
		{
			/// Transfer has failed ... now we send an error back ... 
			/// NativeBuffer should drop automatically
			return DeviceBufferRef();
		});
}

DeviceBufferRef Device::AddNewRef_Internal(DeviceBuffer* Buffer)
{
	check(this != Device_Null::Get());
	DeviceBufferRef BufferRef = DeviceBufferRef(DeviceBufferPtr(Buffer));
	return AddInternal(BufferRef);
}

void Device::TransferAborted(DeviceBufferRef Buffer)
{
	AddInternal(Buffer);
}

void Device::ClearCache()
{
	check(IsInGameThread());
	check(TextureGraphEngine::IsTestMode());

	/// Check all the queues have been flushed
	check(NativePreExecWait.empty());
	check(NativePreExec.empty());
	check(NativeExec.empty());
	check(NativePostExec.empty());

	UpdateAwaitingCollection();
	GarbageCollect();

	Cache.GetCache().Empty(10000);

	AwaitingCollection.clear();
	GCTargets.clear();
}

DeviceBufferRef Device::AddInternal(DeviceBufferRef& Buffer)
{
	CHashPtr Hash = Buffer->Hash(false);
	//auto collectorFn = std::bind(&Device::Collect, this, std::placeholders::_1);

	if (Buffer->IsTransient() ||
		!Buffer->IsValid() ||
		Buffer->IsNull() ||
		!Hash || 
		!Hash->IsFinal())
		return Buffer;

	check(this != Device_Null::Get());
	{
		FScopeLock Lock(Cache.GetMutex());

		check(Hash && Hash->IsFinal());

		DeviceBufferRef Cached = Find(Hash->Value(), false);

		/// Do we already have this
		if (Cached)
		{
			/// Copy from Cached => Buffer (replaces the underlying object)
			UE_LOGF(LogDevice, Log, "DeviceBuffer with Hash: %llu already exists [0x%llx] [Will replace: 0x%llx]", Hash->Value(), Cached.get(), Buffer.get());
			Buffer = Cached;

			return Cached;
		}

		MemUsed += Buffer->MemSize();
		MemUsedNative += Buffer->DeviceNative_MemSize();

		//ObserverSource->AddBuffer(Buffer.get(), Hash->Value(), Buffer->Descriptor()); /// notify the creation of a new DeviceBuffer

		DeviceBufferPtr BufferRef = Buffer; ///!owningRef ? std::shared_ptr<DeviceBuffer>(Buffer, Device::CollectBuffer) : owningRef;
		if (TextureGraphEngine::GetBlobber()->IsCacheEnabled())
			Cache.Insert(Hash->Value(), BufferRef);
		UE_LOGF(LogDevice, VeryVerbose, "Device count: %d", Cache.GetCache().Num());
	}

	return Buffer;
}

DeviceBufferRef Device::Find(HashType Hash, bool Touch)
{
	FScopeLock Lock(Cache.GetMutex());

	const DeviceBufferPtrW* Buffer = Cache.Find(Hash, Touch);

	/// If we don't find anything then just return a null Buffer
	if (!Buffer)
		return DeviceBufferRef();

	return DeviceBufferRef(Buffer->lock());
}

DeviceBufferRef Device::Create(RawBufferPtr Raw)
{
	check(BufferFactory);
	check(Raw->Hash() && Raw->Hash()->IsFinal());

	DeviceBuffer* Buffer = BufferFactory->CreateFromRaw(Raw);

	check(Buffer != nullptr);
	check(Buffer->Hash() && Buffer->Hash()->IsFinal());

	return AddNewRef_Internal(Buffer);
}

DeviceBufferRef Device::Create(BufferDescriptor Desc, CHashPtr Hash)
{
	check(BufferFactory);

	if (!Hash)
		Hash = std::make_shared<CHash>(DataUtil::GNullHash, false);

	/// Ensure that the user isn't messing about by telling us that this is a final Hash
	/// it cannot possibly be!
	//check(!Hash->IsFinal());

	DeviceBuffer* Buffer = BufferFactory->CreateFromDesc(Desc, Hash);
	check(Buffer != nullptr);

	//ObserverSource->AddBuffer(Buffer, Hash->Value(), Desc); /// notify the creation of a new DeviceBuffer, Hash is NOT final

	/// We don't need to add this, since this is temporary for the time being
	return DeviceBufferRef(DeviceBufferPtr(Buffer));
}

void Device::Touch(HashType Hash)
{
	Cache.TouchThreadSafe(Hash);
}

void Device::UpdateAwaitingCollection()
{
	if (TextureGraphEngine::IsDestroying())
		return;

	std::unique_lock<std::mutex> Lock(AwaitingCollectionMutex);

	if (TextureGraphEngine::IsDestroying())
	{
		for (DeviceBuffer* Buffer : AwaitingCollection)
			delete Buffer;
	}
	else
	{
		for (DeviceBuffer* Buffer : AwaitingCollection)
			Buffer->GetOwnerDevice()->Collect(Buffer);			
	}

	AwaitingCollection.clear();
}

void Device::CollectBuffer(DeviceBuffer* Buffer)
{
	if (!Buffer)
		return;

	/// Always delete on the game thread
	if (!TextureGraphEngine::IsDestroying() && !Buffer->IsNull())
	{
		if (Buffer->GetOwnerDevice())
		{
			Device* Dev = Buffer->GetOwnerDevice();
			Dev->AwaitingCollection.push_back(Buffer);
		}
		else
			delete Buffer;
	}
	else
		delete Buffer;
}

void Device::Collect(DeviceBuffer* Buffer)
{
	//check(IsInGameThread());

	//DeviceBuffer* clone = Buffer->Clone();
	FScopeLock Lock(&GCTargetsLock);

	CHashPtr Hash = Buffer->Hash();

	check(Buffer->IsValid());

	/// Trying to double delete?
	check(std::find(GCTargets.begin(), GCTargets.end(), Buffer) == GCTargets.end());

	/// If its a non-final Hash then we don't care about it
	if (!Hash || !Hash->IsFinal())
	{
		GCTargets.push_back(Buffer);
		return;
	}

	/// we remove it from the cache at this point
	Cache.RemoveThreadSafe(Hash->Value());

	/// Make sure that its not marked for transfer if its getting collected
	DeviceTransferServicePtr TransferSvc = TextureGraphEngine::GetScheduler()->GetDeviceTransferService().lock();
	if (TransferSvc)
	{
		TransferSvc->AbortTransfer(Hash->Value());
	}

	BlobRef blob = TextureGraphEngine::GetBlobber()->Find(Hash->Value());

	/// Sanity check (hopefully this should never trigger)
	check(!blob || blob->GetBufferRef().get() != Buffer);

	DeviceBufferRef Cached = Find(Hash->Value(), false);

	if (!Cached)
	{
		GCTargets.push_back(Buffer);
		return;
	}

	if (Cached.get() == Buffer)
	{
		FScopeLock CacheLock(Cache.GetMutex());

		/// remove from the cache
		Cache.GetCache().Remove(Hash->Value());

		/// double delete?
		check(std::find(GCTargets.begin(), GCTargets.end(), Buffer) == GCTargets.end());

		/// add to the gc list ...
		GCTargets.push_back(Cached.get());
	}
	else
		GCTargets.push_back(Buffer);
}

void Device::RemoveInternal(DeviceBuffer* Buffer)
{
	check(IsInGameThread());

	HashType Hash = Buffer->Hash()->Value();

	//ObserverSource->RemoveBuffer(Buffer, Hash, PrevHash);

	/// Remove from the cache
	//Cache.TS_Remove(Hash);

	/// Tell the Buffer to clear up
	Buffer->MarkForCollection();
	Buffer->ReleaseNative();

	UE_LOGF(LogDevice, VeryVerbose, "Deleting DeviceBuffer Ptr: 0x%p", Buffer);

	delete Buffer;
}

size_t Device::GetMaxMemory() const
{
	return MaxMemory;
}

size_t Device::GetMaxThreads() const
{
	return MaxThreads;
}

cti::continuable<int32> Device::Use() const
{
	return cti::make_ready_continuable<int32>(0);
}

void Device::GarbageCollect()
{
	check(IsInGameThread());

	FScopeLock Lock(&GCTargetsLock);

	for (size_t gci = 0; gci < GCTargets.size(); gci++)
		RemoveInternal(GCTargets[gci]);

	GCTargets.clear();

	/// These need to be cleared on the game thread
	{
		FScopeLock TaskLock(&NativeTaskLock);
		FinishedNativeTasks.clear();
	}
}

AsyncJobResultPtr Device::UpdateIdle()
{
	return UpdateDeviceTransfers();
}

void Device::Update(float dt)
{
	UpdateAwaitingCollection();

	GarbageCollect();

	double now = Util::Time();

	if (ShouldPrintStats)
	{
		double statsDelta = now - TimeStatsPrinted;

		if (statsDelta > PrintStatsInterval)
			PrintStats();
	}

	//ObserverSource->Broadcast();

	/// Only create if there are actually jobs to be executed on that Dev
	if (!ExecThread && NativeTasks.size())
	{
		PreExecThreadFunc();
		ExecThreadFunc();
		PostExecThreadFunc();
	}

	UpdateNonAsync();
}

bool Device::ShouldCollect(double TimeNow, DeviceBuffer* Buffer)
{
	float consumption = (float)Cache.GetCache().Num() / (float)Cache.GetCache().Max();

	/// We need to consume at least a minimum before we start collecting
	if (consumption < MinCacheConsumptionBeforeCollection)
		return false;

	if (consumption > MaxCacheConsumptionBeforeCollection)
		return true;

	/// We cannot collect items that haven't been hashed yet
	CHashPtr Hash = Buffer->Hash(false);
	if (!Hash || !Hash->IsFinal())
		return false;

	/// If the Buffer doesn't have a Raw Buffer yet, then it cannot be transferred
	if (!Buffer->HasRaw())
		return false;

	const AccessInfo& BufferAccessInfo = Buffer->GetAccessInfo();
	uint64 batchDelta = JobBatch::CurrentBatchId() - BufferAccessInfo.BatchId;

	if (batchDelta >= MaxBatchDeltaDefault)
	{
		double durationSinceAccess = TimeNow - BufferAccessInfo.Timestamp;
		return TimeNow > MinLastUsageDuration;
	}

	return false;
}

AsyncJobResultPtr Device::UpdateDeviceTransfers()
{
	std::vector<AsyncDeviceBufferRef> Promises;

	/// Over here we evaluate Dev transfers
	{
		FScopeLock Lock(Cache.GetMutex());

		bool DidRemove = true;
		double TimeNow = Util::Time();
		DeviceTransferServicePtr TransferSvc = TextureGraphEngine::GetScheduler()->GetDeviceTransferService().lock();
		FString DeviceName = Name();

		auto& LRU = Cache.GetCache();
		auto Iter = LRU.begin();

		while (DidRemove && Iter != LRU.end())
		{
			auto& Item = *Iter;
			std::shared_ptr<DeviceBuffer> Buffer = Item.lock();
			bool bShouldCollect = false;

			/// Check the Buffer for removal
			if (Buffer)
			{
				DidRemove = ShouldCollect(TimeNow, Buffer.get());

				if (DidRemove)
				{
					HashType BufferHash = Buffer->Hash()->Value();
					Device* NextDevice = Buffer->GetDowngradeDevice();

					if (NextDevice)
					{
						if (NextDevice != Device_Null::Get())
						{
							FString nextDeviceName = NextDevice->Name(); 
							UE_LOGF(LogDevice, Log, "Transfer Buffer: %ls [%ls => %ls]", *Buffer->GetName(), *DeviceName, *nextDeviceName);

							BlobRef BlobObj = TextureGraphEngine::GetBlobber()->Find(BufferHash);

							/// If we find a blob against it then we start the transfer
							if (BlobObj)
							{
								//TransferSvc->QueueTransfer(blob, nextDevice);
								//auto promise = std::move(blob->TransferTo(nextDevice));
								Promises.emplace_back(BlobObj->TransferTo(NextDevice));
								bShouldCollect = false;
							}
							else 
								bShouldCollect = true;
						}
						else
							bShouldCollect = true;
					}
					else
					{
						/// Ok, this is the last Dev that this is destined for ... we move it up the cache
						if (Buffer->IsPersistent())
						{
							LRU.FindAndTouch(BufferHash);

							/// We want to exit the loop at this point and tackle 
							/// TODO: We can improve this later on
							DidRemove = false;
						}
						else
							bShouldCollect = true;
					}

					/// Remove the least recent and reset the iterator
					/// We don't remove the Buffer if its persistent and this is the last 
					/// Dev that its going to be on
					if (DidRemove)
						LRU.RemoveLeastRecent();

					Iter = LRU.begin();
				}
			}
			else
			{
				// This can happen in situations where the blob has been garbage collected by the 
				// blobber. In that case, we should just collect the buffer
				bShouldCollect = true;
				LRU.RemoveLeastRecent();
				Iter = LRU.begin();
			}

			if (bShouldCollect && Buffer)
			{
				/// Basically we ask the blobber to remove it ... if its going to GC, then that should happen automatically
				TextureGraphEngine::GetBlobber()->Remove(Buffer->Hash()->Value());
			}
		}
	}

	if (!Promises.empty())
	{
		/// wait for the promises to finish
		return cti::when_all(Promises.begin(), Promises.end()).then([this](std::vector<DeviceBufferRef> results)
		{
			return cti::make_ready_continuable(std::make_shared<JobResult>());
		});
	}

	return cti::make_ready_continuable(std::make_shared<JobResult>());
}

AsyncDeviceBufferRef Device::CombineFromTiles(const CombineSplitArgs& CombineArgs)
{
	const T_Tiles<DeviceBufferRef>& Tiles = CombineArgs.Tiles; 
	auto Buffer = CombineArgs.Buffer;

	check(Tiles.Rows() > 1 && Tiles.Cols() > 1);

	RawBufferPtrTiles RawTiles(Tiles.Rows(), Tiles.Cols());
	T_Tiles<CHashPtr> Hashes(Tiles.Rows(), Tiles.Cols());

	for (size_t RowId = 0; RowId < Tiles.Rows(); RowId++)
	{
		for (size_t ColId = 0; ColId < Tiles.Cols(); ColId++)
		{
			RawBufferPtr TileRawBuffer = Tiles[RowId][ColId]->Raw_Now();
			check(TileRawBuffer);
			RawTiles[RowId][ColId] = TileRawBuffer;

			CHashPtr TileHash = Tiles[RowId][ColId]->Hash();

			if (TileHash == 0)
				TileHash = TileRawBuffer->Hash();

			Hashes[RowId][ColId] = TileHash;
		}
	}

	CHashPtr Hash = CHash::ConstructFromSources(Hashes.Tiles());
	
	/// We've already calculated the Hash from all the child blobs
	RawBufferPtr Raw = TextureHelper::CombineRaw_Tiles(RawTiles, Hash, Buffer->IsTransient());

	Buffer->UpdateRaw(Raw);

	return cti::make_ready_continuable(Buffer);
}

AsyncDeviceBufferRef Device::SplitToTiles_Generic(const CombineSplitArgs& SplitArgs)
{
	const T_Tiles<DeviceBufferRef>& Tiles = SplitArgs.Tiles;
	auto Buffer = SplitArgs.Buffer;

	RawBufferPtrTiles RawTileBuffers(0, 0);
	RawBufferPtr Raw = Buffer->Raw_Now();
	check(Raw);

	int32 TileWidth = Raw->Width() / Tiles.Cols();
	int32 TileHeight = Raw->Height() / Tiles.Rows();

	check(TileWidth > 0);
	check(TileHeight > 0);

	TextureHelper::RawFromMem_Tiled(Raw->GetData(), Raw->GetLength(), Raw->GetDescriptor(), (size_t)TileWidth, (size_t)TileHeight, RawTileBuffers);
	
	check(RawTileBuffers.Rows() == Tiles.Rows());
	check(RawTileBuffers.Cols() == Tiles.Cols());

	std::vector<AsyncBufferResultPtr> Promises;
	for (int32 RowId = 0; RowId < Tiles.Rows(); RowId++)
	{
		for (int32 ColId = 0; ColId < Tiles.Cols(); ColId++)
		{
			DeviceBufferRef Tile = Tiles[RowId][ColId];
			RawBufferPtr RawTile = RawTileBuffers[RowId][ColId];
			check(RawTile);

			AsyncBufferResultPtr TilePromise = Tile->UpdateRaw(RawTile);
			Promises.push_back(std::move(TilePromise));
		}
	}

	return cti::when_all(Promises.begin(), Promises.end()).then([Buffer]() mutable
	{
		UE_LOGF(LogDevice, Log, "SplitTiles_Generic Tile updates finished!");
		return Buffer;
	});
}

AsyncDeviceBufferRef Device::SplitToTiles(const CombineSplitArgs& SplitArgs)
{
	return Device::SplitToTiles_Generic(SplitArgs);
}

void Device::MarkTaskFinished(DeviceNativeTaskPtr& Task, bool bRemoveFromOwnerQueue)
{
	/// Add to the finished queue so that it can be cleared on the game thread
	{
		FScopeLock Lock(&NativeTaskLock);

		/// and add to the finalisation queue 
		FinishedNativeTasks.push_back(Task);

		if (bRemoveFromOwnerQueue)
		{
			NativeTasks.remove(Task);
		}

		/// Need to make sure that strong ref is released here under the lock
		/// otherwise a race-condition may occure if this is done outside
		/// the lock where this closure may still hold a strong ref in the event
		/// of a context switch back to game thread. This will then mean that the 
		/// final release of the strong ref happens in a background thread, which
		/// causes issues since the job can contain UE objects that have to be
		/// destroyed on the game thread.
		Task = nullptr;
	}
}

void Device::UpdateNonAsyncPostExec()
{
	check(IsInGameThread());
	DeviceNativeTaskPtrWVec Tasks = NonAsyncPostExecQueue.to_vector_and_clear();

	for (DeviceNativeTaskPtrW& TaskW : Tasks)
	{
		DeviceNativeTaskPtr Task = TaskW.lock();
		check(Task);
		Task->PostExec();
		MarkTaskFinished(Task, true);
	}
}

void Device::UpdateNonAsync()
{
	UpdatePreExec();
	UpdateNonAsyncPostExec();
}

void Device::UpdatePreExec()
{
	DeviceNativeTaskPtrWVec Tasks = NativePreExec.to_vector_and_clear();

	if (!Tasks.empty())
	{
		//std::vector<cti::continuable<int32>> promises; 

		for (DeviceNativeTaskPtrW& TaskW : Tasks)
		{
			DeviceNativeTaskPtr Task = TaskW.lock();
			check(Task);

			/// Pre-Exec and then add to the exec queue afterwards
			Task->PreExecAsync(ENamedThreads::GameThread, ENamedThreads::AnyThread)
				.then([this, Task](int32 Result) mutable
				{
					if (Result == 0 && !Task->IsDone())
						NativeExec.add(Task);
					else
						MarkTaskFinished(Task, true);
				});
		}
	}
}

void Device::PreExecThreadFunc()
{
	check(!PreExecThread);

	if (PreExecThread != nullptr)
		return;

	PreExecThread = std::make_unique<std::thread>([this]()
	{
		if (PreExecThread)
			PreExecThreadId = (size_t)PreExecThread->native_handle(); ///Util::GetCurrentThreadId();

		while (!ShouldTerminate)
		{
			auto Task = NativePreExecWait.waitNext().lock();

			if (Task)
			{
				UE_LOGF(LogDevice, VeryVerbose, "PreExec - %ls", *Task->GetDebugName());

				CurrentWaitTask = Task;
				ON_SCOPE_EXIT
				{
					CurrentWaitTask = nullptr;   // always cleared, even on continue/throw
				};
				
				Task->Wait();
				
				if (Task->IsTerminated())
				{
					// Drop gracefully and finalize bookkeeping
					MarkTaskFinished(Task, /*bRemoveFromOwnerQueue=*/true);
					continue;
				}

				NativePreExec.add(Task);
			}
		}
	});
}

void Device::ExecThreadFunc()
{
	check(!ExecThread);

	if (ExecThread != nullptr)
		return;

	ExecThread = std::make_unique<std::thread>([this]()
		{
			if (ExecThread)
				ExecThreadId = (size_t)ExecThread->native_handle(); ///Util::GetCurrentThreadId();

			while (!ShouldTerminate)
			{
				/// Just wait for at least one Task to become available
				auto Task = NativeExec.waitNext().lock();

				if (Task)
				{
					UE_LOGF(LogDevice, VeryVerbose, "Exec - %ls", *Task->GetDebugName());

					/// If the Task is not async, then it needs to exec in the ExecThreadType background UE thread.
					/// Otherwise we can just run it in the current thread.
					if (!Task->IsAsync())
					{ 
						Util::OnThread(ExecThreadType, [this, Task]() mutable
						{
							Task->Exec();

							/// Only add if its not an exec only Task
							if (!Task->IsExecOnly() && !Task->IsDone())
							{
								if (Task->IsAsync())
									NativePostExec.add(Task);
								else
									NonAsyncPostExecQueue.add(Task);
							}
							else
								MarkTaskFinished(Task, true);
						});
					}
					else
					{
						/// Just run it right here
						Task->ExecAsync(ENamedThreads::AnyThread, ENamedThreads::AnyThread)
							.then([this, Task](int32) mutable
							{
								UE_LOGF(LogDevice, VeryVerbose, "[%llu] END - Device::ExecAsync Task: %ls [SUCCESS]", Task->GetTaskId(), *Task->GetName());
								if (!Task->IsDone())
								{
									if (Task->IsAsync())
										NativePostExec.add(Task);
									else
										NonAsyncPostExecQueue.add(Task);
								}
								else
									MarkTaskFinished(Task, true);
							})
							.fail([this, Task](std::exception_ptr e) mutable
							{
								UE_LOGF(LogDevice, VeryVerbose, "[%llu] END - Device::ExecAsync Task: %ls [FAIL]", Task->GetTaskId(), *Task->GetName());
								if (!Task->IsDone())
								{
									if (Task->IsAsync())
										NativePostExec.add(Task);
									else
										NonAsyncPostExecQueue.add(Task);
								}
								else
									MarkTaskFinished(Task, true);
							});
					}
				}
			}
		});
}

void Device::PostExecThreadFunc()
{
	check(!PostExecThread);

	if (PostExecThread != nullptr)
		return;

	PostExecThread = std::make_unique<std::thread>([this]()
		{
			if (PostExecThread)
				PostExecThreadId = (size_t)PostExecThread->native_handle(); ///Util::GetCurrentThreadId();

			while (!ShouldTerminate)
			{
				NativePostExec.wait();

				DeviceNativeTaskPtrWVec Tasks = NativePostExec.to_vector_and_clear();

				UE_LOGF(LogDevice, VeryVerbose, "[%ls] PostExec Task list: %d", *Name(), (int32)Tasks.size());

				for (DeviceNativeTaskPtrW& taskW : Tasks)
				{
					DeviceNativeTaskPtr Task = taskW.lock();

					if (Task)
					{
						UE_LOGF(LogDevice, Log, "PostExec - %ls", *Task->GetDebugName());

						Task->PostExec();
						MarkTaskFinished(Task, true);
					}
				}
			}
		});
}

void Device::UpdateNative()
{
}

#define DEVICE_NATIVE_TASKS_CHECK_DEPENDENCY_PRESENT 0

void Device::AddNativeTask(DeviceNativeTaskPtr Task)
{
	if (ShouldTerminate || TextureGraphEngine::IsDestroying())
	{
		// Drop the task gracefully; don’t enqueue.
		MarkTaskFinished(Task, /*bRemoveFromOwnerQueue=*/true);
		return;
	}
	
	// check(IsInGameThread());
	check(!Task->IsDone());

	/// Keep a strong ref here
	NativeTasks.add(Task);

	//if (Task->IsAsync())
	//{
		/// and add to the pre-exec queue
		if (!Task->IsExecOnly())
		{
			/// Debug check
#if (DEVICE_NATIVE_TASKS_CHECK_DEPENDENCY_PRESENT == 1)
			const DeviceNativeTaskPtrVec& prev = Task->Prev();
			NativePreExecWait.lock();
			auto& v = NativePreExecWait.accessInnerVector_Unsafe();

			for (DeviceNativeTaskPtr p : prev)
			{
				bool found = false;
				for (auto iter = v.begin(); iter != v.end() && !found; iter++)
				{
					if ((*iter).lock() == p)
						found = true;
				}

				check(found);
			}
			NativePreExecWait.unlock();
#endif 
			NativePreExecWait.add(Task);
		}
		else
			NativeExec.add(Task);
	//}
	//else
	//{
	//	NativePreExec.add(Task);
	//}
}

void Device::PrintStats()
{
	TimeStatsPrinted = Util::Time();

	FScopeLock Lock(Cache.GetMutex());

	auto& LRU = Cache.GetCache();
	size_t NumObjects = LRU.Num();

	/// we don't print stats for devices that have no objects at all
	if (!NumObjects)
		return;

	size_t NumNativeObjects = 0;
	size_t MemUsage = 0;
	size_t NativeMemUsage = 0;

	for (auto ItemW : LRU)
	{
		auto Item = ItemW.lock();

		if (Item)
		{
			size_t ItemMemSize = Item->MemSize();
			size_t ItemNativeMemSize = Item->DeviceNative_MemSize();

			MemUsage += ItemMemSize;

			if (ItemNativeMemSize)
			{
				NativeMemUsage += ItemNativeMemSize;
				NumNativeObjects++;
			}
		}
	}

	static const float MBConv = 1024.0f * 1024.0f;
	float MemUsageMB = (float)MemUsage / MBConv;
	float NativeMemUsageMB = (float)NativeMemUsage / MBConv;

	FString DeviceName = Name();

	UE_LOGF(LogDevice, Log, "===== BEGIN Device: %ls STATS =====", *DeviceName);
	UE_LOGF(LogDevice, Log, "Total Objects     : %llu", NumObjects);
	UE_LOGF(LogDevice, Log, "Native Objects    : %llu", NumNativeObjects);
	UE_LOGF(LogDevice, Log, "Raw Mem Usage     : %0.2f MB", MemUsageMB);
	UE_LOGF(LogDevice, Log, "Native Mem Usage  : %0.2f MB", NativeMemUsageMB);

	UE_LOGF(LogDevice, Log, "===== END Device  : %ls STATS =====", *DeviceName); 
}

uint32 Device::TraverseBufferCache(Visitor visitor)
{
	FScopeLock Lock(Cache.GetMutex());

	auto& LRU = Cache.GetCache();
	if (!LRU.Num())
		return 0;

	uint32 index = 0;
	for (auto item_ : LRU)
	{
		auto item = item_.lock();

		if (item)
		{
			visitor(item, index);
		}
		++index;
	}

	return index;
}

void Device::RegisterObserverSource(const DeviceObserverSourcePtr& observerSource)
{
	if (observerSource)
	{
		ObserverSource = observerSource;
	}
	else
	{
		ObserverSource = std::make_shared<DeviceObserverSource>();
	}
}

void DeviceObserverSource::AddBuffer(const DeviceBuffer* Buffer, HashType Hash, const BufferDescriptor& Desc)
{
	HashNDesc HashDesc = { (uint64)Buffer, Hash, Desc };
	FScopeLock observerLock(&ObserverLock);
	AddedBufferStack.emplace_back(HashDesc);
}


void DeviceObserverSource::RemoveBuffer(const DeviceBuffer* Buffer, HashType Hash, HashType PrevHash)
{
	FScopeLock Lock(&ObserverLock);
	RemovedBufferStack.emplace_back((uint64)Buffer);
	RemovedBufferStack.emplace_back(Hash);
	RemovedBufferStack.emplace_back(PrevHash);
}

void DeviceObserverSource::Broadcast()
{
	HashNDescArray AddedBuffers;
	HashArray RemovedBuffers;
	{
		FScopeLock observerLock(&ObserverLock);
		AddedBuffers = std::move(AddedBufferStack);
		RemovedBuffers = std::move(RemovedBufferStack);
		AddedBufferStack.clear();
		RemovedBufferStack.clear();
	}

	if (AddedBuffers.size() || RemovedBuffers.size())
	{
		DeviceBuffersUpdated(std::move(AddedBuffers), std::move(RemovedBuffers));
		Version++;
	}
}
