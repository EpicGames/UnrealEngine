// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/LruCache.h"
#include "CoreMinimal.h"
#include "Data/BinCache.h"
#include "Data/RawBuffer.h"
#include "DeviceBuffer.h"
#include "DeviceType.h"
#include "Helper/Promise.h"
#include "TaskPriorityQueue.h"
#include <list>
#include <memory>
#include <vector>
#include <queue>

#define UE_API TEXTUREGRAPHENGINE_API


class Blob;
class DeviceNativeTask;
typedef std::shared_ptr<DeviceNativeTask>	DeviceNativeTaskPtr;
typedef std::weak_ptr<DeviceNativeTask>		DeviceNativeTaskPtrW;

struct JobResult;
typedef std::shared_ptr<JobResult>			JobResultPtr;
typedef cti::continuable<JobResultPtr>		AsyncJobResultPtr;

class DeviceObserverSource;
typedef std::shared_ptr<DeviceObserverSource> DeviceObserverSourcePtr;

using AsyncBool = cti::continuable<bool>;

DECLARE_LOG_CATEGORY_EXTERN(LogDevice, Log, Verbose);


struct CombineSplitArgs
{
	DeviceBufferRef					Buffer;						/// Main buffer that should be used for combining to or split from
	const T_Tiles<DeviceBufferRef>& Tiles;						/// Tiles that needs to be merged or split
	bool							IsArray = false;			/// if operating buffer should be treated as Array 
	CHashPtr						BufferHash;					/// Precalculated buffer hash that might be available. Otherwise nullptr
};

//////////////////////////////////////////////////////////////////////////
/// Represents a general compute device that is capable of running some
/// computation on a blob.
//////////////////////////////////////////////////////////////////////////
class Device
{
	friend class					DeviceBuffer;
	friend class					Blob;
	friend class					Blobber;

public:
	DECLARE_EVENT(Device, Event_ResourceUpdate);

	Event_ResourceUpdate			ResourceUpdateEvent;			/// Triggered when history is updated
	
	static UE_API FString					DeviceType_Names(DeviceType DevType);

	typedef BinCache<HashType, DeviceBufferPtrW> BufferCache;
	typedef std::vector<DeviceBuffer*> DeviceBufferPList;
	typedef std::vector<DeviceNativeTaskPtr> DeviceNativeTaskPtrVec;
	typedef std::vector<DeviceNativeTaskPtrW> DeviceNativeTaskPtrWVec;

protected:
	friend class DeviceManager;

	static UE_API const double				PrintStatsInterval;		/// Print stats every <this many> milliseconds
	static UE_API const double				DefaultBufferTransferInternal; /// Buffer transfers are updated every few milliseconds represented by this variable

	static UE_API const float				DefaultMaxCacheConsumptionBeforeCollection;
	static UE_API const float				DefaultMinCacheConsumptionBeforeCollection;
	static UE_API const uint64				DefaultMaxBatchDeltaDefault;
	static UE_API const double				DefaultMinLastUsageDuration;

	//////////////////////////////////////////////////////////////////////////
	float							MaxCacheConsumptionBeforeCollection 
										= DefaultMaxCacheConsumptionBeforeCollection;	/// Maximum allowed consumption (percentage) of the cache before collection. 
																						/// The cache will keep on collecting until current consumption goes below max consumption

	float							MinCacheConsumptionBeforeCollection 
										= DefaultMinCacheConsumptionBeforeCollection;	/// Minimum allowed consumption (percentage) of the cache before collection. 
																						/// The cache will not start collecting until this amount of consumption is hit

	uint64							MaxBatchDeltaDefault = DefaultMaxBatchDeltaDefault;	/// Maximum batch delta allow before we start collecting the device
	double							MinLastUsageDuration = DefaultMinLastUsageDuration;	/// A DeviceBuffer must NOT have been used for a certain time before it can be collected
																						/// by the Device. This represents that duration in milliseconds

	//////////////////////////////////////////////////////////////////////////
	BufferCache						Cache;							/// The buffer cache

	DeviceBufferPList				GCTargets;						/// Items that are prime targets for garbage collection
	mutable FCriticalSection		GCTargetsLock;					/// Lock for GCTargets 

	size_t							MaxMemory = 0;					/// The max memory available to us
	size_t							MaxThreads = 1;					/// Max concurrent threads
	DeviceType						Type = DeviceType::Count;		/// What is the type of this device
	DeviceBuffer*					BufferFactory = nullptr;		/// The device buffer factory
	double							TimeStatsPrinted = 0;			/// The last time when stats were printed
	bool							ShouldPrintStats = false;		/// Whether to print stats all all

	size_t							MemUsed = 0;					/// How much main memory are we using 
	size_t							MemUsedNative = 0;				/// How much native memory are we using (this EXCLUDES main memory so that its counted twice)

	mutable FCriticalSection		NativeTaskLock;					/// Lock for _nativeTasks vector
	DeviceNativeTask_PriorityQueueStrong NativeTasks;				/// The tasks that are queued up to be executed over here
	DeviceNativeTaskPtrVec			FinishedNativeTasks;			/// The tasks that are finished. We clear this queue on the game thread

	DeviceNativeTaskPtr				CurrentWaitTask;				/// The task that we're currently waiting on (for debugging purposes only)

	std::atomic_bool				ShouldTerminate = false;		/// Whether the device is getting terminated or not

	DeviceNativeTask_PriorityQueue	NativePreExecWait;				/// Tasks queued in the pre-exec queue. This runs in a separate thread and makes sure that
																	/// all the previous dependencies of a particular task have been finished

	DeviceNativeTask_PriorityQueue	NativePreExec;					/// Tasks queued in the pre-exec queue. These are executed on the game thread
	std::unique_ptr<std::thread>	PreExecThread = nullptr;		/// Thread running pre-exec
	size_t							PreExecThreadId = 0;			/// The thread id of the pre-exec thread (used mostly for debugging and error checking)


	DeviceNativeTask_PriorityQueue	NativeExec;						/// Tasks queued in the exec queue
	std::unique_ptr<std::thread>	ExecThread = nullptr;			/// Thread running exec
	size_t							ExecThreadId = 0;				/// The thread id of the exec thread (used mostly for debugging and error checking)
	ENamedThreads::Type				ExecThreadType					= ENamedThreads::AnyBackgroundThreadNormalTask;
																	/// The thread on which the exec queue will run

	DeviceNativeTask_PriorityQueue	NativePostExec;					/// Tasks queued in the post-exec queue
	std::unique_ptr<std::thread>	PostExecThread = nullptr;		/// Thread running post-exec
	size_t							PostExecThreadId = 0;			/// The thread id of the post-exec thread (used mostly for debugging and error checking)

	DeviceNativeTask_PriorityQueue	NonAsyncPostExecQueue;			/// Non asynchronous post-exec tasks are queued here
	DeviceObserverSourcePtr			ObserverSource;					/// Device observer source that sends of different events happening within the device

	mutable std::mutex				AwaitingCollectionMutex;		/// Mutex for the list below
	DeviceBufferPList				AwaitingCollection;				/// Device buffers that are awaiting collection

	UE_API virtual void					UpdateAwaitingCollection();
	UE_API virtual void					UpdateNonAsyncPostExec();
	UE_API virtual void					UpdateNonAsync();

									UE_API Device(DeviceType InType, DeviceBuffer* InBufferFactory);
	UE_API virtual							~Device();

	// Method to wait for all the queued tasks. 
	// Can be extremely slow and should not
	// be used outside of tests
	UE_API virtual AsyncBool				WaitForQueuedTasks(ENamedThreads::Type ReturnThread = ENamedThreads::UnusedAnchor);
	UE_API virtual	void					CallDeviceUpdate();
	UE_API virtual void					GarbageCollect();

	UE_API virtual void					UpdatePreExec();

	UE_API virtual bool					ShouldCollect(double TimeNow, DeviceBuffer* Buffer);
	UE_API virtual AsyncJobResultPtr		UpdateDeviceTransfers();

	/// To be called from DeviceBuffer only (on destruction)
public:
	static UE_API void						CollectBuffer(DeviceBuffer* Buffer);

protected:
	UE_API virtual void					Collect(DeviceBuffer* Buffer);

	UE_API DeviceBufferRef					AddNewRef_Internal(DeviceBuffer* Buffer);
	UE_API virtual DeviceBufferRef			AddInternal(DeviceBufferRef& Buffer);
	UE_API virtual void					RemoveInternal(DeviceBuffer* Buffer);
	UE_API virtual void					Touch(HashType Hash);
	UE_API virtual void					PrintStats();

	UE_API virtual void					Terminate();

	UE_API virtual void					UpdateNative();
	UE_API virtual void					PreExecThreadFunc();
	UE_API virtual void					ExecThreadFunc();
	UE_API virtual void					PostExecThreadFunc();
	UE_API virtual void					MarkTaskFinished(DeviceNativeTaskPtr& Task, bool bRemoveFromOwnerQueue);

	/// Flushes a specific number of tasks from the _nativeTasks vector.
	/// Set maxCount = 0 to run all the tasks currently queued up. This
	/// function is thread safe.
	/// maxCount: Maximum number of tasks to run
	/// minCount: Minimum number of tasks to run. This is ude in conjunction with timeInterval
	/// timeInterval (milliseconds): Run as many minCount tasks (in batches) 
	/// until the timeInterval is expired or the queue has been emptied
	//virtual void					FlushNativeTasks(size_t maxCount, size_t minCount, double timeInterval);
	//virtual DeviceNativeTaskPtrVec	FetchNativeTasks(size_t maxCount, size_t minCount);
	//virtual void					FlushNativeTasks_Internal(DeviceNativeTaskPtrVec& tasks);

	UE_API virtual void					Free();

public:
	UE_API virtual DeviceBufferRef			Create(RawBufferPtr Raw);
	UE_API virtual DeviceBufferRef			Create(BufferDescriptor Desc, CHashPtr Hash);

	UE_API virtual AsyncDeviceBufferRef	CombineFromTiles(const CombineSplitArgs& CombineArgs);
	UE_API virtual AsyncDeviceBufferRef	SplitToTiles(const CombineSplitArgs& SplitArgs);


	/// Acquire the device for use
	UE_API virtual cti::continuable<int32>	Use() const;

	UE_API virtual size_t					GetMaxMemory() const;
	UE_API virtual size_t					GetMaxThreads() const;

	UE_API virtual DeviceBufferRef			CreateCompatible_Copy(DeviceBufferRef source);
	UE_API virtual AsyncDeviceBufferRef	Transfer(DeviceBufferRef source);
	UE_API virtual DeviceBufferRef			Find(HashType Hash, bool Touch);
	UE_API virtual void					Update(float dt);
	UE_API virtual AsyncJobResultPtr		UpdateIdle();

	virtual FString					Name() const { return "Generic"; }
	
	/// Adds a new task to the _nativeTasks vector. Thread-safe
	UE_API virtual void					AddNativeTask(DeviceNativeTaskPtr Task);

	UE_API virtual void					TransferAborted(DeviceBufferRef Buffer);

	using							Visitor = std::function<void (const std::shared_ptr<DeviceBuffer>&, uint32)>; /// Visitor callable receiving the bufferDevice visited and the index in the lru Cache
	UE_API uint32							TraverseBufferCache(Visitor visitor);

	UE_API void							RegisterObserverSource(const DeviceObserverSourcePtr& observerSource);
	UE_API bool							CanFree() const;
	UE_API virtual void					ClearCache();

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE bool				IsMultithreaded() const { return GetMaxThreads() > 1; }
	FORCEINLINE bool				GetShouldPrintStats() const { return ShouldPrintStats; }
	FORCEINLINE void				SetShouldPrintStats(bool Value) { ShouldPrintStats = Value; }
	FORCEINLINE DeviceType			GetType() const { return Type; }
	FORCEINLINE uint64				GetMemUsed() const { return MemUsed; }
	FORCEINLINE uint64				GetNumDeviceBuffersUsed() const { return Cache.GetCache().Num(); }
	FORCEINLINE uint64				GetNumDeviceBuffersMax() const { return Cache.GetCache().Max(); }

	FORCEINLINE DeviceObserverSourcePtr GetObserverSource() const { return ObserverSource; }
	FORCEINLINE size_t				GetPreExecThreadId() const { return PreExecThreadId; }
	FORCEINLINE size_t				GetExecThreadId() const { return ExecThreadId; }
	FORCEINLINE size_t				GetPostExecThreadId() const { return PostExecThreadId; }

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API AsyncDeviceBufferRef		SplitToTiles_Generic(const CombineSplitArgs& splitArgs);

	//////////////////////////////////////////////////////////////////////////
	/// Events
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE Event_ResourceUpdate& E_ResourceUpdate() { return ResourceUpdateEvent; }
};

#undef UE_API
