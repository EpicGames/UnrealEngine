// Copyright Epic Games, Inc. All Rights Reserved.
#include "DeviceBuffer.h"
#include "Device.h"
#include "Transform/BlobTransform.h"
#include "Job/TempHashService.h"

#include "Data/Blobber.h"
#include "TextureGraphEngine.h"
#include "DeviceManager.h"
#include "DeviceNativeTask.h"
#include <TextureGraphEngine.h>

//////////////////////////////////////////////////////////////////////////
const DeviceTransferChain DeviceBuffer::DefaultTransferChain = { DeviceType::FX, DeviceType::Mem, DeviceType::MemCompressed, DeviceType::Disk };
const DeviceTransferChain DeviceBuffer::FXOnlyTransferChain = { DeviceType::FX };
const DeviceTransferChain DeviceBuffer::PersistentTransferChain = {DeviceType::FX, DeviceType::Mem, DeviceType::MemCompressed, DeviceType::Disk, DeviceType::Null };

DeviceBuffer::DeviceBuffer(Device* Dev, BufferDescriptor NewDesc, CHashPtr NewHash) 
	: OwnerDevice(Dev)
	, Desc(NewDesc)
	, HashValue(NewHash)
{
	SetDeviceTransferChain(DefaultTransferChain);
}

DeviceBuffer::DeviceBuffer(Device* Dev, RawBufferPtr RawObj) 
	: OwnerDevice(Dev)
	, Desc(RawObj->GetDescriptor())
	, HashValue(RawObj->Hash())
	, RawData(RawObj)
{
	SetDeviceTransferChain(DefaultTransferChain);
}

DeviceBuffer::~DeviceBuffer()
{
	check(OwnerDevice != nullptr);
	RawData = nullptr;
}

DeviceBuffer* DeviceBuffer::CopyFrom(DeviceBuffer* RHS)
{ 
	/// Just use the default
	*this = *RHS;
	return this;
}

bool DeviceBuffer::IsTransient() const
{
	return Desc.bIsTransient;
}

DeviceBuffer* DeviceBuffer::Clone()
{
	DeviceBuffer* Clone = nullptr;
	if (RawData)
		Clone = CreateFromRaw(RawData);
	else 
		Clone = CreateFromDesc(Desc, HashValue);

	check(Clone);
	*Clone = *this;

	return Clone;
}

void DeviceBuffer::Touch(uint64 BatchId)
{
	check(IsInGameThread());

	UpdateAccessInfo(BatchId);
	OwnerDevice->Touch(HashValue->Value());
}

bool DeviceBuffer::IsValid() const
{
	return true;
}

bool DeviceBuffer::IsNull() const
{
	return false;
}

void DeviceBuffer::ReleaseNative()
{
}

AsyncBufferResultPtr DeviceBuffer::Unbind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) 
{
	/// No-Op by default
	return cti::make_ready_continuable(std::make_shared<BufferResult>());
}

AsyncRawBufferPtr DeviceBuffer::Raw()
{
	check(IsInGameThread());

	if (RawData)
		return cti::make_ready_continuable(RawData);

	check(!IsNull());
	FetchingRaw = true;

	return GetOwnerDevice()->Use()
		.then([this](int32) mutable
		{
			RawBufferPtr RawObj = Raw_Now();
			check(RawObj);

			CHashPtr NewHash = RawObj->Hash();

			/// Add to blobber
			NewHash = TextureGraphEngine::GetBlobber()->AddGloballyUniqueHash(NewHash);
			FetchingRaw = false;

			return PromiseUtil::OnGameThread();

		})
		.then([this]()
		{
			CHashPtr NewHash = Raw_Now()->Hash();
			
			/// Update the NewHash
			check(NewHash && NewHash->IsFinal());
			SetHash(NewHash);

			return RawData;
		});
}

AsyncRawBufferPtr DeviceBuffer::GetRawOrMaketIt()
{
	check(IsInGameThread());
	if (!IsFetchingRaw())
	{
		return Raw();
	}
	else
	{
		return GetOwnerDevice()->Use().then([this](int32) mutable
			{
				RawBufferPtr RawObj = Raw_Now();

				// Return to game thread with the payload
				return PromiseUtil::OnGameThread(std::move(RawObj));
			});
	}
}

CHashPtr DeviceBuffer::CalcHash()
{
	/// Don't recalculate if the NewHash has already been calculated
	if (HashValue)
		return HashValue;

	/// Default implementation will just calculate off the RawObj buffer
	RawBufferPtr RawObj = Raw_Now();

	/// Raw buffer might not be ready yet
	if (!RawObj)
		return nullptr;

	HashValue = CHash::UpdateHash(RawObj->Hash(), HashValue);

	return HashValue;
}

AsyncBufferResultPtr DeviceBuffer::Flush(const ResourceBindInfo& BindInfo)
{
	/// No-Op by default
	return cti::make_ready_continuable(std::make_shared<BufferResult>());
}

bool DeviceBuffer::IsCompatible(Device* Dev) const
{
	/// Simple base implementation
	return Dev == OwnerDevice;
}

CHashPtr DeviceBuffer::Hash(bool Calculate /* = true */)
{
	/// If we already have a valid NewHash then just give that
	if (HashValue || !Calculate)
		return HashValue;

	/// Otherwise, try to calculate and send
	return CalcHash();
}

Device* DeviceBuffer::GetDowngradeDevice() const
{
	int32 CurrentChainIndex = (int32)OwnerDevice->GetType();
	int32 MaxChainIndex = (int32)DeviceType::Count;

	for (int32 ChainIndex = CurrentChainIndex + 1; ChainIndex < MaxChainIndex; ChainIndex++)
	{
		if (Chain[ChainIndex])
			return TextureGraphEngine::GetDeviceManager()->GetDevice((DeviceType)ChainIndex);
	}

	return nullptr;
}

Device* DeviceBuffer::GetUpgradeDevice() const
{
	int32 ChainIndex = (int32)OwnerDevice->GetType();

	for (int32 ci = ChainIndex - 1; ci >= 0; ci--)
	{
		if (Chain[ci])
			return TextureGraphEngine::GetDeviceManager()->GetDevice((DeviceType)ci);
	}

	/// This should technically never happen
	check(false);

	return nullptr;
}

DeviceTransferChain DeviceBuffer::GetDeviceTransferChain(bool* Persistent /* = nullptr */) const
{
	if (Persistent)
		*Persistent = bIsPersistent;

	DeviceTransferChain TransferChain;
	for (int32 DeviceIndex = 0; DeviceIndex < (int32)DeviceType::Count; DeviceIndex++)
	{
		if (Chain[DeviceIndex])
			TransferChain.push_back((DeviceType)DeviceIndex);
	}

	return TransferChain;
}

void DeviceBuffer::SetDeviceTransferChain(const DeviceTransferChain& TransferChain, bool bPersistent /* = false */)
{
	for (auto c : TransferChain)
		Chain[(int32)c] = true;

	/// Having it on null device makes it peristent automatically
	if (Chain[(int)DeviceType::Null])
	{ 
		bPersistent = true;
		Chain[(int)DeviceType::Null] = false;
	}

	bIsPersistent = bPersistent;
}

AsyncPrepareResult DeviceBuffer::PrepareForWrite(const ResourceBindInfo& BindInfo)
{
	/// Default is no-op
	return cti::make_ready_continuable(0);
}

void DeviceBuffer::SetHash(CHashPtr NewHash)
{
	check(NewHash);

	check(NewHash->IsFinal() || NewHash->IsTemp());

	CHashPtr PrevHash = HashValue;

	if (NewHash->Value() == HashValue->Value() || !HashValue->IsValid())
	{
		HashValue = NewHash;
		return;
	}

	/// Should only be done through the game thread. Not thread-safe otherwise
	check(IsInGameThread());

	HashValue = CHash::UpdateHash(NewHash, PrevHash);
}

AsyncBufferResultPtr DeviceBuffer::UpdateRaw(RawBufferPtr RawObj)
{
	/// Copy the descriptor over. We don't want any discrepancies
	Desc = RawObj->GetDescriptor();
	HashValue = CHash::UpdateHash(RawObj->Hash(), HashValue);

	check(HashValue && HashValue->IsValid() && HashValue->IsFinal());
	return cti::make_ready_continuable(std::make_shared<BufferResult>());
}

//////////////////////////////////////////////////////////////////////////
DeviceBufferPtr::DeviceBufferPtr(DeviceBuffer* buffer) : std::shared_ptr<DeviceBuffer>(buffer, Device::CollectBuffer)
{
}

DeviceBufferPtr::DeviceBufferPtr(const DeviceBufferPtr& RHS) : std::shared_ptr<DeviceBuffer>(RHS)
{
}

DeviceBufferPtr::DeviceBufferPtr(const std::shared_ptr<DeviceBuffer>& RHS) : std::shared_ptr<DeviceBuffer>(RHS)
{
}

DeviceBufferPtr::~DeviceBufferPtr()
{
}

//////////////////////////////////////////////////////////////////////////

DeviceBufferRef::~DeviceBufferRef()
{
}

DeviceBufferRef::DeviceBufferRef(DeviceBufferPtr RHS) : Buffer(RHS) 
{
	/// Check erroneous assignment
	if (Buffer)
	{
		CHashPtr NewHash = Buffer->Hash(false);
		if (NewHash && NewHash->IsFinal())
		{
			BlobRef BlobRef = TextureGraphEngine::GetBlobber()->FindSingle(NewHash->Value());
			BlobPtr BlobObj = BlobRef.get();

			check(
				!BlobObj || 
				BlobObj->GetBufferRef().get() != Buffer.get() ||
				BlobObj->GetBufferRef()->GetOwnerDevice() != Buffer->GetOwnerDevice() ||
				BlobObj->GetBufferRef().GetPtr().use_count() == Buffer.use_count()
			);
		}
	}
}

DeviceBufferRef::DeviceBufferRef(DeviceBuffer*) 
{
	/// Not allowed
	check(false);
}

void DeviceBufferRef::Clear()
{
	Buffer = nullptr;
}

DeviceBufferRef& DeviceBufferRef::operator = (const DeviceBufferRef& RHS)
{
	Buffer = RHS.Buffer;
	return *this;
}

DeviceType DeviceBufferRef::GetDeviceType() const
{
	if (IsValid())
		return Buffer->GetOwnerDevice()->GetType();
	else
		return DeviceType::Null;
}