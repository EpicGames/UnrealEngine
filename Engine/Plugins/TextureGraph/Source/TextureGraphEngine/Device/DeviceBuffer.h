// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Data/RawBuffer.h"
#include "Helper/DataUtil.h"
#include "DeviceType.h"
#include "Data/G_Collectible.h"

#define UE_API TEXTUREGRAPHENGINE_API


class Device;
class Device_Mem;
class BlobTransform;
struct ResourceBindInfo;

struct BufferResult
{
	std::exception_ptr				ExInner;			/// Original exception that was raised by the action
	int32							ErrorCode = 0;		/// What is the error code
};

//////////////////////////////////////////////////////////////////////////

typedef std::shared_ptr<BufferResult>			BufferResultPtr;
typedef cti::continuable<BufferResultPtr>		AsyncBufferResultPtr;
typedef std::function<void(BufferResultPtr)>	BufferResult_Callback;
typedef cti::continuable<int32>					AsyncPrepareResult;

class DeviceBufferRef;

typedef std::vector<DeviceType>					DeviceTransferChain;

class DeviceNativeTask;
typedef std::shared_ptr<DeviceNativeTask>		DeviceNativeTaskPtr;
typedef std::vector<DeviceNativeTaskPtr>		DeviceNativeTaskPtrVec;

//////////////////////////////////////////////////////////////////////////
class DeviceBuffer : public G_Collectible
{
	friend class Device;
	friend class Blob;
	friend class Blobber;

public:
	static UE_API const DeviceTransferChain DefaultTransferChain;		/// Default transfer Chain
	static UE_API const DeviceTransferChain FXOnlyTransferChain;		/// Only exist on the FX device
	static UE_API const DeviceTransferChain PersistentTransferChain;	/// Persistent


protected:
	Device*							OwnerDevice = nullptr;		/// The device that this belongs to
	BufferDescriptor				Desc;						/// The descriptor for this buffer

	mutable CHashPtr				HashValue;					/// The actual RawObj buffer HashValue
	RawBufferPtr					RawData;					/// Raw buffer pointer that we keep cached (If we happen to have it)
	bool							FetchingRaw = false;		/// Whether we're already fetching raw or not

	bool							bMarkedForCollection = false; /// Whether this instance is marked for collection or not
	bool							bIsPersistent = false;		/// Whether this is persistent or not
	bool							Chain[static_cast<int32>(DeviceType::Count)] = {false}; /// Device transfer Chain

	UE_API virtual CHashPtr				CalcHash();

	/// Only accessible from Device. Should not be used elsewhere
	virtual DeviceBuffer*			CreateFromRaw(RawBufferPtr RawObj) = 0;
	virtual DeviceBuffer*			CreateFromDesc(BufferDescriptor Desc, CHashPtr HashValue) = 0;
	UE_API virtual DeviceBuffer*			CopyFrom(DeviceBuffer* RHS);
	UE_API virtual DeviceBuffer*			Clone();

	/// DO NOT MAKE THIS METHOD PUBLIC!
	/// Its not meant to be accessible outside
	UE_API virtual AsyncBufferResultPtr	UpdateRaw(RawBufferPtr RawObj);

	/// To be called from the Device owner only!
	UE_API virtual void					ReleaseNative();

	UE_API virtual void					Touch(uint64 BatchId);

	FORCEINLINE void				MarkForCollection() { bMarkedForCollection = true; }

public:
									UE_API DeviceBuffer(Device* Dev, BufferDescriptor NewDesc, CHashPtr NewHash);
									UE_API DeviceBuffer(Device* Dev, RawBufferPtr RawObj);
									DeviceBuffer(const DeviceBuffer& RHS) = delete;	/// non-copyable
	UE_API virtual							~DeviceBuffer() override;

	//////////////////////////////////////////////////////////////////////////
	/// Get the RawObj memory buffer against this device buffer. Note that
	/// it can be very expensive to get the RawObj buffer against a given
	/// optimised device buffer. Use this with caution.
	/// These are generally meant to be either used by device native buffers
	/// (e.g. Tex class) or background idle services. If you're calling these
	/// inplace in the middle of the rendering cycle, you're most probably 
	/// doing something terribly wrong. Please speak to someone before 
	/// attempting
	//////////////////////////////////////////////////////////////////////////
	virtual RawBufferPtr			Raw_Now() = 0;	/// SLOW: Read above
	UE_API virtual AsyncRawBufferPtr		Raw();			/// SLOW: Read above [use Blob::Raw instead. This is to be used from Blob only]

	UE_API AsyncRawBufferPtr				GetRawOrMaketIt();

protected:
	virtual AsyncBufferResultPtr	TransferFrom(DeviceBufferRef& Source) = 0;

public:

	//////////////////////////////////////////////////////////////////////////
	virtual size_t					MemSize() const = 0;
	virtual size_t					DeviceNative_MemSize() const { return 0; }

	virtual AsyncBufferResultPtr	Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) = 0;
	UE_API virtual AsyncBufferResultPtr	Unbind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo);

	UE_API virtual AsyncBufferResultPtr	Flush(const ResourceBindInfo& BindInfo);
	UE_API virtual bool					IsValid() const;
	UE_API virtual bool					IsNull() const;

	/// With base implementations
	UE_API virtual CHashPtr				Hash(bool Calculate = true);
	UE_API virtual bool					IsCompatible(Device* Dev) const;

	virtual RawBufferPtr			Min() { return nullptr; }
	virtual RawBufferPtr			Max() { return nullptr; }

	UE_API virtual void					SetHash(CHashPtr NewHash);
	UE_API virtual AsyncPrepareResult		PrepareForWrite(const ResourceBindInfo& BindInfo);

	UE_API void							SetDeviceTransferChain(const DeviceTransferChain& Chain, bool bPersistent = false);
	UE_API DeviceTransferChain				GetDeviceTransferChain(bool* Persistent = nullptr) const;
	UE_API virtual Device*					GetDowngradeDevice() const;
	UE_API virtual Device*					GetUpgradeDevice() const;
	UE_API virtual bool					IsTransient() const;

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE Device*				GetOwnerDevice() const { return OwnerDevice; }
	FORCEINLINE const 
		BufferDescriptor&			Descriptor() const { return Desc; };
	FORCEINLINE FString&			GetName() { return Desc.Name; };
	FORCEINLINE bool				HasRaw() const { return RawData != nullptr; }

	FORCEINLINE const bool*			TransferChain() const { return Chain; }
	FORCEINLINE bool				IsPersistent() const { return bIsPersistent; }
	FORCEINLINE bool				IsFetchingRaw() const { return FetchingRaw; }
};

//typedef std::shared_ptr<DeviceBuffer>	DeviceBufferPtr;

class DeviceBufferPtr : public std::shared_ptr<DeviceBuffer>
{
public:
	DeviceBufferPtr() : std::shared_ptr<DeviceBuffer>() {}
	UE_API DeviceBufferPtr(DeviceBuffer* buffer);
	UE_API DeviceBufferPtr(const DeviceBufferPtr& RHS);
	UE_API DeviceBufferPtr(const std::shared_ptr<DeviceBuffer>& RHS);
	UE_API ~DeviceBufferPtr();
};

typedef std::weak_ptr<DeviceBuffer>		DeviceBufferPtrW;
//typedef std::unique_ptr<DeviceBuffer>	DeviceBufferUPtr;

//typedef DeviceBufferPtr					DeviceBufferRef;
//typedef DeviceBufferPtrW				DeviceBufferRefW;

//////////////////////////////////////////////////////////////////////////
/// DeviceBufferRef
//////////////////////////////////////////////////////////////////////////
class DeviceBufferRef
{
public:
	

private:
	friend class Device;
	friend class Blob;
	friend class TiledBlob;

	DeviceBufferPtr					Buffer;				/// The buffer info pointer

	UE_API void							Clear();
public:
									DeviceBufferRef() {}
	UE_API explicit						DeviceBufferRef(DeviceBufferPtr RHS);
	UE_API explicit						DeviceBufferRef(DeviceBuffer* RHS);

									UE_API ~DeviceBufferRef();

	UE_API DeviceBufferRef&				operator = (const DeviceBufferRef& RHS);

	UE_API DeviceType						GetDeviceType() const; /// Need complete Device Definition so cannot be inlined here

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	//FORCEINLINE DeviceBufferRef		operator = (DeviceBuffer* RHS) { _buffer = DeviceBufferPtr(RHS); }
	FORCEINLINE bool				operator == (const DeviceBufferRef& RHS) const { return Buffer == RHS.Buffer; }
	FORCEINLINE bool				operator == (const DeviceBuffer* RHS) const { return Buffer.get() == RHS; }
	FORCEINLINE bool				operator != (const DeviceBufferRef& RHS) const { return Buffer != RHS.Buffer; }
	FORCEINLINE						operator bool () const { return IsValid(); }
	FORCEINLINE						operator DeviceBufferPtr() const { return Buffer; }
	FORCEINLINE DeviceBuffer*		operator -> () const { return Buffer.get(); }
	FORCEINLINE bool				IsValid() const { return Buffer != nullptr && Buffer->IsValid() ? true : false; }
	FORCEINLINE DeviceBuffer*		Get() const { return Buffer.get(); }
	FORCEINLINE DeviceBufferPtr		GetPtr() const { return Buffer; }

	/// STL compatibility
	FORCEINLINE DeviceBuffer*		get() const { return Buffer.get(); }	/// Compaibility with std::shared_ptr
	FORCEINLINE void				reset() { Clear(); }				/// Compaibility with std::shared_ptr
};

typedef cti::continuable<DeviceBufferRef>		AsyncDeviceBufferRef;
typedef std::function<void(DeviceBufferRef)>	DeviceBufferRef_Callback;

#undef UE_API
