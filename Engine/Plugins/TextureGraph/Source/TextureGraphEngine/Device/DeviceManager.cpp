// Copyright Epic Games, Inc. All Rights Reserved.
#include "DeviceManager.h"
#include "Device.h"
#include "Device/Null/Device_Null.h"
#include "Device/FX/Device_FX.h"
#include "Device/Mem/Device_Mem.h"
#include "Device/MemCM/Device_MemCM.h"
#include "Device/Disk/Device_Disk.h"
#include "Profiling/StatGroup.h"

DECLARE_CYCLE_STAT(TEXT("DeviceManager-Update"), STAT_DeviceManager_Update, STATGROUP_TextureGraphEngine);

DeviceManager::DeviceManager()
{
	InitDevices();
}

DeviceManager::~DeviceManager()
{
	ReleaseDevices();
}

void DeviceManager::InitDevices()
{
	Devices[(int32)DeviceType::Null] = new Device_Null();
	Devices[(int32)DeviceType::FX] = new Device_FX();
	Devices[(int32)DeviceType::Mem] = new Device_Mem();
	Devices[(int32)DeviceType::MemCompressed] = new Device_MemCM();
	Devices[(int32)DeviceType::Disk] = new Device_Disk();
}

void DeviceManager::ReleaseDevices()
{
	for (int32 DeviceIndex = 0; DeviceIndex < GetNumDevices(); DeviceIndex++)
	{
		if (Devices[DeviceIndex])
		{
			Devices[DeviceIndex]->Free();
			delete Devices[DeviceIndex];
			Devices[DeviceIndex] = nullptr;
		}
	}
}

void DeviceManager::Update(float Delta)
{
	SCOPE_CYCLE_COUNTER(STAT_DeviceManager_Update);
	for (int32 dvi = 0; dvi < (int32)DeviceType::Count; dvi++)
	{
		if (Devices[dvi])
			Devices[dvi]->Update(Delta);
	}
}

AsyncBool DeviceManager::WaitForQueuedTasks(ENamedThreads::Type ReturnThread/*= ENamedThreads::UnusedAnchor*/)
{
	std::vector<AsyncBool> Promises;
	for (int32 DeviceIndex = 0; DeviceIndex < (int32)DeviceType::Count; DeviceIndex++)
	{
		if (Devices[DeviceIndex])
			Promises.push_back(Devices[DeviceIndex]->WaitForQueuedTasks(ReturnThread));
	}

	return cti::when_all(Promises.begin(), Promises.end()).then([=](std::vector<bool>) mutable
	{
		return true;
	});
}

Device* DeviceManager::GetDevice(DeviceType Type) const
{
	return GetDevice((size_t)Type);
}

Device* DeviceManager::GetDevice(size_t Index) const
{
	check(Index < (size_t)DeviceType::Count);
	Device* Dev = Devices[(int32)Index];
	return Dev;
}
