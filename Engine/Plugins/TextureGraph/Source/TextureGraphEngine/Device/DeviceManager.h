// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "DeviceType.h"
#include "Profiling/StatGroup.h"
#include "Helper/Promise.h"
#include <memory>

#define UE_API TEXTUREGRAPHENGINE_API

class Device;
class Device_Mem;

using AsyncBool = cti::continuable<bool>;

class DeviceManager
{
private:
	Device*							Devices[(size_t)DeviceType::Count] = {};	/// The devices available within the system
	Device_Mem*						DefaultDevice = nullptr;					/// What is the default device on the system. This is always the CPU device

	UE_API void							InitDevices();
	UE_API void							ReleaseDevices();

public:
									UE_API DeviceManager();
	UE_API virtual							~DeviceManager();

	UE_API void							Update(float dt);
	UE_API AsyncBool						WaitForQueuedTasks(ENamedThreads::Type returnThread = ENamedThreads::UnusedAnchor);
	UE_API Device*							GetDevice(DeviceType type) const;
	UE_API Device*							GetDevice(size_t Index) const;

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE size_t				GetNumDevices() const { return (size_t)DeviceType::Count; }
	FORCEINLINE Device_Mem*			GetDefaultDevice() const { return DefaultDevice; }
};

typedef std::unique_ptr<DeviceManager> DeviceManagerPtr;

#undef UE_API
