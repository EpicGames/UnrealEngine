// Copyright Epic Games, Inc. All Rights Reserved.
#include "DeviceNativeTask.h"
#include "Device.h"
#include "Transform/BlobTransform.h"
#include "Job/TempHashService.h"

#include <TextureGraphEngine.h>

std::atomic<int64_t> DeviceNativeTask::GTaskId;

//////////////////////////////////////////////////////////////////////////
int64_t DeviceNativeTask::GetNewTaskId()
{
	return GTaskId++;
}

DeviceNativeTask::DeviceNativeTask(int32 TaskPriority, FString TaskName) 
	: TaskId(GetNewTaskId())
	, Priority(TaskPriority)
	, Name(TaskName)
{
	Future = Promise.get_future();
}

DeviceNativeTask::DeviceNativeTask(DeviceNativeTaskPtrVec& PrevTasks, int32 TaskPriority, FString TaskName)
	: DeviceNativeTask(TaskPriority, TaskName)
{
	Prev = PrevTasks;
}

DeviceNativeTask::~DeviceNativeTask()
{
}

cti::continuable<int32> DeviceNativeTask::GenericExecAsync(ENamedThreads::Type CallingThread, ENamedThreads::Type ReturnThread)
{
	return cti::make_continuable<int32>([this, CallingThread, ReturnThread](auto&& ThisPromise)
	{
		AsyncTask(CallingThread, [this, CallingThread, ReturnThread, ThisPromise = std::forward<decltype(ThisPromise)>(ThisPromise)]() mutable
		{
			std::exception_ptr Ex = nullptr;
			int32 RetVal = -1;

			try
			{
				RetVal = Exec();
			}
			catch (const std::exception& exc)
			{
				ThisPromise.set_exception(std::make_exception_ptr(std::exception(exc)));
			}

			if (CallingThread == ReturnThread || (ReturnThread & ENamedThreads::AnyThread))
			{
				if (!Ex) 
					ThisPromise.set_value(RetVal);
				else
					ThisPromise.set_exception(Ex);
			}

			/// Otherwise, get on the thread 
			/// TODO: There's a better way of doing this in the continuable library. Do that later on
			AsyncTask(ReturnThread, [this, ReturnThread, RetVal, Ex, ThisPromise = std::forward<decltype(ThisPromise)>(ThisPromise)]() mutable
			{
				if (!Ex)
					ThisPromise.set_value(RetVal);
				else
					ThisPromise.set_exception(Ex);
			});
		});
	});
}

cti::continuable<int32> DeviceNativeTask::PreExecAsync(ENamedThreads::Type CallingThread, ENamedThreads::Type ReturnThread)
{
	check(IsInGameThread());

	if (CallingThread == ENamedThreads::GameThread)
	{
		PreExec();
		return cti::make_ready_continuable<int32>(0);
	}

	return GenericExecAsync(CallingThread, ReturnThread);
}

cti::continuable<int32> DeviceNativeTask::ExecAsync(ENamedThreads::Type CallingThread, ENamedThreads::Type ReturnThread)
{
	ThreadId = std::hash<std::thread::id>{}(std::this_thread::get_id());
	return GenericExecAsync(CallingThread, ReturnThread);
}

ENamedThreads::Type DeviceNativeTask::GetExecutionThread() const
{
	return ENamedThreads::AnyThread;
}

FString DeviceNativeTask::GetDebugName() const
{
	return FString::Printf(TEXT("%s [%" INT64_FMT ", %d]"), *GetName(), GetTaskId(), GetPriority());
}

FString DeviceNativeTask::GetName() const
{
	return Name;
}

void DeviceNativeTask::Terminate()
{
	/// Recursively try to terminate the dependencies that we're waiting on
	for (DeviceNativeTaskPtr& PrevTasks : Prev) 
	{
		if (PrevTasks->GetFuture().wait_for(std::chrono::seconds(0)) != std::future_status::ready)
		{
			PrevTasks->Terminate();
			bTerminate = true;
			return;
		}
	}

	/// If we get to here, it means that this native task is the offending task.
	/// Must terminate self in this case!
	bTerminate = true;
}

bool DeviceNativeTask::IsHigherPriorityThan(const DeviceNativeTask& RHS) const
{
	if (BatchId == RHS.BatchId)
	{
		if (Priority != RHS.Priority)
			return Priority > RHS.Priority;
		return TaskId < RHS.TaskId;
	}

	return BatchId < RHS.BatchId;
}

void DeviceNativeTask::SetPriority(DeviceNativeTask* Ref)
{
	int32 existingPriority = Priority;
	Priority = Ref->Priority + 1;

	AdjustPrevPriority(Ref, this);
}

void DeviceNativeTask::SetPriority(int32 TaskPriority, bool AdjustPrev)
{
	int32 existingPriority = Priority;
	Priority = TaskPriority;

	/// Adjust priorities of the PrevTasks dependencies if we've been asked to do so OR
	/// if the current TaskPriority is higher (LESS THAN) the TaskPriority that we had before
	/// meaning that the TaskPriority of this particular task has increased!
	if (AdjustPrev || existingPriority > Priority)
		AdjustPrevPriority(nullptr, this);
}

#if (DEVICE_NATIVE_TASKS_CHECK_CYCLES == 1)

void DeviceNativeTask::CheckQueued(DeviceNativeTask* Ref)
{
	if (Ref && Ref != this)
	{
		check(this->IsHigherPriorityThan(*Ref));
	}

	for (DeviceNativeTaskPtr PrevTasks : Prev)
	{
		PrevTasks->CheckQueued(Ref);
		check(_batchId == PrevTasks->_batchId && PrevTasks->IsHigherPriorityThan(*this));

		if (Ref)
		{
			check(PrevTasks->_batchId == Ref->_batchId && PrevTasks->IsHigherPriorityThan(*Ref));
		}

		check(!PrevTasks->IsCulled());
	}
}

void DeviceNativeTask::DebugVerifyDepsQueued(const std::vector<DeviceNativeTaskPtr>& Queue, const DeviceNativeTaskPtrVec& AllItems)
{
	for (auto PrevTasks : Prev)
	{
		if (!PrevTasks->IsDone())
		{
			auto Iter = std::find(Queue.begin(), Queue.end(), PrevTasks);

			if (Iter == Queue.end())
			{
				auto Iter2 = std::find(AllItems.begin(), AllItems.end(), PrevTasks);
				check(Iter2 != AllItems.end());
				size_t Index = std::distance(AllItems.begin(), Iter2);
				check(false);
			}
			PrevTasks->DebugVerifyDepsQueued(Queue, AllItems);
		}
	}
}
#endif 

void DeviceNativeTask::Reset()
{
	// Reset the promise and future
	Promise = std::promise<int32>();
	Future = Promise.get_future();

	bIsDone = false;
}

void DeviceNativeTask::WaitSelf()
{
	Future.wait();
}

void DeviceNativeTask::Wait()
{
#if (DEVICE_NATIVE_TASKS_CHECK_CYCLES == 1)
	CheckQueued(this);
#endif 

	while (!WaitFor(1))
	{
		if (bTerminate.load(std::memory_order_acquire))
		{
			// Cancelled by Device::Terminate(); exit immediately.
			return;
		}
		std::this_thread::sleep_for(std::chrono::milliseconds(1));
	}
}

bool DeviceNativeTask::WaitSelfFor(int32 Seconds)
{
	std::future_status Status = Future.wait_for(std::chrono::seconds(Seconds));

	if (Status == std::future_status::timeout)
		return false;
	return true;
}

void DeviceNativeTask::CheckCyclicalDependency(std::unordered_set<DeviceNativeTask*>& TransferChain)
{
	for (DeviceNativeTaskPtr PrevTasks : Prev)
	{
		DeviceNativeTask* PrevTask = PrevTasks.get();
		check(TransferChain.find(PrevTask) == TransferChain.end());

		for (auto Iter = TransferChain.begin(); Iter != TransferChain.end(); Iter++)
		{
			DeviceNativeTask* pp = *Iter;
			check(IsHigherPriorityThan(*pp));
		}

		auto myChain = TransferChain;
		myChain.insert(PrevTask);

		PrevTask->CheckCyclicalDependency(TransferChain);
	}
}

FString DeviceNativeTask::DebugWaitChain()
{
	FString debugWaitChain = GetDebugName();

	for (DeviceNativeTaskPtr& PrevTasks : Prev)
	{
		std::future_status Status = PrevTasks->GetFuture().wait_for(std::chrono::seconds(0));

		if (Status == std::future_status::timeout)
		{
			debugWaitChain += FString::Printf(TEXT(" -> %s"), *PrevTasks->DebugWaitChain());
		}
	}

	return debugWaitChain;
}

bool DeviceNativeTask::WaitFor(int32 Seconds)
{
	check(!IsInGameThread());

	for (auto& PrevTask : Prev)
	{
		/// Culled jobs should never be added even though their promise has been resolved
		if (!PrevTask->IsCulled())
		{
			check(!PrevTask->IsCulled());

			/// Must be higher TaskPriority than this particular task
			check(PrevTask->IsHigherPriorityThan(*this));

			UE_LOGF(LogDevice, VeryVerbose, "[%llu] Begin Wait::%ls", PrevTask->GetTaskId(), *PrevTask->GetName());
			std::future_status Status = PrevTask->GetFuture().wait_for(std::chrono::seconds(Seconds));

			if (Status == std::future_status::timeout)
			{
				FString WaitChain = DebugWaitChain();
				UE_LOGF(LogDevice, VeryVerbose, "[%llu] Timed out wait for task::%ls. Chain: %ls", PrevTask->GetTaskId(), *PrevTask->GetName(), *WaitChain);
				return false;
			}

			UE_LOGF(LogDevice, VeryVerbose, "[%llu] End Wait::%ls", PrevTask->GetTaskId(), *PrevTask->GetName());
		}
	}

	return true;
}

bool DeviceNativeTask::DebugCompleteCheck()
{
	std::future_status Status = Future.wait_for(std::chrono::seconds(0));
	check(Status == std::future_status::ready);
	return true;
}

void DeviceNativeTask::FixPriorities()
{
	AdjustPrevPriority(this, this);

#if (DEVICE_NATIVE_TASKS_CHECK_CYCLES == 1)
	CheckQueued(this);
#endif 
}

void DeviceNativeTask::AdjustPrevPriority(DeviceNativeTask* Ref, DeviceNativeTask* Parent, DeviceNativeTask* PrevTasks)
{
	if (Ref->BatchId == PrevTasks->BatchId && Ref->IsHigherPriorityThan(*PrevTasks))
		PrevTasks->SetPriority(Ref);

	if (Parent && Parent != Ref && Parent->BatchId == PrevTasks->BatchId && Parent->IsHigherPriorityThan(*PrevTasks))
		PrevTasks->SetPriority(Parent);

	for (auto p : Prev)
	{
		p->AdjustPrevPriority(Ref, this);
	}
}

void DeviceNativeTask::AdjustPrevPriority(DeviceNativeTask* Ref, DeviceNativeTask* Parent)
{
	if (Ref == nullptr)
		Ref = this;

	//AdjustPrevPriority(Ref, this);

	if (BatchId == Ref->BatchId && Ref->IsHigherPriorityThan(*this))
		this->SetPriority(Ref);

	if (Parent && Parent != Ref && BatchId == Parent->BatchId && Parent->IsHigherPriorityThan(*this))
		this->SetPriority(Parent);

	for (auto PrevTasks : Prev)
	{
		PrevTasks->AdjustPrevPriority(Ref, this);
	}
}

void DeviceNativeTask::SetPrev(const DeviceNativeTaskPtrVec& PrevTasks) 
{ 
	check(Prev.empty()); 
	Prev = PrevTasks; 

#if (DEVICE_NATIVE_TASKS_CHECK_CYCLES == 1)
	std::unordered_set<DeviceNativeTask*> TransferChain;
	CheckCyclicalDependency(TransferChain);
#endif 

	AdjustPrevPriority(this, this);

#if (DEVICE_NATIVE_TASKS_CHECK_CYCLES == 1)
	CheckQueued(this);
#endif 
}

void DeviceNativeTask::AddPrev(DeviceNativeTaskPtr PrevTasks) 
{ 
	if (PrevTasks->IsCulled())
		return;

	PrevTasks->AdjustPrevPriority(this, this);

	Prev.push_back(PrevTasks); 

#if (DEVICE_NATIVE_TASKS_CHECK_CYCLES == 1)
	CheckQueued(this);
#endif 

	PrevTasks->AdjustPrevPriority(this, this);

#if (DEVICE_NATIVE_TASKS_CHECK_CYCLES == 1)
	CheckQueued(this);
#endif 


#if (DEVICE_NATIVE_TASKS_CHECK_CYCLES == 1)
	std::unordered_set<DeviceNativeTask*> TransferChain;
	CheckCyclicalDependency(TransferChain);
#endif 
}

void DeviceNativeTask::AddPrev(const DeviceNativeTaskPtrVec& Tasks)
{ 
	if (Tasks.empty())
		return;

	DeviceNativeTaskPtrVec nonCulledTasks;
	nonCulledTasks.reserve(Tasks.size());

	for (auto t : Tasks)
	{
		if (!t->IsDone())
		{
			t->AdjustPrevPriority(this, this);

			nonCulledTasks.push_back(t);
		}
	}

	if (!nonCulledTasks.empty())
		Prev.insert(Prev.end(), nonCulledTasks.begin(), nonCulledTasks.end());

#if (DEVICE_NATIVE_TASKS_CHECK_CYCLES == 1)
	CheckQueued(this);

	std::unordered_set<DeviceNativeTask*> TransferChain;
	CheckCyclicalDependency(TransferChain);
#endif 
}

void DeviceNativeTask::PreExec()
{
	/// Nothing to be done if already in game thread. The Device has already waited for our
	/// stuff to be ready before calling PreExec here
	if (IsInGameThread())
		return;

	/// No dependency ...
	if (Prev.empty())
		return;

	Device* thisDevice = GetTargetDevice();
	ENamedThreads::Type thisThreadId = GetExecutionThread();

	bool wait = false;

	/// or dependency on the same device, then we're good
	for (size_t i = 0; i < Prev.size() && !wait; i++)
	{
		if (Prev[i])
		{
			if (Prev[i]->GetTargetDevice() != thisDevice)
				wait = true;
			else
			{
				/// If we have a PrevTasks task or if its from a different device then we need to wait for it 
				/// to finish, but first we check if we're running on the same thread because that'll 
				/// block this
				ENamedThreads::Type prevThreadId = Prev[i]->GetExecutionThread();
				if (prevThreadId == thisThreadId &&
					!(prevThreadId & ENamedThreads::AnyThread) && !(thisThreadId & ENamedThreads::AnyThread))
				{
					/// Invalid condition
					FString errorStr = FString::Printf(TEXT("Error where previous device task and this device task are on different devices and yet have the same thread id. This will cause them to block. ThreadId: %d"), thisThreadId);
					UE_LOGF(LogDevice, Error, "%ls", *errorStr);
					throw std::runtime_error(TCHAR_TO_UTF8(*errorStr));
				}
				else
					wait = true;
			}
		}
	}

	if (wait)
		Wait();
}

void DeviceNativeTask::SetPromise(int32 Value)
{
	ErrorCode = Value;
	bIsDone = true;
	Promise.set_value(Value);
}

void DeviceNativeTask::PostExec()
{
	SetPromise(ErrorCode);
}

cti::continuable<int32> DeviceNativeTask::WaitAsync(ENamedThreads::Type ReturnThread /* = ENamedThreads::UnusedAnchor */)
{
#if (DEVICE_NATIVE_TASKS_CHECK_CYCLES == 1)
	CheckQueued(this);
#endif 

	return cti::make_continuable<int32>([this, ReturnThread](auto&& ThisPromise) mutable
	{
		Util::OnBackgroundThread([this, ReturnThread, FWD_PROMISE(ThisPromise)]() mutable 
		{
			Wait();

			/// If no thread is specified, then just return whatever
			if (ReturnThread == ENamedThreads::UnusedAnchor)
			{
				ThisPromise.set_value(0);
				return;
			}

			/// Otherwise, we resolve the promise from the correct thread
			Util::OnThread(ReturnThread, [=, FWD_PROMISE(ThisPromise)]() mutable
			{
				ThisPromise.set_value(0);
			});
		});
	});
}

//////////////////////////////////////////////////////////////////////////
DeviceNativeTask_Generic::DeviceNativeTask_Generic(Device* TargetDevice, FString TaskName, ENamedThreads::Type TaskThreadId /* = ENamedThreads::AnyThread */, bool IsTaskAsync /* = false */, 
	int32 TaskPriority /* = JobPriority::kNormal */)
	: DeviceNativeTask(TaskPriority, TaskName)
	, Dev(TargetDevice)
	, ThreadId(TaskThreadId)
{
	DeviceNativeTask::bIsAsync = IsTaskAsync;
}

DeviceNativeTask_Generic::DeviceNativeTask_Generic(DeviceNativeTaskPtrVec& PrevTasks, Device* TargetDevice, FString TaskName, ENamedThreads::Type TaskThreadId /* = ENamedThreads::AnyThread */, 
	bool IsTaskAsync /* = false */, int32 TaskPriority /* = JobPriority::kNormal */)
	: DeviceNativeTask(PrevTasks, TaskPriority, TaskName)
	, Dev(TargetDevice)
	, ThreadId(TaskThreadId)
{
	DeviceNativeTask::bIsAsync = IsTaskAsync;
}

//////////////////////////////////////////////////////////////////////////
DeviceNativeTask_Lambda::DeviceNativeTask_Lambda(DeviceNativeTask_Lambda_Func LambdaCallback, Device* Dev, FString TaskName, 
	ENamedThreads::Type TaskThreadId /* = ENamedThreads::AnyThread */, bool IsTaskAsync /* = false */, int32 TaskPriority /* = JobPriority::kNormal */)
	: DeviceNativeTask_Generic(Dev, TaskName, TaskThreadId, IsTaskAsync, TaskPriority)
	, Callback(LambdaCallback)
{
	DeviceNativeTask::bExecOnly = true;
}

int32 DeviceNativeTask_Lambda::Exec()
{
	check(Callback);
	ErrorCode = Callback();
	SetPromise(ErrorCode);
	return ErrorCode;
}

std::shared_ptr<DeviceNativeTask> DeviceNativeTask_Lambda::Create(Device* device, int32 TaskPriority, FString TaskName, DeviceNativeTask_Lambda_Func callback)
{
	return Create(device, ENamedThreads::AnyThread, false, TaskPriority, TaskName, callback);
}

std::shared_ptr<DeviceNativeTask> DeviceNativeTask_Lambda::Create(Device* device, ENamedThreads::Type threadId, bool isAsync,
	int32 TaskPriority, FString TaskName, DeviceNativeTask_Lambda_Func callback)
{
	auto task = std::make_shared<DeviceNativeTask_Lambda>(callback, device, TaskName, threadId, isAsync, TaskPriority);
	device->AddNativeTask(task);
	return std::static_pointer_cast<DeviceNativeTask>(task);
}

//cti::continuable<int32> DeviceNativeTask_Lambda::Create_Promise(Device* device, int32 TaskPriority, DeviceNativeTask_Lambda_Func callback)
//{
//	return Create_Promise(device, ENamedThreads::AnyThread, false, TaskPriority, callback);
//}
//
//cti::continuable<int32> DeviceNativeTask_Lambda::Create_Promise(Device* device, ENamedThreads::Type threadId, 
//	bool isAsync, int32 TaskPriority, DeviceNativeTask_Lambda_Func callback)
//{
//	auto task = std::make_shared<DeviceNativeTask_Lambda>(callback, device, threadId, isAsync, TaskPriority);
//	device->AddNativeTask(task);
//
//	return cti::make_continuable<int32>([task](auto&& promise)
//	{
//		Util::OnBackgroundThread([task, FWD_PROMISE(promise)]() mutable
//		{
//			/// The task must not be running on the same thread as the one that the promise is waiting on
//			/// otherwise, it'll lead to an application freeze!
//			int32 retVal = task->Future().get();
//			promise.set_value(retVal);
//		});
//	});
//}

//////////////////////////////////////////////////////////////////////////
