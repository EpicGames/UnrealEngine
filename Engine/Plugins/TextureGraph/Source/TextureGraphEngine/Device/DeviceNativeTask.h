// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Helper/DataUtil.h"
#include <vector>
#include <unordered_set>
#include "Helper/Util.h"
#include "Async/TaskGraphInterfaces.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Device;
class DeviceNativeTask;
typedef cti::continuable<DeviceNativeTask>	AsyncDeviceNativeTask;
typedef cti::continuable<DeviceNativeTask*>	AsyncDeviceNativeTaskP;
typedef std::shared_ptr<DeviceNativeTask>	DeviceNativeTaskPtr;
typedef std::vector<DeviceNativeTaskPtr>	DeviceNativeTaskPtrVec;

#define DEVICE_NATIVE_TASKS_CHECK_CYCLES 0

//////////////////////////////////////////////////////////////////////////
/// Represents a Task that runs natively on a Device (e.g. Rendering 
/// thread on an FX device or CPU and GPU threads on an OpenCL device)
//////////////////////////////////////////////////////////////////////////
class DeviceNativeTask
{
private:
	static UE_API std::atomic<int64_t>		GTaskId;				/// Task ID
	static UE_API int64_t					GetNewTaskId();

	std::promise<int32>				Promise;				/// Our promise
	std::shared_future<int32>		Future;					/// The future for our promise

protected:
	DeviceNativeTaskPtrVec			Prev;					/// The previous Tasks that this is dependent on
	std::atomic<bool>				bTerminate = false;		/// Terminate this process
	size_t							ThreadId = 0;			/// What is the actual thread ID
	int64_t							TaskId;					/// The ID of the Task
	int32							Priority;				/// The TaskPriority of the job. Must be in range [E_Priority::kHighest, E_Priority::kLowest]
	std::atomic<int>				ErrorCode = 0;			/// The error code for this native Task. This will be set in PostExec. 
															/// Derived implementations must set this correcly to the error code of the job/Task

	FString							Name;					/// The TaskName of this DeviceNativeTask

	bool							bExecOnly = false;		/// Whether its an exec only Task with no pre and post exec. This is to avoid waiting for system Tasks
	bool							bIsCulled = false;		/// Whether the Task is culled or not
	bool							bIsDone = false;		/// Whether the Task is done or not
	bool							bIsAsync = true;		/// Whether the native task is async or not

	int64_t							BatchId = 0;			/// What's the batch id of this native Task

	UE_API void							Reset();
	UE_API void							WaitSelf();
	UE_API bool							WaitSelfFor(int32 Seconds);

	UE_API void							SetPromise(int32 Value);
	UE_API FString							DebugWaitChain();

	UE_API virtual cti::continuable<int32>	GenericExecAsync(ENamedThreads::Type CallingThread, ENamedThreads::Type ReturnThread);

	UE_API void							AdjustPrevPriority(DeviceNativeTask* Ref, DeviceNativeTask* Parent);
	UE_API void							AdjustPrevPriority(DeviceNativeTask* Ref, DeviceNativeTask* Parent, DeviceNativeTask* PrevDeps);
	UE_API virtual void					SetPriority(DeviceNativeTask* Ref);

public:
									UE_API DeviceNativeTask(int32 TaskPriority, FString TaskName);
									UE_API DeviceNativeTask(DeviceNativeTaskPtrVec& PrevTasks, int32 TaskPriority, FString TaskName);
	UE_API virtual							~DeviceNativeTask();

	UE_API virtual bool					WaitFor(int32 Seconds);
	UE_API virtual void					Wait();
	UE_API virtual cti::continuable<int32>	PreExecAsync(ENamedThreads::Type CallingThread, ENamedThreads::Type ReturnThread);
	UE_API virtual void					PreExec();
	virtual int32					Exec() = 0;
	UE_API virtual cti::continuable<int32>	ExecAsync(ENamedThreads::Type CallingThread, ENamedThreads::Type ReturnThread);
	UE_API virtual void					PostExec();
	UE_API virtual cti::continuable<int32>	WaitAsync(ENamedThreads::Type ReturnThread = ENamedThreads::UnusedAnchor);

	virtual Device*					GetTargetDevice() const = 0;

#if (DEVICE_NATIVE_TASKS_CHECK_CYCLES == 1)
	UE_API void							CheckQueued(DeviceNativeTask* Ref);
	UE_API void							DebugVerifyDepsQueued(const DeviceNativeTaskPtrVec& Queue, const DeviceNativeTaskPtrVec& AllItems);
#endif 

	UE_API void							FixPriorities();

	/// Waits for the _prev Task to finish before executing. Its the responsibility of the system
	/// to ensure that both the _prev
	UE_API virtual ENamedThreads::Type		GetExecutionThread() const;
	UE_API virtual FString					GetName() const;
	UE_API virtual FString					GetDebugName() const;
	UE_API virtual void					Terminate();

	UE_API virtual void					AddPrev(DeviceNativeTaskPtr PrevTasks);
	UE_API virtual void					AddPrev(const DeviceNativeTaskPtrVec& Tasks);
	UE_API virtual void					SetPrev(const DeviceNativeTaskPtrVec& PrevTasks);
	UE_API virtual void					SetPriority(int32 TaskPriority, bool AdjustPrev);
	UE_API bool							IsHigherPriorityThan(const DeviceNativeTask& RHS) const;

	UE_API void							CheckCyclicalDependency(std::unordered_set<DeviceNativeTask*>& TransferChain);
	UE_API virtual bool					DebugCompleteCheck();
	virtual bool					IsAsync() const { return bIsAsync; }

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE std::shared_future<int32> GetFuture() { return Future; }
	FORCEINLINE size_t				GetActualThreadId() const { return ThreadId; }
	FORCEINLINE int64_t				GetTaskId() const { return TaskId; }
	FORCEINLINE int64_t				GetBatchId() const { return BatchId; }
	FORCEINLINE void				SetBatchId(int64_t NewBatchId) { BatchId = NewBatchId; }
	FORCEINLINE int32				GetPriority() const { return Priority; }
	FORCEINLINE bool				operator < (const DeviceNativeTask& RHS) const { return Priority < RHS.Priority; }
	FORCEINLINE bool				IsExecOnly() const { return bExecOnly; }
	FORCEINLINE bool				IsCulled() const { return bIsCulled; }
	FORCEINLINE bool				IsDone() const { return bIsDone || IsCulled(); }
	FORCEINLINE void				SetIsAsync(bool bIsAsyncIn) { bIsAsync = bIsAsyncIn; }
	FORCEINLINE const DeviceNativeTaskPtrVec& GetPrev() const { return Prev; }
	FORCEINLINE bool				IsTerminated() const	{ return bTerminate.load(std::memory_order_acquire); }
};

//////////////////////////////////////////////////////////////////////////
/// Below are some generic helper implementations
/// Note that if this starts getting too big, then we'll need to move
/// these to their individual files under a Device/Task/ sub-directory
//////////////////////////////////////////////////////////////////////////

//////////////////////////////////////////////////////////////////////////
class DeviceNativeTask_Generic : public DeviceNativeTask
{
	friend class Device;

protected:
	Device*							Dev = nullptr;			/// The target device that this is associated with
	ENamedThreads::Type				ThreadId = ENamedThreads::AnyThread; /// What thread is this going to run on

public:
									UE_API DeviceNativeTask_Generic(Device* TargetDevice, FString TaskName, ENamedThreads::Type TaskThreadId = ENamedThreads::AnyThread, 
										bool IsTaskAsync = false, int32 TaskPriority = (int32)E_Priority::kNormal);
									UE_API DeviceNativeTask_Generic(DeviceNativeTaskPtrVec& PrevDeps, Device* TargetDevice, FString TaskName, ENamedThreads::Type TaskThreadId = ENamedThreads::AnyThread, 
										bool IsTAskAsync = false, int32 TaskPriority = (int32)E_Priority::kNormal);

	virtual Device*					GetTargetDevice() const override { return Dev; }

	/// Waits for the _prev Task to finish before executing. Its the responsibility of the system
	/// to ensure that both the _prev
	virtual ENamedThreads::Type		GetExecutionThread() const override { return ThreadId; }

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE ENamedThreads::Type& GetThreadId() { return ThreadId; }
};

//////////////////////////////////////////////////////////////////////////
typedef TFunction<int32()>			DeviceNativeTask_Lambda_Func;
class DeviceNativeTask_Lambda : public DeviceNativeTask_Generic
{
protected:
	DeviceNativeTask_Lambda_Func	Callback = nullptr;	/// The lambda callback

	UE_API virtual int32					Exec() override;

public:
									UE_API DeviceNativeTask_Lambda(DeviceNativeTask_Lambda_Func LambdaCallback, Device* Dev, FString TaskName, 
										ENamedThreads::Type TaskThreadId = ENamedThreads::AnyThread, bool IsTaskAsync = false, int32 TaskPriority = (int32)E_Priority::kNormal);

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API std::shared_ptr<DeviceNativeTask> Create(Device* device, int32 TaskPriority, FString TaskName, DeviceNativeTask_Lambda_Func callback);
	static UE_API std::shared_ptr<DeviceNativeTask> Create(Device* device, ENamedThreads::Type threadId, bool isAsync, int32 TaskPriority, FString TaskName, DeviceNativeTask_Lambda_Func callback);

	//static cti::continuable<int32>	Create_Promise(Device* device, int32 TaskPriority, DeviceNativeTask_Lambda_Func callback);
	//static cti::continuable<int32>	Create_Promise(Device* device, ENamedThreads::Type threadId, bool isAsync, int32 TaskPriority, DeviceNativeTask_Lambda_Func callback);
};

#undef UE_API
