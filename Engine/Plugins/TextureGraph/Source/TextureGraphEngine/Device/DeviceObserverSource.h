// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Data/RawBuffer.h"
#include "DeviceBuffer.h"
#include <memory>
#include <vector>

#define UE_API TEXTUREGRAPHENGINE_API


class Blob;
class DeviceNativeTask;
typedef std::shared_ptr<DeviceNativeTask>	DeviceNativeTaskPtr;
typedef std::weak_ptr<DeviceNativeTask>		DeviceNativeTaskPtrW;

struct JobResult;
typedef std::shared_ptr<JobResult>	JobResultPtr;
typedef cti::continuable<JobResultPtr> AsyncJobResultPtr;

//////////////////////////////////////////////////////////////////////////
class DeviceObserverSource
{
public:
	struct HashNDesc {
		uint64						Raw = 0;
		HashType					First = 0;
		BufferDescriptor			Second;
	};
	using HashNDescArray			= std::vector<HashNDesc>;
	using HashArray					= std::vector<HashType>;

	UE_API void							AddBuffer(const DeviceBuffer* Buffer, HashType Hash, const BufferDescriptor& Desc);
	UE_API void							RemoveBuffer(const DeviceBuffer* buffer, HashType hash, HashType prevHash);

protected:
	/// Protected interface of emitters called by the scheduler to notify the observers
	friend class Device;

	FCriticalSection				ObserverLock;

	HashNDescArray					AddedBufferStack;
	HashArray						RemovedBufferStack;

	uint32							Version = 1;

	/// Trigger the broadcast of the changes, this call is issued from the Device::Update function
	UE_API void							Broadcast();

	/// The customisable handler method triggered from Broadcast if any buffers were added or removed
	virtual void					DeviceBuffersUpdated(HashNDescArray&& AddedBuffers, HashArray&& RemovedBuffers) {}

public:
	DeviceObserverSource()			= default;
	virtual							~DeviceObserverSource() = default;

	uint32							GetVersion() const { return Version; } /// THe version of the current state of the Device Cache. Incremented when Broadcast trigger
};
typedef std::shared_ptr<DeviceObserverSource> DeviceObserverSourcePtr;

#undef UE_API
