// Copyright Epic Games, Inc. All Rights Reserved.
#include "DeviceBuffer_Disk.h"
#include "Device/Device.h"
#include "Device_Disk.h"

//////////////////////////////////////////////////////////////////////////
DeviceBuffer_Disk::DeviceBuffer_Disk(Device_Disk* Dev, RawBufferPtr NewRawObj) 
	: DeviceBuffer(Dev, NewRawObj)
	, RawObj(NewRawObj)
{
}

DeviceBuffer_Disk::DeviceBuffer_Disk(Device_Disk* Dev, BufferDescriptor Desc, CHashPtr HashValue)
	: DeviceBuffer(Dev, Desc, HashValue)
{
}

DeviceBuffer_Disk::~DeviceBuffer_Disk()
{
	RawObj = nullptr;
}

bool DeviceBuffer_Disk::IsCompatible(Device* Dev) const
{
	/// Raw CPU buffers are compatible with every other device
	return true;
}

AsyncBufferResultPtr DeviceBuffer_Disk::TransferFrom(DeviceBufferRef& Source)
{
	return Source->Raw()
		.then([this, Source](RawBufferPtr NewRawObj)
		{
			RawObj = NewRawObj;
			FString filename = Device_Disk::Get()->GetCacheFilename(RawObj->Hash()->Value());
			return RawObj->WriteToFile(filename);
		})
		.then([this](RawBuffer*)
		{
			return std::make_shared<BufferResult>();
		});
}

RawBufferPtr DeviceBuffer_Disk::Raw_Now()
{
	return RawObj;
}

size_t DeviceBuffer_Disk::MemSize() const
{
	if (!RawObj)
		return 0;

	return RawObj->GetLength();
}

AsyncBufferResultPtr DeviceBuffer_Disk::Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) 
{
	/// These are not bindable
	check(false);
	throw std::runtime_error("DeviceBuffer_Disk is unbindable");
}

DeviceBuffer* DeviceBuffer_Disk::CreateFromRaw(RawBufferPtr NewRawObj)
{
	return new DeviceBuffer_Disk(static_cast<Device_Disk*>(OwnerDevice), NewRawObj);
}

DeviceBuffer* DeviceBuffer_Disk::CreateFromDesc(BufferDescriptor NewDesc, CHashPtr NewHashValue)
{
	return new DeviceBuffer_Disk(static_cast<Device_Disk*>(OwnerDevice), NewDesc, NewHashValue);
}
