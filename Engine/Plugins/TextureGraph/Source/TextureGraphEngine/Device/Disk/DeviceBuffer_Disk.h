// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Device/DeviceBuffer.h"
#include "Data/RawBuffer.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Device;
class Device_Disk;

//////////////////////////////////////////////////////////////////////////
/// This is simply a wrapper around RawBuffer. Must be owned by the 
/// CPU device (though that can change in the future)
//////////////////////////////////////////////////////////////////////////
class DeviceBuffer_Disk : public DeviceBuffer
{
protected:
	RawBufferPtr					RawObj;				/// The underlying raw buffer

	UE_API virtual DeviceBuffer*			CreateFromRaw(RawBufferPtr NewRawObj) override;
	UE_API virtual DeviceBuffer*			CreateFromDesc(BufferDescriptor NewDesc, CHashPtr NewHashValue) override;

	UE_API virtual AsyncBufferResultPtr	TransferFrom(DeviceBufferRef& Source) override;

public:
									UE_API DeviceBuffer_Disk(Device_Disk* Dev, BufferDescriptor Desc, CHashPtr HashValue);
									UE_API DeviceBuffer_Disk(Device_Disk* Dev, RawBufferPtr RawObj);
	UE_API virtual							~DeviceBuffer_Disk() override;

	//////////////////////////////////////////////////////////////////////////
	/// DeviceBuffer implementation
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual RawBufferPtr			Raw_Now() override;
	UE_API virtual size_t					MemSize() const override;
	UE_API virtual AsyncBufferResultPtr	Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) override;

	/// With base implementations
	//virtual void					Release() override;
	UE_API virtual bool					IsCompatible(Device* Dev) const override;

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
