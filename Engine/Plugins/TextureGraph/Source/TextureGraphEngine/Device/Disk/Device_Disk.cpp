// Copyright Epic Games, Inc. All Rights Reserved.
#include "Device_Disk.h"
#include "Misc/Paths.h"
#include "TextureGraphEngine.h"
#include "DeviceBuffer_Disk.h"
#include "Device/DeviceManager.h"

Device_Disk::Device_Disk() : Device_Mem(DeviceType::Disk, new DeviceBuffer_Disk(this, BufferDescriptor(), std::make_shared<CHash>(0xDeadBeef, true)))
{
	ShouldPrintStats = false;
	MaxThreads = FTaskGraphInterface::Get().GetNumWorkerThreads();

	/// This should be configurable from the application somehow
	MaxMemory = 1 * 1024 * 1024 * 1024;

	MinLastUsageDuration = DefaultMinLastUsageDuration * 100;

	/// By default we just use the temp directory for the platform
	BaseDir = FPlatformProcess::UserTempDir();
}

Device_Disk::~Device_Disk()
{
}

FString Device_Disk::SetBaseDirectory(const FString& InBaseDir, bool MigrateExisting /* = true */)
{
	FString PrevBaseDir = BaseDir;
	BaseDir = InBaseDir;

	check(FPaths::ValidatePath(BaseDir));

	/// TODO: migrate existing blobs

	return PrevBaseDir;
}

FString Device_Disk::GetCacheFilename(HashType HashValue) const
{
	FString CacheFilename = FPaths::Combine(BaseDir, FString::Printf(TEXT("%llu"), HashValue));
	return CacheFilename;
}

void Device_Disk::Update(float Delta)
{
	check(IsInGameThread());
	Device::Update(Delta);
}

Device_Disk* Device_Disk::Get()
{
	Device* Dev = TextureGraphEngine::GetDeviceManager()->GetDevice(DeviceType::Disk);
	check(Dev);
	return static_cast<Device_Disk*>(Dev);
}
