// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Device/Mem/Device_Mem.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Device_Disk : public Device_Mem
{
protected:
	FString							BaseDir;				/// The base directory for 

public:
									UE_API Device_Disk();
	UE_API virtual							~Device_Disk() override;

	virtual FString					Name() const override { return "Device_Disk"; }
	UE_API virtual void					Update(float Delta) override;
	UE_API virtual FString					SetBaseDirectory(const FString& InBaseDir, bool MigrateExisting = true);
	UE_API FString							GetCacheFilename(HashType HashValue) const;

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API Device_Disk*				Get();

	//////////////////////////////////////////////////////////////////////////
	/// Inline function
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE const FString&		GetBaseDirectory() const { return BaseDir; }
};

#undef UE_API
