// Copyright Epic Games, Inc. All Rights Reserved.
#include "DeviceBuffer_FX.h"
#include "Device_FX.h"
#include "FxMat/RenderMaterial.h"
#include "TextureGraphEngineGameInstance.h"
#include "Device/Mem/Device_Mem.h"
#include "2D/Tex.h"
#include "TextureGraphEngine.h"
#include "Device/DeviceNativeTask.h"
#include "2D/TexArray.h"
#include "Data/Blobber.h"

DeviceBuffer_FX::DeviceBuffer_FX(Device_FX* Dev, BufferDescriptor Desc, CHashPtr NewHashValue) 
	: DeviceBuffer(Dev, Desc, NewHashValue)
{
}

DeviceBuffer_FX::DeviceBuffer_FX(Device_FX* Dev, TexPtr TextureObj, RawBufferPtr RawObj) 
	: DeviceBuffer(Dev, RawObj)
	, Texture(TextureObj)
{
}

DeviceBuffer_FX::DeviceBuffer_FX(Device_FX* Dev, TexPtr TextureObj, BufferDescriptor Desc, CHashPtr NewHashValue)
	: DeviceBuffer(Dev, Desc, NewHashValue)
	, Texture(TextureObj)
{
}

DeviceBuffer_FX::DeviceBuffer_FX(Device_FX* Dev, RawBufferPtr RawObj)
	: DeviceBuffer(Dev, RawObj)
{
}

DeviceBuffer_FX::~DeviceBuffer_FX()
{
	UE_LOGF(LogDevice, VeryVerbose, "DeviceBuffer_FX Destructor: %ls [NewHashValue: %llu, this: 0x%p]", *Desc.Name, HashValue->Value(), this);
	check(!Texture || Texture.use_count() > 0);
}

RawBufferPtr DeviceBuffer_FX::Raw_Now()
{
	if (RawData)
		return RawData;

	check(!TextureGraphEngine::IsDestroying());
	check(IsInRenderingThread());
	check(Desc.IsValid());

	RawData = Texture->Raw(&Desc);
	check(RawData);

	return RawData;
}

size_t DeviceBuffer_FX::MemSize() const
{
	return RawData ? RawData->GetLength() : 0;
}

size_t DeviceBuffer_FX::DeviceNative_MemSize() const
{
	if (Texture)
		return Texture->GetMemSize();
	return 0;
}

void DeviceBuffer_FX::ReleaseNative()
{
	UE_LOGF(LogDevice, VeryVerbose, "DeviceBuffer_FX ReleaseNative: %ls [Mem: %llu, N-Mem: %llu, Dim: %dx%d]", *Desc.Name, MemSize(), DeviceNative_MemSize(), Desc.Width, Desc.Height);

	//We dont want to clear RT in TextureObj that we might have just created (e.g InitTiles_RenderTarget)
	if (Texture && bMarkedForCollection)
	{
		/// If the app is still running, then we need to manually manage _tex life and add it to RTGcList 
		/// (we need this to stop RT from turning black)
		/// Otherwise we can rely on its own destructor 
		/// This was required to prevent ScopeLock wait while the calling object was destructed
		if (!TextureGraphEngine::IsDestroying())
			(static_cast<Device_FX*>(OwnerDevice))->MarkForCollection(Texture);
	}

	Texture = nullptr;
}

AsyncBufferResultPtr DeviceBuffer_FX::UpdateRaw(RawBufferPtr RawObj)
{
	/// cannot update if there's already a RawObj buffer or texture against this resource
	check(!RawData);

	if (Texture)
	{
		if (Texture->IsRenderTarget())
		{
			Device_FX::Get()->MarkForCollection(Texture);
			Texture = nullptr;
		}
	}

	RawData = RawObj;

	return cti::make_continuable<BufferResultPtr>([this, RawObj](auto&& UpdatePromise)
	{
		DeviceBuffer::UpdateRaw(RawObj).then([this, FWD_PROMISE(UpdatePromise)](BufferResultPtr Result) mutable
		{
			if (Texture)
			{
				Util::OnGameThread([this, Result, FWD_PROMISE(UpdatePromise)]() mutable
				{
					Texture->UpdateRaw(RawData);
					UpdatePromise.set_value(Result);
				});

				return;
			}

			AllocateTexture().then([this, FWD_PROMISE(UpdatePromise)](BufferResultPtr Result) mutable
			{
				UpdatePromise.set_value(Result);
			});
		});
	});
}

AsyncBufferResultPtr DeviceBuffer_FX::AllocateTexture()
{
	if (Texture)
	{
		UE_LOGF(LogDevice, VeryVerbose, "Already cached texture: %ls", *Desc.Name);
		return cti::make_ready_continuable(std::make_shared<BufferResult>());
	}

	check(RawData);
	UE_LOGF(LogDevice, VeryVerbose, "Read BIND: %ls", *Desc.Name);

	check(IsInGameThread());

	TexDescriptor TextureDesc(RawData->GetDescriptor()); 
	Texture = std::make_shared<Tex>(TextureDesc);
	return Texture->LoadRaw(RawData).then([this]() mutable
	{
		check(Texture && Texture->GetTexture());
		return std::make_shared<BufferResult>();
	});
}

AsyncBufferResultPtr DeviceBuffer_FX::AllocateRenderTarget()
{
	/// If its already a render target, then we don't have to do anything
	if (Texture && Texture->IsRenderTarget())
		return cti::make_ready_continuable(std::make_shared<BufferResult>());

	check(!RawData);
	check(Desc.Width > 0 && Desc.Height > 0);

	UE_LOGF(LogDevice, VeryVerbose, "Write BIND: %ls", *Desc.Name);

	auto TextureObj = Device_FX::Get()->AllocateRenderTarget(Desc);
	UpdateRenderTarget(TextureObj);

	return cti::make_ready_continuable(std::make_shared<BufferResult>());
}

AsyncPrepareResult DeviceBuffer_FX::PrepareForWrite(const ResourceBindInfo& BindInfo)
{
	check(IsInGameThread());

	if (BindInfo.bIsCombined)
	{
		if (!Texture || !Texture->IsArray())
		{
			Texture = Device_FX::Get()->AllocateRenderTargetArray(Desc, BindInfo.NumTilesX, BindInfo.NumTilesY);
			return cti::make_ready_continuable(0);
		}
	}

	/// already prepared ... nothing to do over here
	if (Texture && Texture->GetTexture())
		return cti::make_ready_continuable(0);

	check(BindInfo.bWriteTarget);

	if (!BindInfo.bStaticBuffer)
	{
		Texture = Device_FX::Get()->AllocateRenderTarget(Desc);
		return cti::make_ready_continuable(0);
	}

	TexDescriptor TextureDesc(Desc);
	Texture = std::make_shared<Tex>(TextureDesc);
	
	return Texture->LoadRaw(nullptr).then([=](ActionResultPtr Result)
	{
		return 0;
	});
}

void DeviceBuffer_FX::UpdateRenderTarget(TexPtr TextureObj)
{
	check(IsInGameThread());

	Texture = TextureObj;

	/// This is largely redundant and a remanent of when we used to wait for things to be ready
	/// before proceeding. Now, because we have a pipeline, better suited for the multi-threaded
	/// UE approach, we don't need to do this. I'm keeping this for the time being for future 
	/// reference. In time it can be deprecated
#if 0 /// To be deprecated at some point
	check(_tex->RenderTarget()->Resource);
	DeviceNativeTask_Lambda::Create(_ownerDevice, [this]() -> int32
	{
		UTextureRenderTarget2D* rt = _tex->RenderTarget();
		rt->Resource->InitResource();

		FTextureRenderTarget2DResource* rtRes = (FTextureRenderTarget2DResource*)rt->GetRenderTargetResource();
		check(rtRes);

		FTextureRHIRef rhiTexture = rtRes->GetTextureRHI();
		check(rhiTexture);
		rt->UpdateResourceImmediate();

		return 0;
	});
#endif 
}

AsyncBufferResultPtr DeviceBuffer_FX::Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) 
{
	check(IsInGameThread());
	check(Transform || BindInfo.bWriteTarget);

	UE_LOGF(LogDevice, VeryVerbose, "Try Bind: %ls", *Desc.Name);

	Touch(BindInfo.BatchId);

	/// We must have the texture allocated by now
	return (!BindInfo.bWriteTarget ? AllocateTexture() : AllocateRenderTarget())
		.then([this, Transform, BindInfo](BufferResultPtr Result) 
		{
			check(Texture);
			check(IsInGameThread())

			if (!BindInfo.bWriteTarget)
			{
				const RenderMaterial* mat = static_cast<const RenderMaterial*>(Transform);

				check(mat);
				check(!BindInfo.Target.IsEmpty());

				mat->SetTexture(*BindInfo.Target, Texture);
			}
			else
			{
				/// Make sure it has a render target and a resource attached with it
				check(Texture->IsRenderTarget() && Texture->GetRenderTarget()->GetResource());
			}

			return Result;
		});
}

DeviceBuffer* DeviceBuffer_FX::CopyFrom(DeviceBuffer* RHS)
{
	/// Just use the default
	*this = *(static_cast<DeviceBuffer_FX*>(RHS));
	return this;
}

DeviceBuffer* DeviceBuffer_FX::CreateFromRaw(RawBufferPtr RawObj)
{
	return new DeviceBuffer_FX(static_cast<Device_FX*>(OwnerDevice), RawObj);
}

DeviceBuffer* DeviceBuffer_FX::CreateFromDesc(BufferDescriptor NewDesc, CHashPtr NewHashValue)
{
	return new DeviceBuffer_FX(static_cast<Device_FX*>(OwnerDevice), NewDesc, NewHashValue);
}

AsyncBufferResultPtr DeviceBuffer_FX::TransferFrom(DeviceBufferRef& Source)
{
	check(IsInGameThread());

	return Source->Raw()
		.then([this](RawBufferPtr RawObj) mutable
		{
			RawData = RawObj;
			return RawData->LoadRawBuffer(true);
		})
		.then([this, Source](RawBuffer*)
		{
			/// Now that we have a RawObj buffer we can create a texture out of it
			check(IsInGameThread());
			return AllocateTexture();
		});
}

CHashPtr DeviceBuffer_FX::CalcHash()
{
	if (HashValue && HashValue->IsFinal())
		return HashValue;

	if (!RawData)
	{
		/// Its understood at this point that this is a temp NewHashValue
		if (HashValue)
		{
			/// We don't calculate NewHashValue from render targets unless they have been flushed already
			/// so we just return the temp NewHashValue. 
			if (!Texture || (Texture && Texture->IsRenderTarget()))
				return HashValue;
		}

		/// We only calculate NewHashValue on the rendering thread
		if (!IsInRenderingThread())
			return HashValue;

		/// Otherwise we get the RawObj buffer from the texture and then calculate the NewHashValue off that
		RawData = Texture->Raw();
	}

	HashValue = RawData->Hash();

	return HashValue;
}

AsyncBufferResultPtr DeviceBuffer_FX::Unbind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) 
{
	check(IsInGameThread());

	if (BindInfo.bWriteTarget)
	{
		UE_LOGF(LogDevice, Log, "Write UN-BIND: %ls", *Desc.Name);

		if (Texture->GetDescriptor().bMipMaps)
			Texture->GenerateMips();
	}

	return DeviceBuffer::Unbind(Transform, BindInfo);
}

bool DeviceBuffer_FX::IsNull() const
{
	return DeviceBuffer::IsNull() || !Texture || Texture->IsNull();
}

AsyncBufferResultPtr DeviceBuffer_FX::Flush(const ResourceBindInfo& BindInfo)
{
	check(IsInRenderingThread());
	check(!RawData);
	check(Texture && Texture->IsRenderTarget());

	UE_LOGF(LogDevice, Warning, "Flushing buffer: %ls [this = %p][RT = 0x%p]", *Desc.Name, this, Texture->GetRenderTarget());

	/// Fetch the data back from the GPU and calculate the NewHashValue
	RawData = Texture->Raw();
	HashValue = RawData->Hash();

	//UE_LOGF(LogDevice, Log, "Clearing render target: %ls", *_desc.name);
	//Device_FX::Get()->MarkForCollection(_tex);
	//_tex = nullptr;

	///// Now turn into a texture
	//return AllocateTexture();

	/// No need for the render target anymore
	//return cti::make_continuable<BufferResultPtr>([this](auto&& promise) mutable
	//{
	//	Util::OnGameThread([this, promise = std::forward<decltype(promise)>(promise)]() mutable
	//	{
	//		/// de allocate texture
	//		UE_LOGF(LogDevice, Log, "Clearing render target: %ls", *_desc.name);
	//		Device_FX::Get()->MarkForCollection(_tex);
	//		_tex = nullptr;

	//		/// Now turn into a texture
	//		_tex = 
	//		Util::OnRenderingThread([this, promise = std::forward<decltype(promise)>(promise)](FRHICommandListImmediate&) mutable
	//		{ 
	//			promise.set_value(std::make_shared<BufferResult>());
	//		});
	//	});
	//});

	return DeviceBuffer::Flush(BindInfo);
}
