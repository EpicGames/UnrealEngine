// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Device/DeviceBuffer.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Device_FX;
class DeviceBuffer_Mem;

class Tex;
typedef std::shared_ptr<Tex>		TexPtr;

class DeviceBuffer_FX : public DeviceBuffer
{
	friend class Device_FX;
protected: 
	TexPtr							Texture;				/// The underlying texture object

	UE_API AsyncBufferResultPtr			AllocateTexture();
	UE_API AsyncBufferResultPtr			AllocateRenderTarget();
	UE_API void							UpdateRenderTarget(TexPtr TextureObj);

	UE_API virtual DeviceBuffer*			CreateFromRaw(RawBufferPtr RawObj) override;
	UE_API virtual DeviceBuffer*			CreateFromDesc(BufferDescriptor NewDesc, CHashPtr NewHashValue) override;
	UE_API virtual DeviceBuffer*			CopyFrom(DeviceBuffer* RHS) override;
	UE_API virtual AsyncBufferResultPtr	UpdateRaw(RawBufferPtr RawObj) override;

	UE_API virtual CHashPtr				CalcHash() override;

	UE_API virtual AsyncBufferResultPtr	TransferFrom(DeviceBufferRef& Source) override;

public:
									UE_API DeviceBuffer_FX(Device_FX* Dev, BufferDescriptor Desc, CHashPtr NewHashValue);
									UE_API DeviceBuffer_FX(Device_FX* Dev, TexPtr TextureObj, RawBufferPtr RawObj);
									UE_API DeviceBuffer_FX(Device_FX* Dev, TexPtr TextureObj, BufferDescriptor Desc, CHashPtr NewHashValue);
									UE_API DeviceBuffer_FX(Device_FX* Dev, RawBufferPtr RawObj);
	UE_API virtual							~DeviceBuffer_FX() override;

	//////////////////////////////////////////////////////////////////////////
	/// DeviceBuffer Implementation
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual RawBufferPtr			Raw_Now() override;
	UE_API virtual size_t					MemSize() const override;
	UE_API virtual size_t					DeviceNative_MemSize() const override;
	UE_API virtual AsyncBufferResultPtr	Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) override;
	UE_API virtual AsyncBufferResultPtr	Unbind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) override;
	UE_API virtual AsyncBufferResultPtr	Flush(const ResourceBindInfo& BindInfo) override;
	UE_API virtual AsyncPrepareResult		PrepareForWrite(const ResourceBindInfo& BindInfo) override;
	UE_API virtual bool					IsNull() const override;

	UE_API virtual void					ReleaseNative() override;

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE TexPtr				GetTexture() { return Texture; }
};

typedef std::shared_ptr<DeviceBuffer_FX> DeviceBuffer_FXPtr;

#undef UE_API
