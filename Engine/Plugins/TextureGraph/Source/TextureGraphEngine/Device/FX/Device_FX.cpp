// Copyright Epic Games, Inc. All Rights Reserved.
#include "Device_FX.h"
#include "CommonRenderResources.h"
#include "TextureGraphEngine.h"
#include "DeviceBuffer_FX.h"
#include "Device/Mem/Device_Mem.h"
#include "Device/Mem/DeviceBuffer_Mem.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "TextureGraphEngineGameInstance.h"
#include "Stats/Stats.h"
#include "FxMat/MaterialManager.h"
#include "Engine/Canvas.h"
#include "FxMat/RenderMaterial_FX.h"
#include "2D/Tex.h"
#include "Device/DeviceManager.h"
#include "Device/DeviceNativeTask.h"
#include "2D/TexArray.h"
#include <Engine/TextureRenderTarget2DArray.h>
#include <ScreenRendering.h>
#include <TextureResource.h>
#include <Engine/TextureRenderTarget2D.h>
#include "ClearQuad.h"

DECLARE_GPU_STAT_NAMED(Stat_CombineToTiles, TEXT("CombineToTilesProfile"));
DECLARE_GPU_STAT_NAMED(Stat_SplitFromTiles, TEXT("SplitFromTilesProfile"));
DECLARE_CYCLE_STAT(TEXT("Device_FX_CombineToTiles"), STAT_Device_FX_CombineFromTiles, STATGROUP_TextureGraphEngine);
DECLARE_CYCLE_STAT(TEXT("Device_FX_FillTextureArray"), STAT_Device_FX_FillTextureArray, STATGROUP_TextureGraphEngine);
DECLARE_CYCLE_STAT(TEXT("Device_FX_DrawTilesToBuffer"), STAT_Device_FX_DrawTilesToBuffer, STATGROUP_TextureGraphEngine);
DECLARE_CYCLE_STAT(TEXT("Device_FX_SplitToTiles"), STAT_Device_FX_SplitToTiles, STATGROUP_TextureGraphEngine);
DECLARE_CYCLE_STAT(TEXT("Device_FX_AllocateRendertarget"), STAT_Device_FX_AllocateRendertarget, STATGROUP_TextureGraphEngine);

size_t Device_FX::s_maxRenderBatch = 16;

Device_FX::Device_FX() : Device(DeviceType::FX, new DeviceBuffer_FX(this, BufferDescriptor(), std::make_shared<CHash>(0xDeadBeef, true)))
{
	ShouldPrintStats = false;
	ExecThreadType = ENamedThreads::ActualRenderingThread;
	RendererModule = FModuleManager::GetModulePtr<IRendererModule>("Renderer");
}

Device_FX::~Device_FX()
{
}

void Device_FX::Free()
{
	/// clear the RT cache
	FreeCacheInternal(RTCache);
	FreeCacheInternal(RTArrayCache);

	FreeRTList(RTUsed);
	GCTextures();

	Device::Free();
}

void Device_FX::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto Iter : RTCache)
	{
		RTList* List = Iter.second;
		if (List)
		{
			for (auto& RT : *List)
			{
				Collector.AddReferencedObject(RT);
			}
		}
	}

	for (auto Iter : RTArrayCache)
	{
		RTList* List = Iter.second;
		if (List)
		{
			for (auto& RT : *List)
			{
				Collector.AddReferencedObject(RT);
			}
		}
	}

	for (auto& RT : RTUsed)
	{
		Collector.AddReferencedObject(RT);
	}
}

FString Device_FX::GetReferencerName() const
{
	return Name();
}

void Device_FX::FreeCacheInternal(RenderTargetCache& TargetRTCache)
{
	for (auto Iter : TargetRTCache)
	{
		RTList* List = Iter.second;
		if (List)
		{
			for (UTextureRenderTarget2D* RT : *List)
			{
				check(RT);
			}

			List->clear();

			delete List;
		}
	}

	TargetRTCache.clear();
}

void Device_FX::FreeRTList(RTList& RTList)
{
	//for (UTextureRenderTarget2D* RT : RTList)
	//{
	//	check(RT);
	//}

	RTList.clear();
}

cti::continuable<int32> Device_FX::Use() const
{
	if (IsInRenderingThread())
	{
		const_cast<Device_FX*>(this)->CallDeviceUpdate();
		return cti::make_ready_continuable(0);
	}

	return cti::make_continuable<int32>([this](auto&& Promise) mutable
		{
			ENQUEUE_RENDER_COMMAND(SceneDrawCompletion)([this, Promise = std::forward<decltype(Promise)>(Promise)](FRHICommandListImmediate& DevRHI) mutable
				{
					const_cast<Device_FX*>(this)->CallDeviceUpdate(); // OK as long as the type object isn't const
					Promise.set_value(0);
				});
		});
}

void Device_FX::GetStatArray(TArray<FString>& ResourceArrayTiled,
	TArray<FString>& ResourceArrayUnTiled,
	TArray<FString>& TooltipListTiled,
	TArray<FString>& TooltipListUnTiled,
	FString& TotalStats)
{
	size_t RTMemUnused = 0;
	size_t RTCountUnused = 0;
	size_t RTMemUsed = 0;

	for (auto Iter = RTCache.begin(); Iter != RTCache.end(); Iter++)
	{
		RTList* RenderTargets = Iter->second;
		if (RenderTargets)
		{
			for (UTextureRenderTarget2D* RT : *RenderTargets)
			{
				auto RTFormat = RT->RenderTargetFormat;
				auto PixelFormat = TextureHelper::GetPixelFormatFromRenderTargetFormat(RTFormat);

				size_t Size = RT->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);

				FString Info(FString::Printf(TEXT("[Unused] Name : %s "), *RT->GetName()));
				FString Tooltip(FString::Printf(TEXT("Ptr : %llx , Hash : %llu, Memory Size : %0.2fKB"), RT, Iter->first, (float)Size / (1024.0f)));

				ResourceArrayUnTiled.Add(Info);
				TooltipListUnTiled.Add(Tooltip);

				RTMemUnused += Size;
				RTCountUnused++;
			}
		}
	}

	for (auto& RT : RTUsed)
	{
		auto RTFormat = RT->RenderTargetFormat;
		auto PixelFormat = TextureHelper::GetPixelFormatFromRenderTargetFormat(RTFormat);
		size_t Size = RT->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);
		FString Info(FString::Printf(TEXT("[Used] Name : %s"), *RT->GetName()));
		FString Tooltip(FString::Printf(TEXT("Ptr : %llx , Memory Size : %0.2fKB"), RT.Get(), (float)Size / (1024.0f)));

		ResourceArrayUnTiled.Add(Info);
		TooltipListUnTiled.Add(Tooltip);

		RTMemUsed += Size;
	}

	float usedRTMemory = (float)RTMemUsed / (1024.0f * 1024.0f);
	float unusedRTMemory = (float)RTMemUnused / (1024.0f * 1024.0f);

	TotalStats = FString::Printf(TEXT("Total Resources [UNUSED] : %llu, Total Memory [UNUSED] : %0.2fMB, Total Resources [USED] : %llu, Total Memory [USED] : %0.2f MB"), RTCountUnused, unusedRTMemory, RTUsed.size(), usedRTMemory);
}

AsyncDeviceBufferRef Device_FX::CombineFromTiles(const CombineSplitArgs& CombineArgs)
{
	check(IsInGameThread());

	SCOPE_CYCLE_COUNTER(STAT_Device_FX_CombineFromTiles);

	const T_Tiles<DeviceBufferRef>& Tiles = CombineArgs.Tiles;
	auto Buffer = CombineArgs.Buffer;
	auto IsArray = CombineArgs.IsArray;
	CHashPtr BufferHash = CombineArgs.BufferHash;

	check(Tiles.Rows() > 1 && Tiles.Cols() > 1);
	bool bIsCompatible = true;
	BufferDescriptor Desc = Tiles[0][0]->Descriptor();

	Desc.Width = Desc.Width * Tiles.Cols();
	Desc.Height = Desc.Height * Tiles.Rows();

	/// If any of the buffers is not compatible, then we fallback to the default implementation
	if (!bIsCompatible)
		return Device::CombineFromTiles(CombineArgs);

	if (!Buffer)
		Buffer = Create(Desc, BufferHash);

	DeviceBuffer_FX* FXBuffer = static_cast<DeviceBuffer_FX*>(Buffer.get());

	AsyncBufferResultPtr Promise = cti::make_ready_continuable(std::make_shared<BufferResult>());

	if (!IsArray)
		Promise = FXBuffer->AllocateRenderTarget();

	return (std::move(Promise)).then([this, Tiles, Buffer, IsArray]()
		{
			check(IsInGameThread());
			if (IsArray)
				return FillTextureArray_Deferred(Buffer, Tiles);

			return DrawTilesToBuffer_Deferred(Buffer, Tiles);
		});
}

AsyncDeviceBufferRef Device_FX::FillTextureArray_Deferred(DeviceBufferRef Buffer, const T_Tiles<DeviceBufferRef>& Tiles)
{
	DeviceNativeTaskPtr Task = DeviceNativeTask_Lambda::Create(this, (int32)E_Priority::kSystem, TEXT("FillTextureArray"), [this, Buffer, Tiles]() mutable -> int32
		{
			SCOPE_CYCLE_COUNTER(STAT_Device_FX_FillTextureArray);
			check(IsInRenderingThread());

			DeviceBuffer_FX* FXBuffer = static_cast<DeviceBuffer_FX*>(Buffer.get());

			/// This texture must have been created beforehand AND it must be a render target
			TexPtr DstTex = FXBuffer->GetTexture();
			check(DstTex);

			UTextureRenderTarget2D* RTDest = (UTextureRenderTarget2D*)DstTex->GetTexture();
			FTextureRHIRef RTResDest = ((FTextureRenderTarget2DResource*)RTDest->GetResource())->GetTextureRHI();

			FRHICommandListImmediate& DevRHI = RHI();

			int32 Index = 0;
			for (int32 RowId = 0; RowId < Tiles.Rows(); RowId++)
			{
				for (int32 ColId = 0; ColId < Tiles.Cols(); ColId++, Index++)
				{
					DeviceBuffer_FX* TileBuffer = static_cast<DeviceBuffer_FX*>(Tiles[RowId][ColId].get());
					TexPtr TileTex = TileBuffer->GetTexture();

					check(TileTex);

					int32 TileWidth = TileTex->GetWidth();
					int32 TileHeight = TileTex->GetHeight();

					UTextureRenderTarget2D* TileRT = TileTex->GetRenderTarget();
					FRHITexture* TileResource = TileTex->GetRHITexture();

					FRHICopyTextureInfo CopyInfo;
					CopyInfo.Size = TileResource->GetSizeXYZ();
					CopyInfo.DestSliceIndex = Index;

					DevRHI.CopyTexture(TileResource, RTResDest, CopyInfo);
				}
			}

			return 0;
		});

	return cti::make_ready_continuable(Buffer);
}

AsyncDeviceBufferRef Device_FX::DrawTilesToBuffer_Deferred(DeviceBufferRef Buffer, const T_Tiles<DeviceBufferRef>& Tiles, const DrawTilesSettings* DrawSettings, FLinearColor* ClearColor)
{
	TOptional<DrawTilesSettings> DrawSettingsCopy;
	if (DrawSettings != nullptr)
	{
		DrawSettingsCopy = *DrawSettings;
	}

	TOptional<FLinearColor> ClearColorCopy;
	if (ClearColor != nullptr)
	{
		ClearColorCopy = *ClearColor;
	}

	DeviceNativeTaskPtr Task = DeviceNativeTask_Lambda::Create(this, (int32)E_Priority::kSystem, TEXT("DrawTilesToBuffer"), [this, Buffer, Tiles, DrawSettingsCopy, ClearColorCopy]() mutable -> int32
		{
			SCOPE_CYCLE_COUNTER(STAT_Device_FX_DrawTilesToBuffer);
			check(IsInRenderingThread());

			DeviceBuffer_FX* FXBuffer = static_cast<DeviceBuffer_FX*>(Buffer.get());

			/// This texture must have been created beforehand AND it must be a render target
			TexPtr DstTex = FXBuffer->GetTexture();
			check(DstTex);

			UTextureRenderTarget2D* RTDest = DstTex->GetRenderTarget();
			FTextureRHIRef RTResDest = ((FTextureRenderTarget2DResource*)RTDest->GetResource())->GetTextureRHI();

			FRHICommandListImmediate& DevRHI = RHI();

			const uint32 ViewportWidth = DstTex->GetWidth();
			const uint32 ViewportHeight = DstTex->GetHeight();
			const FIntPoint TargetSize(ViewportWidth, ViewportHeight);

			//Render target is about to be renderered into, transition to RTV 
			DevRHI.Transition(FRHITransitionInfo(RTResDest, ERHIAccess::Unknown, ERHIAccess::RTV));

			FRHIRenderPassInfo RenderPassInfo(RTResDest, ERenderTargetActions::Load_Store);
			DevRHI.BeginRenderPass(RenderPassInfo, TEXT("CopyTexture"));

			if (ClearColorCopy.IsSet())
				DrawClearQuad(DevRHI, *ClearColorCopy);

			{
				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				DevRHI.ApplyCachedRenderTargets(GraphicsPSOInit);

				const auto FeatureLevel = GMaxRHIFeatureLevel;
				auto ShaderMap = GetGlobalShaderMap(FeatureLevel);
				TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
				TShaderMapRef<FScreenPS> PixelShader(ShaderMap);
				FxMaterial::InitPSO_Default(GraphicsPSOInit, VertexShader.GetVertexShader(), PixelShader.GetPixelShader());

				GraphicsPSOInit.PrimitiveType = PT_TriangleList;
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;

				FRHISamplerState* PixelSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
				bool bUseInputPositions = DrawSettingsCopy.IsSet();

				/// Verify that there are either no DrawSettings or they are setup correctly to match the input array dimensions
				check(!DrawSettingsCopy.IsSet() || (DrawSettingsCopy->Position.Rows() == Tiles.Rows() && DrawSettingsCopy->Position.Cols() == Tiles.Cols()));
				check(!DrawSettingsCopy.IsSet() || (DrawSettingsCopy->Size.Rows() == Tiles.Rows() && DrawSettingsCopy->Size.Cols() == Tiles.Cols()));

				SetGraphicsPipelineState(DevRHI, GraphicsPSOInit, 0);

				for (int32 RowId = 0; RowId < Tiles.Rows(); RowId++)
				{
					uint32 XOffset = 0;

					for (int32 ColId = 0; ColId < Tiles.Cols(); ColId++)
					{
						DeviceBuffer_FX* TileBuffer = static_cast<DeviceBuffer_FX*>(Tiles(RowId, ColId).get());
						TexPtr TileTex = TileBuffer->GetTexture();

						UE_LOGF(LogDevice, Verbose, "Combine tiles: %ls [%d, %d] => %llu [%ls]", *FXBuffer->GetName(), RowId, ColId, TileBuffer->Hash(false)->Value(), *TileBuffer->GetName());

						check(TileTex);

						UTextureRenderTarget2D* TileRT = TileTex->GetRenderTarget();
						FRHITexture* TileResource = TileTex->GetRHITexture();

						float SrcTextureWidth = TileTex->GetWidth();
						float SrcTextureHeight = TileTex->GetHeight();

						uint32 X1 = XOffset; ///ColId * SrcTextureWidth;
						uint32 Y1 = 0; ///RowId * SrcTextureHeight;

						if (!bUseInputPositions)
						{
							for (int32 TAIndex = 0; TAIndex < RowId; TAIndex++)
							{
								DeviceBuffer_FX* TileAbove = static_cast<DeviceBuffer_FX*>(Tiles(TAIndex, ColId).get());
								Y1 += TileAbove->Descriptor().Height;
							}
						}
						else
						{
							const FIntPoint& TilePos = DrawSettingsCopy->Position(RowId, ColId);
							X1 = (uint32)TilePos.X;
							Y1 = (uint32)TilePos.Y;

							const FIntPoint& TileSize = DrawSettingsCopy->Size(RowId, ColId);
							SrcTextureWidth = TileSize.X;
							SrcTextureHeight = TileSize.Y;
						}

						DevRHI.SetViewport(X1, Y1, 0, X1 + ViewportWidth, Y1 + ViewportHeight, 0);

						SetShaderParametersLegacyPS(DevRHI, PixelShader, PixelSampler, TileResource);

						RendererModule->DrawRectangle(
							DevRHI,
							0, 0,
							SrcTextureWidth, SrcTextureHeight,
							0.0f, 0.0f,
							1, 1,
							TargetSize,
							FIntPoint(1, 1),
							VertexShader,
							EDRF_Default);

						XOffset += SrcTextureWidth;
					}
				}
			}
			DevRHI.EndRenderPass();

			//Render target is done rendering transition to Read 
			DevRHI.Transition(FRHITransitionInfo(RTResDest, ERHIAccess::RTV, ERHIAccess::SRVMask));

			return 0;
		});

	return cti::make_ready_continuable(Buffer);
}

AsyncDeviceBufferRef Device_FX::SplitToTiles_Internal(const CombineSplitArgs& SplitArgs)
{
	const T_Tiles<DeviceBufferRef>& Tiles = SplitArgs.Tiles;
	auto Buffer = SplitArgs.Buffer;
	auto IsArray = SplitArgs.IsArray;

	DeviceNativeTask_Lambda::Create(this, (int32)E_Priority::kSystem, TEXT("SplitToTiles_Internal"), [this, Buffer, Tiles, IsArray]() -> int32
		{
			DeviceBuffer_FX* FXBuffer = static_cast<DeviceBuffer_FX*>(Buffer.get());

			/// This texture must have been created beforehand AND it must be a render target
			TexPtr SrcTex = FXBuffer->GetTexture();
			FTextureRHIRef SourceRes = ((UTextureRenderTarget2D*)SrcTex->GetTexture())->GetRenderTargetResource()->GetRenderTargetTexture();

			for (int32 RowId = 0; RowId < Tiles.Rows(); RowId++)
			{
				for (int32 ColId = 0; ColId < Tiles.Cols(); ColId++)
				{
					/// Using RHI to split Tiles to Buffer
					DeviceBuffer_FX* TileBuffer = static_cast<DeviceBuffer_FX*>(Tiles[RowId][ColId].get());
					TexPtr TileTex = TileBuffer->GetTexture();
					check(TileTex);
					check(TileTex->GetRenderTarget());

					FTextureRHIRef Texture2DRes = ((FTextureRenderTarget2DResource*)TileTex->GetRenderTarget()->GetResource())->GetTextureRHI();

					FRHICopyTextureInfo CopyInfo;
					CopyInfo.SourcePosition = FIntVector(RowId * TileTex->GetHeight(), ColId * TileTex->GetWidth(), 0);
					CopyInfo.DestPosition = FIntVector(0, 0, 0);
					CopyInfo.Size = FIntVector(TileTex->GetWidth(), TileTex->GetHeight(), 0);

					FRHICommandListImmediate& DevRHI = GRHICommandList.GetImmediateCommandList();
					DevRHI.CopyTexture(SourceRes, Texture2DRes, CopyInfo);
				}
			}

			return 0;
		});

	return cti::make_ready_continuable(Buffer);
}

AsyncDeviceBufferRef Device_FX::SplitToTiles(const CombineSplitArgs& SplitArgs)
{
	SCOPE_CYCLE_COUNTER(STAT_Device_FX_SplitToTiles)
		//check(IsInRenderingThread());

#define NATIVE_SPLIT_TILES
		const T_Tiles<DeviceBufferRef>& Tiles = SplitArgs.Tiles;
	auto Buffer = SplitArgs.Buffer;
	auto IsArray = SplitArgs.IsArray;

	const BufferDescriptor& Desc = Buffer->Descriptor();
	BufferDescriptor TileDesc = Desc;
	TileDesc.Width = Desc.Width / Tiles.Rows();
	TileDesc.Height = Desc.Height / Tiles.Cols();

	std::vector<AsyncBufferResultPtr> promises;

	/// first of all we need to ensure that all the Tiles are ready

	for (int32 RowId = 0; RowId < Tiles.Rows(); RowId++)
	{
		for (int32 ColId = 0; ColId < Tiles.Cols(); ColId++)
		{
			if (!Tiles[RowId][ColId])
			{
				DeviceBufferRef Tile = Tiles[RowId][ColId];
				TileDesc.Name = FString::Format(TEXT("{0}-{1}.{2}"), { Buffer->Descriptor().Name, RowId, ColId });
				DeviceBuffer_FX* FXBuffer = new DeviceBuffer_FX(this, TileDesc, nullptr);
				DeviceBufferRef FXBufferRef = AddNewRef_Internal(FXBuffer);

#ifdef NATIVE_SPLIT_TILES
				AsyncBufferResultPtr Promise = FXBuffer->AllocateRenderTarget();
				promises.push_back(std::move(Promise));
#endif /// NATIVE_SPLIT_TILES

				Tile = FXBufferRef;
			}
		}
	}

#ifdef NATIVE_SPLIT_TILES
	/// If there are no pending promises, then we can be ready immediately
	if (!promises.size())
		return SplitToTiles_Internal(SplitArgs);

	/// Otherwise, we wait ...
	return cti::when_all(promises.begin(), promises.end()).then([this, SplitArgs]()
		{
			return cti::make_continuable<DeviceBufferRef>([this, SplitArgs](auto&& Promise)
				{
					//Util::OnRenderingThread([this, Buffer, &Tiles, Promise = std::forward<decltype(Promise)>(Promise)](FRHICommandListImmediate& DevRHI) mutable
					//{
					/// And then, once we're ready, we split
					SplitToTiles_Internal(SplitArgs).then([this, Promise = std::forward<decltype(Promise)>(Promise)](DeviceBufferRef Result) mutable
						{
							Promise.set_value(Result);
						});
					//});
				});
		});
#else 
	return cti::make_continuable<DeviceBufferRef>([this, Buffer, &Tiles](auto&& Promise)
		{
			Device::SplitToTiles_Generic(Buffer, Tiles).then([this, Promise = std::forward<decltype(Promise)>(Promise)](DeviceBufferRef Result) mutable
				{
					Promise.set_value(Result);
				});
		});
#endif /// NATIVE_SPLIT_TILES
}

UTextureRenderTarget2D* Device_FX::CreateRenderTarget(const BufferDescriptor& Desc)
{
	check(Desc.Width && Desc.Height);

	FName Name = *FString::Printf(TEXT("%s [RT]"), *Desc.Name);
	auto Package = Util::GetRenderTargetPackage();
	UTextureRenderTarget2D* RT = NewObject<UTextureRenderTarget2D>(Package, Name, Tex::Flags);
	check(RT);

	RT->ClearColor = FLinearColor::Black;

	RT->bForceLinearGamma = true;
	RT->RenderTargetFormat = TextureHelper::GetRenderTargetFormatFromPixelFormat(Desc.PixelFormat());
	RT->SizeX = Desc.Width;
	RT->SizeY = Desc.Height;

	//UE_LOGF(LogTexture, Log, "Mips number: %d", _rt->GetNumMips());
	RT->LODBias = 0;
	//_rt->bAutoGenerateMips = false;

	//_rt->InitCustomFormat(_desc.width, _desc.height, _desc.format, forceLinearGamma);
	//UE_LOGF(LogTexture, Log, "Creating render target: %ls", *_desc.Name);

	RT->UpdateResource();

	return RT;
}

void Device_FX::FreeRenderTarget(HashType HashValue, UTextureRenderTarget2D* RT)
{
	FScopeLock lock(&GC_RTCache);
	RTUsed.remove(RT);
	UE_LOGF(LogDevice, VeryVerbose, "Reclaimed RT: Name : %ls : %llu [Ptr: 0x%llu, Size: %dx%d]", *RT->GetName(), HashValue, RT, RT->SizeX, RT->SizeY);

	/// We don't wanna be keeping these around in test mode.
	auto Iter = RTCache.find(HashValue);

	if (Iter == RTCache.end())
	{
		/// If the List wasnt previously found, that means RT was not created through AllocateRenderTarget
		/// Hence there is no need to save this RT , so we simply release the resources
		/// This is possible since FreeRenderTarget is called on deallocation, and deallocation can occur on shared ptr operator overloading as well
		RT->ReleaseResource();
		return;
	}

	RTList* List = Iter->second;
	List->push_back(RT);
}

TexPtr Device_FX::AllocateRenderTarget(const BufferDescriptor& Desc)
{
	const uint16 Depth = 1;
	const uint16 ArraySize = 1;
	uint8        NumMips = 1;
	uint8        NumSamples = 1;
	uint32       ExtData = 0;

	FRHITextureCreateInfo rhiDesc = FRHITextureDesc(ETextureDimension::Texture2D, ETextureCreateFlags::RenderTargetable, Desc.PixelFormat(),
		FClearValueBinding(Desc.DefaultValue), FIntPoint(Desc.Width, Desc.Height), Depth, ArraySize, NumMips, NumSamples, ExtData);

	check(IsInGameThread());

	SCOPE_CYCLE_COUNTER(STAT_Device_FX_AllocateRendertarget)

	RTList* List = nullptr;
	HashType Hash = Desc.FormatHashValue();

	{
		FScopeLock lock(&GC_RTCache);
		auto Iter = RTCache.find(Hash);

		if (Iter == RTCache.end())
		{
			List = new RTList();
			RTCache[Hash] = List;
		}
		else
			List = Iter->second;
	}

	if (!List->empty())
	{
		/// otherwise, we can just pull it from the List
		UTextureRenderTarget2D* RT = List->back();

		List->pop_back();

		FString existingName = RT->GetName();
		//UE_LOGF(LogDevice, Verbose, "Rename: %ls => %ls", *existingName, *Desc.Name);
		//RT->Rename(*Desc.Name);

		/// The RT resource is gone! then we need to create a new one
		if (RT->GetResource())
		{
			check(IsInGameThread());
			if (TextureGraphEngine::GetBlobber()->IsCacheEnabled())
				RTUsed.push_back(RT);

			TexPtr TextureObj = std::make_shared<Tex>(RT);
			return TextureObj;
		}
	}

	check(IsInGameThread());

	TexDescriptor TextureDesc(Desc);
	TexPtr TextureObj = std::make_shared<Tex>(TextureDesc);
	TextureObj->InitRT();

	check(TextureObj->GetRenderTarget());

	if (TextureGraphEngine::GetBlobber()->IsCacheEnabled())
		RTUsed.push_back(TextureObj->GetRenderTarget());

	DeviceNativeTask_Lambda_Func allocateFunction = [this, TextureObj]() { return AllocateRTResource(TextureObj); };
	auto Task = DeviceNativeTask_Lambda::Create(this, (int32)E_Priority::kSystem, TEXT("AllocateRenderTarget"), allocateFunction);

	return TextureObj;
}

TexPtr Device_FX::AllocateRenderTargetArray(const BufferDescriptor& Desc, int32 NumTilesX, int32 NumTilesY)
{
	check(IsInGameThread());

	SCOPE_CYCLE_COUNTER(STAT_Device_FX_AllocateRendertarget)

		RTList* List = nullptr;

	HashType Hash = Desc.FormatHashValue();
	{
		FScopeLock lock(&GC_RTCache);
		auto Iter = RTArrayCache.find(Hash);

		if (Iter == RTArrayCache.end())
		{
			List = new RTList();
			RTArrayCache[Hash] = List;
		}
		else
			List = Iter->second;
	}

	check(IsInGameThread());
	TexDescriptor TextureDesc(Desc);
	TexArrayPtr TexArrayObj = std::make_shared<TexArray>(TextureDesc, NumTilesX, NumTilesY);

	/// check if rendering target resources have been created correctly
	/// This has to be done in GameThread since UE5
	UTextureRenderTarget2DArray* RTArray = TexArrayObj->GetRenderTargetArray();
	RTArray->SetResource(RTArray->CreateResource());

	/// The rest of it can stay in RenderingThread
	DeviceNativeTask_Lambda_Func AllocateFunction = [this, TexArrayObj]() { return AllocateRTArrayResource(TexArrayObj); };
	auto Task = DeviceNativeTask_Lambda::Create(this, (int32)E_Priority::kSystem, TEXT("AllocateRenderTargetArray"), AllocateFunction);

	List->push_back((UTextureRenderTarget2D*)TexArrayObj->GetRenderTargetArray());

	return TexArrayObj;
}

int32 Device_FX::InitRTResource(TexPtr TextureObj, UTextureRenderTarget* RT)
{
	check(IsInRenderingThread());
	check(RT->GetResource());

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	RT->GetResource()->InitResource(RHICmdList);
	
	FDeferredUpdateResource::UpdateResources(RHICmdList);

	FTextureRenderTargetResource* RTRes = RT->GetRenderTargetResource();
	check(RTRes);

	UE_LOGF(LogDevice, Verbose, "New RT allocation: %ls %llu [Ptr: 0x%p, Size: %ux%u]", *TextureObj->GetDescriptor().Name,
		TextureObj->GetDescriptor().Format_HashValue(), RT, RTRes->GetSizeX(), RTRes->GetSizeY());
	
	FTextureRHIRef RHITexture = RTRes->GetTextureRHI();
	check(RHITexture);
	RHICmdList.BindDebugLabelName(RHITexture, *TextureObj->GetDescriptor().Name);
	
	return 0;
}

int32 Device_FX::AllocateRTResource(TexPtr TextureObj)
{
	return InitRTResource(TextureObj, TextureObj->GetRenderTarget());
}

int32 Device_FX::AllocateRTArrayResource(TexArrayPtr TexArrayObj)
{
	return InitRTResource(std::static_pointer_cast<Tex>(TexArrayObj), TexArrayObj->GetRenderTargetArray());
}

void Device_FX::MarkForCollection(TexPtr TextureObj)
{
	FScopeLock lock(&GCLock);

#if 0
	if (TextureObj->IsRenderTarget())
	{
		UTextureRenderTarget2D* RT = TextureObj->RenderTarget();

		/// Add to RT Display List
		HashType Hash = TextureObj->Descriptor().ToBufferDescriptor().Format_HashValue();
		FreeRenderTarget(Hash, RT);
		TextureObj->ReleaseRT();
	}
#endif 

	UE_LOGF(LogDevice, VeryVerbose, "[Device_FX] Marking Tex for collection: %ls", *TextureObj->GetDescriptor().Name);

	GCTargetTextures.push_back(TextureObj);
}

DeviceBufferRef Device_FX::CreateFromTex(TexPtr TextureObj, bool bInitRaw)
{
	BufferDescriptor Desc = TextureObj->GetDescriptor().ToBufferDescriptor();
	HashType Hash = (HashType)TextureObj->GetTexture(); ///Raw->Hash();

	/// TODO: Put this in the mem device
	//DeviceBufferRef memBuffer = Device_Mem::Get()->Create(Raw);
	DeviceBuffer_FX* FXBuffer;

	if (bInitRaw)
	{
		RawBufferPtr Raw = TextureObj->Raw();
		FXBuffer = new DeviceBuffer_FX(this, TextureObj, Raw);
	}
	else
		FXBuffer = new DeviceBuffer_FX(this, TextureObj, Desc, nullptr);

	return AddNewRef_Internal(FXBuffer);
}

DeviceBufferRef Device_FX::CreateFromTexAndRaw(TexPtr TextureObj, RawBufferPtr Raw)
{
	CHashPtr Hash = Raw->Hash();

	/// We need to lock this entire scope for thread safety reasons, so we just 
	/// use this mutex as a way to achieve that
	FScopeLock lock(&GCLock);

	/// Find whether we already have an object like this
	DeviceBufferRef RefBuffer = Find(*Hash, true);

	if (RefBuffer && !RefBuffer.IsValid())
	{
		/// We already have something
		GCTargetTextures.push_back(TextureObj);

		UE_LOGF(LogDevice, VeryVerbose, "[Device_FX] HASH ALREADY FOUND TextureObj: %ls Hash:%llu Previously:%ls", *TextureObj->GetDescriptor().Name, Hash->Value(), *RefBuffer->Descriptor().Name);

		return RefBuffer;
	}

	/// Otherwise we create a new one

	/// TODO: Put this in the mem device
	//DeviceBufferRef memBuffer = Device_Mem::Get()->Create(Raw);

	UE_LOGF(LogDevice, VeryVerbose, "[Device_FX] TextureObj: %ls Hash:%llu", *TextureObj->GetDescriptor().Name, Hash->Value());

	/// Now that we have the in-memory Buffer. We can use re-use the render target 
	/// for our device buffer
	DeviceBuffer_FX* FXBuffer = new DeviceBuffer_FX(this, TextureObj, Raw);

	return AddNewRef_Internal(FXBuffer);
}

DeviceBufferRef Device_FX::CreateFromTexture(UTexture2D* texture, const BufferDescriptor& Desc)
{
	check(IsInRenderingThread());

	/// Get the Raw data from render target
	RawBufferPtr Raw = TextureHelper::RawFromTexture(texture, Desc);
	return CreateFromTexAndRaw(std::make_shared<Tex>(texture), Raw);
}

DeviceBufferRef Device_FX::CreateFromRT(UTextureRenderTarget2D* RT, const BufferDescriptor& Desc)
{
	check(IsInRenderingThread());

	/// Get the Raw data from render target
	RawBufferPtr Raw = TextureHelper::RawFromRT(RT, Desc);
	return CreateFromTexAndRaw(std::make_shared<Tex>(RT), Raw);
}

void Device_FX::ClearCache()
{
	Device::ClearCache();
	GCTextures();
}

void Device_FX::GCTextures()
{
	check(IsInGameThread());

	TextureNodeList Tmp;
	{
		FScopeLock lock(&GCLock);
		Tmp = GCTargetTextures;
		GCTargetTextures.clear();
	}

	for (TexPtr t : Tmp)
		check(t.use_count());

	Tmp.clear();
}

void Device_FX::AddNativeTask(DeviceNativeTaskPtr Task)
{
	/// All Device_FX tasks must run in rendering thread, otherwise these
	/// need to be Device_Mem tasks
	//check(!Task->IsAsync());

	Device::AddNativeTask(Task);
}

void Device_FX::Update(float Delta)
{
	check(IsInGameThread());

	GCTextures();

	Device::Update(Delta);
}

void Device_FX::PrintStats()
{
	size_t RTMemUnused = 0;
	size_t RTCountUnused = 0;

	for (auto Iter = RTCache.begin(); Iter != RTCache.end(); Iter++)
	{
		RTList* RenderTargets = Iter->second;
		if (RenderTargets != nullptr)
		{
			for (UTextureRenderTarget2D* RT : *RenderTargets)
			{
				auto RTFormat = RT->RenderTargetFormat;
				auto PixelFormat = TextureHelper::GetPixelFormatFromRenderTargetFormat(RTFormat);
				size_t Size = RT->SizeX * RT->SizeY * TextureHelper::GetBppFromPixelFormat(PixelFormat) / 8;
				RTMemUnused += Size;
				RTCountUnused++;
			}
		}
	}

	size_t RTMemUsed = 0;
	for (auto RT : RTUsed)
	{
		auto RTFormat = RT->RenderTargetFormat;
		auto PixelFormat = TextureHelper::GetPixelFormatFromRenderTargetFormat(RTFormat);
		size_t Size = RT->SizeX * RT->SizeY * TextureHelper::GetBppFromPixelFormat(PixelFormat) / 8;
		RTMemUsed += Size;
	}

	UE_LOGF(LogDevice, Log, "===== BEGIN Device: FX STATS (Native) =====");

	UE_LOGF(LogDevice, Log, "[USED] RT Count   : %llu", RTUsed.size());
	UE_LOGF(LogDevice, Log, "[USED] RT Mem     : %0.2f MB", (float)RTMemUsed / (1024.0f * 1024.0f));

	UE_LOGF(LogDevice, Log, "[Unused] RT Count : %llu", RTCountUnused);
	UE_LOGF(LogDevice, Log, "[Unused] RT Mem   : %0.2f MB", (float)RTMemUnused / (1024.0f * 1024.0f));

	UE_LOGF(LogDevice, Log, "===== END Device  : FX STATS (Native) =====");

	Device::PrintStats();
}

//////////////////////////////////////////////////////////////////////////
Device_FX* Device_FX::Get()
{
	if (TextureGraphEngine::IsDestroying())
		throw std::runtime_error("Engine is shutting down!");

	Device* Dev = TextureGraphEngine::GetDeviceManager()->GetDevice(DeviceType::FX);
	check(Dev);
	return static_cast<Device_FX*>(Dev);
}

void Device_FX::InitTiles_Texture(T_Tiles<BlobPtr>* Tiles, const BufferDescriptor& InTileDesc, bool bInitRaw)
{
	size_t NumRows = Tiles->Rows();
	size_t NumCols = Tiles->Cols();

	BufferDescriptor TileDesc = InTileDesc;
	TileDesc.Format = TextureHelper::FindOptimalSupportedFormat(InTileDesc.Format);

	for (int32 RowId = 0; RowId < NumRows; RowId++)
	{
		for (int32 ColId = 0; ColId < NumCols; ColId++)
		{
			TileDesc.Name = TextureHelper::CreateTileName(InTileDesc.Name, RowId, ColId);

			TexDescriptor TextureTileDesc(TileDesc);
			TexPtr TileTex = std::make_shared<Tex>(TextureTileDesc);
			TileTex->InitTexture(nullptr, 0);
			/// prepare over here
			//DeviceBuffer_FX* FXBuffer = new DeviceBuffer_FX(Device_FX::Get(), TileTex, TileDesc.ToBufferDescriptor(), (CHashPtr)nullptr);
			//DeviceBuffer_FXPtr FXBuffer = std::make_shared<DeviceBuffer_FX>;
			BlobPtr TileBlob = std::make_shared<Blob>(Device_FX::Get()->CreateFromTex(TileTex, bInitRaw));
			(*Tiles)[RowId][ColId] = TileBlob;
		}
	}
}

void Device_FX::InitTiles_RenderTargets(BlobUPtr* Tiles, size_t NumRows, size_t NumCols, const BufferDescriptor& InTileDesc, bool bInitRaw)
{
	for (int32 RowId = 0; RowId < NumRows; RowId++)
	{
		for (int32 ColId = 0; ColId < NumCols; ColId++)
		{
			TexDescriptor TileDesc(InTileDesc);
			TileDesc.Name = TextureHelper::CreateTileName(InTileDesc.Name, RowId, ColId);

			TexPtr TileTex = std::make_shared<Tex>(TileDesc);
			TileTex->InitRT();

			/// prepare over here
			//DeviceBuffer_FXPtr FXBuffer = std::make_shared<DeviceBuffer_FX>(Device_FX::Get(), TileTex, TileDesc.ToBufferDescriptor(), (CHashPtr)nullptr);
			BlobUPtr TileBlob = std::make_unique<Blob>(Device_FX::Get()->CreateFromTex(TileTex, bInitRaw));
			Tiles[RowId * NumCols + ColId] = std::move(TileBlob);
		}
	}
}
