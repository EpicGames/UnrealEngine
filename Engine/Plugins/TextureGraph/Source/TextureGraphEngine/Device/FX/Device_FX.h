// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Data/Blob.h"
#include "Device/Device.h"
#include "DeviceBuffer_FX.h"
#include "RenderTargetPool.h"
#include "Engine/TextureRenderTarget2D.h"
#include "UObject/GCObject.h"
#include <list>
#include <unordered_map>

#define UE_API TEXTUREGRAPHENGINE_API

typedef cti::continuable<TexPtr>	AsyncTexPtr;
class RenderMaterial_FX;
typedef std::shared_ptr<RenderMaterial_FX> RenderMaterial_FXPtr;
typedef std::shared_ptr<class TexArray>	TexArrayPtr;
class UTextureRenderTarget;
class IRendererModule;

struct DrawTilesSettings
{
	T_Tiles<FIntPoint>				Position;						/// The position where to put the tiles
	T_Tiles<FIntPoint>				Size;							/// The size of the tiles
};

class Device_FX : public Device,  public FGCObject
{
private: 
	static UE_API size_t					s_maxRenderBatch;

	typedef std::list<TexPtr>		TextureNodeList;

	mutable FCriticalSection		GCLock;							/// Lock to the GC lists
	TextureNodeList					GCTargetTextures;				/// Textures that are GC targets right now

	typedef std::list<TObjectPtr<UTextureRenderTarget2D>> RTList;
	typedef std::unordered_map<HashType, RTList*> RenderTargetCache;

	mutable FCriticalSection		GC_RTCache;						/// Lock to the GC lists

	RenderTargetCache				RTCache;						/// The render target cache that we are currently maintaining
	RenderTargetCache				RTArrayCache;					/// The render target array cache that we are currently maintaining

	RTList							RTUsed;							/// Used render targets

	IRendererModule*				RendererModule;					/// Useful rendering functionalities wrapped in this module

	UE_API void							GCTextures();
	UE_API virtual void					Free() override;
	UE_API virtual void					ClearCache() override;

	//////////////////////////////////////////////////////////////////////////
	/// FGCObject
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual void					AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual FString					GetReferencerName() const override;

	UE_API void							FreeCacheInternal(RenderTargetCache& TargetRTCache);
	UE_API void							FreeRTList(RTList& RTList);
	UE_API void							FreeRenderTarget(HashType HashValue, UTextureRenderTarget2D* RT);

	static UE_API UTextureRenderTarget2D*	CreateRenderTarget(const BufferDescriptor& Desc);
	UE_API AsyncDeviceBufferRef			SplitToTiles_Internal(const CombineSplitArgs& SplitArgs);

	UE_API int32							AllocateRTArrayResource(TexArrayPtr TexArrayObj);
	UE_API int32							AllocateRTResource(TexPtr TextureObj);
	UE_API int32							InitRTResource(TexPtr TextureObj, UTextureRenderTarget* RT);

protected:
	UE_API virtual void					PrintStats() override;

public:
									UE_API Device_FX();
	UE_API virtual							~Device_FX() override;

	UE_API virtual cti::continuable<int32>	Use() const override;
	UE_API void							GetStatArray(TArray<FString>& ResourceArrayTiled, TArray<FString>& ResourceArrayUnTiled, TArray<FString>& TooltipListTiled, TArray<FString>& TooltipListUnTiled, FString& TotalStats);
	/// Must be called from the rendering thread
	UE_API DeviceBufferRef					CreateFromRT(UTextureRenderTarget2D* RT, const BufferDescriptor& Desc);
	UE_API DeviceBufferRef					CreateFromTexture(UTexture2D* TextureObj, const BufferDescriptor& Desc);
	UE_API DeviceBufferRef					CreateFromTex(TexPtr TextureObj, bool bInitRaw);
	UE_API DeviceBufferRef					CreateFromTexAndRaw(TexPtr TextureObj, RawBufferPtr RawObj);

	UE_API virtual AsyncDeviceBufferRef	CombineFromTiles(const CombineSplitArgs& CombineArgs) override;

	UE_API AsyncDeviceBufferRef			DrawTilesToBuffer_Deferred(DeviceBufferRef buffer, const T_Tiles<DeviceBufferRef>& Tiles, const DrawTilesSettings* DrawSettings = nullptr, FLinearColor* ClearColor = nullptr);
	UE_API AsyncDeviceBufferRef			FillTextureArray_Deferred(DeviceBufferRef buffer, const T_Tiles<DeviceBufferRef>& Tiles);

	UE_API virtual AsyncDeviceBufferRef	SplitToTiles(const CombineSplitArgs& splitArgs) override;

	UE_API TexPtr							AllocateRenderTarget(const BufferDescriptor& Desc);
	UE_API TexPtr							AllocateRenderTargetArray(const BufferDescriptor& Desc, int32 NumTilesX, int32 NumTilesY);



	UE_API virtual void					Update(float Delta) override;
	UE_API void							MarkForCollection(TexPtr TextureObj);
	virtual FString					Name() const override { return "Device_FX"; }
	UE_API virtual void					AddNativeTask(DeviceNativeTaskPtr task) override;

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE FRHICommandListImmediate& RHI() const
	{
		return FRHICommandListImmediate::Get();
	}

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API void						InitTiles_Texture(T_Tiles<BlobPtr>* Tiles, const BufferDescriptor& InTileDesc, bool bInitRaw);
	static UE_API void						InitTiles_RenderTargets(BlobUPtr* Tiles, size_t NumRows, size_t NumCols, const BufferDescriptor& InTileDesc, bool bInitRaw);

	static UE_API Device_FX*				Get();
};

#undef UE_API
