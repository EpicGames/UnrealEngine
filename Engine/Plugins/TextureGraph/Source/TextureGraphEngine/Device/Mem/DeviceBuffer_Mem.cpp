// Copyright Epic Games, Inc. All Rights Reserved.
#include "DeviceBuffer_Mem.h"
#include "Device/Device.h"
#include "Device_Mem.h"
#include "Transform/BlobTransform.h"

//////////////////////////////////////////////////////////////////////////
DeviceBuffer_Mem::DeviceBuffer_Mem(Device_Mem* Device, RawBufferPtr RawObj) 
	: DeviceBuffer(Device, RawObj)
{
}

DeviceBuffer_Mem::DeviceBuffer_Mem(Device_Mem* Device, BufferDescriptor Desc, CHashPtr HashValue)
	: DeviceBuffer(Device, Desc, HashValue)
{
}

DeviceBuffer_Mem::~DeviceBuffer_Mem()
{
}

bool DeviceBuffer_Mem::IsCompatible(Device* Dev) const
{
	return DeviceBuffer::IsCompatible(Dev);
}

void DeviceBuffer_Mem::Allocate() 
{
	if (RawData)
		return;

	size_t Length = Desc.Size();
	check(Length > 0);

	uint8* Data = new uint8 [Length];
	RawData = std::make_shared<RawBuffer>(Data, Length, Desc, nullptr);
	HashValue = RawData->Hash();
}

AsyncBufferResultPtr DeviceBuffer_Mem::TransferFrom(DeviceBufferRef& Source)
{
	return Source->Raw()
		.then([this](RawBufferPtr RawObj) mutable
		{
			RawData = RawObj;
			return RawData->LoadRawBuffer(true);
		})
		.then([this](RawBuffer*)
		{
			return std::make_shared<BufferResult>();
		});
}

RawBufferPtr DeviceBuffer_Mem::Raw_Now()
{
	if (!RawData)
		Allocate();

	return RawData;
}

size_t DeviceBuffer_Mem::MemSize() const
{
	if (!RawData)
		return 0;

	return RawData->GetLength();
}

AsyncBufferResultPtr DeviceBuffer_Mem::Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) 
{
	check(OwnerDevice == Device_Mem::Get());
	check(BindInfo.Dev == OwnerDevice);

	if (!RawData)
		Allocate();

	return cti::make_ready_continuable(std::make_shared<BufferResult>());
}

DeviceBuffer* DeviceBuffer_Mem::CreateFromRaw(RawBufferPtr RawObj)
{
	return new DeviceBuffer_Mem(static_cast<Device_Mem*>(OwnerDevice), RawObj);
}

DeviceBuffer* DeviceBuffer_Mem::CreateFromDesc(BufferDescriptor NewDesc, CHashPtr NewHashValue)
{
	return new DeviceBuffer_Mem(static_cast<Device_Mem*>(OwnerDevice), NewDesc, NewHashValue);
}
