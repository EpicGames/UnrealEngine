// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Device/DeviceBuffer.h"
#include "Data/RawBuffer.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Device;
class Device_Mem;

//////////////////////////////////////////////////////////////////////////
/// This is simply a wrapper around RawBuffer. Must be owned by the 
/// CPU device (though that can change in the future)
//////////////////////////////////////////////////////////////////////////
class DeviceBuffer_Mem : public DeviceBuffer
{
protected:
	UE_API virtual DeviceBuffer*			CreateFromRaw(RawBufferPtr RawObj) override;
	UE_API virtual DeviceBuffer*			CreateFromDesc(BufferDescriptor NewDesc, CHashPtr NewHashValue) override;

	UE_API void							Allocate();
	UE_API virtual AsyncBufferResultPtr	TransferFrom(DeviceBufferRef& Source) override;

public:
									UE_API DeviceBuffer_Mem(Device_Mem* Device, BufferDescriptor Desc, CHashPtr HashValue);
									UE_API DeviceBuffer_Mem(Device_Mem* Device, RawBufferPtr RawObj);
	UE_API virtual							~DeviceBuffer_Mem() override;

	//////////////////////////////////////////////////////////////////////////
	/// DeviceBuffer implementation
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual RawBufferPtr			Raw_Now() override;
	UE_API virtual size_t					MemSize() const override;
	UE_API virtual AsyncBufferResultPtr	Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) override;

	/// With base implementations
	//virtual void					Release() override;
	UE_API virtual bool					IsCompatible(Device* Dev) const override;

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
