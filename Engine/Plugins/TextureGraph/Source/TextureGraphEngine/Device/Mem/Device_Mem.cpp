// Copyright Epic Games, Inc. All Rights Reserved.
#include "Device_Mem.h"
#include "TextureGraphEngine.h"
#include "DeviceBuffer_Mem.h"
#include "Device/DeviceManager.h"
#include "Device/DeviceNativeTask.h"

Device_Mem::Device_Mem(DeviceType Type, DeviceBuffer* BufferFactory) : Device(Type, BufferFactory)
{
}

Device_Mem::Device_Mem() : Device_Mem(DeviceType::Mem, new DeviceBuffer_Mem(this, BufferDescriptor(), std::make_shared<CHash>(0xDeadBeef, true)))
{
	ShouldPrintStats = false;
	MaxThreads = FTaskGraphInterface::Get().GetNumWorkerThreads();

	auto MemConstants = FPlatformMemory::GetConstants();
	MaxMemory = (size_t)((float)MemConstants.TotalPhysical * 0.5f);
}

Device_Mem::~Device_Mem()
{
}

void Device_Mem::Update(float Delta)
{
	check(IsInGameThread());

	//if (_shouldPrintStats)
	//{
	//	UE_LOGF(LogDevice, Log, "===== BEGIN Device: Mem STATS =====");
	//}

	/// Update the native queue
	Device::Update(Delta);
}

void Device_Mem::AddNativeTask(DeviceNativeTaskPtr Task)
{
	/// Device_Mem tasks MUST always be async
	//check(Task->IsAsync());

	/// Cannot have game thread as the main execution thread because this can
	/// potentially block the game thread, where all the tasks execute
	check(Task->GetExecutionThread() != ENamedThreads::GameThread);

	Device::AddNativeTask(Task);
}

Device_Mem* Device_Mem::Get()
{
	Device* Dev = TextureGraphEngine::GetDeviceManager()->GetDevice(DeviceType::Mem);
	check(Dev);
	return static_cast<Device_Mem*>(Dev);
}
