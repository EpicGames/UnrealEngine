// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Device/Device.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Device_Mem : public Device
{
protected:
									UE_API Device_Mem(DeviceType Type, DeviceBuffer* BufferFactory);

public:
									UE_API Device_Mem();
	UE_API virtual							~Device_Mem() override;

	virtual FString					Name() const override { return "Device_Mem"; }
	UE_API virtual void					Update(float Delta) override;
	UE_API virtual void					AddNativeTask(DeviceNativeTaskPtr Task) override;

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API Device_Mem*				Get();
};

#undef UE_API
