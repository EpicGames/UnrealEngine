// Copyright Epic Games, Inc. All Rights Reserved.
#include "DeviceBuffer_MemCM.h"
#include "Device/Device.h"
#include "Device_MemCM.h"

//////////////////////////////////////////////////////////////////////////
DeviceBuffer_MemCM::DeviceBuffer_MemCM(Device_MemCM* Dev, RawBufferPtr InRawObj) 
	: DeviceBuffer_Mem(Dev, InRawObj)
{
	/// must be compressed
	check(RawData->IsCompressed());
}

DeviceBuffer_MemCM::DeviceBuffer_MemCM(Device_MemCM* Dev, BufferDescriptor InDesc, CHashPtr InHashValue)
	: DeviceBuffer_Mem(Dev, InDesc, InHashValue)
{
}

DeviceBuffer_MemCM::~DeviceBuffer_MemCM()
{
}

bool DeviceBuffer_MemCM::IsCompatible(Device* Dev) const
{
	/// Raw CPU buffers are compatible with every other device
	return true;
}

AsyncBufferResultPtr DeviceBuffer_MemCM::TransferFrom(DeviceBufferRef& Source)
{
	return Source->Raw()
		.then([this](RawBufferPtr RawObj) mutable
		{
			RawData = RawObj;
			return RawData->LoadRawBuffer(false);
		})
		.then([this, Source](RawBuffer*)
		{
			return RawData->Compress();
		})
		.then([this](RawBuffer*)
		{
			return std::make_shared<BufferResult>();
		});
}

RawBufferPtr DeviceBuffer_MemCM::Raw_Now()
{
	return DeviceBuffer_Mem::Raw_Now();
}

size_t DeviceBuffer_MemCM::MemSize() const
{
	if (!RawData)
		return 0;

	return RawData->GetLength();
}

AsyncBufferResultPtr DeviceBuffer_MemCM::Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) 
{
	/// These are not bindable
	check(false);
	throw std::runtime_error("DeviceBuffer_MemCM is unbindable");
}

DeviceBuffer* DeviceBuffer_MemCM::CreateFromRaw(RawBufferPtr raw)
{
	return new DeviceBuffer_MemCM(static_cast<Device_MemCM*>(OwnerDevice), raw);
}

DeviceBuffer* DeviceBuffer_MemCM::CreateFromDesc(BufferDescriptor InDesc, CHashPtr InHashValue)
{
	return new DeviceBuffer_MemCM(static_cast<Device_MemCM*>(OwnerDevice), InDesc, InHashValue);
}
