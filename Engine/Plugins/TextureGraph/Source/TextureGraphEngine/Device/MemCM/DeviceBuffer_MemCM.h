// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Device/Mem/DeviceBuffer_Mem.h"
#include "Data/RawBuffer.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Device;
class Device_MemCM;

//////////////////////////////////////////////////////////////////////////
/// This is simply a wrapper around RawBuffer. Must be owned by the 
/// CPU device (though that can change in the future)
//////////////////////////////////////////////////////////////////////////
class DeviceBuffer_MemCM : public DeviceBuffer_Mem
{
protected:
	UE_API virtual DeviceBuffer*			CreateFromRaw(RawBufferPtr InRawObj) override;
	UE_API virtual DeviceBuffer*			CreateFromDesc(BufferDescriptor InDesc, CHashPtr InHashValue) override;

	UE_API virtual AsyncBufferResultPtr	TransferFrom(DeviceBufferRef& Source) override;
public:
									UE_API DeviceBuffer_MemCM(Device_MemCM* Dev, BufferDescriptor InDesc, CHashPtr InHashValue);
									UE_API DeviceBuffer_MemCM(Device_MemCM* Dev, RawBufferPtr InRawObj);
	UE_API virtual							~DeviceBuffer_MemCM() override;

	//////////////////////////////////////////////////////////////////////////
	/// DeviceBuffer implementation
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual RawBufferPtr			Raw_Now() override;
	UE_API virtual size_t					MemSize() const override;
	UE_API virtual AsyncBufferResultPtr	Bind(const BlobTransform* Transform, const ResourceBindInfo& BindInfo) override;

	/// With base implementations
	//virtual void					Release() override;
	UE_API virtual bool					IsCompatible(Device* Dev) const override;

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
