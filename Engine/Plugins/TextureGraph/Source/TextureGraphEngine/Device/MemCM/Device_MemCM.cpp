// Copyright Epic Games, Inc. All Rights Reserved.
#include "Device_MemCM.h"
#include "TextureGraphEngine.h"
#include "DeviceBuffer_MemCM.h"
#include "Device/DeviceManager.h"

Device_MemCM::Device_MemCM() : Device_Mem(DeviceType::MemCompressed, new DeviceBuffer_MemCM(this, BufferDescriptor(), std::make_shared<CHash>(0xDeadBeef, true)))
{
	ShouldPrintStats = false;
	MaxThreads = FTaskGraphInterface::Get().GetNumWorkerThreads();

	auto memConstants = FPlatformMemory::GetConstants();
	MaxMemory = (size_t)((float)memConstants.TotalPhysical * 0.5f);

	MinLastUsageDuration = DefaultMinLastUsageDuration * 10;
}

Device_MemCM::~Device_MemCM()
{
}

void Device_MemCM::Update(float Delta)
{
	check(IsInGameThread());

	//if (_shouldPrintStats)
	//{
	//	UE_LOGF(LogDevice, Log, "===== BEGIN Device: MemCM STATS =====");
	//}

	Device::Update(Delta);
}

Device_MemCM* Device_MemCM::Get()
{
	Device* dev = TextureGraphEngine::GetDeviceManager()->GetDevice(DeviceType::MemCompressed);
	check(dev);
	return static_cast<Device_MemCM*>(dev);
}
