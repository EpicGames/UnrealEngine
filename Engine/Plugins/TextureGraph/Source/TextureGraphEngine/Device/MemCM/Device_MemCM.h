// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Device/Mem/Device_Mem.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Device_MemCM : public Device_Mem
{
public:
									UE_API Device_MemCM();
	UE_API virtual							~Device_MemCM() override;

	virtual FString					Name() const override { return "Device_MemCM"; }
	UE_API virtual void					Update(float Delta) override;

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API Device_MemCM*			Get();
};

#undef UE_API
