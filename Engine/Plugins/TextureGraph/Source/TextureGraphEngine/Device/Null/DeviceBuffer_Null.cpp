// Copyright Epic Games, Inc. All Rights Reserved.
#include "DeviceBuffer_Null.h"
#include "Device_Null.h"

DeviceBuffer_Null::DeviceBuffer_Null(Device_Null* device, BufferDescriptor desc, CHashPtr hash) 
	: DeviceBuffer(device, desc, hash)
{
}

DeviceBuffer_Null::~DeviceBuffer_Null()
{
}

DeviceBuffer* DeviceBuffer_Null::CreateFromRaw(RawBufferPtr rawBuffer)
{
	/// This is unsupported
	return nullptr;
}

AsyncBufferResultPtr DeviceBuffer_Null::TransferFrom(DeviceBufferRef& source)
{
	/// Nothing to do here ... buffer should automatically get deleted (this should technically never get called anyway)
	return cti::make_ready_continuable(std::make_shared<BufferResult>());
}

DeviceBuffer* DeviceBuffer_Null::CreateFromDesc(BufferDescriptor desc, CHashPtr hash)
{
	return new DeviceBuffer_Null(static_cast<Device_Null*>(OwnerDevice), desc, hash);
}

RawBufferPtr DeviceBuffer_Null::Raw_Now() 
{
	return nullptr;
}

size_t DeviceBuffer_Null::MemSize() const
{
	return 0;
}

AsyncBufferResultPtr DeviceBuffer_Null::Bind(const BlobTransform* transform, const ResourceBindInfo& bindInfo) 
{
	return cti::make_ready_continuable(std::make_shared<BufferResult>());
}

bool DeviceBuffer_Null::IsValid() const
{
	return false;
}

bool DeviceBuffer_Null::IsNull() const
{
	return true;
}

bool DeviceBuffer_Null::IsTransient() const
{
	return true;
}
