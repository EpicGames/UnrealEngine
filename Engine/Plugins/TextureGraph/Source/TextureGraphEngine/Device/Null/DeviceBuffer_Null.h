// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Device/DeviceBuffer.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Device_Null;

class DeviceBuffer_Null : public DeviceBuffer
{
protected:
	UE_API virtual DeviceBuffer*			CreateFromRaw(RawBufferPtr rawBuffer) override;
	UE_API virtual DeviceBuffer*			CreateFromDesc(BufferDescriptor desc, CHashPtr hash) override;

	UE_API virtual AsyncBufferResultPtr	TransferFrom(DeviceBufferRef& source);

public:
									UE_API DeviceBuffer_Null(Device_Null* device, BufferDescriptor desc, CHashPtr hash);
	UE_API virtual							~DeviceBuffer_Null();

	UE_API virtual RawBufferPtr			Raw_Now();
	UE_API virtual size_t					MemSize() const;
	UE_API virtual AsyncBufferResultPtr	Bind(const BlobTransform* transform, const ResourceBindInfo& bindInfo) override;
	UE_API virtual bool					IsValid() const;
	UE_API virtual bool					IsNull() const;
	UE_API virtual bool					IsTransient() const override;
};

#undef UE_API
