// Copyright Epic Games, Inc. All Rights Reserved.
#include "Device_Null.h"
#include "TextureGraphEngine.h"
#include "DeviceBuffer_Null.h"
#include "Device/DeviceManager.h"

Device_Null::Device_Null() : Device(DeviceType::Null, new DeviceBuffer_Null(this, BufferDescriptor(), std::make_shared<CHash>(0xDeadBeef, true)))
{
	ShouldPrintStats = false;
}

Device_Null::~Device_Null()
{
}

Device_Null* Device_Null::Get()
{
	Device* Dev = TextureGraphEngine::GetDeviceManager()->GetDevice(DeviceType::Null);
	check(Dev);
	return static_cast<Device_Null*>(Dev);
}

void Device_Null::Collect(DeviceBuffer* Buffer)
{
	delete Buffer;
}
