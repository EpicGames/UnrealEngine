// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Device/Device.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Device_Null : public Device
{
protected:

	UE_API virtual void					Collect(DeviceBuffer* Buffer) override;

public:
									UE_API Device_Null();
	UE_API virtual							~Device_Null() override;

	virtual FString					Name() const override { return "Device_Null"; }

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API Device_Null*				Get();
};

#undef UE_API
