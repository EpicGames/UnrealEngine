// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/LruCache.h"
#include "CoreMinimal.h"
#include "Data/RawBuffer.h"
#include "DeviceBuffer.h"
#include <memory>
#include <vector>
#include <queue>


class Blob;
class DeviceNativeTask;

//////////////////////////////////////////////////////////////////////////
struct DeviceNativeTask_CompareStrong
{
	bool operator()(const std::shared_ptr<DeviceNativeTask>& LHS, const std::shared_ptr<DeviceNativeTask>& RHS);
};

//////////////////////////////////////////////////////////////////////////
struct DeviceNativeTask_CompareWeak
{
	bool operator()(const std::weak_ptr<DeviceNativeTask>& InLHS, const std::weak_ptr<DeviceNativeTask>& InRHS);
};

//////////////////////////////////////////////////////////////////////////
template<typename PtrType, typename Comparer>
class GenericTaskPriorityQueue : public std::priority_queue<PtrType, std::vector<PtrType>, Comparer>
{
	using PriorityQueue = std::priority_queue<PtrType, std::vector<PtrType>, Comparer>;

private:
	/// We don't want people using push
	using PriorityQueue::push;

protected:
	mutable std::mutex		Mutex;
	std::condition_variable CondVar;

public:
	bool remove(PtrType value) 
	{
		std::unique_lock<std::mutex> lock(Mutex);

		auto it = std::find(this->c.begin(), this->c.end(), value);

		if (it != PriorityQueue::c.end())
		{
			PriorityQueue::c.erase(it);
			return true;
		}
		else
			return false;
	}

	void add(PtrType p)
	{
		std::lock_guard<std::mutex> lock(Mutex);
		PriorityQueue::push(p);
		CondVar.notify_one();
	}

	void lock()
	{
		Mutex.lock();
	}

	void unlock()
	{
		Mutex.unlock();
	}

	void reserve(size_t size)
	{
		std::unique_lock<std::mutex> lock(Mutex);
		PriorityQueue::c.reserve(size);
	}

	void resize(size_t size)
	{
		std::unique_lock<std::mutex> lock(Mutex);
		PriorityQueue::c.resize(size);
	}

	size_t size() const
	{
		std::unique_lock<std::mutex> lock(Mutex);
		return PriorityQueue::size();
	}

	const std::vector<PtrType>& accessInnerVector_Unsafe() const
	{
		return PriorityQueue::c;
	}

	PtrType waitNext()
	{
		std::unique_lock<std::mutex> lock(Mutex);

		/// Only start waiting if there's nothing there
		if (!PriorityQueue::size())
			CondVar.wait(lock);

		auto next = PriorityQueue::top();
		PriorityQueue::pop();

		return next;
	}

	void wait()
	{
		std::unique_lock<std::mutex> lock(Mutex);

		/// Only start waiting if there's nothing there
		if (!PriorityQueue::size())
			CondVar.wait(lock);
	}

	std::vector<PtrType> to_vector() const
	{
		std::unique_lock<std::mutex> lock(Mutex);
		PriorityQueue copy = *this;

		std::vector<PtrType> vec(copy.size());


		size_t index = 0;
		while (!copy.empty())
		{
			vec[index++] = copy.top();
			copy.pop();
		}

		return vec;
	}

	void addBeginAndEnd(PtrType begin, PtrType end)
	{
		std::unique_lock<std::mutex> lock(Mutex);

		PriorityQueue::c.insert(PriorityQueue::c.begin(), begin);
		PriorityQueue::c.push_back(end);
	}

	void clear()
	{
		std::unique_lock<std::mutex> lock(Mutex);
		PriorityQueue::c.clear();
	}

	std::vector<PtrType> to_vector_and_clear()
	{
		std::unique_lock<std::mutex> lock(Mutex);

		size_t index = 0;
		std::vector<PtrType> vec(PriorityQueue::size());

		while (!PriorityQueue::empty())
		{
			vec[index++] = PriorityQueue::top();
			PriorityQueue::pop();
		}

		return vec;
	}
};

typedef GenericTaskPriorityQueue<std::shared_ptr<DeviceNativeTask>, DeviceNativeTask_CompareStrong>	DeviceNativeTask_PriorityQueueStrong;
typedef GenericTaskPriorityQueue<std::weak_ptr<DeviceNativeTask>, DeviceNativeTask_CompareWeak>	DeviceNativeTask_PriorityQueue;
