// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_ExportAsUAsset.h"
#include "2D/Tex.h"
#include "TextureGraphEngine.h"
#include "Model/Mix/Mix.h"

#include "FxMat/MaterialManager.h"
#include "Device/Mem/Device_Mem.h"
#include "Data/Blobber.h"
#include "Job/Scheduler.h"
#include "Transform/Utility/T_MipMap.h"
#include "Transform/Utility/T_MinMax.h"
#include <AssetRegistry/AssetRegistryModule.h>

#if WITH_EDITOR
#include "Editor.h"
#endif

#include "CoreGlobals.h"
#include "Framework/Application/SlateApplication.h"

//////////////////////////////////////////////////////////////////////////
//// Exporting as UAsset job
//////////////////////////////////////////////////////////////////////////
FString GetJobName(FString name)
{
	return FString::Printf(TEXT("T_ExportAsUAsset_%s"), *name);
}

Job_ExportAsUAsset::Job_ExportAsUAsset(UMixInterface* Mix, int32 TargetId, FExportMapSettings InSetting, FString OutputFolder, UObject* InErrorOwner /* = nullptr*/, uint16 Priority /*= (uint16)E_Priority::kHigh*/, uint64 id /*= 0*/)
	: Job(Mix,TargetId, std::make_shared<Null_Transform>(Device_Mem::Get(), TEXT("T_ExportAsUAsset"), true, false), InErrorOwner, Priority)
	, Setting(InSetting)
	, OutFolder(OutputFolder)
{
	DeviceNativeTask::Name = GetJobName(InSetting.Name.ToString());
}

cti::continuable<int32> Job_ExportAsUAsset::PreExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type returnThread)
{
	auto hash = Job::Hash();
	BlobPtr cachedResult = TextureGraphEngine::GetBlobber()->Find(hash->Value());

	//TODO: Need to cache the Result so that the job do not execute every time with matching arguments
	if (cachedResult)
	{
		bIsCulled = true;
		bIsDone = true;
		MarkJobDone();

		return cti::make_ready_continuable(0);
	}

	return cti::make_ready_continuable(0);
}

cti::continuable<int32> Job_ExportAsUAsset::ExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type returnThread)
{
#if WITH_EDITOR
	if (TextureGraphEngine::BlockCommandlets())
		return cti::make_ready_continuable(0);

	UE_LOGF(LogData, Log, "Exporting As UAsset: %ls [Width: %d Height: %d]", *(Setting.Name.ToString()), Setting.Width, Setting.Height);

	if (IsDone())
	{
		MarkJobDone();
		return cti::make_ready_continuable(0);
	}

	RawBufferPtr rawPtr;

	return Job::BeginNative(RunInfo)
		.then([]()
		{
			return PromiseUtil::OnGameThread();
		})
		.then([this](int32)
		{
			return Setting.Map->CombineTiles(false,false);
		})
		.then([this](BufferResultPtr bufferPtr)
		{
			UE_LOGF(LogData, Log, "[Export As UAsset - %ls] Calling Export ...", *Setting.Name.ToString());
			DeviceBufferRef Buffer = Setting.Map->GetBufferRef();
			return Buffer->Raw();
		})
		.then([this](RawBufferPtr RawObj)
		{
			FName fileName = Setting.Name;

			check(RawObj->GetData());

			UE_LOGF(LogData, Display, "   - Writing RawObj Buffer to filename: %ls", *fileName.ToString());
			return TextureExporter::ExportRawAsUAsset(RawObj, Setting, OutFolder, fileName);
		})
		.then([this](int32) mutable
		{
			Setting.OnDone.ExecuteIfBound(Setting);
			//promise.set_value(0);
			UE_LOGF(LogData, Display, "Exporting finished");
			return EndNative();
		})
		.then([this](JobResultPtr)
		{
			SetPromise(0);
			return 0;
		})
		.fail([this](std::exception_ptr e) mutable
		{
			UE_LOGF(LogData, Log, "[Exporting As UAsset - %ls] Promise failure!", *Setting.Name.ToString());
			Setting.OnDone.ExecuteIfBound(Setting);
			EndNative();
			return -1;
		});
#else
	return cti::make_ready_continuable(0);
#endif 
}

T_ExportAsUAsset::T_ExportAsUAsset()
{
}

T_ExportAsUAsset::~T_ExportAsUAsset()
{
}

void T_ExportAsUAsset::ExportAsUAsset(MixUpdateCyclePtr cycle, int32 targetId, FExportMapSettings exportMapSetting, FString outFolder)
{
	std::unique_ptr<Job_ExportAsUAsset> job = std::make_unique<Job_ExportAsUAsset>(cycle->GetMix(),0, exportMapSetting, outFolder);
	
	job->AddArg(ARG_STRING(outFolder, "OutputFolder"));
	job->AddArg(ARG_STRING(exportMapSetting.Name.ToString(), "Name"));
	job->AddArg(ARG_INT(exportMapSetting.Width, "Width"));
	job->AddArg(ARG_INT(exportMapSetting.Height, "Height"));

	cycle->AddJob(targetId, std::move(job));
}

