// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h" 
#include "Job/Job.h"
#include "TextureExporter.h"

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
class Job_ExportAsUAsset : public Job
{
private:
	FExportMapSettings				Setting;
	FString							OutFolder;

public:
									UE_API Job_ExportAsUAsset(UMixInterface* Mix, int32 TargetId, FExportMapSettings InSetting, FString OutputFolder, UObject* InErrorOwner = nullptr, uint16 Priority = (uint16)E_Priority::kHigh, uint64 id = 0);

protected:
	virtual void					GetDependencies(JobPtrVec& prior, JobPtrVec& after, JobRunInfo runInfo) override { RunInfo = runInfo; Prev.clear(); }
	virtual AsyncPrepareResult		PrepareTargets(JobBatch* batch) override { return cti::make_ready_continuable(0); }
	virtual AsyncPrepareResult		PrepareResources(JobBatch* batch) override { return cti::make_ready_continuable(0); }

	virtual int32					Exec() override { return 0; }
	virtual cti::continuable<int32>	BindArgs_All(JobRunInfo runInfo) override { return cti::make_ready_continuable(0); }
	virtual cti::continuable<int32>	UnbindArgs_All(JobRunInfo runInfo) override { return cti::make_ready_continuable(0); }
	virtual bool					CanHandleTiles() const override { return false; }

	UE_API virtual cti::continuable<int32>	PreExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type returnThread) override;
	UE_API virtual cti::continuable<int32>	ExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type returnThread) override;
};

class T_ExportAsUAsset
{
public:
									UE_API T_ExportAsUAsset();
	UE_API virtual							~T_ExportAsUAsset();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API void						ExportAsUAsset(MixUpdateCyclePtr cycle, int32 targetId, FExportMapSettings exportMap, FString outFolder);
};

#undef UE_API
