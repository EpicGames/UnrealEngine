// Copyright Epic Games, Inc. All Rights Reserved.
#include "TextureExporter.h"
#include "2D/Tex.h"
#include "Async/ParallelFor.h"
#include "Data/TiledBlob.h"
#include "Job/JobBatch.h"
#include "Job/Scheduler.h"
#include "Misc/Paths.h"
#include "TextureGraphEngine.h"
#include "Model/Mix/MixSettings.h"
#include "Model/Mix/MixUpdateCycle.h"
#include "T_ExportAsUAsset.h"
#include "UObject/SavePackage.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include <AssetRegistry/AssetRegistryModule.h>
#include <Engine/Texture2D.h>
#include "2D/TextureHelper.h"
#include "TextureResource.h"
#if WITH_EDITOR
#include "FileHelpers.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#endif
#include <Misc/PackageName.h>
#include "Misc/FileHelper.h"
#include "TextureCompiler.h"

DEFINE_LOG_CATEGORY(ExportLogs);
bool TextureExporter::IsPackageNameValid(FString Path, FString AssetName)
{
	return FPackageName::TryConvertLongPackageNameToFilename(Path, AssetName);
}

bool TextureExporter::IsFileNameValid(FName FileName, FText& Reason)
{
	FileName.IsValidObjectName(Reason);
	FFileHelper::IsFilenameValidForSaving(FileName.ToString(), Reason);

	return Reason.IsEmpty();
}

bool TextureExporter::IsFolderPathValid(FString FolderPath, FText& Reason)
{
	FName::IsValidXName(FolderPath, INVALID_OBJECTPATH_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS, &Reason);
	
	if (!Reason.IsEmpty())
	{
		Reason = FText::FromString(Reason.ToString().Replace(TEXT("Name"), TEXT("Path")));
	}

	FPaths::ValidatePath(FolderPath, &Reason);
	return Reason.IsEmpty();
}

bool TextureExporter::IsFilePathValid(const FName InFileName, const FName InFolderPath, FString& OutErrors)
{
	FText FileNameError;
	IsFileNameValid(InFileName, FileNameError);
		
	if(!FileNameError.IsEmpty())
	{
		FString Message = "Invalid file name. " + FileNameError.ToString();
		OutErrors += Message;
	}
		
	FText FullPathError;
	const FString AssetName = FString::Printf(TEXT("/%s.%s"), *InFileName.ToString(), *InFileName.ToString());
	const FString FullPath = FPaths::Combine(InFolderPath.ToString(), AssetName);

#if WITH_EDITOR
	if(!AssetViewUtils::IsValidObjectPathForCreate(FullPath, FullPathError, true))
	{
		FString Message = "Invalid file path. " + FullPathError.ToString();
		OutErrors += Message;
	}
#endif
	
	FText PathValidError;
	IsFolderPathValid(InFolderPath.ToString(), PathValidError);
		
	if(!PathValidError.IsEmpty())
	{
		FString Message = "Invalid folder path Error: " + PathValidError.ToString();
		OutErrors += Message;
	}

	return OutErrors.IsEmpty();
}
AsyncInt TextureExporter::ExportAllMapsAsUAsset(UMixInterface* MixObj, const FString& OutputFolder, TSharedRef<FExportSettings> SettingsRef)
{
	FInvalidationDetails Details;
	Details.Mix = MixObj;
	Details.All();
	JobBatchPtr Batch = JobBatch::Create(Details);
	MixUpdateCyclePtr Cycle = Batch->GetCycle();
	const ExportMaps Maps = SettingsRef->ExportPreset;
	for (const auto& ExportMap : Maps)
	{
		auto& MapSettings = ExportMap.second;
		FString Path = MapSettings.Path;
		if (ExportMap.second.UseOverridePath)
		{
			Path = OutputFolder;
		}
		T_ExportAsUAsset::ExportAsUAsset(Cycle, 0, ExportMap.second, Path);
	}
	TextureGraphEngine::GetScheduler()->AddBatch(Batch);
	return cti::make_continuable<int32>([Batch, SettingsRef](auto&& Promise) mutable
	{
		Batch->OnDone([SettingsRef, FWD_PROMISE(Promise)](JobBatch*) mutable
		{
			Promise.set_value(SettingsRef->MapsExported);
		});
	});
}
AsyncInt TextureExporter::ExportRawAsUAsset(RawBufferPtr RawObj,const FExportMapSettings& Setting, const FString& CompletePath, const FName& TextureName)
{
	FText PathError;
	FPaths::ValidatePath(CompletePath, &PathError);
	if (CompletePath.IsEmpty())
	{
		UE_LOGF(LogTexture, Warning, "Provide path is empty!");
		return cti::make_ready_continuable(-1);
	}
	else if (!PathError.IsEmpty())
	{
		UE_LOGF(LogTexture, Warning, "Invalid file path provided: %ls", *(PathError.ToString()));
		return cti::make_ready_continuable(-1);
	}
		
	if (!RawObj || RawObj->GetData() == nullptr)
	{
		UE_LOGF(LogTexture, Warning, "Can't export with invalid RawBuffer: %ls", (RawObj != nullptr) ? *RawObj->GetName() : TEXT("nullptr"));
		return cti::make_ready_continuable(-1);
	}
		
	return cti::make_continuable<int32>([RawObj,Setting, CompletePath, TextureName](auto Promise) mutable
	{
		
		const EPixelFormat PixelFormat = BufferDescriptor::BufferPixelFormat(RawObj->GetDescriptor().Format, RawObj->GetDescriptor().ItemsPerPoint);
		const ETextureSourceFormat SourceFormat = TextureHelper::GetTextureSourceFormat(RawObj->GetDescriptor().Format, RawObj->GetDescriptor().ItemsPerPoint);
		FString PackageName = FPaths::Combine(CompletePath, TextureName.ToString());
		FString SuggestedName = TextureName.ToString();
		FPackagePath PackagePath;
#if WITH_EDITOR
		bool bCanOverriteTexture = true;

		// Get the Asset Registry Module
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		// Get all asset data in the Content Browser
		TArray<FAssetData> AssetData;
		AssetRegistryModule.Get().GetAssetsByPackageName(FName(*PackageName), AssetData);

		if (AssetData.Num() > 0)
		{
			bool bIsTextureAsset = false;
			FString AssetType = "";
			// Iterate through the asset data
			for (const FAssetData& Asset : AssetData)
			{
				if (Asset.IsInstanceOf(UTexture::StaticClass()))
				{
					bIsTextureAsset = true;
					break;
				}
			}
			// It is an error if we are trying to replace an object of a different class
			if (!bIsTextureAsset)
			{
				bCanOverriteTexture = false;
				UE_LOGF(LogTexture, Warning, "Creating a Unique name for the output texture as another asset of different type %ls exists with same name %ls", *AssetType, *PackageName);
			}
		}
		//Using override paths when exporting from blueprints
		if (!Setting.OverwriteTextures || !bCanOverriteTexture)
		{
			//Helps creating a unique Name if a file with same name exists
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
			AssetToolsModule.Get().CreateUniqueAssetName(PackageName, TEXT(""), PackageName, SuggestedName);
		}

		const uint8* RawBufferData = RawObj->GetData();
		size_t RawBufferSize = RawObj->GetUnpaddedSize();
		UPackage* Package = CreatePackage( *PackageName);
		Package->FullyLoad();
		UTexture2D* NewTexture = nullptr;
		int32 WidthToUse = Setting.Width == 0 ? RawObj->Width() : Setting.Width;
		int32 HeightToUse = Setting.Height == 0 ? RawObj->Height() : Setting.Height;
		
		const int32 NumBlocksX = WidthToUse / GPixelFormats[PixelFormat].BlockSizeX;
		const int32 NumBlocksY = HeightToUse / GPixelFormats[PixelFormat].BlockSizeY;
		uint32 BufferSizeRequired = NumBlocksX * GPixelFormats[PixelFormat].BlockBytes * NumBlocksY;

		if (BufferSizeRequired > RawBufferSize)
		{
			UE_LOGF(LogTexture, Warning, "Export dimensions are invalid and require buffer size larger than is available. Expected to be less than: %zu [Asked: %d, %dx%d]. Clamping ...",
				RawBufferSize, BufferSizeRequired, WidthToUse, HeightToUse);
			WidthToUse = RawObj->Width();
			HeightToUse = RawObj->Height();
		}

		if ((WidthToUse % GPixelFormats[PixelFormat].BlockSizeX) == 0 &&
			(HeightToUse % GPixelFormats[PixelFormat].BlockSizeY) == 0)
		{
			NewTexture = NewObject<UTexture2D>(Package, FName(SuggestedName), RF_Public | RF_Standalone | RF_MarkAsRootSet);
		
			// if there is padding, we need to remove that
			if (RawObj->IsPadded())
			{
				FTexturePlatformData* PlatformData = NewTexture->GetPlatformData();
				if (!PlatformData)
				{	
					PlatformData = new FTexturePlatformData();
					NewTexture->SetPlatformData(PlatformData);
				}
				PlatformData->SizeX = WidthToUse;
				PlatformData->SizeY = HeightToUse;
				PlatformData->SetNumSlices(1);
				PlatformData->PixelFormat = PixelFormat;
   
				// Get First MipMap
				//
				FTexture2DMipMap* Mip;
				if (PlatformData->Mips.IsEmpty())
				{
					Mip = new FTexture2DMipMap(0, 0);
					PlatformData->Mips.Add(Mip);
				}
				else
				{
					Mip = &PlatformData->Mips[0];
        
				}
				Mip->SizeX = WidthToUse;
				Mip->SizeY = HeightToUse;
				Mip->SizeZ = 1;
				
				const size_t DestSize = RawObj->GetUnpaddedSize();
				
				Mip->BulkData.Lock(LOCK_READ_WRITE);
				uint8* TextureData = Mip->BulkData.Realloc(DestSize);
				RawObj->CopyUnpaddedBytes(TextureData);
				Mip->BulkData.Unlock();
				
				RawBufferData = TextureData;
			}
		}
		else
		{
			UE_LOGF(LogTexture, Warning, "Cannot Export specified Pixel Format %d as UTexture2D", PixelFormat);
			Promise.set_value(-1);
			return;
		}
		
		NewTexture->SRGB = Setting.IsSRGB;
		NewTexture->LODGroup = Setting.LODGroup;
		NewTexture->CompressionSettings = Setting.Compression;

		NewTexture->Source.Init(WidthToUse, HeightToUse, 1,1, SourceFormat, RawBufferData);
		NewTexture->UpdateResource();
		FAssetRegistryModule::AssetCreated(NewTexture);
		Package->FullyLoad();
		FTextureCompilingManager::Get().FinishCompilation({NewTexture});
		
		bool Success = true;
		if (Setting.bSave)
		{
			Success = UEditorLoadingAndSavingUtils::SavePackages({ Package }, false);
		}
		else
		{
			Package->SetDirtyFlag(true);
		}

		Promise.set_value(Success);
#else
		Promise.set_value(-1);
#endif
	});
}
AsyncInt TextureExporter::ExportAsUAsset(UMixInterface* MixObj, TSharedRef<FExportSettings> ExportSettings, FString CurrentPath)
{
	return ExportAllMapsAsUAsset(MixObj, CurrentPath, ExportSettings);
}
FExportMapSettings TextureExporter::GetExportSettingsForTarget(FExportSettings& ExportSettings, TiledBlobPtr BlobObj, FName Name)
{
	if (TextureGraphEngine::BlockCommandlets())
		return FExportMapSettings();
	ExportSettings.PresetName = "ExportAsUAssetPreset";
	
	FExportMapSettings TargetMap;
	TargetMap.Name = Name;
	TargetMap.IsSRGB = BlobObj->GetDescriptor().bIsSRGB;
	TargetMap.Map = BlobObj;
	TargetMap.Width = TargetMap.Map->GetWidth();
	TargetMap.Height = TargetMap.Map->GetHeight();
	TargetMap.Compression = TextureCompressionSettings::TC_Default;
	TargetMap.OnDone.BindRaw(&ExportSettings, &FExportSettings::OnMapExportDone);
	return TargetMap;
}
void TextureExporter::ExportAsUAsset(UMixInterface* MixObj, FName AssetName, FString CurrentPath, TSharedRef<FExportSettings> Setting)
{
	ExportAllMapsAsUAsset(MixObj, CurrentPath, Setting);
}