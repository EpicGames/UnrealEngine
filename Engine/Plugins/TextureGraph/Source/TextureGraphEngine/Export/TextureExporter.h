// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "2D/TargetTextureSet.h"
#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Model/Mix/MixInterface.h"
THIRD_PARTY_INCLUDES_START
#include "continuable/continuable.hpp"
THIRD_PARTY_INCLUDES_END
#include <vector>

#if WITH_EDITOR
#include "Editor.h"
#endif

#include "CoreGlobals.h"
#include "Framework/Application/SlateApplication.h"

DECLARE_LOG_CATEGORY_EXTERN(ExportLogs, Log, Verbose);

struct FExportMapSettings
{
	bool bIsEnabled = true;
	FName Name;
	FString Path;
	bool UseOverridePath;
	bool OverwriteTextures;
	TiledBlobPtr Map;
	bool IsSRGB;
	bool UseMipMaps;
	int32 Width;
	int32 Height;
	TextureCompressionSettings Compression;
	TextureGroup LODGroup;
	bool bSave;

	DECLARE_DELEGATE_OneParam(OnExportMapDone, FExportMapSettings MapSettings);
	OnExportMapDone OnDone;
};

typedef std::vector<std::pair<FName, FExportMapSettings>> ExportMaps;

class FExportSettings
{
public:
	FString PresetName;
	ExportMaps ExportPreset;

	DECLARE_DELEGATE(OnExportDone);
	OnExportDone OnDone;

	int MapsExported = 0;
	
	int GetMapCount()
	{
		int MapCount = 0;

		
		if(ExportPreset.size() > 0)
		{
			for (const auto& Map : ExportPreset)
			{
				if(Map.second.bIsEnabled)
				{
					MapCount++;
				}
			}
		}

		return MapCount;
	}

	void Reset()
	{
		MapsExported = 0;
		ExportPreset.clear();
	}

	void OnMapExportDone(FExportMapSettings MapSetting)
	{
		if (TextureGraphEngine::BlockCommandlets())
			return;

		MapsExported++;
		if (GetMapCount() == MapsExported)
		{
			OnDone.ExecuteIfBound();
		}

		if(MapSetting.OnDone.IsBound())
			MapSetting.OnDone.Unbind();
	}
};

struct TextureExporter
{
	static TEXTUREGRAPHENGINE_API FExportMapSettings	GetExportSettingsForTarget(FExportSettings& ExportSettings, TiledBlobPtr BlobObj, FName Name);
	static TEXTUREGRAPHENGINE_API AsyncInt				ExportAllMapsAsUAsset(UMixInterface* MixObj, const FString& OutputFolder, TSharedRef<FExportSettings> SettingsRef);
	static TEXTUREGRAPHENGINE_API AsyncInt				ExportAsUAsset(UMixInterface* MixObj, TSharedRef<FExportSettings> SettingsRef, FString C);
	static TEXTUREGRAPHENGINE_API void					ExportAsUAsset(UMixInterface* MixObj, FName AssetName, FString CurrentPath, TSharedRef<FExportSettings> SettingsRef);
	static TEXTUREGRAPHENGINE_API AsyncInt				ExportRawAsUAsset(RawBufferPtr RawObj,const FExportMapSettings& Setting, const FString& CompletePath, const FName& TextureName);
	static TEXTUREGRAPHENGINE_API bool					IsPackageNameValid(FString Path, FString AssetName);
	static TEXTUREGRAPHENGINE_API bool 					IsFileNameValid(FName FileName, FText& Reason);
	static TEXTUREGRAPHENGINE_API bool 					IsFolderPathValid(FString FolderPath, FText& Reason);
	static TEXTUREGRAPHENGINE_API bool 					IsFilePathValid(const FName InFileName, const FName InFolderPath, FString& OutErrors);
};
