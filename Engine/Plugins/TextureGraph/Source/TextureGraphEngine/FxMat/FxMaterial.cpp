// Copyright Epic Games, Inc. All Rights Reserved.
#include "FxMaterial.h"
#include "CommonRenderResources.h"
#include "ShaderParameters.h" 
#include "Device/Device.h"
#include "EngineModule.h"
#include "RHIResourceUtils.h"
#include "TextureResource.h"

IMPLEMENT_GLOBAL_SHADER(VSH_Simple, "/Plugin/TextureGraph/Simple.usf", "VSH_Simple", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FSH_Simple, "/Plugin/TextureGraph/Simple.usf", "FSH_Simple", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_SimpleVT, "/Plugin/TextureGraph/SimpleVT.usf", "FSH_SimpleVT", SF_Pixel);

//template <>
//void SetupDefaultParameters(FSH_Simple::FParameters& params)
//{
//	FStandardSamplerStates_Setup(params.SamplerStates);
//}

//////////////////////////////////////////////////////////////////////////
FxMaterial::FxMaterial(TShaderRef<VSH_Base> VSH, TShaderRef<FSH_Base> FSH)
{
	FString VSHHashStr = VSH->GetHash().ToString();
	FString FSHHashStr = FSH->GetHash().ToString();

	CHashPtr VSHHash = std::make_shared<CHash>(DataUtil::Hash((uint8*)&VSHHashStr.GetCharArray(), VSHHashStr.Len() * sizeof(TCHAR)), true);
	CHashPtr FSHHash = std::make_shared<CHash>(DataUtil::Hash((uint8*)&FSHHashStr.GetCharArray(), FSHHashStr.Len() * sizeof(TCHAR)), true);
	CHashPtrVec Hashes = { VSHHash, FSHHash };
	HashValue = CHash::ConstructFromSources(Hashes);
}

FxMaterial::FxMaterial(TShaderRef<VSH_Base> VSH)
{
	FString VSHHashStr = VSH->GetHash().ToString();

	CHashPtr VSHHash = std::make_shared<CHash>(DataUtil::Hash((uint8*)&VSHHashStr.GetCharArray(), VSHHashStr.Len() * sizeof(TCHAR)), true);
	CHashPtrVec Hashes = { VSHHash };
	HashValue = CHash::ConstructFromSources(Hashes);
}

FxMaterial::FxMaterial(FString VSHHashStr, FString FSHHashStr)
{
	CHashPtr VSHHash = std::make_shared<CHash>(DataUtil::Hash((uint8*)&VSHHashStr.GetCharArray(), VSHHashStr.Len() * sizeof(TCHAR)), true);
	CHashPtr FSHHash = std::make_shared<CHash>(DataUtil::Hash((uint8*)&FSHHashStr.GetCharArray(), FSHHashStr.Len() * sizeof(TCHAR)), true);
	CHashPtrVec Hashes = { VSHHash, FSHHash };
	HashValue = CHash::ConstructFromSources(Hashes);
}


std::unique_ptr<FxMaterial::MemberLUT>& FxMaterial::GetParamsLUT()
{
	if (ParamsLUT)
		return ParamsLUT;

	ParamsLUT = std::make_unique<MemberLUT>();

	FxMetadataSet MetadataSet = GetMetadata();

	for (FxMetadata& MetaInfo : MetadataSet)
	{
		const TArray<FShaderParametersMetadata::FMember>& members = MetaInfo.Metadata->GetMembers();
		char* base = MetaInfo.StartAddress;

		for (size_t MemberIndex = 0; MemberIndex < members.Num(); MemberIndex++)
		{
			const FShaderParametersMetadata::FMember& Member = members[MemberIndex];
			FName memberName = Member.GetName();

			MemberInfo MemInfo = { Member, base + Member.GetOffset() };
			ParamsLUT->Add(memberName, MemInfo);
		}
	}

	return ParamsLUT;
}

FxMaterial::MemberInfo FxMaterial::GetMember(FName MemberName)
{
	auto& paramsLUT = GetParamsLUT();

	/// Now find the Member
	MemberInfo* MemInfo = paramsLUT->Find(MemberName);
	check(MemInfo != nullptr);

	return *MemInfo;
}

void FxMaterial::SetArrayTextureParameterValue(FName Name, const ArrayTexture& Value)
{
	auto MemInfo = GetMember(Name);
	const FShaderParametersMetadata::FMember& Member = MemInfo.Member;

	char* Arg = MemInfo.RawPtr;
	check(Arg);

	UE_LOGF(LogDevice, VeryVerbose, "Bind Array Texture: %ls", *Name.ToString());
	//memcpy(Arg, (const char*)&tex, sizeof(FRHITexture**));

	/// We actually bind all of these when we're actually rendering
	Textures.push_back({ Name, Arg, nullptr, Value });
}

void FxMaterial::SetTextureParameterValue(FName Name, const UTexture* Value)
{
	auto MemInfo = GetMember(Name);
	const FShaderParametersMetadata::FMember& Member = MemInfo.Member;

	char* Arg = MemInfo.RawPtr;
	check(Arg);

	UE_LOGF(LogDevice, VeryVerbose, "Bind Texture: %ls", *Name.ToString());
	//memcpy(Arg, (const char*)&tex, sizeof(FRHITexture**));

	/// We actually bind all of these when we're actually rendering
	Textures.push_back({ Name, Arg, Value });
}

void FxMaterial::BindTexturesForBlitting()
{
	for (auto& BoundTex : Textures)
	{
		if (BoundTex.Texture)
		{
			check(BoundTex.Texture->GetResource());

			FRHITexture* tex = BoundTex.Texture->GetResource()->TextureRHI;
			check(tex);

			FRHITexture* texture2DArray = tex->GetTexture2DArray();

			if (!texture2DArray)
			{
				memcpy(BoundTex.Arg, (const char*)&tex, sizeof(FRHITexture**));
			}
			else
			{
				memcpy(BoundTex.Arg, (const char*)&texture2DArray, sizeof(FRHITexture**));
			}
		}
		else if (BoundTex.tiles.size())
		{
			auto DestArg = BoundTex.Arg;
			for (const auto& ti : BoundTex.tiles)
			{
				FRHITexture* TextureRHI = ti->GetResource()->TextureRHI;
				memcpy(DestArg, (const char*)&TextureRHI, sizeof(FRHITexture**));
				DestArg += sizeof(FRHITexture**) ;
			}
		}
	}
}

void FxMaterial::SetScalarParameterValue(FName Name, float Value)
{
	auto MemInfo = GetMember(Name);
	const FShaderParametersMetadata::FMember& Member = MemInfo.Member;

	check(Member.GetNumColumns() == 1 && Member.GetNumRows() == 1 && Member.GetBaseType() == EUniformBufferBaseType::UBMT_FLOAT32);
	float* Arg = reinterpret_cast<float*>(MemInfo.RawPtr);

	check(Arg);
	*Arg = Value;
}

void FxMaterial::SetScalarParameterValue(FName Name, int32 Value)
{
	auto MemInfo = GetMember(Name);
	const FShaderParametersMetadata::FMember& Member = MemInfo.Member;

	check(Member.GetNumColumns() == 1 && Member.GetNumRows() == 1 && Member.GetBaseType() == EUniformBufferBaseType::UBMT_INT32);
	int* Arg = reinterpret_cast<int*>(MemInfo.RawPtr);

	check(Arg);
	*Arg = Value;
}

void FxMaterial::SetVectorParameterValue(FName Name, const FLinearColor& Value)
{
	auto MemInfo = GetMember(Name);
	const FShaderParametersMetadata::FMember& Member = MemInfo.Member;

	/// We only support RGBA/XYZW vectors right now. This is to avoid
	/// trying to write 4 float values to something that isn't the 
	/// correct size
	check(Member.GetNumColumns() <= 4 && Member.GetBaseType() == EUniformBufferBaseType::UBMT_FLOAT32);
	FLinearColor* Arg = reinterpret_cast<FLinearColor*>(MemInfo.RawPtr);

	check(MemInfo.RawPtr);
	memcpy(MemInfo.RawPtr, (const void*)&Value, sizeof(float) * Member.GetNumColumns());
}

void FxMaterial::SetVectorParameterValue(FName Name, const FIntVector4& Value)
{
	auto MemInfo = GetMember(Name);
	const FShaderParametersMetadata::FMember& Member = MemInfo.Member;

	/// We support int32 XYZW vectors in this function. This is to avoid
	/// trying to write 4 Int values to something that isn't the 
	/// correct size
	check(Member.GetNumColumns() <= 4 && Member.GetBaseType() == EUniformBufferBaseType::UBMT_INT32);
	FIntVector4* Arg = reinterpret_cast<FIntVector4*>(MemInfo.RawPtr);

	check(MemInfo.RawPtr);
	memcpy(MemInfo.RawPtr, (const void*)&Value, sizeof(float) * Member.GetNumColumns());
}

void FxMaterial::SetStructParameterValue(FName Name, const char* Value, size_t StructSize)
{
	size_t memberPosition = 0;
	auto MemInfo = GetMember(Name);
	char* Arg = reinterpret_cast<char*>(MemInfo.RawPtr);
	auto InputSize = MemInfo.Member.GetStructMetadata()->GetSize() ;
	check(StructSize == InputSize)
	memcpy(Arg, Value, InputSize);
}

void FxMaterial::SetArrayParameterValue(FName Name, const char* startAddress, size_t TypeSize, size_t ArraySize)
{
	auto MemInfo = GetMember(Name);
	auto Member = MemInfo.Member;

	char* Arg = MemInfo.RawPtr;
	check(Arg);
	uint32 MemberSize = Member.GetMemberSize();
	uint32 ElementSize = TypeSize;
	uint32 ExpandingRange = MemberSize / ElementSize;
	uint32 Skip = ExpandingRange > 0 ? ArraySize/ExpandingRange : 0;

	int ValueIndex = 0;
	for (uint32 i = 0; i < ExpandingRange;i += Skip)
	{
		check(ValueIndex < ArraySize)

		if(ValueIndex < ArraySize)
		{
			memcpy(Arg + (i * ElementSize), startAddress + (ValueIndex * ElementSize), ElementSize);
			ValueIndex++;
		}
	}
}

void FxMaterial::SetMatrixParameterValue(FName Name, const FMatrix& Value)
{
	auto MemInfo = GetMember(Name);
	const FShaderParametersMetadata::FMember& Member = MemInfo.Member;

	/// Must be the correct size and dimension
	check(Member.GetNumColumns() == 4 && Member.GetNumRows() == 4 && Member.GetBaseType() == EUniformBufferBaseType::UBMT_FLOAT32);
	FMatrix* Arg = reinterpret_cast<FMatrix*>(MemInfo.RawPtr);

	check(Arg);
	*Arg = Value;
}

bool FxMaterial::DoesMemberExist(FName MemberName)
{
	auto& ParamsLUTObj = GetParamsLUT();

	/// Now find the Member
	auto* MemInfo = ParamsLUTObj->Find(MemberName);
	return (MemInfo != nullptr);
}

//////////////////////////////////////////////////////////////////////////
void QuadScreenBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	//FVertexDeclarationElementList Elements;
	//uint16 Stride = sizeof(FFilterVertex);
	//Elements.Add(FVertexElement(0, STRUCT_OFFSET(FFilterVertex, Position), VET_Float4, 0, Stride));
	//Elements.Add(FVertexElement(0, STRUCT_OFFSET(FFilterVertex, UV), VET_Float2, 1, Stride));
	//VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);

	const FFilterVertex Vertices[] =
	{
		FFilterVertex { FVector4f(-1, 1, 0, 1), FVector2f(0, 0) },
		FFilterVertex { FVector4f( 1, 1, 0, 1), FVector2f(1, 0) },
		FFilterVertex { FVector4f(-1,-1, 0, 1), FVector2f(0, 1) },
		FFilterVertex { FVector4f( 1,-1, 0, 1), FVector2f(1, 1) },
	};

	VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("FxMaterial_VB"), EBufferUsageFlags::Static, MakeConstArrayView(Vertices));
}

//////////////////////////////////////////////////////////////////////////
TGlobalResource<QuadScreenBuffer> FxMaterial::GQuadBuffer;

void FxMaterial::InitPSO_Default(FGraphicsPipelineStateInitializer& PSO)
{
	PSO.BlendState = TStaticBlendState<>::GetRHI();
	PSO.RasterizerState = TStaticRasterizerState<>::GetRHI();
	PSO.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	PSO.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
	PSO.PrimitiveType = PT_TriangleStrip;
}

void FxMaterial::InitPSO_Default(FGraphicsPipelineStateInitializer& PSO, FRHIVertexShader* VSH, FRHIPixelShader* FSH)
{
	InitPSO_Default(PSO);
	PSO.BoundShaderState.VertexShaderRHI = VSH;
	PSO.BoundShaderState.PixelShaderRHI = FSH;
}

//////////////////////////////////////////////////////////////////////////
