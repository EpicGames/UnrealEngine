// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "3D/RenderMesh.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "CoreMinimal.h"
#include "GenerateMips.h"
#include "GlobalShader.h"
#include "Helper/DataUtil.h"
#include "PixelShaderUtils.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "RHIUtilities.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SamplerStates_FX.h"
#include "Serialization/JsonSerializerMacros.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "UniformBuffer.h"

#include "GlobalShader.h"
#include "SimpleElementShaders.h"
#include "ShaderParameterUtils.h"

#include <memory>

#define UE_API TEXTUREGRAPHENGINE_API


template <class SH_TypeParams> void SetupDefaultParameters(SH_TypeParams& params) {
}

//////////////////////////////////////////////////////////////////////////
class VSH_Base : public FGlobalShader
{
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& params)
	{
		return true;
	}

	VSH_Base() {}
	explicit VSH_Base(const ShaderMetaType::CompiledShaderInitializerType& initType) : FGlobalShader(initType) {}
};

//////////////////////////////////////////////////////////////////////////
class FSH_Base : public FGlobalShader
{
public:
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& params)
	{
		return true;
	}

	FSH_Base() {}
	explicit FSH_Base(const ShaderMetaType::CompiledShaderInitializerType& initType) : FGlobalShader(initType) {}
};

//////////////////////////////////////////////////////////////////////////
template <const int ThreadGroupSize_X = 16, const int ThreadGroupSize_Y = 16, const int ThreadGroupSize_Z = 1>
class CmpSH_Base : public FGlobalShader
{
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& params)
	{
		return true;
	}

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
		FGlobalShader::ModifyCompilationEnvironment(params, env);

		env.SetDefine(TEXT("THREADGROUPSIZE_X"), ThreadGroupSize_X);
		env.SetDefine(TEXT("THREADGROUPSIZE_Y"), ThreadGroupSize_Y);
		env.SetDefine(TEXT("THREADGROUPSIZE_Z"), ThreadGroupSize_Z);
	}

	CmpSH_Base() {}
	CmpSH_Base(const ShaderMetaType::CompiledShaderInitializerType& initType) : FGlobalShader(initType) {}

	FORCEINLINE constexpr FIntVector ThreadGroupSize() const { return FIntVector(ThreadGroupSize_X, ThreadGroupSize_Y, ThreadGroupSize_Z); }
};

////////////////////////////////////////////////////////////////////////////
class VSH_Simple : public VSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(VSH_Simple, UE_API);
	SHADER_USE_PARAMETER_STRUCT(VSH_Simple, VSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& params)
	{
		return true;
	}
};

////////////////////////////////////////////////////////////////////////////
class FSH_Simple : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_Simple, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_Simple, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& params)
	{
		return true;// IsFeatureLevelSupported(params.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
	}
};

//////////////////////////////////////////////////////////////////////////
class FSH_SimpleVT : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_SimpleVT, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_SimpleVT, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_SAMPLER(SamplerState, InTextureSampler)
		SHADER_PARAMETER_ARRAY(FUintVector4, VTPackedPageTableUniform, [2])
		SHADER_PARAMETER(FUintVector4, VTPackedUniform)
		SHADER_PARAMETER_SRV(Texture2D, InPhysicalTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, InPageTableTexture0)
		SHADER_PARAMETER_TEXTURE(Texture2D, InPageTableTexture1)
		//SHADER_PARAMETER(FVector4f, PackedParams)
		END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& params)
	{
		if (!IsFeatureLevelSupported(params.Platform, ERHIFeatureLevel::SM5))
		{
			return false;
		}

		if (IsConsolePlatform(params.Platform))
		{
			return false;
		}

		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
	}
};

//////////////////////////////////////////////////////////////////////////
class QuadScreenBuffer : public FVertexBuffer
{
	UE_API void							InitRHI(FRHICommandListBase& RHICmdList) override;
};

//////////////////////////////////////////////////////////////////////////
class FxMaterial
{
public:
	struct MemberInfo
	{
		const FShaderParametersMetadata::FMember& 
									Member;
		char*						RawPtr = nullptr;
	};

	typedef TMap<FName, MemberInfo> MemberLUT;

	struct FxMetadata
	{
		const FShaderParametersMetadata* 
									Metadata = nullptr;
		char*						StartAddress = nullptr;
	};

	typedef std::vector<FxMetadata> FxMetadataSet;

	using ArrayTexture = std::vector<const UTexture*>;

protected:
	struct BoundTextures
	{
		FName						Name;				/// The Name of the argument
		char*						Arg = nullptr;		/// The pointer in the arg struct
		const UTexture*				Texture = nullptr;	/// The underlying texture
		ArrayTexture				tiles;
	};

	std::unique_ptr<MemberLUT>		ParamsLUT;			/// Parameters lookup table
	CHashPtr						HashValue;			/// Hash for the FxMaterial

	std::vector<BoundTextures>		Textures;			/// Textures that are bound with this material. We bind them in the end

	UE_API std::unique_ptr<MemberLUT>&		GetParamsLUT();
	UE_API MemberInfo						GetMember(FName MemberName);

	UE_API void							BindTexturesForBlitting();

	/// Default constructor is only accessible to derived classes
	FxMaterial() = default;

public:

	UE_API FxMaterial(TShaderRef<VSH_Base> VSH, TShaderRef<FSH_Base> FSH);
	UE_API FxMaterial(TShaderRef<VSH_Base> VSH);
	UE_API FxMaterial(FString VSHHashStr, FString FSHHashStr);

	//////////////////////////////////////////////////////////////////////////
	/// Global static data and functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API TGlobalResource<QuadScreenBuffer> GQuadBuffer;
	static UE_API void						InitPSO_Default(FGraphicsPipelineStateInitializer& PSO);
	static UE_API void						InitPSO_Default(FGraphicsPipelineStateInitializer& PSO, FRHIVertexShader* VSH, FRHIPixelShader* FSH);

	//////////////////////////////////////////////////////////////////////////
	/// Must be overridden by the implementation
	//////////////////////////////////////////////////////////////////////////
	virtual							~FxMaterial() = default;
	
	virtual std::shared_ptr<FxMaterial>	
									Clone() = 0;
	virtual FxMetadataSet			GetMetadata() const = 0;
	virtual void					Blit(FRHICommandListImmediate& RHI, FRHITexture* Target, const RenderMesh* MeshObj, int32 TargetId, FGraphicsPipelineStateInitializer* PSO = nullptr) = 0;

	//////////////////////////////////////////////////////////////////////////
	/// Mimicing UMaterialInstanceDynamic
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual void					SetArrayTextureParameterValue(FName Name, const ArrayTexture& Value);
	UE_API virtual void					SetTextureParameterValue(FName Name, const UTexture* Value);
	UE_API virtual void					SetScalarParameterValue(FName Name, float Value);
	UE_API virtual void					SetScalarParameterValue(FName Name, int32 Value);
	UE_API virtual void					SetVectorParameterValue(FName Name, const FLinearColor& Value);
	UE_API virtual void					SetVectorParameterValue(FName Name, const FIntVector4& Value);
	UE_API virtual void					SetStructParameterValue(FName Name, const char* Value, size_t StructSize); 
	UE_API virtual void					SetArrayParameterValue(FName Name, const char* startAddress, size_t TypeSize, size_t ArraySize);
	UE_API virtual void					SetMatrixParameterValue(FName Name, const FMatrix& Value);
	UE_API virtual bool					DoesMemberExist(FName MemberName);
	
	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE CHashPtr			Hash() const { return HashValue; }
};

typedef std::shared_ptr<FxMaterial>	FxMaterialPtr;

//////////////////////////////////////////////////////////////////////////
template <typename CmpSH_Type>
class FxMaterial_Compute : public FxMaterial
{
public:
	// type Name for the permutation domain
	using CmpSHPermutationDomain		= typename CmpSH_Type::FPermutationDomain;

protected:
	static constexpr int				GDefaultNumThreadsXY = 1024;		/// Default number of threads
	FString								OutputId;							/// The ID of the output of the compute shader

	CmpSHPermutationDomain				PermutationDomain;					/// Compute shader Permutation Domain Value
	typename CmpSH_Type::FParameters	Params;								/// Params for the shader
	int									NumThreadsX = GDefaultNumThreadsXY;	/// How many X threads
	int									NumThreadsY = GDefaultNumThreadsXY;	/// How many Y threads
	int									NumThreadsZ = 1;					/// How many Z threads
	FUnorderedAccessViewRHIRef			UnorderedAccessView = nullptr;		/// The access view for the compute shader. If this is null then 
																			/// the Blit function will create a one at the time of blitting

public:
	explicit FxMaterial_Compute(FString InOutputId, const CmpSHPermutationDomain* InPermDomain = nullptr, 
	                            int InNumThreadsX = GDefaultNumThreadsXY, int InNumThreadsY = GDefaultNumThreadsXY, 
								int InNumThreadsZ= 1, FUnorderedAccessViewRHIRef InUnorderedAccessView = nullptr) 
		: FxMaterial()
		, OutputId(InOutputId)
		, NumThreadsX(InNumThreadsX)
		, NumThreadsY(InNumThreadsY)
		, NumThreadsZ(InNumThreadsZ)
		, UnorderedAccessView(InUnorderedAccessView)
	{
		if (InPermDomain)
			PermutationDomain = *InPermDomain;

		TShaderRef<CmpSH_Type> CSH = TShaderMapRef<CmpSH_Type>(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationDomain);
		FString CSHHashStr = CSH->GetHash().ToString();
		CHashPtr CSHHash = std::make_shared<CHash>(DataUtil::Hash((uint8*)&CSHHashStr.GetCharArray(), CSHHashStr.Len() * sizeof(TCHAR)), true);
		HashValue = CSHHash;
	}

	virtual std::shared_ptr<FxMaterial> Clone() override
	{
		return std::static_pointer_cast<FxMaterial>(std::make_shared<FxMaterial_Compute<CmpSH_Type>>(OutputId, &PermutationDomain, 
			NumThreadsX, NumThreadsY, NumThreadsZ, UnorderedAccessView));
	}

	FORCEINLINE typename CmpSH_Type::FParameters& GetParams() { return Params; }

	virtual FxMetadataSet GetMetadata() const override
	{
		return { { CmpSH_Type::FParameters::FTypeInfo::GetStructMetadata(), (char*)&Params } };
	}

	void SetUAVParameterValue(FName Name, FUnorderedAccessViewRHIRef UAV)
	{
		auto MemInfo = GetMember(Name);

		char* Arg = MemInfo.RawPtr;
		check(Arg);

		FUnorderedAccessViewRHIRef* UAVRef = (FUnorderedAccessViewRHIRef*)Arg;
		*UAVRef = UAV;
		Params.Result = UAV;
	}

	virtual void Blit(FRHICommandListImmediate& RHI, FRHITexture* Target, const RenderMesh* MeshObj, int32 TargetId, FGraphicsPipelineStateInitializer* PSO = nullptr) override
	{
		BindTexturesForBlitting();

		QUICK_SCOPE_CYCLE_COUNTER(STAT_ShaderPlugin_ComputeShader); // Used to gather CPU profiling data for the UE4 session frontend
		SCOPED_DRAW_EVENT(RHI, ShaderPlugin_Compute); // Used to profile GPU activity and add metadata to be consumed by for example RenderDoc

		//UnbindRenderTargets(RHI);

		if (!OutputId.IsEmpty())
		{
			FUnorderedAccessViewRHIRef RenderTargetUAV = UnorderedAccessView;

			if (!RenderTargetUAV)
			{
				RenderTargetUAV = RHI.CreateUnorderedAccessView(Target, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(Target));

				/// These access flags are taken from UE4 where and match what ERHIAccess::ERWNoBarrier was 
				/// defined as. The definition has been discarded in UE5
				ERHIAccess AccessFlags = ERHIAccess::UAVMask;

				RHI.Transition(FRHITransitionInfo(Target, ERHIAccess::Unknown, AccessFlags));
				//RHI.TransitionResource(EResourceTransitionAccess::ERWBarrier, EResourceTransitionPipeline::EGfxToCompute, rtUAV);
			}

			SetUAVParameterValue(FName(*OutputId), RenderTargetUAV);
		}

		TShaderMapRef<CmpSH_Type> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationDomain);
		FIntVector GroupSize = ComputeShader->ThreadGroupSize();

		FComputeShaderUtils::Dispatch(RHI, ComputeShader, Params, 
			FIntVector(
				FMath::DivideAndRoundUp(NumThreadsX, GroupSize.X),
				FMath::DivideAndRoundUp(NumThreadsY, GroupSize.Y), 
				FMath::DivideAndRoundUp(NumThreadsZ, GroupSize.Z)
			)
		);

		// UAV target has been rendered, transition to the default SRV state for read
		RHI.Transition(FRHITransitionInfo(Target, ERHIAccess::UAVMask, ERHIAccess::SRVMask));
	}
};

//////////////////////////////////////////////////////////////////////////
template <typename VSH_Type, typename FSH_Type>
class FxMaterial_Normal : public FxMaterial
{
public: 
	// type Name for the permutation domain
	using VSHPermutationDomain = typename VSH_Type::FPermutationDomain;
	using FSHPermutationDomain = typename FSH_Type::FPermutationDomain;

protected:
	VSHPermutationDomain			VSHPermDomain;		/// Vertex shader Permutation Domain Value
	FSHPermutationDomain			FSHPermDomain;		/// Fragment shader Permutation Domain Value
	typename VSH_Type::FParameters	VSHParams;			/// Params for the vertex shader
	typename FSH_Type::FParameters	FSHParams;			/// Params for the fragment shader
	//bool							_paramsSet = false;	/// Whether the params have been set or not

public:

	FxMaterial_Normal()
		: FxMaterial(	TShaderMapRef<VSH_Type>(GetGlobalShaderMap(GMaxRHIFeatureLevel), VSHPermutationDomain()),
						TShaderMapRef<FSH_Type>(GetGlobalShaderMap(GMaxRHIFeatureLevel), FSHPermutationDomain()))
	{
	}

	FxMaterial_Normal(const VSHPermutationDomain& InVSHPermutationDomain, const FSHPermutationDomain& InFSHPermutationDomain)
		: FxMaterial(	TShaderMapRef<VSH_Type>(GetGlobalShaderMap(GMaxRHIFeatureLevel), InVSHPermutationDomain),
						TShaderMapRef<FSH_Type>(GetGlobalShaderMap(GMaxRHIFeatureLevel), InFSHPermutationDomain)),
		VSHPermDomain(InVSHPermutationDomain),
		FSHPermDomain(InFSHPermutationDomain)
	{
	}

	FxMaterial_Normal(const typename FSH_Type::FParameters InFSHParams)
		: FxMaterial(TShaderMapRef<VSH_Type>(GetGlobalShaderMap(GMaxRHIFeatureLevel), VSHPermutationDomain()),
			TShaderMapRef<FSH_Type>(GetGlobalShaderMap(GMaxRHIFeatureLevel), FSHPermutationDomain())),
		FSHParams(InFSHParams)
	{
	}

	virtual std::shared_ptr<FxMaterial> Clone() override
	{
		return std::static_pointer_cast<FxMaterial>(std::make_shared<FxMaterial_Normal<VSH_Type, FSH_Type>>(VSHPermDomain, FSHPermDomain));
	}

	FORCEINLINE typename VSH_Type::FParameters& VSH_Params() { return VSHParams; }
	FORCEINLINE typename FSH_Type::FParameters& FSH_Params() { return FSHParams; }

	virtual FxMetadataSet GetMetadata() const override
	{
		return { 
			{ VSH_Type::FParameters::FTypeInfo::GetStructMetadata(), (char*)&VSHParams },
			{ FSH_Type::FParameters::FTypeInfo::GetStructMetadata(), (char*)&FSHParams } 
		};
	}

	virtual void Blit(FRHICommandListImmediate& RHI, FRHITexture* Target, const RenderMesh* MeshObj, int32 InTargetId, FGraphicsPipelineStateInitializer* InPSO = nullptr) override
	{
		BindTexturesForBlitting();

		//const typename FSH_Type::FParameters* params = reinterpret_cast<typename FSH_Type::FParameters*>(params_);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_ShaderPlugin_PixelShader); // Used to gather CPU profiling data for the UE4 session frontend
		SCOPED_DRAW_EVENT(RHI, ShaderPlugin_Pixel); // Used to profile GPU activity and add metadata to be consumed by for example RenderDoc

		//check(target->IsRenderTarget());

		// Target render target needs to transition to RTV for rendering
		RHI.Transition(FRHITransitionInfo(Target, ERHIAccess::Unknown, ERHIAccess::RTV));

		FRHIRenderPassInfo passInfo(Target, ERenderTargetActions::Clear_Store);
		RHI.BeginRenderPass(passInfo, TEXT("FxMaterial_Render"));


		//RHI.BindDebugLabelName(target, *target->GetName().ToString());
		auto shaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<VSH_Type> VSH(shaderMap, VSHPermDomain);
		TShaderMapRef<FSH_Type> FSH(shaderMap, FSHPermDomain);

		// Set the graphic pipeline state.
		FGraphicsPipelineStateInitializer PSO;

		if (!InPSO)
			InitPSO_Default(PSO, VSH.GetVertexShader(), FSH.GetPixelShader());
		else
			PSO = *InPSO;

		if (MeshObj)
		{
			MeshObj->Init_PSO(PSO);
		}

		//PSO.BoundShaderState.VertexShaderRHI = VSH.GetVertexShader();
		//PSO.BoundShaderState.PixelShaderRHI = FSH.GetPixelShader();

		RHI.ApplyCachedRenderTargets(PSO);
		SetGraphicsPipelineState(RHI, PSO, 0);
		SetupDefaultParameters<typename FSH_Type::FParameters>(FSHParams);
		SetShaderParameters(RHI, VSH, VSH.GetVertexShader(), VSHParams);
		SetShaderParameters(RHI, FSH, FSH.GetPixelShader(), FSHParams);

		if (!MeshObj)
		{
			RHI.SetStreamSource(0, GQuadBuffer.VertexBufferRHI, 0);
			RHI.DrawPrimitive(0, 2, 1);
		}
		else
		{
			MeshObj->Render_Now(RHI, InTargetId);
		}
		
		RHI.EndRenderPass();

		// Target render target has been rendered, transition to the default SRV state for read
		RHI.Transition(FRHITransitionInfo(Target, ERHIAccess::RTV, ERHIAccess::SRVMask));
	}
};

typedef FxMaterial_Normal<VSH_Simple, FSH_Simple>	Fx_FullScreenCopy;
typedef FxMaterial_Normal<VSH_Simple, FSH_SimpleVT>	Fx_FullScreenCopyVT;

#define DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(Name, BaseName) class TEXTUREGRAPHENGINE_API Name : public BaseName \
{ \
public: \
	SHADER_USE_PARAMETER_STRUCT(Name, BaseName); \
	DECLARE_GLOBAL_SHADER(Name); \
};

#define DECLARE_EMPTY_GLOBAL_SHADER(Name) DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(Name, FSH_Base)

#define TEXTURE_ENGINE_DEFAULT_PERMUTATION \
static bool	ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) \
{ \
	return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData); \
}

#define TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV \
static void						ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env) {}

#undef UE_API
