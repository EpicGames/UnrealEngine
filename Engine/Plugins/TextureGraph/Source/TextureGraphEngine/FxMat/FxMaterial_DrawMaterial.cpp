// Copyright Epic Games, Inc. All Rights Reserved
#include "FxMaterial_DrawMaterial.h"

#include "ContentStreaming.h"
#include "EngineModule.h"
#include "SceneView.h"
#include "VT/VirtualTextureChunkManager.h"
#include "VT/VirtualTextureFeedbackResource.h"

IMPLEMENT_MATERIAL_SHADER_TYPE(, FTextureGraphMaterialShaderVS, TEXT("/Plugin/TextureGraph/TextureGraphMaterialShader.usf"), TEXT("TextureGraphMaterialShaderVS"), SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(, FTextureGraphMaterialShaderPS, TEXT("/Plugin/TextureGraph/TextureGraphMaterialShader.usf"), TEXT("TextureGraphMaterialShaderPS"), SF_Pixel);

FName FTextureGraphMaterialShaderPS::PSCONTROL_ARG = TEXT("PSControl");

static int32 GTG_VirtualTextureNumWarmupFrames = 8;
static FAutoConsoleVariableRef CVarTGVirtualTextureNumWarmupFrames(
	TEXT("TG.VirtualTexture.NumWarmupFrames"),
	GTG_VirtualTextureNumWarmupFrames,
	TEXT("Number of warmup render frames when sampling from virtual textures."),
	ECVF_RenderThreadSafe);

static int32 GVirtualTextureFeedbackFactor = 16;
static FAutoConsoleVariableRef CVarVirtualTextureFeedbackFactor(
	TEXT("TG.VirtualTexture.FeedbackFactor"),
	GVirtualTextureFeedbackFactor,
	TEXT("Texel size of tile used to generate one item of virtual texture feedback."),
	ECVF_RenderThreadSafe);

FSceneView* FxMaterial_DrawMaterialBase::CreateSceneView(FSceneViewFamilyContext& ViewFamily, UTextureRenderTarget2D* RenderTarget, const FIntPoint& TargetSizeXY, int32 RenderIndex)
{
	FIntRect ViewRect(FIntPoint(0, 0), TargetSizeXY);

	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = &ViewFamily;
	ViewInitOptions.SetViewRectangle(ViewRect);
	ViewInitOptions.ViewOrigin = FVector::ZeroVector;
	ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;

	// Set up an orthographic projection matrix
	constexpr float AspectRatio = 1.0f;
	constexpr float OrthoWidth = 2.0f;
	constexpr float OrthoHeight = OrthoWidth * AspectRatio;
	const float NearPlane = GNearClippingPlane;
	const float FarPlane = GNearClippingPlane * 10000.0f;
	ViewInitOptions.ProjectionMatrix = FReversedZOrthoMatrix(OrthoWidth, OrthoHeight, NearPlane, FarPlane);

	ViewInitOptions.BackgroundColor = FLinearColor::Black;
	ViewInitOptions.OverlayColor = FLinearColor::White;

	FSceneView* View = new FSceneView(ViewInitOptions);
	ViewFamily.Views.Add(View);

	{
		// Create the view's uniform buffer.
		FViewUniformShaderParameters ViewUniformShaderParameters;
		ViewUniformShaderParameters.VTFeedbackBuffer = GEmptyStructuredBufferWithUAV->UnorderedAccessViewRHI;

		View->SetupCommonViewUniformBufferParameters(
			ViewUniformShaderParameters,
			TargetSizeXY,
			1,
			ViewRect,
			View->ViewMatrices,
			FViewMatrices()
		);

		// TODO LWC
		ViewUniformShaderParameters.RelativeWorldViewOriginTO = (FVector3f)View->ViewMatrices.GetViewOrigin();

		// Slate materials need this scale to be positive, otherwise it can fail in querying scene textures (e.g., custom stencil)
		ViewUniformShaderParameters.BufferToSceneTextureScale = FVector2f(1.0f, 1.0f);

		ERHIFeatureLevel::Type RHIFeatureLevel = View->GetFeatureLevel();

		ViewUniformShaderParameters.MobilePreviewMode = 0.0f;

		// Setup virtual texture feedback.
		// Use RenderIndex as the frame counter to vary feedback location independent of global render frame count.
		VirtualTexture::FFeedbackShaderParams VirtualTextureFeedbackShaderParams;
		VirtualTexture::GetFeedbackShaderParams(RenderIndex, GVirtualTextureFeedbackFactor, VirtualTextureFeedbackShaderParams);
		VirtualTexture::UpdateViewUniformShaderParameters(VirtualTextureFeedbackShaderParams, ViewUniformShaderParameters);

		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_Slate_CreateViewUniformBufferImmediate);
			View->ViewUniformBuffer = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(ViewUniformShaderParameters, UniformBuffer_SingleFrame);
		}
	}

	return View;
}

bool FxMaterial_DrawMaterialBase::HasVirtualTextureFeedback(FMaterial const& RenderMaterial, EShaderPlatform ShaderPlatform) const
{
	// todo: Would be better if there was a bGeneratesVirtualTextureFeedback flag on the cached material data, since it
	// is possible to have virtual texture samples that don't generate feedback.
	return UseVirtualTexturing(ShaderPlatform) && !RenderMaterial.GetUniformVirtualTextureExpressions().IsEmpty();
}

int32 FxMaterial_DrawMaterialBase::GetDefaultVirtualTextureWarmupFrames() const
{
	return FMath::Max(GTG_VirtualTextureNumWarmupFrames, 1);
}

void FxMaterial_DrawMaterialBase::VirtualTextureFeedbackBegin(FRDGBuilder& GraphBuilder, FIntPoint TargetSize, ERHIFeatureLevel::Type FeatureLevel) const
{
	VirtualTexture::BeginFeedback(GraphBuilder, TargetSize, GVirtualTextureFeedbackFactor, /*bInExtendFeedbackForDebug*/false, FeatureLevel);
}

void FxMaterial_DrawMaterialBase::VirtualTextureFeedbackEnd(FRDGBuilder& GraphBuilder) const
{
	VirtualTexture::EndFeedback(GraphBuilder);
}

void FxMaterial_DrawMaterialBase::VirtualTextureFeedbackSync(FRHICommandListImmediate& RHICmdList, ERHIFeatureLevel::Type FeatureLevel) const
{
	// Need a flush before we can read back the virtual texture feedback buffer.
	GetRendererModule().SyncVirtualTextureUpdates(RHICmdList, FeatureLevel);
	// Update virtual texture streaming to trigger release of upload buffers.
	// todo: Remove the need to access private implementation detail requiring private header access (enabled in our build.cs file).
	IStreamingManager::Get().GetVirtualTextureStreamingManager().UpdateResourceStreaming(0.f);
}

void FxMaterial_DrawMaterialBase::EndFrame(FRHICommandListImmediate& RHICmdList)
{
	// Increment frame number so that FVirtualTextureUploadCache releases upload buffers.
	GFrameNumberRenderThread++;
	// Tick RDG pool, so that render target memory can be recycled.
	FRDGBuilder::TickPoolElements();
	// EndFrame and flush to give GPU allocator opportunity to release memory.
	RHICmdList.EndFrame();
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
}

bool FxMaterial_DrawMaterialBase::ValidateMaterialShaderMap(UMaterial * InMaterial, FShaderType* InPixelShaderType)
{
	// Pixel shader combined with Material
	const FMaterialRenderProxy* MaterialProxy = InMaterial->GetRenderProxy();

	MaterialProxy->UpdateUniformExpressionCacheIfNeeded(GMaxRHIFeatureLevel);
	FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions();

	const FMaterial& RenderMaterial = MaterialProxy->GetMaterialWithFallback(GMaxRHIFeatureLevel, MaterialProxy);
	FMaterialShaderMap* MaterialShaderMap = RenderMaterial.GetRenderingThreadShaderMap();
	const FMaterialShaderMapContent* MaterialShaderMapContent = MaterialShaderMap->GetContent();
	bool NeedSceneTextures = MaterialShaderMap->NeedsSceneTextures();

	
	auto PixelShader = (MaterialShaderMap->GetShader(InPixelShaderType));
	FRHIPixelShader* RHIPixelShader = PixelShader.GetPixelShader();

	bool IsSupported = true;

	if (NeedSceneTextures)
	{
		UE_LOGF(LogTexture, Warning, "Material [%ls]: Require the <SceneTexturesStructs> which is not supported by the TextureGraph pipeline", *InMaterial->GetName());
		IsSupported = false;
	}
	if (MaterialShaderMap->NeedsGBuffer())
	{
		UE_LOGF(LogTexture, Warning, "Material [%ls]: Require the <GBuffer> which is not supported by the TextureGraph pipeline", *InMaterial->GetName());
		IsSupported = false;
	}

//#define ONLY_IN_DEBUG_CHECK_SHADER_DATA
#if defined(ONLY_IN_DEBUG_CHECK_SHADER_DATA)
	//auto Output = MaterialShaderMapContent->MaterialCompilationOutput
	//UE_LOGF(LogTexture, Warning, "Material[%ls]:\n[%ls]", *InMaterial->GetName(), MaterialShaderMap->GetDebugDescription());


	const FShaderMapBase* ShaderMap = PixelShader.GetShaderMap();
	
	//UE_LOGF(LogTexture, Warning, "Material[%ls]:\n[%ls]", *InMaterial->GetName(), *ShaderMap->ToString());

	int i = 0;
/*	for (auto& Name : RHIVertexShader->Debug.UniformBufferNames)
	{
		// DEBUG: Log the list of uniform required
		UE_LOGF(LogTexture, Warning, "Vertex Shader [%d]: %ls", i, *Name.ToString());
		++i;
	}*/
	i = 0;
	bool RequireSceneTexturesStruct = false;
	for (auto& Name : RHIPixelShader->Debug.UniformBufferNames)
	{
		// DEBUG: Log the list of uniform required
		// UE_LOGF(LogTexture, Warning, "Pixel Shader [%d]: %ls", i, *Name.ToString());
		if (Name == TEXT("SceneTexturesStruct"))
		{
			IsSupported = false;
			UE_LOGF(LogTexture, Warning, "Pixel Shader [%d]: Require the <SceneTexturesStructs> which is not supported by the TextureGraph pipeline", i, *Name.ToString());
		}
		++i;
	}
#endif

	return IsSupported;

}