// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include <Materials/Material.h>
#include <Materials/MaterialRenderProxy.h>
#include <MaterialDomain.h>
#include <TextureResource.h>
#include <SceneManagement.h>
#include <PostProcess/DrawRectangle.h>
#include <EngineModule.h>

#include <DataDrivenShaderPlatformInfo.h>
#include <MaterialShader.h>
#include <MaterialShared.h>
#include "Engine/TextureRenderTarget2D.h"

#include "FxMaterial.h"
#include "SceneView.h"
#include "TileInfo_FX.h"

#include "Device/Device.h"

#define UE_API TEXTUREGRAPHENGINE_API

class FTextureGraphMaterialShaderVS : public FMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FTextureGraphMaterialShaderVS, Material)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()

	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FTextureGraphMaterialShaderVS, FMaterialShader);

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static bool ShouldCache(EShaderPlatform Platform)
	{
		// Only cache the shader for the given platform if it is supported
		return IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View)
	{
		UE::Renderer::PostProcess::SetDrawRectangleParameters(BatchedParameters, this, View);
		FMaterialShader::SetViewParameters(BatchedParameters, View, View.ViewUniformBuffer);
	}
};

class FTextureGraphMaterialShaderPS : public FMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FTextureGraphMaterialShaderPS, Material)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FLinearColor, PSControl)
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
	END_SHADER_PARAMETER_STRUCT()

	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FTextureGraphMaterialShaderPS, FMaterialShader);

	static UE_API FName PSCONTROL_ARG;

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static bool ShouldCache(EShaderPlatform Platform)
	{
		// Only cache the shader for the given platform if it is supported
		return IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const FMaterialRenderProxy* MaterialProxy, const FMaterial& Material)
	{
		auto& PrimitivePS = GetUniformBufferParameter<FPrimitiveUniformShaderParameters>();
		SetUniformBufferParameter(BatchedParameters, PrimitivePS, GIdentityPrimitiveUniformBuffer);
		FMaterialShader::SetViewParameters(BatchedParameters, View, View.ViewUniformBuffer);
		FMaterialShader::SetParameters(BatchedParameters, MaterialProxy, Material, View);
	}
};

class FxMaterial_DrawMaterialBase : public FxMaterial
{
public:
	FxMaterial_DrawMaterialBase()
		: FxMaterial(TEXT("VSH_TypeVSH_TypeVSH_TypeVSH_Type"), TEXT("FSH_TypeFSH_TypeFSH_TypeFSH_Type"))
	{
	}

	FSceneView* CreateSceneView(FSceneViewFamilyContext& ViewFamily, UTextureRenderTarget2D* RenderTarget, const FIntPoint& TargetSizeXY, int32 RenderIndex);

	bool HasVirtualTextureFeedback(FMaterial const& RenderMaterial, EShaderPlatform ShaderPlatform) const;
	int32 GetDefaultVirtualTextureWarmupFrames() const;
	void VirtualTextureFeedbackBegin(FRDGBuilder& GraphBuilder, FIntPoint TargetSize, ERHIFeatureLevel::Type FeatureLevel) const;
	void VirtualTextureFeedbackEnd(FRDGBuilder& GraphBuilder) const;
	void VirtualTextureFeedbackSync(FRHICommandListImmediate& RHICmdList, ERHIFeatureLevel::Type FeatureLevel) const;
	void EndFrame(FRHICommandListImmediate& RHICmdList);

	static bool ValidateMaterialShaderMap(UMaterial* InMaterial, FShaderType* InPixelShaderType);
};

template <typename VSH_Type, typename FSH_Type>
class FxMaterial_DrawMaterial : public FxMaterial_DrawMaterialBase
{
public:

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(VSH_Type::FParameters, VS)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSH_Type::FParameters, PS)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()


	// type name for the permutation domain
	using VSHPermutationDomain = typename VSH_Type::FPermutationDomain;
	using FSHPermutationDomain = typename FSH_Type::FPermutationDomain;

	TObjectPtr<UMaterialInterface>	Material;

protected:
	VSHPermutationDomain			VSHPermDomain;		/// Vertex shader Permutation Domain value
	FSHPermutationDomain			FSHPermDomain;		/// Fragment shader Permutation Domain value
	typename VSH_Type::FParameters	VSHParams;			/// Params for the vertex shader
	typename FSH_Type::FParameters	FSHParams;			/// Params for the fragment shader

public:
	// If Material uses VT, we need to run several frames to make sure the VT is fully loaded.
	// 0 indicate to use the actual default value specified from the CVar <TG.VirtualTexture.NumWarmupFrames>
	int32							VirtualTextureNumWarmupFrames = 0;

public:

	FxMaterial_DrawMaterial()
		: FxMaterial_DrawMaterialBase()
	{
	}

	FxMaterial_DrawMaterial(UMaterialInterface* InMaterial, const VSHPermutationDomain& InVSHPermutationDomain, const FSHPermutationDomain& InFSHPermutationDomain, int32 InVirtualTextureNumWarmupFrames)
		: 	FxMaterial_DrawMaterialBase(),
		Material(InMaterial),
		VSHPermDomain(InVSHPermutationDomain),
		FSHPermDomain(InFSHPermutationDomain),
		VirtualTextureNumWarmupFrames(InVirtualTextureNumWarmupFrames)
	{
	}

	virtual std::shared_ptr<FxMaterial> Clone() override
	{
		return std::static_pointer_cast<FxMaterial>(std::make_shared<FxMaterial_DrawMaterial<VSH_Type, FSH_Type>>(Material, VSHPermDomain, FSHPermDomain, VirtualTextureNumWarmupFrames));
	}
	
	int32 GetVirtualTextureWarmupFrames() const
	{
		// If the instance value is invalid, then fallback to the default
		if (VirtualTextureNumWarmupFrames < 1)
		{
			return GetDefaultVirtualTextureWarmupFrames();
		}
		else
		{
			return VirtualTextureNumWarmupFrames;
		}
	}

	virtual FxMetadataSet GetMetadata() const override
	{
		return {
			{ VSH_Type::FParameters::FTypeInfo::GetStructMetadata(), (char*)&VSHParams }, 
			{ FSH_Type::FParameters::FTypeInfo::GetStructMetadata(), (char*)&FSHParams }
		};
	}

	virtual void Blit(FRHICommandListImmediate& RHI, FRHITexture* Target, const RenderMesh* MeshObj, int32 TargetId, FGraphicsPipelineStateInitializer* InPSO = nullptr) override
	{
		
	}

	void MyBlit(FRHICommandListImmediate& RHI, UTextureRenderTarget2D* RenderTarget, FRHITexture* Target, const RenderMesh* MeshObj, int32 TargetId, FGraphicsPipelineStateInitializer* InPSO = nullptr)
	{
		const ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;

		BindTexturesForBlitting();

		// Pixel shader combined with Material
		const FMaterialRenderProxy* MaterialProxy = Material->GetRenderProxy();
		MaterialProxy->UpdateUniformExpressionCacheIfNeeded(FeatureLevel);
		FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions();

		const FMaterial& RenderMaterial = MaterialProxy->GetMaterialWithFallback(FeatureLevel, MaterialProxy);
		const FMaterialShaderMap* const MaterialShaderMap = RenderMaterial.GetRenderingThreadShaderMap();

		TShaderRef<VSH_Type> VertexShader = MaterialShaderMap->GetShader<VSH_Type>();
		TShaderRef<FSH_Type> PixelShader = MaterialShaderMap->GetShader<FSH_Type>();

		const bool bUseVirtualTexturing = HasVirtualTextureFeedback(RenderMaterial, GMaxRHIShaderPlatform);
		const int32 RenderCount = bUseVirtualTexturing ? GetVirtualTextureWarmupFrames() : 1;

		QUICK_SCOPE_CYCLE_COUNTER(STAT_ShaderPlugin_PixelShader); // Used to gather CPU profiling data for the session frontend
		SCOPED_DRAW_EVENT(RHI, ShaderPlugin_Pixel); // Used to profile GPU activity and add metadata to be consumed by for example RenderDoc

		for (int32 RenderIndex = 0; RenderIndex < RenderCount; ++RenderIndex)
		{
			FRDGBuilder GraphBuilder(RHI);
			FRDGTextureRef TargetRDG = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Target, TEXT("FxMaterial_Target")));

			// Call VirtualTextureFeedbackBegin() before CreateSceneView() so that we have feedback buffer ready for creation of scene uniform buffer.
			if (bUseVirtualTexturing)
			{
				VirtualTextureFeedbackBegin(GraphBuilder, Target->GetSizeXY(), FeatureLevel);
			}

			FSceneViewFamilyContext SceneViewFamilyContext(
				FSceneViewFamily::ConstructionValues(RenderTarget->GetRenderTargetResource(), nullptr, ESFIM_Game)
				.SetTime(FGameTime())
				.SetRealtimeUpdate(true));
			FSceneView* SceneView = CreateSceneView(SceneViewFamilyContext, RenderTarget, Target->GetSizeXY(), RenderIndex);

			auto* PassParameters = GraphBuilder.AllocParameters<FParameters>();
			PassParameters->VS = VSHParams;
			PassParameters->PS = FSHParams;
			PassParameters->RenderTargets[0] = FRenderTargetBinding(TargetRDG, ERenderTargetLoadAction::EClear);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("FxMaterial (TargetId: %d, RenderIndex: %d)", TargetId, RenderIndex),
				PassParameters,
				ERDGPassFlags::Raster,
				[&](FRHICommandList& RHICmdList)
			{
				FGraphicsPipelineStateInitializer PSO;
				InitPSO_Default(PSO, VertexShader.GetVertexShader(), PixelShader.GetPixelShader());
				RHICmdList.ApplyCachedRenderTargets(PSO);
				SetGraphicsPipelineState(RHICmdList, PSO, 0);

				SetShaderParametersMixedVS(RHICmdList, VertexShader, PassParameters->VS, *SceneView);
				SetShaderParametersMixedPS(RHICmdList, PixelShader, PassParameters->PS, *SceneView, MaterialProxy, RenderMaterial);

				RHICmdList.SetStreamSource(0, GQuadBuffer.VertexBufferRHI, 0);
				RHICmdList.DrawPrimitive(0, 2, 1);
			});

			if (bUseVirtualTexturing)
			{
				VirtualTextureFeedbackEnd(GraphBuilder);
			}

			GraphBuilder.Execute();

			if (bUseVirtualTexturing)
			{
				VirtualTextureFeedbackSync(RHI, FeatureLevel);
			}

			EndFrame(RHI);
		}
	}

	static bool ValidateMaterial(UMaterialInterface* InMaterial)
	{
		check(IsInRenderingThread());

		UMaterial* Material = InMaterial->GetMaterial();

		return ValidateMaterialShaderMap(Material, &FSH_Type::GetStaticType());
	}
};

typedef FxMaterial_DrawMaterial< FTextureGraphMaterialShaderVS, FTextureGraphMaterialShaderPS> FxMaterial_QuadDrawMaterial;
typedef std::shared_ptr<FxMaterial_QuadDrawMaterial>	FxMaterialQuadDrawMaterialPtr;


#undef UE_API
