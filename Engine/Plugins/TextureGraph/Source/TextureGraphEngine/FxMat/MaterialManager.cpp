// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialManager.h"
#include "Model/ModelObject.h"
#include "2D/Tex.h"
#include "Engine/Canvas.h"
#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialManager)

UMaterialManager::UMaterialManager()
{
}

UMaterialManager::~UMaterialManager()
{
}

void UMaterialManager::InitEssentialMaterials()
{
	LoadMaterial(TEXT("Util/CopyTexture"));
}

UMaterial* UMaterialManager::LoadMaterial(const FString& Path)
{
	TObjectPtr<UMaterial>* ExistingMat = Materials.Find(Path);

	if (ExistingMat)
		return ExistingMat->Get();

	check(IsInGameThread());

	FString FullPath = TEXT("Materials/") + Path;
	UMaterial* Mat = Cast<UMaterial>(LoadObjectFromPath(FullPath));

	if (Mat)
	{
		Materials.Add(FullPath, Mat);
		Materials.Add(Path, Mat);
	}

	return Mat;
}

RenderMaterial_BPPtr UMaterialManager::CreateMaterial_BP(FString Name, const FString& Path, int32 InNumWarmupFrames)
{
	UMaterial* Mat = LoadMaterial(Path);
	return CreateMaterial_BP(Name, Mat, InNumWarmupFrames);
}

RenderMaterial_BPPtr UMaterialManager::CreateMaterial_BP(FString InName, UMaterial* InMaterial, int32 InNumWarmupFrames)
{
	return std::make_shared<RenderMaterial_BP>(InName, InMaterial, InNumWarmupFrames);
}

RenderMaterial_BP_NoTilePtr UMaterialManager::CreateMaterial_BP_NoTile(FString Name, const FString& Path, int32 InNumWarmupFrames)
{
	UMaterial* Mat = LoadMaterial(Path);
	return std::make_shared<RenderMaterial_BP_NoTile>(Name, Mat, InNumWarmupFrames);
}

RenderMaterial_ThumbPtr UMaterialManager::CreateMaterial_Thumbnail(FString Name, const FString& Path)
{
	UMaterial* Mat = LoadMaterial(Path);
	return std::make_shared<RenderMaterial_Thumbnail>(Name, Mat);
}

RenderMaterial_BP_TileArgsPtr UMaterialManager::CreateMaterialOfType_BP_TileArgs(FString Name, const FString& Path, int32 InNumWarmupFrames)
{
	UMaterial* Mat = LoadMaterial(Path);
	return std::make_shared<RenderMaterial_BP_TileArgs>(Name, Mat, InNumWarmupFrames);
}
