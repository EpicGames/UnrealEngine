// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "FxMat/FxMaterial.h"
#include "RenderMaterial_BP.h"
#include "RenderMaterial_FX.h"
#include "RenderMaterial_FX_Combined.h"
#include "RenderMaterial_BP_NoTile.h"
#include "RenderMaterial_Thumbnail.h"
#include "Model/ModelObject.h"
#include "MaterialManager.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

UCLASS(MinimalAPI)
class UMaterialManager : public UModelObject
{
	GENERATED_BODY()

private:
	UPROPERTY(transient)
	TMap<FString, TObjectPtr<UMaterial>> Materials;				/// The materials that have been loaded so far

	UE_API void							InitEssentialMaterials();
	

public:
									UE_API UMaterialManager();
	UE_API virtual							~UMaterialManager() override;
	
	UE_API class UMaterial*				LoadMaterial(const FString& Path);	

	UE_API RenderMaterial_BPPtr			CreateMaterial_BP(FString Name, const FString& Path, int32 InNumWarmupFrames = 0);
	UE_API RenderMaterial_BPPtr			CreateMaterial_BP(FString InName, UMaterial* InMaterial, int32 InNumWarmupFrames = 0);
	UE_API RenderMaterial_BP_NoTilePtr		CreateMaterial_BP_NoTile(FString Name, const FString& Path, int32 InNumWarmupFrames = 0);
	UE_API RenderMaterial_ThumbPtr			CreateMaterial_Thumbnail(FString Name, const FString& Path);
	UE_API RenderMaterial_BP_TileArgsPtr	CreateMaterialOfType_BP_TileArgs(FString Name, const FString& Path, int32 InNumWarmupFrames = 0);

	//////////////////////////////////////////////////////////////////////////

	template <typename VSH_Type, typename FSH_Type>
	RenderMaterial_FXPtr			CreateMaterial_FX(FString Name)
	{
		std::shared_ptr<FxMaterial_Normal<VSH_Type, FSH_Type>> Mat = std::make_shared<FxMaterial_Normal<VSH_Type, FSH_Type>>();
		return std::make_shared<RenderMaterial_FX>(Name, std::static_pointer_cast<FxMaterial>(Mat));
	}

	template <typename VSH_Type, typename FSH_Type>
	RenderMaterial_FXPtr			CreateMaterial_FX(FString Name, const typename VSH_Type::FPermutationDomain& InVSHPermutationDomain, const typename FSH_Type::FPermutationDomain& InFSHPermutationDomain)
	{
		std::shared_ptr<FxMaterial_Normal<VSH_Type, FSH_Type>> Mat = std::make_shared<FxMaterial_Normal<VSH_Type, FSH_Type>>(InVSHPermutationDomain, InFSHPermutationDomain);
		return std::make_shared<RenderMaterial_FX>(Name, std::static_pointer_cast<FxMaterial>(Mat));
	}

	template <typename VSH_Type, typename FSH_Type>
	RenderMaterial_FXPtr			CreateMaterial_FX(FString Name, const typename FSH_Type::FPermutationDomain& InFSHPermutationDomain)
	{
		std::shared_ptr<FxMaterial_Normal<VSH_Type, FSH_Type>> Mat = std::make_shared<FxMaterial_Normal<VSH_Type, FSH_Type>>(typename VSH_Type::FPermutationDomain(), InFSHPermutationDomain);
		return std::make_shared<RenderMaterial_FX>(Name, std::static_pointer_cast<FxMaterial>(Mat));
	}

	template <typename CmpSH_Type>
	RenderMaterial_FXPtr			CreateMaterial_CmpFX(FString Name, FString OutputId, int NumThreadsX, int NumThreadsY, int NumThreadsZ = 1)
	{
		std::shared_ptr<FxMaterial_Compute<CmpSH_Type>> Mat = std::make_shared<FxMaterial_Compute<CmpSH_Type>>(OutputId, nullptr, NumThreadsX, NumThreadsY, NumThreadsZ);
		return std::make_shared<RenderMaterial_FX>(Name, std::static_pointer_cast<FxMaterial>(Mat));
	}

	template <typename CmpSH_Type>
	RenderMaterial_FXPtr			CreateMaterial_CmpFX(FString Name, FString OutputId, const typename CmpSH_Type::FPermutationDomain& cmpshPermutationDomain, 
		int NumThreadsX, int NumThreadsY, int NumThreadsZ = 1)
	{
		std::shared_ptr<FxMaterial_Compute<CmpSH_Type>> Mat = std::make_shared<FxMaterial_Compute<CmpSH_Type>>(OutputId, &cmpshPermutationDomain, NumThreadsX, NumThreadsY, NumThreadsZ);
		return std::make_shared<RenderMaterial_FX>(Name, std::static_pointer_cast<FxMaterial>(Mat));
	}

	template <typename FxMatType>
	RenderMaterial_FXPtr			CreateMaterialOfType_FX(FString Name)
	{
		std::shared_ptr<FxMatType> Mat = std::make_shared<FxMatType>();
		return std::make_shared<RenderMaterial_FX>(Name, std::static_pointer_cast<FxMaterial>(Mat));
	}

	template <typename VSH_Type, typename FSH_Type>
	RenderMaterial_FX_CombinedPtr			CreateMaterial_FX_Combined(FString Name)
	{
		std::shared_ptr<FxMaterial_Normal<VSH_Type, FSH_Type>> Mat = std::make_shared<FxMaterial_Normal<VSH_Type, FSH_Type>>();
		return std::make_shared<RenderMaterial_FX_Combined>(Name, std::static_pointer_cast<FxMaterial>(Mat));
	}

	template <typename VSH_Type, typename FSH_Type>
	RenderMaterial_FX_CombinedPtr			CreateMaterial_FX_Combined(FString Name, const typename VSH_Type::FPermutationDomain& InVSHPermutationDomain, const typename FSH_Type::FPermutationDomain& InFSHPermutationDomain)
	{
		std::shared_ptr<FxMaterial_Normal<VSH_Type, FSH_Type>> Mat = std::make_shared<FxMaterial_Normal<VSH_Type, FSH_Type>>(InVSHPermutationDomain, InFSHPermutationDomain);
		return std::make_shared<RenderMaterial_FX_Combined>(Name, std::static_pointer_cast<FxMaterial>(Mat));
	}

	template <typename VSH_Type, typename FSH_Type>
	RenderMaterial_FX_CombinedPtr			CreateMaterial_FX_Combined(FString Name, const typename FSH_Type::FPermutationDomain& InFSHPermutationDomain)
	{
		std::shared_ptr<FxMaterial_Normal<VSH_Type, FSH_Type>> Mat = std::make_shared<FxMaterial_Normal<VSH_Type, FSH_Type>>(typename VSH_Type::FPermutationDomain(), InFSHPermutationDomain);
		return std::make_shared<RenderMaterial_FX_Combined>(Name, std::static_pointer_cast<FxMaterial>(Mat));
	}
};

#undef UE_API
