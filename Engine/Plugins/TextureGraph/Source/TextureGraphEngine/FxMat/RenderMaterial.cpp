// Copyright Epic Games, Inc. All Rights Reserved.
#include "RenderMaterial.h"
#include "MaterialManager.h"
#include "Model/ModelObject.h"
#include "2D/Tex.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "Engine/Canvas.h"
#include "TextureGraphEngine.h"
#include "Device/FX/Device_FX.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include <TextureResource.h>

RenderMaterial::RenderMaterial(FString Name) : BlobTransform(Name)
{
}

RenderMaterial::~RenderMaterial()
{
}

Device* RenderMaterial::TargetDevice(size_t Index) const
{
	return Device_FX::Get();
}

ENamedThreads::Type RenderMaterial::ExecutionThread() const
{
	return ENamedThreads::ActualRenderingThread;
}

bool RenderMaterial::CanHandleTiles() const
{
	return bCanHandleTiles;
}

AsyncTransformResultPtr RenderMaterial::Exec(const TransformArgs& Args)
{
	check(IsInRenderingThread());

	ResourceBindInfo BindInfo;

	BindInfo.Dev = Args.Dev;
	BindInfo.bWriteTarget = true;

	BlobPtr TargetBlob = Args.Target.lock();
	check(TargetBlob);

	check(TargetBlob->GetBufferRef().GetDeviceType() == DeviceType::FX);
	DeviceBuffer_FX* TargetBuffer = static_cast<DeviceBuffer_FX*>(TargetBlob->GetBufferRef().get());

	Tex* TexObj = TargetBuffer->GetTexture().get();
	check(TexObj);
	
	UE_LOGF(LogDevice, VeryVerbose, "Rendering material: %ls [Target: %ls, Blob: 0x%p, RT: 0x%p]", *Name, *TargetBlob->Name(), TargetBlob.get(), TexObj->GetRenderTarget());

	FTextureRenderTarget2DResource* RTRes = (FTextureRenderTarget2DResource*)TexObj->GetRenderTarget()->GetRenderTargetResource();
	check(RTRes != nullptr);

	auto& RHI = Device_FX::Get()->RHI();
	BlitTo(RHI, TexObj, Args.Mesh, Args.TargetId);

	//target->Unbind(this, bindInfo);

	TransformResultPtr Result = std::make_shared<TransformResult>();
	Result->Target = TargetBlob;

	return cti::make_ready_continuable(Result);
}

void RenderMaterial::SetSourceTexture(const Tex* Texture) const
{ 
	SetTexture(FName("SourceTexture"), Texture->GetTexture()); 
}

void RenderMaterial::SetTexture(FName InName, std::shared_ptr<Tex> TexObj) const
{ 
	SetTexture(InName, TexObj->GetTexture()); 
}

void RenderMaterial::SetArrayTexture(FName InName, const std::vector<std::shared_ptr<Tex>>& TexObj) const
{
	std::vector<const UTexture*> Textures;
	for (const auto& t : TexObj)
	{
		Textures.emplace_back(t->GetTexture());
	}
	SetArrayTexture(InName, Textures);
}

void RenderMaterial::BlitTo(FRHICommandListImmediate& RHI, Tex* Dst, const RenderMesh* MeshObj /* = nullptr */, int32 TargetId /* = -1 */) const
{ 
	BlitTo(RHI, Dst->GetRenderTarget(), MeshObj, TargetId); 
}

void RenderMaterial::BlitTo(FRHICommandListImmediate& RHI, UTextureRenderTarget2D* Dst) const
{
	return BlitTo(RHI, Dst, nullptr, -1);
}
