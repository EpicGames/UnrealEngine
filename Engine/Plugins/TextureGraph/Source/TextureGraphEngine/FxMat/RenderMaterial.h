// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Transform/BlobTransform.h"
#include "Engine/TextureRenderTarget2D.h"

#define UE_API TEXTUREGRAPHENGINE_API


class Tex;
class Device;
class RenderMesh;
class FRHICommandListImmediate;
class UTextureRenderTarget2D;
class UTexture;

class RenderMaterial : public BlobTransform
{
protected:
	bool							bCanHandleTiles = true;		/// Whether this material can handle tiles or not

public:
	UE_API explicit						RenderMaterial(FString Name);
	UE_API virtual							~RenderMaterial() override;

	virtual void					BlitTo(FRHICommandListImmediate& RHI, UTextureRenderTarget2D* Dst, const RenderMesh* MeshObj, int32 TargetId) const = 0;
	UE_API void							BlitTo(FRHICommandListImmediate& RHI, UTextureRenderTarget2D* Dst) const;

	//////////////////////////////////////////////////////////////////////////
	/// BlobTransform overrides
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual Device*					TargetDevice(size_t Index) const override;
	UE_API virtual AsyncTransformResultPtr	Exec(const TransformArgs& Args) override;
	UE_API virtual ENamedThreads::Type		ExecutionThread() const override;
	UE_API virtual bool					CanHandleTiles() const override;

	virtual void					SetTexture(FName Name, const UTexture* Texture) const = 0;
	virtual void					SetArrayTexture(FName Name, const std::vector<const UTexture*>& Textures) const = 0;
	virtual void					SetInt(FName Name, int32 Value) const = 0;
	virtual void					SetFloat(FName Name, float Value) const = 0;
	virtual void					SetColor(FName Name, const FLinearColor& Value) const = 0;
	virtual void					SetIntVector4(FName Name, const FIntVector4& Value) const = 0;
	virtual void					SetMatrix(FName Name, const FMatrix& Value) const = 0;
	UE_API void							SetSourceTexture(const Tex* Texture) const;
	UE_API void							SetTexture(FName InName, std::shared_ptr<Tex> TexObj) const;
	UE_API void							SetArrayTexture(FName InName, const std::vector<std::shared_ptr<Tex>>& TexObj) const;
	UE_API void							BlitTo(FRHICommandListImmediate& RHI, Tex* Dst, const RenderMesh* MeshObj = nullptr, int32 TargetId = -1) const;
	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE void				SetSourceTexture(const UTexture* Texture) const { SetTexture(FName("SourceTexture"), Texture); }
	FORCEINLINE bool&				CanHandleTiles() { return bCanHandleTiles; }
};

typedef std::shared_ptr<RenderMaterial>	RenderMaterialPtr;

#undef UE_API
