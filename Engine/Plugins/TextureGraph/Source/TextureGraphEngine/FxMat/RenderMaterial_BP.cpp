// Copyright Epic Games, Inc. All Rights Reserved.
#include "RenderMaterial_BP.h"
#include "2D/Tex.h"
#include "Kismet/KismetRenderingLibrary.h"
//#include "3D/ProceduralMeshActor.h"
#include "Engine/Canvas.h"
#include "TextureGraphEngine.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <Materials/Material.h>
#include <Materials/MaterialInstanceDynamic.h>
#include <Materials/MaterialInstanceConstant.h>
#include <Engine/World.h>
#include <TextureResource.h>
#include <UObject/UObjectGlobals.h>
#include <UObject/Package.h>
#include "Async/Async.h"
#include "FxMaterial_DrawMaterial.h"
#include "Job/Job.h"

FMaterialInstanceStaticSwitchPermutationMap FMaterialInstanceStaticSwitchPermutationMap::Create(UMaterialInterface* InMaterial)
{
	FMaterialInstanceStaticSwitchPermutationMap MISSPMap;
	if (InMaterial)
	{
		UMaterialInstanceConstant* MIC = NewObject<UMaterialInstanceConstant>();
#if WITH_EDITOR
		if (GIsEditor || IsRunningCommandlet())
		{
			InMaterial->GetStaticParameterValues(MISSPMap.DefaultStaticParameterSet);
			MISSPMap.DefaultKey = MISSPMap.KeyFromStaticSwitchParameters(MISSPMap.DefaultStaticParameterSet.StaticSwitchParameters);

			MIC->SetParentEditorOnly(InMaterial);
			MIC->ClearParameterValuesEditorOnly();
			MIC->SetFlags(RF_Transient);
			MIC->PreEditChange(NULL);
			MIC->PostEditChange();
		}
#endif
		MISSPMap.PermutationsMap.Add(MISSPMap.DefaultKey, MIC);
	}
	return MISSPMap;
}

UMaterialInterface* FMaterialInstanceStaticSwitchPermutationMap::GetRootMaterial()
{
	TObjectPtr<UMaterialInstanceConstant>* MICP = PermutationsMap.Find(DefaultKey);
	if (MICP)
	{
		// Use the FIRST MIC that initiated the MISWPermutationMap as the "root Material" 
		// in order to preserve the default values
		return *MICP;
	}
	return nullptr;
}

int32 FMaterialInstanceStaticSwitchPermutationMap::KeyFromStaticSwitchParameters(const TArray<FStaticSwitchParameter>& Parameters)
{
	int32 Key = 0;
	for (int32 Index = 0; Index < Parameters.Num(); ++Index)
	{
		int32 ParamBitfield = ((int32)(Parameters[Index].Value) << Index);
		Key |= ParamBitfield;
	}

	return Key;
}

UMaterialInstanceConstant* FMaterialInstanceStaticSwitchPermutationMap::GetMaterialInstance(const TArray<FStaticSwitchParameter>& Parameters)
{
	check(DefaultStaticParameterSet.StaticSwitchParameters.Num() == Parameters.Num());

	int32 Key = KeyFromStaticSwitchParameters(Parameters);

	TObjectPtr<UMaterialInstanceConstant>* MICP = PermutationsMap.Find(Key);
	if (!MICP)
	{
		UMaterialInstanceConstant* MIC = NewObject<UMaterialInstanceConstant>();
#if WITH_EDITOR
		if (GIsEditor || IsRunningCommandlet())
		{
			MIC->SetParentEditorOnly(GetRootMaterial());
			MIC->ClearParameterValuesEditorOnly();
			MIC->SetFlags(RF_Transient);

			FStaticParameterSet InstanceStaticParameterSet = DefaultStaticParameterSet;
			InstanceStaticParameterSet.StaticSwitchParameters = Parameters;
			MIC->UpdateStaticPermutation(InstanceStaticParameterSet);
			MIC->PreEditChange(NULL);
			MIC->PostEditChange();
		}
#endif
		PermutationsMap.Add(Key, MIC);

		return MIC;
	}
	else
	{
		return *MICP;
	}
}

bool FMaterialInstanceStaticSwitchPermutationMap::IsValid() const
{
	// If a non-null material is provided to Create, it will be in the PermutationsMap, so if 
	// PermutationsMap has no entries, it implies Create was not used to initialize this struct.
	return !PermutationsMap.IsEmpty();
}

bool RenderMaterial_BP::ValidateMaterialCompatible(UMaterialInterface* InMaterial)
{
	return FxMaterial_QuadDrawMaterial::ValidateMaterial(InMaterial);
}

RenderMaterial_BP::RenderMaterial_BP(FString Name, UMaterialInterface* InMaterial, int32 InNumWarmupFrames)
	: RenderMaterial(!Name.IsEmpty() ? Name : InMaterial->GetName())
	, Material(InMaterial)
	, FXMaterialObj(std::make_shared<FxMaterial_QuadDrawMaterial>())
	, VirtualTextureNumWarmupFrames(InNumWarmupFrames)
{
	// Cast to or build a MIC to be used by this RenderMaterial and its duplicates
	if (Material && Material->IsA< UMaterialInstanceConstant>())
	{
		MaterialInstance = TStrongObjectPtr<UMaterialInstanceConstant>(Cast<UMaterialInstanceConstant>(Material));
	}
	else if (Material && !MaterialInstance)
	{

		MaterialInstance = TStrongObjectPtr<UMaterialInstanceConstant>(NewObject<UMaterialInstanceConstant>());
#if WITH_EDITOR
		MaterialInstance->SetParentEditorOnly(Material);
		MaterialInstance->ClearParameterValuesEditorOnly();
		MaterialInstance->SetFlags(RF_Transient);
		MaterialInstance->PreEditChange(NULL);
		MaterialInstance->PostEditChange();
#endif
	}

	check(Material && MaterialInstance);


	UWorld* world = Util::GetGameWorld();
	Canvas = world ? world->GetCanvasForDrawMaterialToRenderTarget() : NewObject<UCanvas>(GetTransientPackage(), NAME_None);

	/// The material pointer will be the same for every material of the same type (even 
	/// though instances can be different so we can just use the address as the hash
	/// for the sake of completeness
	HashValue = std::make_shared<CHash>(DataUtil::Hash((uint8*)Material, sizeof(UMaterial*)), true);

	std::static_pointer_cast<FxMaterial_QuadDrawMaterial>(FXMaterialObj)->Material = MaterialInstance.Get();
	std::static_pointer_cast<FxMaterial_QuadDrawMaterial>(FXMaterialObj)->VirtualTextureNumWarmupFrames = VirtualTextureNumWarmupFrames;
}

RenderMaterial_BP::~RenderMaterial_BP()
{
}

CHashPtr RenderMaterial_BP::Hash() const
{
	return HashValue;
}

void RenderMaterial_BP::SetTexture(FName InName, const UTexture* Texture) const
{
	check(Material && MaterialInstance);
#if WITH_EDITOR
	MaterialInstance->SetTextureParameterValueEditorOnly(InName, const_cast<UTexture*>(Texture));
#endif
}

void RenderMaterial_BP::SetArrayTexture(FName InName, const std::vector<const UTexture*>& Textures) const
{
	check(false);
}

void RenderMaterial_BP::SetInt(FName InName, int32 Value) const
{
	// Same as set scalar for material instance constant
	SetFloat(InName, Value);
}

void RenderMaterial_BP::SetFloat(FName InName, float Value) const
{
	check(Material && MaterialInstance);
#if WITH_EDITOR
	MaterialInstance->SetScalarParameterValueEditorOnly(InName, Value);
#endif
}

void RenderMaterial_BP::SetColor(FName InName, const FLinearColor& Value) const
{
	check(Material && MaterialInstance);
#if WITH_EDITOR
	MaterialInstance->SetVectorParameterValueEditorOnly(InName, Value);
#endif
}

void RenderMaterial_BP::SetIntVector4(FName InName, const FIntVector4& Value) const
{
	check(Material && MaterialInstance);
#if WITH_EDITOR
	MaterialInstance->SetVectorParameterValueEditorOnly(InName, FLinearColor(Value.X, Value.Y, Value.Z, Value.W));
#endif
}

void RenderMaterial_BP::SetMatrix(FName InName, const FMatrix& Value) const
{
	// UMaterialInstanceConstant does not support assigning matrix parameters
}

std::shared_ptr<BlobTransform> RenderMaterial_BP::DuplicateInstance(FString InName)
{
	if (InName.IsEmpty())
		InName = Name;

	RenderMaterial_BPPtr MaterialBP = std::make_shared<RenderMaterial_BP>(InName, MaterialInstance.Get(), GetVirtualTextureNumWarmupFrames()); //Reuse the same material instance for every invocations

	MaterialBP->MaterialInstanceValidated = MaterialInstanceValidated;
	std::shared_ptr<BlobTransform> Result = std::static_pointer_cast<RenderMaterial>(MaterialBP);
	
	return Result;
}

AsyncPrepareResult RenderMaterial_BP::PrepareResources(const TransformArgs& Args)
{
	if (RequestMaterialValidation) // Only if the Validation of the Material is requested
	{
		UObject* ErrorOwner = Args.JobObj->GetErrorOwner();
		UMixInterface* Mix = Args.Cycle->GetMix();
		RenderMaterial_BP* Self = this;
		return cti::make_continuable<int32>([Self, Mix, ErrorOwner](auto&& Promise) mutable
			{


				Util::OnRenderingThread([FWD_PROMISE(Promise), Self, Mix, ErrorOwner](FRHICommandListImmediate& RHI) mutable
					{
						bool ValidationResult = ValidateMaterialCompatible(Self->MaterialInstance.Get());
						if (ValidationResult)
						{
							Self->MaterialInstanceValidated = true;
						}
						else
						{
							Self->MaterialInstanceValidated = false;
							const FString ErrorMsg = FString::Format(TEXT("Material '{0}' is not supported in TextureGraph."), { Self->GetName() });
							TextureGraphEngine::GetErrorReporter(Mix)->ReportError(static_cast<int32>(ETextureGraphErrorType::UNSUPPORTED_MATERIAL), ErrorMsg, ErrorOwner);
						}

						Util::OnGameThread([FWD_PROMISE(Promise)]() mutable
							{
								Promise.set_value(0);
							});
					});

			});
	}
	else
	{
		return cti::make_ready_continuable<int32>(0);
	}
}

void RenderMaterial_BP::DrawMaterial(UMaterialInterface* RenderMaterial, FVector2D ScreenPosition, FVector2D ScreenSize,
	FVector2D CoordinatePosition, FVector2D CoordinateSize, float Rotation, FVector2D PivotPoint) const
{
	if (RenderMaterial 
		&& ScreenSize.X > 0.0f 
		&& ScreenSize.Y > 0.0f 
		// Canvas can be NULL if the user tried to draw after EndDrawCanvasToRenderTarget
		&& Canvas->Canvas)
	{
		FCanvasTileItem TileItem(ScreenPosition, RenderMaterial->GetRenderProxy(), ScreenSize, CoordinatePosition, CoordinatePosition + CoordinateSize);
		TileItem.Rotation = FRotator(0, Rotation, 0);
		TileItem.PivotPoint = PivotPoint;
		TileItem.SetColor(Canvas->DrawColor);
		Canvas->DrawItem(TileItem);
	}
}

void RenderMaterial_BP::BlitTo(FRHICommandListImmediate& RHI, UTextureRenderTarget2D* RT, const RenderMesh* MeshObj, int32 TargetId) const
{
	if (!MaterialInstanceValidated)
		return;

	if (!MeshObj && RT)
	{
		FTextureRenderTargetResource* RTRes = RT->GetRenderTargetResource();
		check(RTRes);

		FTextureRHIRef TextureRHI = RTRes->GetTextureRHI();
		check(TextureRHI);

		TextureRHI->SetName(FName(*RT->GetName()));

		std::static_pointer_cast<FxMaterial_QuadDrawMaterial>(FXMaterialObj)->MyBlit(RHI, RT, TextureRHI, MeshObj, TargetId);
	}
	else
	{
	/*	throw std::runtime_error(std::string("RenderMesh_BP : Need to update this code."));

		// TODO : Need to update this.
		/// Removed mesh->Actor(int targetId) function
		/// Each mesh contains multiple actors and all the actors are present in single target.
			
		/// This needs to change into a RenderMesh method
		auto actor = dynamic_cast<AProceduralMeshActor*>(mesh->Actor(targetId));

		/// Assert but don't fail catastrophically if the dynamic_cast fails
		check(actor);

		if (actor)
			actor->BlitTo(RT, _instance);*/
	}
}

void RenderMaterial_BP::Bind(int32 Value, const ResourceBindInfo& BindInfo)
{
	SetInt(*BindInfo.Target, Value);
}

void RenderMaterial_BP::Bind(float Value, const ResourceBindInfo& BindInfo)
{
	SetFloat(*BindInfo.Target, Value);	
}

void RenderMaterial_BP::Bind(const FLinearColor& Value, const ResourceBindInfo& BindInfo)
{
	SetColor(*BindInfo.Target, Value);

	FName BindName(BindInfo.Target);
	if (BindName == FTextureGraphMaterialShaderPS::PSCONTROL_ARG)
		std::static_pointer_cast<FxMaterial_QuadDrawMaterial>(FXMaterialObj)->SetVectorParameterValue(*BindInfo.Target, Value);
}

void RenderMaterial_BP::Bind(const FIntVector4& Value, const ResourceBindInfo& BindInfo)
{
	SetIntVector4(*BindInfo.Target, Value);
}

void RenderMaterial_BP::Bind(const FMatrix& Value, const ResourceBindInfo& BindInfo)
{
	SetMatrix(*BindInfo.Target, Value);
}

void RenderMaterial_BP::BindStruct(const char* ValueAddress, size_t StructSize, const ResourceBindInfo& BindInfo)
{
	std::static_pointer_cast<FxMaterial_QuadDrawMaterial>(FXMaterialObj)->SetStructParameterValue(*BindInfo.Target, ValueAddress, StructSize);
}
