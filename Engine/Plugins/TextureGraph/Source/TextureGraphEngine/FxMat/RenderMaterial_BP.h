// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RenderMaterial.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInstanceConstant.h"

#include "RenderMaterial_BP.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

class FxMaterial;
typedef std::shared_ptr< FxMaterial> FxMaterialPtr;

// Helper container of MIC for the different static switch permutations of a Material
// Used by the caller of a RenderMaterial_BP to pick a specific permutation different from the default set.
// See TG_Expression_MaterialBase for a usage example
USTRUCT()
struct FMaterialInstanceStaticSwitchPermutationMap
{
	GENERATED_BODY()

	int32												DefaultKey = 0;
	
	UPROPERTY()
	FStaticParameterSet									DefaultStaticParameterSet;
	
	UPROPERTY()
	TMap<int32, TObjectPtr<UMaterialInstanceConstant>>	PermutationsMap;

	static UE_API FMaterialInstanceStaticSwitchPermutationMap Create(UMaterialInterface* InMaterial);
	UE_API UMaterialInterface*				GetRootMaterial();
	UE_API int32							KeyFromStaticSwitchParameters(const TArray<FStaticSwitchParameter>& Parameters);
	UE_API UMaterialInstanceConstant*		GetMaterialInstance(const TArray<FStaticSwitchParameter>& Parameters);
	UE_API bool								IsValid() const;
};

class RenderMaterial_BP : public RenderMaterial
{
public:
	// When a Material contains a VirtualTexture, the default num of warmup frames run to stream the appropriate resolution
	// This can be fine tuned by instance of RenderMaterial_BP from the constructor 
	static UE_API int32 GetDefaultVirtualTextureNumWarmupFrames();

protected:

	UMaterialInterface*				Material = nullptr;			/// The base material that is used for this job 
	TStrongObjectPtr<UMaterialInstanceConstant> MaterialInstance;			/// An instance of the material

	CHashPtr						HashValue;					/// The hash for this material
	bool							RequestMaterialValidation = true;		/// 
	bool							MaterialInstanceValidated = false;		/// 

	FxMaterialPtr					FXMaterialObj;

	int32 							VirtualTextureNumWarmupFrames = 0;  // Num warmup frames for this instance default to -1

	UCanvas*						Canvas = nullptr;

	UE_API void							DrawMaterial(UMaterialInterface* RenderMaterial, FVector2D ScreenPosition, FVector2D ScreenSize,
									FVector2D CoordinatePosition, FVector2D CoordinateSize=FVector2D::UnitVector, float Rotation=0.f,
									FVector2D PivotPoint=FVector2D(0.5f,0.5f)) const;
public:
									UE_API RenderMaterial_BP(FString Name, UMaterialInterface* InMaterial, int32 InNumWarmupFrames);

	UE_API virtual							~RenderMaterial_BP() override;

	static UE_API bool						ValidateMaterialCompatible(UMaterialInterface* InMaterial);


	UE_API virtual AsyncPrepareResult		PrepareResources(const TransformArgs& Args) override;

	//////////////////////////////////////////////////////////////////////////
	/// BlobTransform Implementation
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual void					Bind(int32 Value, const ResourceBindInfo& BindInfo) override;
	UE_API virtual void					Bind(float Value, const ResourceBindInfo& BindInfo) override;
	UE_API virtual void					Bind(const FLinearColor& Value, const ResourceBindInfo& BindInfo) override;
	UE_API virtual void					Bind(const FIntVector4& Value, const ResourceBindInfo& BindInfo) override;
	UE_API virtual void					Bind(const FMatrix& Value, const ResourceBindInfo& BindInfo) override;
	UE_API virtual void					BindStruct(const char* ValueAddress, size_t StructSize, const ResourceBindInfo& BindInfo) override;
	UE_API virtual CHashPtr				Hash() const override;
	UE_API virtual std::shared_ptr<BlobTransform> DuplicateInstance(FString InName) override;

	//////////////////////////////////////////////////////////////////////////
	/// RenderMaterial Implementation
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual void					BlitTo(FRHICommandListImmediate& RHI, UTextureRenderTarget2D* DstRT, const RenderMesh* MeshObj, int32 TargetId) const override;
	UE_API virtual void					SetTexture(FName InName, const UTexture* Texture) const override;
	UE_API virtual void					SetArrayTexture(FName InName, const std::vector<const UTexture*>& Textures) const override;
	UE_API virtual void					SetInt(FName InName, int32 Value) const override;
	UE_API virtual void					SetFloat(FName InName, float Value) const override;
	UE_API virtual void					SetColor(FName InName, const FLinearColor& Value) const override;
	UE_API virtual void					SetIntVector4(FName InName, const FIntVector4& Value) const override;
	UE_API virtual void					SetMatrix(FName InName, const FMatrix& Value) const override;
	
	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	////////////////////////////////////////////////////////////////////////// 
	FORCEINLINE UMaterialInterface*					GetMaterial() { return Material; }
	FORCEINLINE UMaterialInstanceConstant*			Instance() { return MaterialInstance.Get(); }
	FORCEINLINE const UMaterialInstanceConstant*	Instance() const { return MaterialInstance.Get(); }
	FORCEINLINE int32 GetVirtualTextureNumWarmupFrames() const { return VirtualTextureNumWarmupFrames; }
};

typedef std::shared_ptr<RenderMaterial_BP> RenderMaterial_BPPtr;

#undef UE_API
