// Copyright Epic Games, Inc. All Rights Reserved.
#include "RenderMaterial_BP_NoTile.h"
#include "Job/JobArgs.h"
#include "Job/Job.h"

void RenderMaterial_BP_TileArgs::AddTileArgs(TransformArgs& Args)
{
	FTileInfo TileInfo;
	auto TileArg = ARG_TILEINFO(TileInfo, "TileInfo");
	const_cast<Job*>(Args.JobObj)->AddArg(TileArg);
}

AsyncPrepareResult RenderMaterial_BP_TileArgs::PrepareResources(const TransformArgs& Args)
{
	AddTileArgs(const_cast<TransformArgs&>(Args));
	return cti::make_ready_continuable(0);
}

std::shared_ptr<BlobTransform> RenderMaterial_BP_TileArgs::DuplicateInstance(FString NewName)
{
	if (NewName.IsEmpty())
		NewName = Name;

	RenderMaterial_BP_TileArgsPtr MaterialBP = std::make_shared<RenderMaterial_BP_TileArgs>(NewName, Instance(), GetVirtualTextureNumWarmupFrames()); //We would want a new instance every time
	std::shared_ptr<BlobTransform> Result = std::static_pointer_cast<RenderMaterial_BP_TileArgs>(MaterialBP);

	return Result;
}
