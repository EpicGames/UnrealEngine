// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RenderMaterial_BP.h"

#define UE_API TEXTUREGRAPHENGINE_API

/**
 * 
 */
class RenderMaterial_BP_NoTile : public RenderMaterial_BP
{

public:
	RenderMaterial_BP_NoTile(FString InName, UMaterialInterface* InMaterial, int32 InVTNumWarmUpFrames) : RenderMaterial_BP(InName, InMaterial, InVTNumWarmUpFrames) {}
	virtual bool					CanHandleTiles() const override { return false; };
};

typedef std::shared_ptr<RenderMaterial_BP_NoTile> RenderMaterial_BP_NoTilePtr;

class RenderMaterial_BP_TileArgs : public RenderMaterial_BP
{
private:
	FName							MemberName = "TileInfo";

public:
	RenderMaterial_BP_TileArgs(FString InName, UMaterialInterface* InMaterial, int32 InVTNumWarmUpFrames) : RenderMaterial_BP(InName, InMaterial, InVTNumWarmUpFrames) {}
	UE_API void							AddTileArgs(TransformArgs& Args);
	UE_API virtual AsyncPrepareResult		PrepareResources(const TransformArgs& Args) override;
	virtual bool					CanHandleTiles() const override { return true; };
	UE_API virtual std::shared_ptr<BlobTransform> DuplicateInstance(FString NewName) override;
};

typedef std::shared_ptr<RenderMaterial_BP_TileArgs> RenderMaterial_BP_TileArgsPtr;

#undef UE_API
