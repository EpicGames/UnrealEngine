// Copyright Epic Games, Inc. All Rights Reserved.
#include "RenderMaterial_FX.h"
#include "FxMat/FxMaterial.h"
#include "2D/Tex.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "TextureGraphEngineGameInstance.h"
#include "Engine/Canvas.h"
#include "TextureGraphEngine.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include <TextureResource.h>

RenderMaterial_FX::RenderMaterial_FX(FString InName, FxMaterialPtr InFXMaterial) 
	: RenderMaterial(InName)
	, FXMaterial(InFXMaterial)
{
	static constexpr int32 MaxName = 256;
	char FullName[MaxName] = {0};
	FMemory::Memcpy((char*)FullName, (const char*)TCHAR_TO_ANSI(*InName), (size_t)std::min((int32)InName.Len(), MaxName - 1));
	HashValue = std::make_shared<CHash>(DataUtil::Hash((const uint8*)FullName, sizeof(FullName)), true);
}

RenderMaterial_FX::~RenderMaterial_FX()
{

}

CHashPtr RenderMaterial_FX::Hash() const
{
	return HashValue;
}

void RenderMaterial_FX::SetTexture(FName InName, const UTexture* Texture) const
{
	FXMaterial->SetTextureParameterValue(InName, Texture);
}

void RenderMaterial_FX::SetArrayTexture(FName InName, const std::vector<const UTexture*>& Textures) const
{
	FXMaterial->SetArrayTextureParameterValue(InName, Textures);
}

void RenderMaterial_FX::SetInt(FName InName, int32 Value) const
{
	FXMaterial->SetScalarParameterValue(InName, Value);
}

void RenderMaterial_FX::SetFloat(FName InName, float Value) const
{
	FXMaterial->SetScalarParameterValue(InName, Value);
}

void RenderMaterial_FX::SetColor(FName InName, const FLinearColor& Value) const
{
	FXMaterial->SetVectorParameterValue(InName, Value);
}

void RenderMaterial_FX::SetIntVector4(FName InName, const FIntVector4& Value) const
{
	FXMaterial->SetVectorParameterValue(InName, Value);
}

void RenderMaterial_FX::SetMatrix(FName InName, const FMatrix& Value) const
{
	FXMaterial->SetMatrixParameterValue(InName, Value);
}

std::shared_ptr<BlobTransform> RenderMaterial_FX::DuplicateInstance(FString NewName)
{
	if (NewName.IsEmpty())
		NewName = Name;

	check(FXMaterial);
	FxMaterialPtr Clone = FXMaterial->Clone();
	check(Clone);
	
	return std::static_pointer_cast<BlobTransform>(std::make_shared<RenderMaterial_FX>(NewName, Clone));
}

void RenderMaterial_FX::BlitTo(FRHICommandListImmediate& RHI, UTextureRenderTarget2D* RenderTarget, const RenderMesh* MeshObj, int32 TargetId) const
{
	check(RenderTarget);

	FTextureRenderTargetResource* RTRes = RenderTarget->GetRenderTargetResource();
	check(RTRes);

	FTextureRHIRef TextureRHI = RTRes->GetTextureRHI();
	check(TextureRHI);

	TextureRHI->SetName(FName(*RenderTarget->GetName()));
	FXMaterial->Blit(RHI, TextureRHI, MeshObj, TargetId);
}


void RenderMaterial_FX::Bind(int32 Value, const ResourceBindInfo& BindInfo)
{
	check(FXMaterial);
	FXMaterial->SetScalarParameterValue(*BindInfo.Target, Value);
}

void RenderMaterial_FX::Bind(float Value, const ResourceBindInfo& BindInfo)
{
	check(FXMaterial);
	FXMaterial->SetScalarParameterValue(*BindInfo.Target, Value);
}

void RenderMaterial_FX::Bind(const FLinearColor& Value, const ResourceBindInfo& BindInfo)
{
	check(FXMaterial);
	FXMaterial->SetVectorParameterValue(*BindInfo.Target, Value);
}

void RenderMaterial_FX::Bind(const FIntVector4& Value, const ResourceBindInfo& BindInfo)
{
	check(FXMaterial);
	FXMaterial->SetVectorParameterValue(*BindInfo.Target, Value);
}

void RenderMaterial_FX::Bind(const FMatrix& Value, const ResourceBindInfo& BindInfo)
{
	check(FXMaterial);
	FXMaterial->SetMatrixParameterValue(*BindInfo.Target, Value);
}

void RenderMaterial_FX::BindStruct(const char* ValueAddress, size_t StructSize, const ResourceBindInfo& BindInfo)
{
	FXMaterial->SetStructParameterValue(*BindInfo.Target, ValueAddress, StructSize);
}

void RenderMaterial_FX::BindScalarArray(const char* StartingAddress, size_t TypeSize, size_t ArraySize, const ResourceBindInfo& BindInfo)
{
	FXMaterial->SetArrayParameterValue(*BindInfo.Target, StartingAddress, TypeSize, ArraySize);
}

