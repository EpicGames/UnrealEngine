// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RenderMaterial.h"
#include "SamplerStates_FX.h"

#define UE_API TEXTUREGRAPHENGINE_API

class FxMaterial;
typedef std::shared_ptr<FxMaterial>	FxMaterialPtr;

class RenderMaterial_FX : public RenderMaterial
{
protected:
	mutable FxMaterialPtr			FXMaterial;			/// The FX material from the shader
	CHashPtr						HashValue;			/// The hash for the material
	
public:
									UE_API RenderMaterial_FX(FString InName, FxMaterialPtr InFXMaterial);
	UE_API virtual							~RenderMaterial_FX() override;

	//////////////////////////////////////////////////////////////////////////
	/// BlobTransform implementation
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual void					Bind(int32 Value, const ResourceBindInfo& BindInfo) override;
	UE_API virtual void					Bind(float Value, const ResourceBindInfo& BindInfo) override;
	UE_API virtual void					Bind(const FLinearColor& Value, const ResourceBindInfo& BindInfo) override;
	UE_API virtual void					Bind(const FIntVector4& Value, const ResourceBindInfo& BindInfo) override;
	UE_API virtual void					Bind(const FMatrix& Value, const ResourceBindInfo& BindInfo) override;
	UE_API virtual void					BindStruct(const char* ValueAddress, size_t StructSize, const ResourceBindInfo& BindInfo) override;
	UE_API virtual void					BindScalarArray(const char* StartingAddress, size_t TypeSize, size_t ArraySize, const ResourceBindInfo& BindInfo) override;

	UE_API virtual CHashPtr				Hash() const override;
	UE_API virtual std::shared_ptr<BlobTransform> DuplicateInstance(FString NewName) override;

	//////////////////////////////////////////////////////////////////////////
	/// RenderMaterial Implementation
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual void					BlitTo(FRHICommandListImmediate& RHI, UTextureRenderTarget2D* RenderTarget, const RenderMesh* MeshObj, int32 TargetId) const override;
	UE_API virtual void					SetTexture(FName InName, const UTexture* Texture) const override;
	UE_API virtual void					SetArrayTexture(FName InName, const std::vector<const UTexture*>& Textures) const override;
	UE_API virtual void					SetInt(FName InName, int32 Value) const override;
	UE_API virtual void					SetFloat(FName InName, float Value) const override;
	UE_API virtual void					SetColor(FName InName, const FLinearColor& Value) const override;
	UE_API virtual void					SetIntVector4(FName InName, const FIntVector4& Value) const override;
	UE_API virtual void					SetMatrix(FName InName, const FMatrix& Value) const override;

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE FxMaterialPtr		FxMaterial() const { return FXMaterial; }
};



typedef std::shared_ptr<RenderMaterial_FX> RenderMaterial_FXPtr;

#undef UE_API
