// Copyright Epic Games, Inc. All Rights Reserved.
#include "RenderMaterial_FX_Combined.h"
#include "FxMat/FxMaterial.h"
#include "Job/JobArgs.h"
#include "Job/Job.h"
#include "Transform/BlobTransform.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Device/FX/Device_FX.h"
#include "2D/TexArray.h"

DEFINE_LOG_CATEGORY(LogRenderMaterial_FX_NoTile);

const FName RenderMaterial_FX_Combined::MemberName = "TileInfo";

RenderMaterial_FX_Combined::RenderMaterial_FX_Combined(FString InName, FxMaterialPtr InFXMaterial) : RenderMaterial_FX(InName, InFXMaterial)
{
	verifyf(InFXMaterial->DoesMemberExist(MemberName), TEXT("Please add member %s to parameter of shader and modify shader enviornment"), *MemberName.ToString());
}

RenderMaterial_FX_Combined::~RenderMaterial_FX_Combined()
{
}

void RenderMaterial_FX_Combined::AddTileArgs(TransformArgs& Args)
{
	FTileInfo TileInfo;
	auto TileArgs = ARG_TILEINFO(TileInfo, TCHAR_TO_ANSI(*(MemberName.ToString())));
	const_cast<Job*>(Args.JobObj)->AddArg(TileArgs);
}

AsyncPrepareResult RenderMaterial_FX_Combined::PrepareResources(const TransformArgs& Args)
{
	AddTileArgs(const_cast<TransformArgs&>(Args));

	std::vector<std::decay_t<AsyncPrepareResult>, std::allocator<std::decay_t<AsyncPrepareResult>>> Promises;

	for (auto BlobToCombine : ToBeCombined)
	{
		int rows = BlobToCombine->Rows();
		int cols = BlobToCombine->Cols();

		ResourceBindInfo BindInfo;
		BindInfo.bIsCombined = true;
		BindInfo.NumTilesX = rows;
		BindInfo.NumTilesY = cols;
		BindInfo.Dev = Device_FX::Get();

		Promises.push_back(BlobToCombine->PrepareForWrite(BindInfo));
	}

	if (Promises.empty())
		return cti::make_ready_continuable(0);

	return cti::make_continuable<int32>([Promises = std::move(Promises)](auto&& Promise) mutable
	{
		cti::when_all(Promises.begin(), Promises.end()).then([FWD_PROMISE(Promise)]() mutable
		{
			/// Resolve the Promise on game thread
			Util::OnGameThread([FWD_PROMISE(Promise)]() mutable
			{
				Promise.set_value(0);
			});
		});
	});
}

std::shared_ptr<BlobTransform> RenderMaterial_FX_Combined::DuplicateInstance(FString NewName)
{
	if (NewName.IsEmpty())
		NewName = Name;

	check(FXMaterial);
	FxMaterialPtr Clone = FXMaterial->Clone();
	check(Clone);
	RenderMaterial_FX_CombinedPtr noTileMat = std::make_shared<RenderMaterial_FX_Combined>(NewName, Clone);
	return std::static_pointer_cast<BlobTransform>(noTileMat);
}

AsyncTransformResultPtr RenderMaterial_FX_Combined::Exec(const TransformArgs& Args)
{
	return RenderMaterial::Exec(Args);
}

TiledBlobPtr RenderMaterial_FX_Combined::AddBlobToCombine(TiledBlobPtr BlobToCombine)
{
	ToBeCombined.Add(BlobToCombine);
	return BlobToCombine;
}

void RenderMaterial_FX_Combined::FreeCombinedBlob()
{
	for (auto BlobToCombine : ToBeCombined)
	{
		DeviceBufferRef SourceBuffer = BlobToCombine->GetBufferRef();
		DeviceBuffer_FX* SourceFXBuffer = static_cast<DeviceBuffer_FX*>(SourceBuffer.get());
		TexArrayPtr SourceTexArray = std::static_pointer_cast<TexArray>(SourceFXBuffer->GetTexture());

		if (SourceTexArray->GetRenderTargetArray()->GetResource())
			SourceTexArray->GetRenderTargetArray()->ReleaseResource();
	}	

	ToBeCombined.Empty();
}

