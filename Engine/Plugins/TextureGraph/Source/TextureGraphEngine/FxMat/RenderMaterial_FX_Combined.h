// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RenderMaterial_FX.h"
#include "Data/TiledBlob.h"

#define UE_API TEXTUREGRAPHENGINE_API

DECLARE_LOG_CATEGORY_EXTERN(LogRenderMaterial_FX_NoTile, All, All);




class RenderMaterial_FX_Combined : public RenderMaterial_FX
{
private:
	
	TArray<TiledBlobPtr>					ToBeCombined;		/// To save under process blobs. This array ensures that multiple SRVs can be passed to the shader

public:
	UE_API RenderMaterial_FX_Combined(FString InName, FxMaterialPtr InFXMaterial);
	UE_API virtual									~RenderMaterial_FX_Combined() override;

	UE_API void									AddTileArgs(TransformArgs& Args);
	UE_API virtual AsyncPrepareResult				PrepareResources(const TransformArgs& Args) override;
	virtual bool							CanHandleTiles() const override { return true; };
	UE_API virtual std::shared_ptr<BlobTransform>	DuplicateInstance(FString NewName) override;
	UE_API virtual AsyncTransformResultPtr			Exec(const TransformArgs& Args) override;
	UE_API TiledBlobPtr							AddBlobToCombine(TiledBlobPtr BlobToCombine);
	UE_API void									FreeCombinedBlob();

	static UE_API const FName						MemberName; /// Default struct parameter for tile
};

typedef std::shared_ptr<RenderMaterial_FX_Combined> RenderMaterial_FX_CombinedPtr;

#undef UE_API
