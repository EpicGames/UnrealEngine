// Copyright Epic Games, Inc. All Rights Reserved.
#include "RenderMaterial_FX_MinMax.h"
#include "FxMat/FxMaterial.h"
#include "Job/JobArgs.h"
#include "Job/Job.h"
#include "Transform/BlobTransform.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include "Device/FX/Device_FX.h"
#include "2D/Tex.h"
#include <TextureResource.h>

DEFINE_LOG_CATEGORY(LogRenderMaterial_FX_MinMax);

RenderMaterial_FX_MinMax::RenderMaterial_FX_MinMax(FString InName, FxMaterialPtr InMaterial, FxMaterialPtr InSecondPassMaterial) : RenderMaterial_FX(InName, InMaterial)
{
	SecondPassMaterial = InSecondPassMaterial;
}

RenderMaterial_FX_MinMax::~RenderMaterial_FX_MinMax()
{
	
}

AsyncPrepareResult RenderMaterial_FX_MinMax::PrepareResources(const TransformArgs& Args)
{
	int Width = SourceDesc.Width;
	int Height = SourceDesc.Height;
	std::vector<std::decay_t<AsyncTexPtr>, std::allocator<std::decay_t<AsyncTexPtr>>> Promises;

	while (Width > 1 && Height > 1)
	{
		Width = FMath::Max(Width >> 1, 1);
		Height = FMath::Max(Height >> 1, 1);

		if (Width > 1 && Height > 1)
		{
			BufferDescriptor Desc;
			Desc.Width = Width;
			Desc.Height = Height;
			Desc.Format = SourceDesc.Format;
			Desc.ItemsPerPoint = SourceDesc.ItemsPerPoint;
			Desc.bIsSRGB = SourceDesc.bIsSRGB;

			DownsampledResultTargets.Add(Device_FX::Get()->AllocateRenderTarget(Desc));
		}
	}

	return cti::make_ready_continuable(0);
}

AsyncTransformResultPtr RenderMaterial_FX_MinMax::Exec(const TransformArgs& Args)
{
	check(IsInRenderingThread());

	TexPtr Source; // Source Texture
	TexPtr Target;

	int Width = SourceDesc.Width;
	int Height = SourceDesc.Height;

	for (int PassIndex = 0; PassIndex < DownsampledResultTargets.Num() + 1; PassIndex++)
	{
		Width = FMath::Max(Width >> 1, 1);
		Height = FMath::Max(Height >> 1, 1);
		
		if (Width == 1 && Height == 1)
		{
			BlobPtr TargetBlob = Args.Target.lock();

			check(TargetBlob);

			check(TargetBlob->GetBufferRef().GetDeviceType() == DeviceType::FX);
			DeviceBuffer_FX* TargetBuffer = static_cast<DeviceBuffer_FX*>(TargetBlob->GetBufferRef().get());

			Target = TargetBuffer->GetTexture();
			check(Target.get());
		}
		else
		{
			Target = DownsampledResultTargets[PassIndex];
		}

		FTextureRenderTarget2DResource* RTRes = (FTextureRenderTarget2DResource*)Target->GetRenderTarget()->GetRenderTargetResource();
		check(RTRes != nullptr);

		if (PassIndex > 0)
		{
			FXMaterial = SecondPassMaterial;

			SetSourceTexture(*Source.get());		
			SetFloat("DX", 1 / (float)Source->GetWidth());
			SetFloat("DY", 1 / (float)Source->GetHeight());
		}
		
		auto& RHI = Device_FX::Get()->RHI();
		
		FTextureRHIRef TextureRHI = RTRes->GetTextureRHI();
		check(TextureRHI);
		TextureRHI->SetName(FName(Target->GetRenderTarget()->GetName()));

		FXMaterial->Blit(RHI, TextureRHI, Args.Mesh, Args.TargetId);

		Source = Target;
	}

	TransformResultPtr Result = std::make_shared<TransformResult>();
	Result->Target = Args.JobObj->GetResultRef();

	return cti::make_ready_continuable(Result);
}


