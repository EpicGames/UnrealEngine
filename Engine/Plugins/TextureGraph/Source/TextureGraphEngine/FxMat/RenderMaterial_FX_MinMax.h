// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RenderMaterial_FX.h"
#include "Data/TiledBlob.h"

#define UE_API TEXTUREGRAPHENGINE_API

DECLARE_LOG_CATEGORY_EXTERN(LogRenderMaterial_FX_MinMax, All, All);

class Tex;
typedef std::shared_ptr<Tex>		TexPtr;

class RenderMaterial_FX_MinMax : public RenderMaterial_FX
{
private:
	
	FxMaterialPtr					SecondPassMaterial;
	BufferDescriptor				SourceDesc;
	TArray<TexPtr>					DownsampledResultTargets;		/// used for creating intermediate targets.
	
public:
	UE_API RenderMaterial_FX_MinMax(FString InName, FxMaterialPtr InMaterial, FxMaterialPtr InSecondPassMaterial);
	UE_API virtual ~RenderMaterial_FX_MinMax() override;

	UE_API virtual AsyncPrepareResult		PrepareResources(const TransformArgs& Args) override;
	UE_API virtual AsyncTransformResultPtr	Exec(const TransformArgs& Args) override;
	
	void							SetDescriptor(const BufferDescriptor& InDesc) { SourceDesc = InDesc;}
};

typedef std::shared_ptr<RenderMaterial_FX_MinMax> RenderMaterial_FX_MinMaxPtr;

#undef UE_API
