// Copyright Epic Games, Inc. All Rights Reserved.
#include "RenderMaterial_Thumbnail.h"
#include "2D/Tex.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include "Device/FX/Device_FX.h"
#include "Engine/Canvas.h"
#include "Engine/Texture2D.h"
#include "Job/JobBatch.h"
#include "TextureResource.h"
#include <CanvasTypes.h>
#include <Engine/World.h>
#include <SceneInterface.h>


////////////////////////////////////////////////////////////////////////// RenderMaterial_Thumbnail //////////////////////////////////////////////////////////////////////////

RenderMaterial_Thumbnail::RenderMaterial_Thumbnail(FString InName, UMaterialInterface* InMaterial) : RenderMaterial_BP(InName, InMaterial, 0)
{
}

bool IsPixelFormatResizeable(EPixelFormat pixelFormat)
{
	return
		pixelFormat == PF_A8R8G8B8 ||
		pixelFormat == PF_R8G8B8A8 ||
		pixelFormat == PF_B8G8R8A8 ||
		pixelFormat == PF_R8G8B8A8_SNORM ||
		pixelFormat == PF_R8G8B8A8_UINT;
}

AsyncPrepareResult RenderMaterial_Thumbnail::PrepareResources(const TransformArgs& Args)
{
	FeatureLevel = GWorld->Scene->GetFeatureLevel();
	return RenderMaterial::PrepareResources(Args);
}

std::shared_ptr<BlobTransform> RenderMaterial_Thumbnail::DuplicateInstance(FString InName)
{
	if (InName.IsEmpty())
		InName = Name;

	std::shared_ptr<RenderMaterial_Thumbnail> MaterialThm = std::make_shared<RenderMaterial_Thumbnail>(InName, Instance()); //We would want a new instance every time

	MaterialThm->FeatureLevel = FeatureLevel;
	MaterialThm->MaterialInstanceValidated = MaterialInstanceValidated;
	std::shared_ptr<BlobTransform> Result = std::static_pointer_cast<RenderMaterial>(MaterialThm);

	return Result;
}

void RenderMaterial_Thumbnail::BlitTo(FRHICommandListImmediate& RHI, UTextureRenderTarget2D* DstRT, const RenderMesh* MeshObj, int32 TargetId) const
{
	check(IsInRenderingThread());
	int32 TargetWidth = DstRT->SizeX;
	int32 TargetHeight = DstRT->SizeY;

	UTextureRenderTarget2D* RenderTargetTexture = DstRT;

	FTextureRenderTargetResource* RTRes = RenderTargetTexture->GetRenderTargetResource();

	check(RTRes);

	TSharedPtr<FCanvas> RenderCanvas = MakeShared<FCanvas>(RTRes, nullptr, FGameTime::GetTimeSinceAppStart(), FeatureLevel);

	Canvas->Init(TargetWidth, TargetHeight, nullptr, RenderCanvas.Get());

	Canvas->Update();

	RTRes->FlushDeferredResourceUpdate(RHI);

	DrawMaterial(MaterialInstance.Get(), FVector2D(0, 0), FVector2D(TargetWidth, TargetHeight), FVector2D(0, 0));

	RenderCanvas->Flush_RenderThread(RHI);
	Canvas->Canvas = nullptr;
}
