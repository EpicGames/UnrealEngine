// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RenderMaterial.h"
#include "UObject/StrongObjectPtr.h"
#include "RenderMaterial_BP.h"

#define UE_API TEXTUREGRAPHENGINE_API

// Subset of RenderMaterial BP, This class will always render a thumbnail from its material instance
class RenderMaterial_Thumbnail : public RenderMaterial_BP
{
public:
	static constexpr uint32	GThumbWidth = 128;			/// Width of standard thumbnail image
	static constexpr uint32	GThumbHeight = 128;		/// Height of the standard thumbnail image

private:
	ERHIFeatureLevel::Type FeatureLevel;

public:
	UE_API RenderMaterial_Thumbnail(FString InName, UMaterialInterface* InMaterial);

	UE_API virtual void BlitTo(FRHICommandListImmediate& RHI, UTextureRenderTarget2D* DstRT, const RenderMesh* MeshObj, int32 TargetId) const override;
	virtual bool CanHandleTiles() const override { return true; };
	UE_API virtual std::shared_ptr<BlobTransform> DuplicateInstance(FString InName) override;
	UE_API virtual AsyncPrepareResult PrepareResources(const TransformArgs& Args) override;
};

typedef std::shared_ptr<RenderMaterial_Thumbnail> RenderMaterial_ThumbPtr;

#undef UE_API
