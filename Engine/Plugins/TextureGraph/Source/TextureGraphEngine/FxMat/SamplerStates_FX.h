// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ShaderParameterMacros.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"

#define DEFAULT_POINT_CLAMP TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI()
#define DEFAULT_POINT_WRAP TStaticSamplerState<SF_Point, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI()
#define DEFAULT_POINT_BORDER TStaticSamplerState<SF_Point, AM_Border, AM_Border, AM_Border>::GetRHI()
#define DEFAULT_LINEAR_CLAMP TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI()
#define DEFAULT_LINEAR_WRAP TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI()
#define DEFAULT_LINEAR_BORDER TStaticSamplerState<SF_Bilinear, AM_Border, AM_Border, AM_Border>::GetRHI()

BEGIN_SHADER_PARAMETER_STRUCT(FStandardSamplerStates, )
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_SAMPLER, TShaderResourceParameterTypeInfo<FRHISamplerState*>, FRHISamplerState*, Clamp, , = DEFAULT_POINT_CLAMP, EShaderPrecisionModifier::Float, TEXT("SamplerState"), false)
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_SAMPLER, TShaderResourceParameterTypeInfo<FRHISamplerState*>, FRHISamplerState*, Wrap, ,  = DEFAULT_POINT_WRAP , EShaderPrecisionModifier::Float, TEXT("SamplerState"), false)
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_SAMPLER, TShaderResourceParameterTypeInfo<FRHISamplerState*>, FRHISamplerState*, NoBorder, , = DEFAULT_POINT_BORDER, EShaderPrecisionModifier::Float, TEXT("SamplerState"), false)
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_SAMPLER, TShaderResourceParameterTypeInfo<FRHISamplerState*>, FRHISamplerState*, Linear_Clamp, , = DEFAULT_LINEAR_CLAMP, EShaderPrecisionModifier::Float, TEXT("SamplerState"), false)
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_SAMPLER, TShaderResourceParameterTypeInfo<FRHISamplerState*>, FRHISamplerState*, Linear_Wrap, , = DEFAULT_LINEAR_WRAP, EShaderPrecisionModifier::Float, TEXT("SamplerState"), false)
	INTERNAL_SHADER_PARAMETER_EXPLICIT(UBMT_SAMPLER, TShaderResourceParameterTypeInfo<FRHISamplerState*>, FRHISamplerState*, Linear_Border, , = DEFAULT_LINEAR_BORDER, EShaderPrecisionModifier::Float, TEXT("SamplerState"), false)
END_SHADER_PARAMETER_STRUCT()


FORCEINLINE void FStandardSamplerStates_Setup(FStandardSamplerStates& params) {}
