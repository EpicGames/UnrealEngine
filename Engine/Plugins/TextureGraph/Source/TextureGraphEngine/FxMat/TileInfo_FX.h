// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ShaderParameterMacros.h"

BEGIN_SHADER_PARAMETER_STRUCT(FTileInfo, )
	SHADER_PARAMETER(float, TileX)
	SHADER_PARAMETER(float, TileCountX)
	SHADER_PARAMETER(float, TileWidth)
	SHADER_PARAMETER(float, TileY)
	SHADER_PARAMETER(float, TileCountY)
	SHADER_PARAMETER(float, TileHeight)
END_SHADER_PARAMETER_STRUCT()

