// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "2D/TextureType.h"
#include "ProceduralMeshComponent.h" 

typedef FVector4			Vector4;
typedef FVector				Vector3;
typedef FVector2D			Vector2;
typedef FProcMeshTangent	Tangent;
