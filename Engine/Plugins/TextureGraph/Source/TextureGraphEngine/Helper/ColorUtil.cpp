// Copyright Epic Games, Inc. All Rights Reserved.
#include "ColorUtil.h"
#include "TextureGraphEngine/Helper/MathUtils.h"

const FLinearColor ColorUtil::DefaultColor(TextureType Type)
{
	switch (Type)
	{
	case TextureType::Diffuse:
		return DefaultDiffuse();
	case TextureType::Specular:
		return DefaultSpecular();
	case TextureType::Albedo:
		return DefaultAlbedo();
	case TextureType::Metalness:
		return DefaultMetalness();
	case TextureType::Normal:
		return DefaultNormal();
	case TextureType::Displacement:
		return DefaultDisplacement();
	case TextureType::Opacity:
		return DefaultOpacity();
	case TextureType::Roughness:
		return DefaultRoughness();
	case TextureType::AO:
		return DefaultAO();
	case TextureType::Curvature:
		return DefaultCurvature();
	case TextureType::Preview:
		return DefaultPreview();
	default:
		return FLinearColor::Black;
	}
}

const FLinearColor ColorUtil::DefaultDiffuse()
{
	return FLinearColor::FromSRGBColor(FColor(127, 127, 127, 255));
}

const FLinearColor ColorUtil::DefaultSpecular()
{
	return FLinearColor::Black;
}

const FLinearColor ColorUtil::DefaultAlbedo()
{
	return FLinearColor::FromSRGBColor(FColor(127, 127, 127, 255));
}

const FLinearColor ColorUtil::DefaultMetalness()
{
	return FLinearColor::Black;
}

const FLinearColor ColorUtil::DefaultNormal()
{
	return FLinearColor(0.5f, 0.5f, 1.0f, 1.0f);
}

const FLinearColor ColorUtil::DefaultDisplacement()
{
	return FLinearColor(0.5f, 0.5f, 0.5f, 1.0f);
}

const FLinearColor ColorUtil::DefaultOpacity()
{
	return FLinearColor::White;
}

const FLinearColor ColorUtil::DefaultRoughness()
{
	return FLinearColor(0.9f, 0.9f, 0.9f, 1);
}

const FLinearColor ColorUtil::DefaultAO()
{
	return FLinearColor::White;
}

const FLinearColor ColorUtil::DefaultCurvature()
{
	return FLinearColor(0.5f, 0.5f, 0.5f, 1);
}
const FLinearColor ColorUtil::DefaultPreview()
{
	return FLinearColor::Black;
}

float ColorUtil::GetHue(const FColor& Color)
{
	float R = Color.R / 255.0f;
	float G = Color.G / 255.0f;
	float B = Color.B / 255.0f;

	float Max = FMath::Max3(R, G, B);
	float Min = FMath::Min3(R, G, B);

	if (Max == Min)
	{
		return 0; // achromatic
	}
	else
	{
		float Hue = 0;
		float Delta = Max - Min;

		if (Max == R)
		{
			Hue = (G - B) / Delta + (G < B ? 6 : 0);
		}
		else if (Max == G)
		{
			Hue = (B - R) / Delta + 2;
		}
		else if (Max == B)
		{
			Hue = (R - G) / Delta + 4;
		}

		return Hue /= 6.0f;
	}
}

float ColorUtil::GetSquaredDistance(FLinearColor Current, FLinearColor Match)
{
	const float RedDifference = Current.R - Match.R;
	const float GreenDifference = Current.G - Match.G;
	const float BlueDifference = Current.B - Match.B;

	return (RedDifference * RedDifference + GreenDifference * GreenDifference + BlueDifference * BlueDifference);
}

bool ColorUtil::IsColorBlack(const FLinearColor& C, bool IgnoreAlpha)
{
	static constexpr float Epsilon = std::numeric_limits<float>::epsilon();

	if (std::abs(C.R) <= Epsilon &&
		std::abs(C.G) <= Epsilon &&
		std::abs(C.B) <= Epsilon &&
		(IgnoreAlpha || (!IgnoreAlpha && std::abs(C.A) <= Epsilon)))
		return true;

	return false;
}

bool ColorUtil::IsColorNear(const FLinearColor& Color, const FLinearColor& Ref, bool IgnoreAlpha)
{
	FLinearColor Diff = Color - Ref;
	return IsColorBlack(Diff, IgnoreAlpha);
}

FLinearColor ColorUtil::HSV2RGB(FLinearColor C)
{
	FVector4f K = FVector4f(1.0f, 2.0f / 3.0f, 1.0f / 3.0f, 3.0f);
	FVector3f Kx(K.X, K.X, K.X);
	FVector3f P;
	P.X = FMath::Abs(FMath::Frac(C.R + K.X) * 6.0 - K.W);
	P.Y = FMath::Abs(FMath::Frac(C.R + K.Y) * 6.0 - K.W);
	P.Z = FMath::Abs(FMath::Frac(C.R + K.Z) * 6.0 - K.W);

	FVector3f D = P - Kx;
	D.X = FMath::Clamp(D.X, 0.0f, 1.0f);
	D.Y = FMath::Clamp(D.Y, 0.0f, 1.0f);
	D.Z = FMath::Clamp(D.Z, 0.0f, 1.0f);

	FVector3f RGB = C.B * FMath::Lerp(Kx, D, C.G);

	return FLinearColor(RGB.X, RGB.Y, RGB.Z, 1.0f);
}

FLinearColor ColorUtil::RGB2HSV(FLinearColor C)
{
	FVector4f K = FVector4f(0.0f, -1.0f / 3.0f, 2.0f / 3.0f, -1.0f);
	FLinearColor P = FMath::Lerp(FLinearColor(C.B, C.G, K.W, K.Z), FLinearColor(C.G, C.B, K.X, K.Y), MathUtils::Step(C.B, C.G));
	FLinearColor Q = FMath::Lerp(FLinearColor(P.R, P.G, P.B, C.R), FLinearColor(C.R, P.G, P.B, P.R), MathUtils::Step(P.R, C.R));
	float D = Q.R - FMath::Min(Q.A, Q.G);
	float E = 1.0e-10f;
	FVector3f HSV(FMath::Abs(Q.B + (Q.A - Q.G) / (6.0 * D + E)), D / (Q.R + E), Q.R);

	return FLinearColor(HSV.X, HSV.Y, HSV.Z, 1.0f);
}

FLinearColor ColorUtil::HSVTweak(FLinearColor C, float H, float S, float V)
{
	FLinearColor ClrHSV = RGB2HSV(C);
	ClrHSV *= FLinearColor(H, S, V, 1.0f);
	FLinearColor Result = HSV2RGB(ClrHSV);
	Result.A = C.A;

	return Result;
}

bool ColorUtil::IsColorWhite(const FLinearColor& Color, bool IgnoreAlpha) { return IsColorNear(Color, FLinearColor::White, IgnoreAlpha); }
bool ColorUtil::IsColorGray(const FLinearColor& Color, bool IgnoreAlpha) { return IsColorNear(Color, FLinearColor::Gray, IgnoreAlpha); }
bool ColorUtil::IsColorRed(const FLinearColor& Color, bool IgnoreAlpha) { return IsColorNear(Color, FLinearColor::Red, IgnoreAlpha); }
bool ColorUtil::IsColorGreen(const FLinearColor& Color, bool IgnoreAlpha) { return IsColorNear(Color, FLinearColor::Green, IgnoreAlpha); }
bool ColorUtil::IsColorBlue(const FLinearColor& Color, bool IgnoreAlpha) { return IsColorNear(Color, FLinearColor::Blue, IgnoreAlpha); }
bool ColorUtil::IsColorYellow(const FLinearColor& Color, bool IgnoreAlpha) { return IsColorNear(Color, FLinearColor::Yellow, IgnoreAlpha); }
bool ColorUtil::IsColorMagenta(const FLinearColor& Color, bool IgnoreAlpha) { return IsColorNear(Color, FLinearColor(1, 0, 1, 1), IgnoreAlpha); }

FString ColorUtil::GetColorName(FLinearColor Color)
{
	auto HSV = Color.LinearRGBToHSV();
	
	FString prefix = "";
	float h = HSV.R;
	float s = HSV.G * 100;
	float v = HSV.B * 100;

	if (v < 1) return "Pitch Black";
	if (v < 6) return "Black";

	if (s < 15) //Grayscales
		{
		if (v < 30) return "Dark Gray";
		if (v < 54) return "Gray";
		if (v < 80) return "Bright Gray";
		return "White";
		}

	if (s < 60) //Desaturated
		{
		if (v > 50)
		{
			prefix = "Pastel ";
		}
		else if (v < 27) prefix = "Murky ";
		else prefix = "Faded ";
		}

	//Saturated
	else if (v > 80) prefix = "Bright ";
	else if (v < 40) prefix = "Dark ";

	if (h < 10)
	{
		return prefix + "Red";
	}
	if (h < 39)
	{
		if (prefix == "Pastel ") return "Beige";
		else if (v < 60 || prefix == "Dark " || prefix == "Murky " || prefix == "Faded ") return prefix + "Brown";
		else return prefix + "Orange";
	}
	if (h < 62) return prefix + "Yellow";
	if (h < 160) return prefix + "Green";
	if (h < 217) return prefix + "Cyan";
	if (h < 259) return prefix + "Blue";
	if (h < 284) return prefix + "Purple";
	if (h < 306) return prefix + "Magenta";
	if (h < 333) return prefix + "Pink";
	return prefix + "Red";
}
