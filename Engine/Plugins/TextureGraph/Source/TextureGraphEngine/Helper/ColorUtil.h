// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "2D/TextureType.h"

#define UE_API TEXTUREGRAPHENGINE_API

class ColorUtil
{
public:
	//Converting from Array to Functions to avoid static fiasco problem with CPP
	static UE_API const FLinearColor			DefaultColor(TextureType Type);
	static UE_API const FLinearColor			DefaultDiffuse();
	static UE_API const FLinearColor			DefaultSpecular();
	static UE_API const FLinearColor			DefaultAlbedo();
	static UE_API const FLinearColor			DefaultMetalness();
	static UE_API const FLinearColor			DefaultNormal();
	static UE_API const FLinearColor			DefaultDisplacement();
	static UE_API const FLinearColor			DefaultOpacity();
	static UE_API const FLinearColor			DefaultRoughness();
	static UE_API const FLinearColor			DefaultAO();
	static UE_API const FLinearColor			DefaultCurvature();
	static UE_API const FLinearColor			DefaultPreview();

	static UE_API float						GetHue(const FColor& Color);
	static UE_API float						GetSquaredDistance(FLinearColor Current, FLinearColor Match);
	static UE_API FString						GetColorName(FLinearColor Color);

	static UE_API FLinearColor					HSV2RGB(FLinearColor C);
	static UE_API FLinearColor					RGB2HSV(FLinearColor C);
	static UE_API FLinearColor					HSVTweak(FLinearColor C, float H, float S, float V);

	static UE_API bool							IsColorBlack(const FLinearColor& Color, bool IgnoreAlpha = true);
	static UE_API bool							IsColorWhite(const FLinearColor& Color, bool IgnoreAlpha = true);
	static UE_API bool							IsColorGray(const FLinearColor& Color, bool IgnoreAlpha = true);
	static UE_API bool							IsColorRed(const FLinearColor& Color, bool IgnoreAlpha = true);
	static UE_API bool							IsColorGreen(const FLinearColor& Color, bool IgnoreAlpha = true);
	static UE_API bool							IsColorBlue(const FLinearColor& Color, bool IgnoreAlpha = true);
	static UE_API bool							IsColorYellow(const FLinearColor& Color, bool IgnoreAlpha = true);
	static UE_API bool							IsColorMagenta(const FLinearColor& Color, bool IgnoreAlpha = true);
	static UE_API bool							IsColorNear(const FLinearColor& Color, const FLinearColor& Ref, bool IgnoreAlpha = true);
};

#undef UE_API
