// Copyright Epic Games, Inc. All Rights Reserved.
#include "DataUtil.h"
#include "Async/ParallelFor.h"
#include "CoreTypes.h"
#include "Data/Blobber.h"
#include "Helper/Promise.h"
#include "TextureGraphEngine.h"
#include "Util.h"

DEFINE_LOG_CATEGORY(LogData);

//////////////////////////////////////////////////////////////////////////
/// Hash
//////////////////////////////////////////////////////////////////////////
HashType DataUtil::Hash_One(const uint8* Data, size_t Length, HashType InitialValue /* = GFNVInit */, HashType Prime /* = GFNVPrime */)
{
	/// Most common conditions go first ...
	if (Length > sizeof(HashType))
	{
		check(Length >= sizeof(HashType));

		/// Calculate the number of iterations
		size_t IterLength = ((size_t)(Length / sizeof(HashType))) * sizeof(HashType);

		HashType HashValue = InitialValue;
		for (size_t i = 0; i < IterLength; i += sizeof(HashType))
			HashValue = MX_HASH_VAL(HashValue, Prime, *((HashType*)(Data + i)));

		/// Now iterate the residual Length
		for (size_t i = IterLength; i < Length; i++)
			HashValue = MX_HASH_VAL(HashValue, Prime, (HashType)(*(Data + i)));

		return HashValue;
	}
	else
	{
		check(Length <= sizeof(HashType));
		HashType H = 0;
		memcpy(&H, Data, Length);
		return MX_HASH_VAL(InitialValue, Prime, H);
	}
}

HashType DataUtil::Hash_GenericString_Name(const FString& Name, HashType InitialHash /* = GFNVInit */, HashType Prime /* = GFNVPrime */)
{
	// Rely on the unreal HashString feature
	// The HashValue produces a 32bit key, so offset to spread the keys across the full 64 bits range
	// TODO: Create a true StrCrc64 version ?
	return ((HashType) FCrc::StrCrc32<TCHAR>(*Name, InitialHash)) << 32;
}

HashType DataUtil::Hash(const uint8* Data, size_t Length, HashType InitialHash /* = GFNVInit */, HashType Prime /* = GFNVPrime */)
{
	HashType HashValue;

	/// Decide whether to chunk the HashValue or not [currently we don't allow chuking anything that isn't rounded to chunk size].
	/// It should be easy to remove this constraint later on!
	if (Length <= GMaxChunk * 2 || Length % GMaxChunk != 0)
		HashValue = Hash_One(Data, Length, InitialHash, Prime);
	else
		HashValue = Hash_Chunked(Data, Length, InitialHash, Prime);

	return HashValue;
}

HashType DataUtil::Hash_Chunked(const uint8* Data, size_t Length, HashType InitialValue /* = GFNVInit */, HashType Prime /* = GFNVPrime */)
{
	/// Upto 2 chunks we calculate in one go. This also avoids the division further down
	size_t MaxChunks = Length / DataUtil::GMaxChunk;
	check(MaxChunks > 1);

	std::vector<HashType> results(MaxChunks);

	ParallelFor(MaxChunks, [&](int32 Index)
	{ 
		HashType HashValue = Hash_One(Data + Index * GMaxChunk, GMaxChunk, InitialValue, Prime);
		results[Index] = HashValue;
	});

	/// Now we combine all the HashValues together
	return Hash_One((const uint8*)&results[0], MaxChunks * sizeof(HashType), InitialValue, Prime);
}

size_t DataUtil::GetOptimalHashingSize(size_t Size)
{
	size_t RoundedDataLength = Size;
	/// Round it up
	size_t Remainder = RoundedDataLength % sizeof(HashType);

	/// We need to pad the src data so that it can be hashed efficiently
	if (Remainder != 0)
	{
		RoundedDataLength = Size + sizeof(HashType) - Remainder;
	}

	/// If we have large data then make sure it's aligned to GMaxChunk for efficient hashing
	if (RoundedDataLength > DataUtil::GMaxChunk)
	{
		size_t ChunkRemainder = RoundedDataLength % DataUtil::GMaxChunk;

		if (ChunkRemainder != 0)
		{
			RoundedDataLength = RoundedDataLength + DataUtil::GMaxChunk - ChunkRemainder;
		}
	}

	return RoundedDataLength;
}

HashType DataUtil::Hash(const HashTypeVec& SubHashes, HashType InitialValue /* = GFNVInit */, HashType Prime /* = GFNVPrime */)
{
	check(SubHashes.size());

	if (SubHashes.size() == 1)
		return SubHashes[0];

	const uint8* Data = (const uint8*)&SubHashes[0];
	size_t Length = SubHashes.size() * sizeof(HashType);
	return Hash_One(Data, Length, InitialValue, Prime);
}

HashType DataUtil::Hash(const CHashPtrVec& InSubHashes, HashType InitialValue /* = GFNVInit */, HashType Prime /* = GFNVPrime */)
{
	HashTypeVec SubHashes(InSubHashes.size());

	for (size_t HashIndex = 0; HashIndex < InSubHashes.size(); HashIndex++)
		SubHashes[HashIndex] = InSubHashes[HashIndex]->Value();

	return Hash(SubHashes, InitialValue, Prime);
}

//////////////////////////////////////////////////////////////////////////
CHash::CHash(HashType Value, bool bInIsFinal) : bIsFinal(bInIsFinal), Timestamp(FDateTime::Now())
{
	if (!bIsFinal)
		TempHashValue = std::make_shared<CHash>(Value, true);
	else
		HashValue = Value;
}

CHash::CHash(const CHashPtrVec& Sources) : Timestamp(FDateTime::Now())
{
	check(Sources.size() > 0);
	bIsFinal = true;

	HashTypeVec HashValues(Sources.size());

	for (size_t si = 0; si < Sources.size(); si++)
	{
		HashValues[si] = Sources[si]->Value();
		bIsFinal &= Sources[si]->IsFinal();
	}

	HashType FinalHash = DataUtil::Hash(HashValues);

	if (!bIsFinal)
	{
		/// IMPORTANT: The reason why we want to send isFinal = true in the ctor and then
		/// set it to false later on is because we don't want the temp has to create another
		/// temp HashValue internally, which is what this ctor used below does. We want to 
		/// make _temp, THE temp HashValue object and then set _isFinal to false so that the 
		/// Svc_TempHash is able to resolve it later on
		TempHashValue = std::make_shared<CHash>(FinalHash, true);
		TempHashValue->HashSources = Sources;
		TempHashValue->bIsFinal = false;
	}
	else
	{
		HashSources = Sources;
		HashValue = FinalHash;
	}
}

CHashPtr CHash::ConstructFromSources(const CHashPtrVec& Sources)
{
	CHashPtr HashValue = CHashPtr(new CHash(Sources));

	if (HashValue->TempHashValue)
	{
		for (CHashPtr source : HashValue->TempHashValue->HashSources)
			source->AddLink(HashValue);
	}

	return HashValue;
}

CHash::CHash(CHashPtr Temp) 
	: bIsFinal(false)
	, TempHashValue(!Temp->IsTemp() ? Temp : Temp->TempHashValue)
	, Timestamp(Util::Time())
{
	check(TempHashValue->HashValue != DataUtil::GNullHash);
}

bool CHash::operator == (const CHash& RHS) const
{
	return Value() == RHS.Value();
}

void CHash::CheckLinkCycles(std::unordered_set<CHashPtr>& Chain)
{
	for (CHashPtrW link_ : Linked)
	{
		CHashPtr Link = link_.lock();

		if (Link)
		{
			check(Chain.find(Link) == Chain.end());
			Chain.insert(Link);
			Link->CheckLinkCycles(Chain);
		}
	}
}

void CHash::AddLink(CHashPtrW Link)
{
	/// If this HashValue is already final then we don't need to add any links
	/// that need resolution later on
	if (IsFinal())
		return;

	for (auto Iter : Linked)
	{
		if (Iter.lock() == Link.lock())
			return;
	}

	Linked.push_back(Link);
}

bool CHash::TryFinalise(HashType FinalHash /* = DataUtil::s_nullHash */, bool UpdateBlobber /* = true */)
{
	if (TextureGraphEngine::IsDestroying())
		return false;

	check(IsInGameThread());

	if (TempHashValue != nullptr)
	{
		/// If there's a temp HashValue then the current HashValue must be null
		check(HashValue == DataUtil::GNullHash);

		/// Save the old Value of the temp HashValue
		HashType OldHash = TempHashValue->HashValue;
		IntermediateHashes = TempHashValue->GetIntermediateHashes();

		bool bDidUpdate = TempHashValue->TryFinalise(FinalHash, false);

		if (bDidUpdate)
		{
			if (TempHashValue->IsFinal())
			{
				bIsFinal = true;
				HashValue = TempHashValue->HashValue;

				UpdateLinks();
			}

			if (HashValue != DataUtil::GNullHash && OldHash != HashValue && UpdateBlobber)
			{
				CHashPtr ThisHash = shared_from_this();

				IntermediateHashes.push_back(OldHash);

				/// Now we try to update the mapping in blobber
				for (HashType IntermediateHash : IntermediateHashes)
					TextureGraphEngine::GetBlobber()->UpdateHash(IntermediateHash, ThisHash);
			}

			return true;
		}

		return bDidUpdate;
	}

	/// Cannot finalise a HashValue that's already been finalised!
	bool bShouldUpdate = false;
	bool bDidUpdate = false;
	HashType CurrentHash = HashValue;
	bool bIsTemp = IsTemp();

	if (!HashSources.empty())
	{
		HashTypeVec Sources(HashSources.size());
		bool bIsFinalHash = true;

		for (size_t HashIndex = 0; HashIndex < HashSources.size(); HashIndex++)
		{
			/// Check whether the source HashValue has been finalised or not
			bIsFinalHash &= HashSources[HashIndex]->IsFinal();
			Sources[HashIndex] = HashSources[HashIndex]->Value();

			if (HashSources[HashIndex]->GetTimestamp() > Timestamp)
				bShouldUpdate = true;
		}

		if (bShouldUpdate || (bIsFinalHash != bIsFinal))
		{
			/// Save the current HashValue as we'll need to update the mapping in blobber
			HashType OldHash = HashValue;

			if (FinalHash != 0)
				HashValue = FinalHash;
			else
				HashValue = DataUtil::Hash(Sources);

			if (OldHash != HashValue && UpdateBlobber)
			{
				/// Now we try to update the mapping in blobber
				TextureGraphEngine::GetBlobber()->UpdateHash(OldHash, shared_from_this());
			}

			bDidUpdate = true;
			Timestamp = FDateTime::Now();

			if (bIsFinalHash)
			{
				bIsFinal = true;
				TempHashValue = nullptr;
			}
		}
	}
	else
	{
		bIsFinal = true;
		TempHashValue = nullptr;

		/// Explicitly set to true so that return Value doesn't change if someone changes
		/// the initialisation of shouldUpdate
		bShouldUpdate = true;
	}

	if (bDidUpdate)
	{
		if (!bIsTemp && CurrentHash != HashValue)
			IntermediateHashes.push_back(CurrentHash);

		UpdateLinks();
	}

	return bDidUpdate;
}

void CHash::UpdateLinks()
{
	if (TextureGraphEngine::IsDestroying())
		return;

	if (Linked.empty())
		return;

	check(IsInGameThread());

	auto ThisHash = shared_from_this();

	for (CHashPtrW& Iter : Linked)
	{ 
		CHashPtr Link = Iter.lock();

		if (Link)
		{
			/// Call the linked to update itself
			Link->HandleLinkUpdated(ThisHash);
		}
	}

	/// Ok, the links should've updated over here and We can clear our linked list now
	/// if the hash is final, as they'll never be used after this. 
	if (IsFinal())
		Linked.clear();
}

HashTypeVec CHash::GetIntermediateHashes() const
{
	if (!TempHashValue)
		return IntermediateHashes;

	HashTypeVec CombinedIntermediateHashes = IntermediateHashes;
	CombinedIntermediateHashes.insert(CombinedIntermediateHashes.end(), TempHashValue->IntermediateHashes.begin(), TempHashValue->IntermediateHashes.end());

	return CombinedIntermediateHashes;
}

CHashPtr CHash::UpdateHash(CHashPtr NewHash, CHashPtr PrevHash)
{
	if (PrevHash)
	{
		check(NewHash != PrevHash);

		HashType PrevHashValue = PrevHash->Value();
		NewHash->IntermediateHashes = PrevHash->GetIntermediateHashes();
		NewHash->IntermediateHashes.push_back(PrevHashValue);

		/// Copy some Data over from the previous HashValue
		NewHash->Linked = PrevHash->Linked;
		if (PrevHash->TempHashValue && !PrevHash->TempHashValue->Linked.empty())
			NewHash->Linked.insert(NewHash->Linked.end(), PrevHash->TempHashValue->Linked.begin(), PrevHash->TempHashValue->Linked.end());

		NewHash->HashSources = PrevHash->Sources();

		/// Replace the Data from previous HashValue so that other HashValues that have linked
		/// that particular pointer can see the latest updates as well
		//*prevHash.get() = *newHash.get()
		PrevHash->TempHashValue = nullptr;
		PrevHash->bIsFinal = true;
		PrevHash->HashValue = NewHash->HashValue;
		PrevHash->Timestamp = NewHash->Timestamp;

		if (PrevHashValue != PrevHash->Value() && !TextureGraphEngine::IsDestroying() && TextureGraphEngine::GetBlobber())
		{
			for (HashType IntermediateHash : NewHash->IntermediateHashes)
				TextureGraphEngine::GetBlobber()->UpdateHash(IntermediateHash, NewHash);

			NewHash->UpdateLinks();
		}
	}

	return NewHash;
}

void CHash::HandleLinkUpdated(CHashPtr LinkUpdated)
{
	/// If this has already been finalised then we don't need to do anything. This can happen
	/// if the BlobHasher service got to a blob before this Link got updated
	if (IsFinal())
		return;

	/// Recalculate the new HashValue. This will also propagate to the 
	/// other HashValues linked to this one
	TryFinalise();
}
