// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include <memory>
#include <vector>
#include <list>
#include <unordered_set>

typedef uint64						HashType;
typedef std::vector<HashType>		HashTypeVec;
typedef std::list<HashType>			HashTypeList;

class CHash;
typedef std::shared_ptr<CHash>		CHashPtr;
typedef std::weak_ptr<CHash>		CHashPtrW;
typedef std::vector<CHashPtr>		CHashPtrVec;
typedef std::vector<CHashPtrW>		CHashPtrWVec;

//struct HashType
//{
//	uint64							HashValue;			/// The actual HashValue Value
//};
DECLARE_LOG_CATEGORY_EXTERN(LogData, Log, All);

#define MX_HASH_VAL(HashValue, Prime, val) ((HashValue * Prime) ^ (val))
#define MX_HASH_VAL_DEF(val)		MX_HASH_VAL(DataUtil::GFNVInit, DataUtil::GFNVPrime, (HashType)val)

struct DataUtil
{
	static constexpr HashType		GNullHash = 0; /// 0xDeadBeef;
	//////////////////////////////////////////////////////////////////////////
	/// Hashing related
	//////////////////////////////////////////////////////////////////////////
	static constexpr HashType		GFNVPrime = 0x00000100000001B3;
	static const HashType			GFNVInit = 0xcbf29ce484222325;

	/// Don't change this ... EVER!
	static const size_t				GMaxChunk = 16 * 1024;

	/// Calculate HashValue for one chunk of data
	static TEXTUREGRAPHENGINE_API HashType					Hash(const uint8* data, size_t Length, HashType InitialHash = GFNVInit, HashType Prime = GFNVPrime);
	static TEXTUREGRAPHENGINE_API HashType					Hash_GenericString_Name(const FString& Name, HashType InitialHash = GFNVInit, HashType Prime = GFNVPrime);
	static TEXTUREGRAPHENGINE_API HashType					Hash(const HashTypeVec& SubHashes, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime);
	static TEXTUREGRAPHENGINE_API HashType					Hash(const CHashPtrVec& InSubHashes, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime);

	/// Automatically decide whether to chunk the HashValue or not
	static TEXTUREGRAPHENGINE_API HashType					Hash_One(const uint8* Data, size_t Length, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime);
	static TEXTUREGRAPHENGINE_API HashType					Hash_Chunked(const uint8* data, size_t Length, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime);
	static TEXTUREGRAPHENGINE_API size_t					GetOptimalHashingSize(size_t Size);

	template <typename T>
	static HashType					Hash_Simple(const T& Value, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime) 
	{
		HashType H = 0;
		check(sizeof(Value) <= sizeof(HashType));
		memcpy(&H, &Value, sizeof(T));
		return MX_HASH_VAL(InitialValue, Prime, H);
	}

	//////////////////////////////////////////////////////////////////////////
	/// Specialised versions
	//////////////////////////////////////////////////////////////////////////
	static HashType					Hash_Simple(const FString& Value, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime)
	{
		return Hash_GenericString_Name(Value, InitialValue, Prime); 
	}

	static HashType					Hash_Float(float Value, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime)
	{
		return Hash_Simple<float>(Value, InitialValue, Prime);
	}

	static HashType					Hash_Int32(int32 Value, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime)
	{
		return Hash_Simple<int32>(Value, InitialValue, Prime);
	}

	static HashType					Hash_Double(double Value, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime)
	{
		return Hash_Simple<double>(Value, InitialValue, Prime);
	}

	static HashType					Hash_Bool(bool Value, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime)
	{
		return Hash_Simple<bool>(Value, InitialValue, Prime);
	}

	static HashType					Hash_Simple(const FVector2D& Vec, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime)
	{
		HashType HashValue = InitialValue;

		HashValue = Hash_Float(Vec.X, HashValue, Prime);
		HashValue = Hash_Float(Vec.Y, HashValue, Prime);

		return HashValue;
	}

	static HashType					Hash_Simple(const FVector& Vec, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime)
	{
		HashType HashValue = InitialValue;

		HashValue = Hash_Float(Vec.X, HashValue, Prime);
		HashValue = Hash_Float(Vec.Y, HashValue, Prime);
		HashValue = Hash_Float(Vec.Z, HashValue, Prime);

		return HashValue;
	}

	static HashType					Hash_Simple(const FColor& Color, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime)
	{
		return Hash_Int32((int32)Color.DWColor());
	}

	static HashType					Hash_Simple(const FLinearColor& Color, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime)
	{
		HashType HashValue = InitialValue;

		HashValue = Hash_Float(Color.R, HashValue, Prime);
		HashValue = Hash_Float(Color.G, HashValue, Prime);
		HashValue = Hash_Float(Color.B, HashValue, Prime);
		HashValue = Hash_Float(Color.A, HashValue, Prime);

		return HashValue;
	}

	template <typename TKey, typename TValue>
	static HashType					Hash_Simple(const TMap<TKey, TValue>& Map, HashType InitialValue = GFNVInit, HashType Prime = GFNVPrime)
	{
		HashTypeVec HashValues(Map.Num() * 2);
		size_t i = 0;

		for (auto& Iter : Map)
		{
			const TKey& key = Iter.Key;
			const TValue& Value = Iter.Value;
			HashValues[i++] = Hash_Simple<TKey>(key);
			HashValues[i++] = Hash_Simple<TValue>(Value);
		}

		return DataUtil::Hash(HashValues, InitialValue, Prime);
	}
};

//////////////////////////////////////////////////////////////////////////
/// Hash data structure
//////////////////////////////////////////////////////////////////////////
class CHash : public std::enable_shared_from_this<CHash>
{
private:
	HashType						HashValue = DataUtil::GNullHash;/// The actual Value of the hash
	CHashPtrVec						HashSources;					/// The sources used to construct this hash
	bool							bIsFinal = false;				/// Whether this is a final (immutable hash)
	CHashPtr						TempHashValue;					/// Whether there's a temporary HashValue attached with this HashValue
	FDateTime						Timestamp = FDateTime::Now();	/// The timestamp of when this was last updated

	/// This structure represents a HashValue link. This is required for cases when a temp HashValue is used in 
	/// the calculation of some other complex HashValue. When the temp HashValue finally resolves, we need to 
	/// update the linked HashValues so that our look-ups to the, now old embedded HashValues is successful.
	/// otherwise we'll have no way of connecting complex-temp hashes to the correct hash and 
	/// determine whether a job has already been done before or not
	CHashPtrWVec					Linked;						/// The HashValues that have this HashValue embedded in it. This is kept
																/// temporarily if THIS HASH is a temp HashValue. When THIS HASH gets
																/// updated, we can update the linked parent HashValues

	HashTypeVec						IntermediateHashes;			/// Intermediate hashes that might have been evaluated before this
																/// hash became finalised

	FORCEINLINE bool				HasTempDependency_Internal() const { return (IsTemp() && !HashSources.empty() && TempHashValue->IsTemp()); }
	TEXTUREGRAPHENGINE_API void							AddLink(CHashPtrW Link);
	TEXTUREGRAPHENGINE_API void							CheckLinkCycles(std::unordered_set<CHashPtr>& Chain);
	TEXTUREGRAPHENGINE_API void							UpdateLinks();
	TEXTUREGRAPHENGINE_API void							HandleLinkUpdated(CHashPtr LinkUpdated);

									/// DO NOT MAKE THIS C'TOR PUBLIC. Use ConstructFromSources instead
	TEXTUREGRAPHENGINE_API explicit						CHash(const CHashPtrVec& Sources);

public:
									TEXTUREGRAPHENGINE_API CHash(HashType Value, bool bInIsFinal);
	TEXTUREGRAPHENGINE_API explicit						CHash(CHashPtr Temp);

	TEXTUREGRAPHENGINE_API bool							TryFinalise(HashType FinalHash = DataUtil::GNullHash, bool UpdateBlobber = true);
	TEXTUREGRAPHENGINE_API bool							operator == (const CHash& RHS) const;
	TEXTUREGRAPHENGINE_API HashTypeVec						GetIntermediateHashes() const;

	static TEXTUREGRAPHENGINE_API CHashPtr					ConstructFromSources(const CHashPtrVec& Sources);
	static TEXTUREGRAPHENGINE_API CHashPtr					UpdateHash(CHashPtr NewHash, CHashPtr PrevHash);

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE bool				IsNull() const { HashType CurrentHashValue = Value(); return CurrentHashValue == DataUtil::GNullHash || CurrentHashValue == 0; }
	FORCEINLINE bool				IsValid() const { return !IsNull(); }
	FORCEINLINE HashType			Value() const { return !TempHashValue ? HashValue : TempHashValue->HashValue; }
	FORCEINLINE operator			HashType() const { return Value(); }
	FORCEINLINE bool				IsFinal() const { return bIsFinal; }
	FORCEINLINE const FDateTime&	GetTimestamp() const { return Timestamp; }
	FORCEINLINE bool				HasUpdatedSince(FDateTime timestamp) const { return timestamp > Timestamp; }
	FORCEINLINE bool				IsTemp() const { return TempHashValue != nullptr; }
	FORCEINLINE CHashPtr			Temp() const { return TempHashValue; }
	FORCEINLINE bool				HasTempDependency() const { return (IsTemp() && TempHashValue->HasTempDependency_Internal()) || (!IsTemp() && HasTempDependency_Internal()); }
	FORCEINLINE const CHashPtrVec&	Sources() const { return !IsTemp() ? HashSources : TempHashValue->HashSources; }
	FORCEINLINE size_t				NumSources() const { return !IsTemp() ? HashSources.size() : TempHashValue->HashSources.size(); }
	FORCEINLINE bool				IsTempFinal() const { return IsFinal() && IsTemp(); }
};

THIRD_PARTY_INCLUDES_START
#include "continuable/continuable.hpp"
THIRD_PARTY_INCLUDES_END
typedef cti::continuable<CHashPtr>	AscynCHashPtr;

//////////////////////////////////////////////////////////////////////////
template <typename Type>
class T_Tiles
{
public:
	typedef std::vector<Type>		TypeVec;

protected:
	TypeVec							TilesVec;			/// The tiles 
	size_t							NumRows = 0;		/// Number of rows
	size_t							NumCols = 0;		/// Number of columns

public:
	T_Tiles() = default;
	T_Tiles(size_t InNumRows, size_t InNumCols) : NumRows(InNumRows), NumCols(InNumCols)
	{
		if (NumRows > 0 && NumCols > 0)
			TilesVec.resize(NumRows * NumCols);
	}

	T_Tiles(const TypeVec& RHS, size_t InNumRows, size_t InNumCols) 
		: TilesVec(RHS)
		, NumRows(InNumRows)
		, NumCols(InNumCols)
	{
	}

	void							Resize(size_t InNumRows, size_t InNumCols)
	{
		NumRows = InNumRows;
		NumCols = InNumCols;
		if (NumRows > 0 && NumCols > 0)
			TilesVec.resize(NumRows * NumCols);
	}

	FORCEINLINE size_t				Length() const { return NumRows * NumCols; }

	template<typename Functor>
	FORCEINLINE void				ForEach(Functor Func) const
	{
		for (size_t RowId = 0; RowId < NumRows; ++RowId)
		{
			for (size_t ColId = 0; ColId < NumCols; ++ColId)
			{
				Func(RowId, ColId);
			}
		}
	}
	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE const Type*			operator[](size_t RowId) const { check(RowId < NumRows); return (TilesVec.data() + (RowId * NumCols)); }
	FORCEINLINE Type*				operator[](size_t RowId) { check(RowId < NumRows); return (TilesVec.data() + (RowId * NumCols)); }
	FORCEINLINE const Type&			operator()(size_t RowId, size_t ColId) const { check(RowId < NumRows && ColId < NumCols); return TilesVec[RowId * NumCols + ColId]; }
	FORCEINLINE Type&				operator()(size_t RowId, size_t ColId) { check(RowId < NumRows && ColId < NumCols); return TilesVec[RowId * NumCols + ColId]; }
	FORCEINLINE size_t				Rows() const { return NumRows; }
	FORCEINLINE size_t				Cols() const { return NumCols; }
	FORCEINLINE const TypeVec&		Tiles() const { return TilesVec; }
	FORCEINLINE TypeVec&			Tiles() { return TilesVec; }
	FORCEINLINE void				Clear() { TilesVec.clear(); }
};

using TileInvalidateMatrix			= T_Tiles<int32>;
