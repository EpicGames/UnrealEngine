// Copyright Epic Games, Inc. All Rights Reserved.
#include "GraphicsUtil.h"
#include <limits>

Vector3 GraphicsUtil::s_boundsAxis[6] =
{
	Vector3(1, 0, 0), Vector3(-1, 0, 0),
	Vector3(0, 1, 0), Vector3(0, -1, 0),
	Vector3(0, 0, 1), Vector3(0, 0, 0),
};

Vector2 GraphicsUtil::CalculateBarycentricCoordinates(const Vector2* UV, const Vector2& P)
{
	Vector3 uv[3] = 
	{
		Vector3(UV[0].X, UV[0].Y, 0),
		Vector3(UV[1].X, UV[1].Y, 0),
		Vector3(UV[2].X, UV[2].Y, 0)
	};

	return CalculateBarycentricCoordinates(uv[0], uv[1], uv[2], Vector3(P.X, P.Y, 0));
}

Vector2 GraphicsUtil::CalculateBarycentricCoordinates(const Vector3& A, const Vector3& B, const Vector3& C, const Vector3& P)
{
	/// Taken from: http://blackpawn.com/texts/pointinpoly/default.html
	auto v0 = B - A;
	auto v1 = C - A;
	auto v2 = P - A;

	// Compute dot products
	auto dot00 = Vector3::DotProduct(v0, v0);
	auto dot01 = Vector3::DotProduct(v0, v1);
	auto dot02 = Vector3::DotProduct(v0, v2);
	auto dot11 = Vector3::DotProduct(v1, v1);
	auto dot12 = Vector3::DotProduct(v1, v2);

	// Compute barycentric coordinates
	auto invDenom = 1.0f / (dot00 * dot11 - dot01 * dot01);
	auto u = (dot11 * dot02 - dot01 * dot12) * invDenom;
	auto v = (dot00 * dot12 - dot01 * dot02) * invDenom;

	return Vector2(u, v);
}

bool GraphicsUtil::IsPointInTriangle(const Vector3& A, const Vector3& B, const Vector3& C, const Vector3& P)
{
	Vector2 uv = CalculateBarycentricCoordinates(A, B, C, P);
	return IsPointInTriangle(uv.X, uv.Y);
}

bool GraphicsUtil::IsPointInTriangle(float u, float v, float tolerance /* = 0.001f */)
{
	return (u >= -tolerance) && (v >= -tolerance) && (u + v < (1.0f + tolerance));
}

bool GraphicsUtil::IsPointInTriangle(const Vector2& uv, float tolerance /* = 0.001f */)
{
	return IsPointInTriangle(uv.X, uv.Y, tolerance);
}



//////////////////////////////////////////////////////////////////////////
float GraphicsUtil::SquaredDistance(const Vector3& p0, const Vector3& p1)
{
	return (p0.X - p1.X) * (p0.X - p1.X) + (p0.Y - p1.Y) * (p0.Y - p1.Y) + (p0.Z - p1.Z) * (p0.Z - p1.Z);
}

float GraphicsUtil::SquaredLength(const Vector3& pt)
{
	return (pt.X * pt.X) + (pt.Y * pt.Y) + (pt.Z * pt.Z);
}

std::array<Vector3, 8> GraphicsUtil::GetBoundsCorners(const FBox& bounds)
{
	std::array<Vector3, 8> corners;

	for (int i = 0; i < 8; i++)
		corners[i] = GetBoundsPoint(bounds, (BoundsCorners)i);

	return corners;
}

Vector3 GraphicsUtil::GetBoundsPoint(const FBox& bounds, BoundsCorners pti)
{
	switch ((BoundsCorners)pti)
	{
	case BoundsCorners::L0_BottomLeft:
		return bounds.Min;
	case BoundsCorners::L0_TopLeft:
		return Vector3(bounds.Min.X, bounds.Min.Y, bounds.Max.Z);
	case BoundsCorners::L0_TopRight:
		return Vector3(bounds.Min.X, bounds.Max.Y, bounds.Max.Z);
	case BoundsCorners::L0_BottomRight:
		return Vector3(bounds.Max.X, bounds.Max.Y, bounds.Max.Z);
	case BoundsCorners::L1_BottomLeft:
		return Vector3(bounds.Min.X, bounds.Max.Y, bounds.Min.Z);
	case BoundsCorners::L1_TopLeft:
		return Vector3(bounds.Min.X, bounds.Max.Y, bounds.Max.Z);
	case BoundsCorners::L1_TopRight:
		return bounds.Max;
	case BoundsCorners::L1_BottomRight:
		return Vector3(bounds.Max.X, bounds.Max.Y, bounds.Min.Z);
	}

	return Vector3();
}

FBox GraphicsUtil::GetChildBounds_QuadTree(const FBox& bounds, BoundsCorners ci)
{
	FBox b(ForceInit);
	Vector3 c(bounds.GetCenter().X, bounds.GetCenter().Y, bounds.Max.Z);

	switch (ci)
	{
	case BoundsCorners::L0_BottomLeft:
		b.Min = bounds.Min;
		b.Max = c;
		break;
	case BoundsCorners::L0_TopLeft:
		b.Min = Vector3(bounds.Min.X, c.Y, bounds.Min.Z);
		b.Max = Vector3(c.X, bounds.Max.Y, bounds.Max.Z);
		break;
	case BoundsCorners::L0_TopRight:
		b.Min = Vector3(c.X, c.Y, bounds.Min.Z);
		b.Max = bounds.Max;
		break;
	case BoundsCorners::L0_BottomRight:
		b.Min = Vector3(c.X, bounds.Min.Y, bounds.Min.Z);
		b.Max = Vector3(bounds.Max.X, c.Y, bounds.Max.Z);
		break;
	}

	return b;
}

FBox GraphicsUtil::GetChildBounds_Octree(const FBox& bounds, BoundsCorners ci)
{
	FBox b(ForceInit);
	auto c = bounds.GetCenter();

	switch (ci)
	{
	case BoundsCorners::L0_BottomLeft:
		b.Min = bounds.Min;
		b.Max = c;
		break;
	case BoundsCorners::L0_TopLeft:
		b.Min = Vector3(bounds.Min.X, bounds.Min.Y, c.Z);
		b.Max = Vector3(c.X, c.Y, bounds.Max.Z);
		break;
	case BoundsCorners::L0_TopRight:
		b.Min = Vector3(c.X, bounds.Min.Y, c.Z);
		b.Max = Vector3(bounds.Max.X, c.Y, bounds.Max.Z);
		break;
	case BoundsCorners::L0_BottomRight:
		b.Min = Vector3(c.X, bounds.Min.Y, bounds.Min.Z);
		b.Max = Vector3(bounds.Max.X, c.Y, c.Z);
		break;
	case BoundsCorners::L1_BottomLeft:
		b.Min = Vector3(bounds.Min.X, c.Y, bounds.Min.Z);
		b.Max = Vector3(c.X, bounds.Max.Y, bounds.Max.Z);
		break;
	case BoundsCorners::L1_TopLeft:
		b.Min = Vector3(bounds.Min.X, c.Y, c.Z);
		b.Max = Vector3(c.X, bounds.Max.Y, bounds.Max.Z);
		break;
	case BoundsCorners::L1_TopRight:
		b.Min = c;
		b.Max = bounds.Max;
		break;
	case BoundsCorners::L1_BottomRight:
		b.Min = Vector3(c.X, c.Y, bounds.Min.Z);
		b.Max = Vector3(bounds.Max.X, bounds.Max.Y, c.Z);
		break;
	}

	return b;
}

bool GraphicsUtil::CheckRayTriangleIntersection(const FRay& ray, const Vector3& p1, const Vector3& p2, const Vector3& p3, float& distance, Vector3& ip)
{
	/// https://en.wikipedia.org/wiki/M%C3%B6ller%E2%80%93Trumbore_intersection_algorithm

	distance = -1.0f;
	ip = Vector3::ZeroVector;

	/// Vectors from p1 to p2/p3 (edges)
	Vector3 e1, e2;
	Vector3 p, q, t;
	float det, invDet, u, v;

	/// Find vectors for two edges sharing vertex/point p1
	e1 = p2 - p1;
	e2 = p3 - p1;

	/// calculating determinant 
	p = Vector3::CrossProduct(ray.Direction, e2);

	det = Vector3::DotProduct(e1, p);

	/// if determinant is near zero, ray lies in plane of triangle otherwise not
	if (det > -std::numeric_limits<float>::epsilon() && det < std::numeric_limits<float>::epsilon())
		return false;

	invDet = 1.0f / det;

	/// calculate distance from p1 to ray origin
	t = ray.Origin - p1;

	/// Calculate u parameter
	u = Vector3::DotProduct(t, p) * invDet;

	/// Check for ray hit
	if (u < 0 || u > 1)
		return false;

	/// Prepare to test v parameter
	q = Vector3::CrossProduct(t, e1);

	/// Calculate v parameter
	v = Vector3::DotProduct(ray.Direction, q) * invDet;

	//Check for ray hit
	if (v < 0 || u + v > 1)
		return false;

	distance = Vector3::DotProduct(e2, q) * invDet;
	if (distance > std::numeric_limits<float>::epsilon())
	{
		/// ray does intersect, calculate the intersection point
		ip = ray.Origin + ray.Direction * distance;
		return true;
	}

	// No hit at all
	return false;
}

