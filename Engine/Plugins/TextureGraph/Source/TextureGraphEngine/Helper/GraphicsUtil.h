// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "2D/TextureType.h"
#include "GraphicsDefs.h"
#include <array>

#define UE_API TEXTUREGRAPHENGINE_API

enum class BoundsCorners
{
	L0_BottomLeft,
	L0_TopLeft,
	L0_TopRight,
	L0_BottomRight,

	L1_BottomLeft,
	L1_TopLeft,
	L1_TopRight,
	L1_BottomRight
};

class GraphicsUtil
{
public:
	static UE_API Vector2				CalculateBarycentricCoordinates(const Vector2* UV, const Vector2& P);
	static UE_API Vector2				CalculateBarycentricCoordinates(const Vector3& A, const Vector3& B, const Vector3& C, const Vector3& P);

	static UE_API bool					IsPointInTriangle(const Vector3& A, const Vector3& B, const Vector3& C, const Vector3& P);
	static UE_API bool					IsPointInTriangle(float u, float v, float tolerance = 0.001f);
	static UE_API bool					IsPointInTriangle(const Vector2& uv, float tolerance = 0.001f);

	//////////////////////////////////////////////////////////////////////////
	static UE_API Vector3				s_boundsAxis[6];
	static UE_API float				SquaredDistance(const Vector3& p0, const Vector3& p1);
	static UE_API float				SquaredLength(const Vector3& pt);

	static UE_API std::array<Vector3, 8> GetBoundsCorners(const FBox& bounds);

	static UE_API Vector3				GetBoundsPoint(const FBox& bounds, BoundsCorners pti);
	static UE_API FBox					GetChildBounds_QuadTree(const FBox& bounds, BoundsCorners ci);
	static UE_API FBox					GetChildBounds_Octree(const FBox& bounds, BoundsCorners ci);
	static UE_API bool					CheckRayTriangleIntersection(const FRay& ray, const Vector3& p1, const Vector3& p2, const Vector3& p3, float& distance, Vector3& ip);

	static FORCEINLINE Vector4	ConvertTangent(const Tangent& t) { return Vector4(t.TangentX.X, t.TangentX.Y, t.TangentX.Z, t.bFlipTangentY ? -1.0f : 1.0f); }
	static FORCEINLINE FLinearColor TangentAsLinearColor(const Tangent& t) { return FLinearColor(ConvertTangent(t)); }
	static FORCEINLINE FLinearColor Vec2AsLinearColor(const Vector2& v) { return FLinearColor(v.X, v.Y, 0.0f, 0.0f); }
	static FORCEINLINE FLinearColor Vec2AsLinearColor(const FVector2f& v) { return FLinearColor(v.X, v.Y, 0.0f, 0.0f); }
	static FORCEINLINE FLinearColor Vec3AsLinearColor(const Vector3& v) { return FLinearColor(v.X, v.Y, v.Z, 0.0f); }
	static FORCEINLINE FLinearColor Vec4AsLinearColor(const Vector4& v) { return FLinearColor(v); }

	//////////////////////////////////////////////////////////////////////////
	template <typename VecType>
	static VecType				Interpolate(const VecType& v0, const VecType& v1, const VecType& v2, Vector2 barycentric)
	{
		float u = barycentric.X;
		float v = barycentric.Y;

		return v0 + u * (v1 - v0) + v * (v2 - v0);
	}
};

#undef UE_API
