// Copyright Epic Games, Inc. All Rights Reserved.
#include "Helper/MathUtils.h"
#include <limits>

const float MathUtils::GMeterToCm = 100.0f;

float MathUtils::MinFloat()
{
	return std::numeric_limits<float>::lowest();
}

float MathUtils::MaxFloat()
{
	return std::numeric_limits<float>::max();
}

FVector MathUtils::MaxFVector()
{
	return FVector(MaxFloat(),MaxFloat(),MaxFloat());
}

FVector MathUtils::MinFVector()
{
	return FVector(MinFloat(), MinFloat(),MinFloat());
}

FVector2f MathUtils::MinFVector2()
{
	return FVector2f(MinFloat(),MinFloat());
}

FVector2f MathUtils::MaxFVector2()
{
	return FVector2f(MaxFloat(),MaxFloat());
}

void MathUtils::UpdateBounds(FBox& Bounds, const FVector& Point)
{
	Bounds.Min.X = FMath::Min(Bounds.Min.X, Point.X);
	Bounds.Min.Y = FMath::Min(Bounds.Min.Y, Point.Y);
	Bounds.Min.Z = FMath::Min(Bounds.Min.Z, Point.Z);
											
	Bounds.Max.X = FMath::Max(Bounds.Max.X, Point.X);
	Bounds.Max.Y = FMath::Max(Bounds.Max.Y, Point.Y);
	Bounds.Max.Z = FMath::Max(Bounds.Max.Z, Point.Z);
}

void MathUtils::EncapsulateBound(FBox& Bounds, FBox& boundsToCompare)
{	
	UpdateBounds(Bounds, boundsToCompare.Min);
	UpdateBounds(Bounds, boundsToCompare.Max);
}

FVector MathUtils::GetDirection(float YZAngle, float XAngle, int XSign /*= 1*/)
{
	FVector Direction = FVector::RightVector;

	UE_LOGF(LogTemp, Log, "MathUtils : Direction : xyAngle {%f}, zAngle {%f} and zSign {%d}", YZAngle, XAngle, XSign);

	YZAngle = YZAngle > 0 ? YZAngle : 180 + (180 - FMath::Abs(YZAngle));
	Direction = FQuat::MakeFromEuler(FVector(0, 0, XAngle)) * Direction;
	Direction = FQuat::MakeFromEuler(FVector(YZAngle, 0, 0)) * Direction;

	Direction.X = XSign * FMath::Abs(Direction.X);

	Direction.Normalize();

	UE_LOGF(LogTemp, Log, "MathUtils : Direction : {%f}, {%f}, {%f}", Direction.X, Direction.Y, Direction.Z);
	
	return Direction;
}

FBox MathUtils::GetCombinedBounds(TArray<FBox> InputBounds)
{
	check(InputBounds.Num() > 0);
	
	if (InputBounds.Num() == 0)
		return FBox::BuildAABB(FVector::ZeroVector,FVector::ZeroVector);

	FVector Min = InputBounds[0].Min;
	FVector Max = InputBounds[0].Max;

	for (int bi = 1; bi < InputBounds.Num(); bi++)
	{
		FBox b = InputBounds[bi];

		Min.X = FMath::Min(Min.X, b.Min.X);
		Min.Y = FMath::Min(Min.Y, b.Min.Y);
		Min.Z = FMath::Min(Min.Z, b.Min.Z);

		Max.X = FMath::Max(Max.X, b.Max.X);
		Max.Y = FMath::Max(Max.Y, b.Max.Y);
		Max.Z = FMath::Max(Max.Z, b.Max.Z);
	}

	FVector size = Max - Min;

	FBox CombinedBounds = FBox::BuildAABB(FVector::ZeroVector,FVector::ZeroVector);
	CombinedBounds.Min = Min;
	CombinedBounds.Max = Max;

	return CombinedBounds;
}

float MathUtils::Step(float Y, float X)
{
	return X >= Y ? 1 : 0;
}

