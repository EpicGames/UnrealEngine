// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#define UE_API TEXTUREGRAPHENGINE_API
/**
 * 
 */
class MathUtils
{
public:
	MathUtils() = delete;

	static UE_API const float					GMeterToCm;

	static UE_API float						MinFloat(); 
	static UE_API float						MaxFloat(); 
	
	static UE_API FVector						MinFVector();
	static UE_API FVector						MaxFVector();
	static UE_API FVector2f					MinFVector2();
	static UE_API FVector2f					MaxFVector2();
	static UE_API void							UpdateBounds(FBox& bounds, const FVector& point);
	static UE_API void							EncapsulateBound(FBox& bounds, FBox& otherBounds);
	static UE_API FVector						GetDirection(float yzAngle, float xAngle, int xSign = 1);
	static UE_API FBox							GetCombinedBounds(TArray<FBox> inputBounds);
	static UE_API float						Step(float Y, float X);
};

#undef UE_API
