// Copyright Epic Games, Inc. All Rights Reserved.
#include "Promise.h"

auto PromiseUtil::OnThread(ENamedThreads::Type ThreadType)
{
	return cti::make_continuable<void>([ThreadType](auto&& Promise)
	{
		AsyncTask(ThreadType, [&Promise]()
		{
			//promise.set_empty();
		});
	});
}