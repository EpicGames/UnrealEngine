// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
THIRD_PARTY_INCLUDES_START
#include "continuable/continuable.hpp"
THIRD_PARTY_INCLUDES_END
#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#include "Util.h"

#define UE_API TEXTUREGRAPHENGINE_API

typedef cti::continuable<int32> AsyncInt;

// This is temporary and will eventually go away
struct ActionResult
{
	std::exception_ptr				ExInner;					/// Original exception that was raised by the action
	int32							ErrorCode = 0;				/// What is the error code

	TMap<FString, FString>			Metadata;					/// Additional metadata that may be associated with an action result. 
	/// This can be used to 'send back' a lot of useful information

	explicit ActionResult(std::exception_ptr Ex = nullptr, int32 ErrCode = 0) : ExInner(Ex), ErrorCode(ErrCode) {}

	FORCEINLINE bool				IsOK() const { return ErrorCode == 0 && !ExInner; }
};

typedef std::shared_ptr<ActionResult>		ActionResultPtr;
typedef cti::continuable<ActionResultPtr>	AsyncActionResultPtr;

//////////////////////////////////////////////////////////////////////////
struct PromiseUtil 
{
	////////////////////////////////////////////////////////////////////////////
	///// Static functions
	////////////////////////////////////////////////////////////////////////////
	static UE_API auto						OnThread(ENamedThreads::Type ThreadType);

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	template<typename T>
	static FORCEINLINE cti::continuable<T> OnThread(ENamedThreads::Type ThreadType, T&& Args)
	{
		return cti::make_continuable<T>([ThreadType, args = std::forward<T>(Args)](auto&& Promise) mutable
		{
			AsyncTask(ThreadType, [args = std::forward<T>(args), promise = std::forward<decltype(Promise)>(Promise)]() mutable
			{
				promise.set_value(std::forward<T>(args));
			});
		});
	}

	template<typename T>
	static cti::continuable<T>		WrapResolvePromiseOnThread(ENamedThreads::Type ThreadType, TUniqueFunction<cti::continuable<T>()> callback)
	{
		return cti::make_continuable<int32>([=, callback = std::move(callback)](auto&& promise) mutable
		{
			return callback().then([=, FWD_PROMISE(promise)](T result) mutable
			{
				Util::OnThread(ThreadType, [=, FWD_PROMISE(promise)]() mutable
				{
					promise.set_value(result);
				});
			});
		});
	}

	static AsyncInt				OnBackgroundThread()
	{
		return cti::make_continuable<int32>([](auto&& Promise)
		{
			AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [promise = std::forward<decltype(Promise)>(Promise)]() mutable
			{
				promise.set_value(0);
			});
		});
	}

	static AsyncInt					OnGameThread()
	{
		if (IsInGameThread())
			return cti::make_ready_continuable(0);

		return cti::make_continuable<int32>([](auto&& Promise)
		{
			AsyncTask(ENamedThreads::GameThread, [promise = std::forward<decltype(Promise)>(Promise)]() mutable
			{
				promise.set_value(0);
			});
		});
	}

	template<typename T>
	static cti::continuable<T>  OnGameThread(T&& Arg)
	{
		if (IsInGameThread())
			return cti::make_ready_continuable(std::forward<T>(Arg));

		return OnThread(ENamedThreads::GameThread, std::forward<T>(Arg));
	}


	static AsyncInt					OnRenderingThread()
	{
		if (IsInRenderingThread())
			return cti::make_ready_continuable(0);

		return cti::make_continuable<int32>([](auto&& Promise)
			{
				AsyncTask(ENamedThreads::ActualRenderingThread, [promise = std::forward<decltype(Promise)>(Promise)]() mutable
					{
						promise.set_value(0);
					});
			});
	}
};

#define FWD_PROMISE(p) p = std::forward<decltype(p)>(p)

#undef UE_API
