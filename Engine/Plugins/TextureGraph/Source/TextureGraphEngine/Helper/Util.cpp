// Copyright Epic Games, Inc. All Rights Reserved.
#include "Util.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/Async.h"
#include "Containers/EnumAsByte.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Misc/FileHelper.h"
#include "TextureGraphEngine.h"
#include "Model/ModelObject.h"
#include "Modules/ModuleManager.h"
#include "RenderingThread.h"
#include "UnrealClient.h"
#include <UObject/MetaData.h>
#include <UObject/UObjectGlobals.h>
#include <HAL/PlatformFileManager.h>
#include <Engine/Engine.h>
#include <Framework/Application/SlateApplication.h>
#include <UObject/Package.h>
#include <Materials/Material.h>
#include "Device/FX/Device_FX.h"

#include "FxMat/MaterialManager.h"
#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#endif

UObject* Util::GetModelPackage()
{
	auto Package = (UObject*)TextureGraphEngine::GetMaterialManager();
	return Package ? Package : GetTransientPackage();
}

UObject* Util::GetTexturesPackage()
{
	return TextureGraphEngine::GetMaterialManager();
}

UObject* Util::GetRenderTargetPackage()
{
	return TextureGraphEngine::GetMaterialManager();
}

UObject* Util::GetMegascansTexturesPackage()
{
	return GetTransientPackage();
}

UObject* Util::GetMaterialsPackage()
{
	return TextureGraphEngine::GetMaterialManager();
}

//////////////////////////////////////////////////////////////////////////
/// Path/File related
//////////////////////////////////////////////////////////////////////////
bool Util::IsFileValid(const FString& FileName)
{
	// Check whether we can read from the file
	const int64 FileSize = FPlatformFileManager::Get().GetPlatformFile().FileSize(*FileName);
	if ((!FPlatformFileManager::Get().GetPlatformFile().FileExists(*FileName)) || (FileSize == 0))
		return false;

	return true;
}

bool Util::IsDirectory(const FString& Path)
{
	return FPlatformFileManager::Get().GetPlatformFile().DirectoryExists(*Path);
}

bool Util::IsDirectoryEmpty(const FString& Path)
{
	DirectoryVisitor Visitor;
	IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();

	FileManager.IterateDirectoryRecursively(*Path, Visitor);
	
	return ((Visitor.FolderCount == 0) && (Visitor.FileCount == 0));
}

//////////////////////////////////////////////////////////////////////////
/// Threading related
//////////////////////////////////////////////////////////////////////////
void Util::OnGameThread(TUniqueFunction<void()> Callback) 
{ 
	check(!TextureGraphEngine::IsDestroying());

	/// If already in game thread then don't wait
	if (IsInGameThread())
	{
		Callback();
		return;
	}

	AsyncTask(ENamedThreads::GameThread, std::move(Callback)); 
}

void Util::OnBackgroundThread(TUniqueFunction<void()> Callback)
{
	check(!TextureGraphEngine::IsDestroying());
	AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, std::move(Callback));
}

void Util::OnAnyThread(TUniqueFunction<void()> Callback)
{
	check(!TextureGraphEngine::IsDestroying());
	AsyncTask(ENamedThreads::AnyThread, std::move(Callback));
}

void Util::OnThread(ENamedThreads::Type thread, TUniqueFunction<void()> Callback)
{
	if (thread == ENamedThreads::ActualRenderingThread || thread == ENamedThreads::RHIThread)
	{
		ENQUEUE_RENDER_COMMAND(SceneDrawCompletion)([Callback = std::move(Callback)](FRHICommandListImmediate&) mutable
		{
			Callback();
		});
	}
	else
	{
		AsyncTask(thread, std::move(Callback));
	}
}

void Util::OnRenderingThread(TUniqueFunction<void(FRHICommandListImmediate&)> Callback)
{
	check(!TextureGraphEngine::IsDestroying());

	if (IsInRenderingThread())
	{
		Callback(Device_FX::Get()->RHI());
		return;
	}

	ENQUEUE_RENDER_COMMAND(SceneDrawCompletion)([Callback = std::forward<decltype(Callback)>(Callback)](FRHICommandListImmediate& RHI) mutable
	{
		Callback(RHI);
	});
}

size_t Util::GetCurrentThreadId()
{
	return std::hash<std::thread::id>{}(std::this_thread::get_id());
}

FRHICommandListImmediate& Util::RHIImmediate()
{
	return FRHICommandListImmediate::Get();
}

//////////////////////////////////////////////////////////////////////////
/// Game objects and engine related
//////////////////////////////////////////////////////////////////////////
extern ENGINE_API UEngine* GEngine;
#if WITH_EDITOR	
extern UNREALED_API UEditorEngine* GEditor;
#endif

const FWorldContext* Util::GetGameWorldContext()
{
	if (GEngine)
	{
		for (const FWorldContext& context : GEngine->GetWorldContexts())
		{
			// Return the first Game/PIE world that we can find
			if (context.WorldType == EWorldType::Game || context.WorldType == EWorldType::PIE)
			{
				return &context;		
			}
		}
	}

#if WITH_EDITOR	
	// If we cannot find Game/PIE(play in editor mode) then it means we are running in editor mode.
	// We can directly access editor world context from GEdtior.
	return &GEditor->GetEditorWorldContext();
#else
	return nullptr;
#endif
}

UWorld* Util::GetGameWorld()
{
	bool hasEditorEngine = false;

#if WITH_EDITOR
	hasEditorEngine = GEditor != nullptr;
#endif

	if (!GEngine && !hasEditorEngine)
		return nullptr;

	const FWorldContext* Context = GetGameWorldContext();
	if (Context)
		return Context->World();

	return nullptr;
}

FString Util::RandomID()
{
	return FGuid::NewGuid().ToString(EGuidFormats::Short);
}

FString Util::ToPascalCase(const FString& strToConvert)
{
	int32 UnderscoreIndex = strToConvert.Find("_");
	if (UnderscoreIndex == -1)
		return strToConvert;

	FString ReturnString = strToConvert.Mid(UnderscoreIndex + 1, 1).ToUpper() + strToConvert.Mid(UnderscoreIndex + 2, strToConvert.Len());
	return ReturnString;
}

void* Util::GetOSWindowHandle()
{
	return FSlateApplication::Get().GetActiveTopLevelWindow()->GetNativeWindow()->GetOSWindowHandle();
}
