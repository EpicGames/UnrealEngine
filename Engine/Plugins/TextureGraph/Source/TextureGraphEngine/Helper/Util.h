// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include <GenericPlatform/GenericPlatformFile.h>
#include <HAL/FileManager.h>
#include <UObject/ObjectMacros.h>
#include <UObject/ReflectedTypeAccessors.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
enum class E_Priority
{
	kLowest = 0,
	kLow = 25,
	kNormal = 50,
	kHigh = 10000,
	kHighest = 100000,
	kSystem = 1000000,
};

struct DirectoryVisitor : public IPlatformFile::FDirectoryVisitor
{
	uint64 	FileCount = 0;
	uint64 	TotalSize = 0;
	uint64 	FolderCount = 0;
	bool	StoreFilenames = false;
	TArray<FString>	FilePaths;

	//This function is called for every file or directory it finds.
	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		// did we find a Directory or a file?
		if (bIsDirectory)
		{
			FolderCount++;
		}
		else
		{
			FileCount++;
			TotalSize += IFileManager::Get().FileSize(FilenameOrDirectory);
			if (StoreFilenames)
				FilePaths.AddUnique(FilenameOrDirectory);
		}
		return true;
	}
};

class UMaterial;
class UPackage;
struct FWorldContext;
class FRHICommandListImmediate;
struct Util
{
	static constexpr int32			GDefaultWidth = 1024;
	static constexpr int32			GDefaultHeight = 1024;

	//////////////////////////////////////////////////////////////////////////
	/// Package related utility functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API UObject*					GetModelPackage();
	static UE_API UObject*					GetTexturesPackage();
	static UE_API UObject*					GetRenderTargetPackage();
	static UE_API UObject*					GetMegascansTexturesPackage();
	static UE_API UObject*					GetMaterialsPackage();

	//////////////////////////////////////////////////////////////////////////
	/// Path related
	//////////////////////////////////////////////////////////////////////////
	static FORCEINLINE FString		GetRuntimeTexturesPackagePath() { return FString("/Game/Maps"); }
	static FORCEINLINE FString		GetRenderTargetPackagePath() { return FString("/Game/Maps"); }
	static FORCEINLINE FString		GetRuntimeTexturesPackagePath(FString suffix) { return GetRuntimeTexturesPackagePath() + suffix; }
	static FORCEINLINE FString		GetRuntimeTexturesPackagePath(const char* suffix) { return GetRuntimeTexturesPackagePath() + FString(suffix); }

	static UE_API bool						IsFileValid(const FString& FileName);
	static UE_API bool						IsDirectory(const FString& Path);
	static UE_API bool						IsDirectoryEmpty(const FString& Path);

	//////////////////////////////////////////////////////////////////////////
	/// Threading related
	//////////////////////////////////////////////////////////////////////////
	static UE_API void						OnGameThread(TUniqueFunction<void()> Callback);
	static UE_API void						OnRenderingThread(TUniqueFunction<void(FRHICommandListImmediate&)> Callback);
	static UE_API void						OnBackgroundThread(TUniqueFunction<void()> Callback);
	static UE_API void						OnAnyThread(TUniqueFunction<void()> Callback);
	static UE_API void						OnThread(ENamedThreads::Type thread, TUniqueFunction<void()> Callback);
	static UE_API size_t					GetCurrentThreadId();

	UE_DEPRECATED(5.8, "Util::RHIImmediate() is deprecated. Use FRHICommandListImmediate::Get() instead.")
	static UE_API FRHICommandListImmediate& RHIImmediate();

	//////////////////////////////////////////////////////////////////////////
	/// Game objects and engine related
	//////////////////////////////////////////////////////////////////////////
	static UE_API const FWorldContext*		GetGameWorldContext();
	static UE_API UWorld*					GetGameWorld();
	static UE_API void*					GetOSWindowHandle();
	static UE_API FString					RandomID();
	static UE_API FString					ToPascalCase(const FString& strToConvert);
	static FORCEINLINE double		Time() { return FPlatformTime::Seconds() * 1000.0; }
	static FORCEINLINE double		TimeDelta(double start) { return Time() - start; }

	//////////////////////////////////////////////////////////////////////////
	//// UObject related
	//////////////////////////////////////////////////////////////////////////

	/**
 * Convert the value of an enum to a string.
 *
 * @param EnumValue
 *	The enumerated type value to convert to a string.
 *
 * @return
 *	The key/name that corresponds to the value in the enumerated type.
 */
	template<typename T>
	static FString EnumToString(const T EnumValue)
	{
		FString Name = StaticEnum<T>()->GetNameStringByValue(static_cast<__underlying_type(T)>(EnumValue));

		check(Name.Len() != 0);

		return Name;
	}

};

#undef UE_API
