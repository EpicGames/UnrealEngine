// Copyright Epic Games, Inc. All Rights Reserved.
#include "BlobHasherService.h"
#include "TextureGraphEngine.h"
#include "Data/Blobber.h"

BlobHasherService::BlobHasherService() : BlobHelperService(TEXT("Blob_Hashing"))
{
	Blobs.reserve(1024);
}

BlobHasherService::~BlobHasherService()
{
}

void BlobHasherService::Add(BlobRef BlobObj)
{
	if (TextureGraphEngine::IsTestMode())
		return;

	check(BlobObj.IsKeepStrong());

	// Should not be adding transient blobs to the BlobObj hasher service
	check(!BlobObj->IsTransient());

	BlobHelperService::Add(BlobObj);
}

AsyncJobResultPtr BlobHasherService::Tick()
{
	UE_LOGF(LogIdle_Svc, VeryVerbose, "Svc_BlobHasher::Tick");

	static constexpr size_t MaxBlobs = 8;
	BlobPtr BlobObjs[MaxBlobs];
	HashType PrevHashes[MaxBlobs] = {0};
	size_t NumBlobs = 0;
	size_t TotalCount = 0;

	auto Iter = Blobs.begin();
	auto StartIter = Iter;

	while (!TextureGraphEngine::IsDestroying() && NumBlobs < MaxBlobs && Blobs.size() > 0 && Iter != Blobs.end())
	{
		BlobPtr BlobObj = *Iter;

		if (BlobObj && BlobObj->CanCalculateHash())
		{
			/// Must've resolved by now
			check(!BlobObj->IsLateBound());
			check(BlobObj->IsFinalised());

			BlobObjs[NumBlobs] = BlobObj;
			PrevHashes[NumBlobs] = BlobObj->Hash()->Value();
			NumBlobs++;

			Iter = Blobs.erase(Iter);
		}
		else
			Iter++;
	}

	TotalCount = Blobs.size();

	if (NumBlobs > 0)
	{
		std::vector<std::decay_t<AscynCHashPtr>, std::allocator<std::decay_t<AscynCHashPtr>>> promises;
		promises.reserve(NumBlobs);

		for (size_t BlobIndex = 0; !TextureGraphEngine::IsDestroying() && BlobIndex < NumBlobs; BlobIndex++)
			promises.push_back(BlobObjs[BlobIndex]->CalcHash());

		if (!TextureGraphEngine::IsDestroying())
		{
			return cti::when_all(promises.begin(), promises.end()).then([this, BlobObjs, PrevHashes, NumBlobs, TotalCount](std::vector<CHashPtr> hashes) mutable
			{
				for (size_t BlobIndex = 0; BlobIndex < NumBlobs; BlobIndex++)
				{
					BlobPtr BlobObj = BlobObjs[BlobIndex];
					CHashPtr CurrentHash = BlobObj->Hash();
					HashType PrevHash = PrevHashes[BlobIndex];

					check(CurrentHash->IsFinal());
					UE_LOGF(LogIdle_Svc, VeryVerbose, "Hashed BlobObj: %ls => %llu [Total Blobs: %llu]", *BlobObj->Name(), CurrentHash->Value(), TotalCount);

					if (!TextureGraphEngine::IsDestroying())
						TextureGraphEngine::GetBlobber()->UpdateBlobHash(PrevHash, BlobObj);
				}

				return JobResult::NullResult;
			});
		}
	}

	return cti::make_ready_continuable<JobResultPtr>(std::make_shared<JobResult>());
}
