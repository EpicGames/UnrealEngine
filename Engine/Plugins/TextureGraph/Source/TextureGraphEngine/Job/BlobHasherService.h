// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "BlobHelperService.h"
#include <vector>

#define UE_API TEXTUREGRAPHENGINE_API

class BlobHasherService : public BlobHelperService
{
public:
									UE_API BlobHasherService();
	UE_API virtual							~BlobHasherService() override;

	UE_API virtual AsyncJobResultPtr		Tick() override;
	UE_API virtual void					Add(BlobRef BlobObj) override;
};

typedef std::shared_ptr<BlobHasherService>	BlobHasherServicePtr;
typedef std::weak_ptr<BlobHasherService>	BlobHasherServicePtrW;

#undef UE_API
