// Copyright Epic Games, Inc. All Rights Reserved.
#include "BlobHelperService.h"
#include "TextureGraphEngine.h"
#include "Data/Blobber.h"
#include "Data/TiledBlob.h"

BlobHelperService::BlobHelperService(const FString& InName) : IdleService(InName)
{
	Blobs.reserve(1024);
}

BlobHelperService::~BlobHelperService()
{
}

void BlobHelperService::Stop()
{
	check(IsInGameThread());
	Blobs.clear();
}

void BlobHelperService::Add(BlobRef InBlobObj)
{
	check(IsInGameThread());
	BlobPtr BlobObj = InBlobObj.lock();

	/// If this doesn't have a buffer (tiled blob) then just queue and return
	if (BlobObj->IsTiled())
	{
		Blobs.push_back(BlobObj);
		return;
	}

	/// Blob and its buffer must be valid!
	check(BlobObj && BlobObj->GetBufferRef() && !BlobObj->IsNull());

	CHashPtr BlobHash = BlobObj->Hash();

	/// Don't allow a non-temp hash
	check(BlobHash);

	/// Don't wanna hash transient buffers
	if (BlobObj->GetBufferRef()->Descriptor().bIsTransient) 
		return;

	//UE_LOGF(LogIdle_Svc, Log, "Added for hashing: %ls [Hash: %llu]", *blob->Name(), blobHash->Value());
	Blobs.push_back(BlobObj);
}
