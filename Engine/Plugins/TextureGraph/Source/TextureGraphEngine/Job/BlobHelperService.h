// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IdleService.h"
#include <vector>

#define UE_API TEXTUREGRAPHENGINE_API

class BlobHelperService : public IdleService
{
protected:
	std::vector<BlobPtr>			Blobs;					/// The blobs that we want to hash

public:
	UE_API explicit						BlobHelperService(const FString& InName);
	UE_API virtual							~BlobHelperService() override;

	UE_API virtual void					Stop() override;

	UE_API virtual void					Add(BlobRef BlobObj);
};

#undef UE_API
