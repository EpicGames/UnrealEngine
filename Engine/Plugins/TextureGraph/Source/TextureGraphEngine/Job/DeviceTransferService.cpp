// Copyright Epic Games, Inc. All Rights Reserved.
#include "DeviceTransferService.h"
#include "TextureGraphEngine.h"
#include "Data/Blobber.h"
#include "Device/Device.h"

DEFINE_LOG_CATEGORY(LogTransfer_Svc);

DeviceTransferService::DeviceTransferService() : IdleService(TEXT("TempHashResolver"))
{
}

DeviceTransferService::~DeviceTransferService()
{
}

AsyncJobResultPtr DeviceTransferService::Tick()
{
	UE_LOGF(LogTransfer_Svc, Verbose, "Svc_DeviceTransfer::Tick");

	/// can only be called from the game thread
	check(IsInGameThread());

	/// First of all we tell the blobber to check for device transfers ...
	/// Then we actually engage the buffers that we've 
	//Engine::Blobber()->UpdateDeviceTransfers();

	UE_LOGF(LogTransfer_Svc, Verbose, "Svc_DeviceTransfer::Tick");

	/// Ok, we now flush upto s_maxFlush
	int32 Count = std::min((int32)GMaxFlush, (int32)Transfers.size());

	if (Count)
	{
		std::vector<AsyncDeviceBufferRef> Promises;
		std::vector<TransferInfo> CurrentTransfers;
		Promises.reserve(Count);

		auto Iter = Transfers.begin();

		while (!TextureGraphEngine::IsDestroying() && (int32)Promises.size() < Count && Iter != Transfers.end())
		{
			const TransferInfo& ThisTransferInfo = Iter->second;
			FString BufferName = ThisTransferInfo.BlobObj->Name();
			CHashPtr BufferHash = ThisTransferInfo.BlobObj->Hash();
			FString OwnerDeviceNameOrg = ThisTransferInfo.OwnerDevice->Name();
			FString TargetDeviceName = ThisTransferInfo.TargetDevice->Name();
			const AccessInfo& ThisAccessInfo = ThisTransferInfo.BlobObj->GetBufferRef()->GetAccessInfo();
			bool AbortThisTransfer = false;

			/// 1. If the owner device has changed in betwwen this->Transfer(...) call and now then give an error ...
			if (ThisTransferInfo.BlobObj->GetBufferRef()->GetOwnerDevice() != ThisTransferInfo.OwnerDevice)
			{
				FString ownerDeviceNameCurr = ThisTransferInfo.BlobObj->GetBufferRef()->GetOwnerDevice()->Name();

				UE_LOGF(LogTransfer_Svc, Warning, "Buffer [Name: %ls, Hash: %llu] owner device changed between transfer initiation and execution. Original = %ls, Current = %ls [Target: %ls]. Aborting transfer for this buffer ...", 
					*BufferName, BufferHash->Value(), *OwnerDeviceNameOrg, *ownerDeviceNameCurr, *TargetDeviceName);

				AbortThisTransfer = true;
			}
			/// 2. If the timestamp has changed in betwwen this->Transfer(...) call and now then give an error ...
			else if (ThisTransferInfo.Access.Timestamp != ThisAccessInfo.Timestamp)
			{
				UE_LOGF(LogTransfer_Svc, Warning, "Buffer [Name: %ls, Hash: %llu] timestamp changed between transfer initiation and execution. Original = %lf, Current = %lf [Device: %ls => %ls]. Aborting transfer for this buffer ...", 
					*BufferName, BufferHash->Value(), ThisTransferInfo.Access.Timestamp.load(), ThisAccessInfo.Timestamp.load(), *OwnerDeviceNameOrg, *TargetDeviceName);

				AbortThisTransfer = true;
			}
			/// Start the transfer
			else
			{
				UE_LOGF(LogTransfer_Svc, Verbose, "Buffer [Name: %ls, Hash: %llu] starting transfer from %ls => %ls", *BufferName, BufferHash->Value(), *OwnerDeviceNameOrg, *TargetDeviceName);
				Promises.push_back(ThisTransferInfo.BlobObj->TransferTo(ThisTransferInfo.TargetDevice));

				CurrentTransfers.push_back(ThisTransferInfo);
			}

			/// if the transfer has been aborted, then we need to put it back
			if (AbortThisTransfer)
			{
				ThisTransferInfo.OwnerDevice->TransferAborted(ThisTransferInfo.BlobObj->GetBufferRef());
			}

			Iter++;
		}

		/// Remove all that we've handled
		Transfers.erase(Transfers.begin(), Iter);

		if (!TextureGraphEngine::IsDestroying() && !Promises.empty())
		{
			return cti::when_all(Promises.begin(), Promises.end())
				.then([this, CurrentTransfers](std::vector<DeviceBufferRef> results) mutable
				{
					/// We wanna keep the reference for blobs so that they don't get collected in 
					/// the meantime
					CurrentTransfers.clear();

					/// Return empty result 
					return std::make_shared<JobResult>();
				})
				.fail([this, CurrentTransfers](std::exception_ptr) mutable
				{
					/// We wanna keep the reference for blobs so that they don't get collected in 
					/// the meantime
					CurrentTransfers.clear();

					/// TODO: Handle the error here
				});
		}
	}

	return cti::make_ready_continuable<JobResultPtr>(std::make_shared<JobResult>());
}

void DeviceTransferService::Stop()
{
	Transfers.clear();
}

void DeviceTransferService::AbortTransfer(HashType hash)
{
	check(IsInGameThread());
	auto iter = Transfers.find(hash);

	if (iter != Transfers.end())
		Transfers.erase(iter);
}

void DeviceTransferService::QueueTransfer(BlobPtr blob, Device* target)
{
	check(IsInGameThread());

	DeviceBufferRef buffer = blob->GetBufferRef();
	CHashPtr hash = buffer->Hash();

	/// Must've had a final hash by now
	check(hash && hash->IsFinal());
	check(buffer->GetOwnerDevice() != target);

	auto iter = Transfers.find(hash->Value());

	/// This has already been added for transfer
	if (iter != Transfers.end())
	{
		/// But just check whether its targetting the same device, otherwise we're in a situation
		/// where multiple devices are trying to transfer the same buffer
		TransferInfo& existingTransfer = iter->second;
		check(existingTransfer.TargetDevice == target);

		UE_LOGF(LogTransfer_Svc, Warning, "Buffer [Hash: %llu] has already been marked for transfer to device: %ls", hash->Value(), *target->Name());

		return;
	}

	/// Now check whether the target device already has this 
	DeviceBufferRef existingBufferOnTarget = target->Find(hash->Value(), false);
	if (existingBufferOnTarget && existingBufferOnTarget->IsValid())
	{
		UE_LOGF(LogTransfer_Svc, Warning, "Buffer [Hash: %llu] is already cached on the target device: %ls", hash->Value(), *target->Name());
		return;
	}

	/// Queue it for transfer
	Transfers[hash->Value()] = TransferInfo { buffer->GetOwnerDevice(), blob, target, buffer->GetAccessInfo() };
}
