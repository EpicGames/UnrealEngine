// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IdleService.h"
#include <unordered_map>

#define UE_API TEXTUREGRAPHENGINE_API

DECLARE_LOG_CATEGORY_EXTERN(LogTransfer_Svc, Log, Verbose);

class DeviceTransferService : public IdleService
{
protected:
	static constexpr int			GMaxFlush = 32;		/// Size is chosen so that TransferInfo

	struct TransferInfo
	{
		Device*						OwnerDevice = nullptr;	/// The owner device at the time when this blob was market for collection
		BlobPtr						BlobObj;					/// The blob that needs to be transferred
		Device*						TargetDevice = nullptr;	/// The device that we're transferring this to

		AccessInfo					Access;					/// This is a snapshot of the access info we save at the time the blob 
															/// is marked for collection. We then check it at the time of collection to 
															/// see whether the buffer is in same state. If it is, we go ahead with the 
															/// transfer, otherwise we abort it.
	};

	std::unordered_map<HashType, TransferInfo>	
									Transfers;				/// The current transfers that we're working on

public:
									UE_API DeviceTransferService();
	UE_API virtual							~DeviceTransferService() override;

	UE_API virtual AsyncJobResultPtr		Tick() override;
	UE_API virtual void					Stop() override;
	virtual bool					ShouldRunDuringBusyUpdate() const override { return true; }

	UE_API virtual	void					QueueTransfer(BlobPtr BlobObj, Device* TargetDevice);
	UE_API virtual void					AbortTransfer(HashType HashValue);
};

typedef std::shared_ptr<DeviceTransferService> DeviceTransferServicePtr;
typedef std::weak_ptr<DeviceTransferService>	DeviceTransferServicePtrW;

#undef UE_API
