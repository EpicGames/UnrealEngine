// Copyright Epic Games, Inc. All Rights Reserved.
#include "HistogramService.h"
#include "../TextureGraphEngine.h"
#include "Scheduler.h"
#include "Transform/Utility/T_TextureHistogram.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"

HistogramService::HistogramService() : IdleService(TEXT("Histogram"))
{
}

HistogramService::~HistogramService()
{
}

void HistogramService::AddHistogramJob(MixUpdateCyclePtr Cycle,JobUPtr JobToAdd ,int32 TargetID , UMixInterface* Mix)
{
	check(Cycle);
	Cycle->AddJob(TargetID, std::move(JobToAdd));
}

JobBatchPtr HistogramService::GetOrCreateNewBatch(UMixInterface* Mix)
{
	check(IsInGameThread());
	check(Mix);

	if (!Batch)
	{
		FInvalidationDetails Details(Mix);
		Details.All();
		Batch = JobBatch::Create(Details);
		Batch->IsIdle() = true;
		SceneTargetUpdatePtr Target = std::make_shared<MixTargetUpdate>(Mix, 0);
		Batch->GetCycle()->AddTarget(Target);
	}

	return Batch;
}

AsyncJobResultPtr HistogramService::Tick()
{
	check(IsInGameThread());

	UE_LOGF(LogIdle_Svc, VeryVerbose, "Svc_Histogram::Tick");

	if (Batch)
	{
		auto LastBatch = Batch;
		Batch = nullptr;

		//TextureGraphEngine::GetScheduler()->AddBatch(LastBatch);
		LastBatch->SetCaptureRenderDoc(bCaptureNextBatch);
		bCaptureNextBatch = false;

		TextureGraphEngine::GetInstance()->GetScheduler()->GetObserverSource()->BatchAdded(LastBatch); // notify observer

		return LastBatch->Exec([=](JobBatch*)	/// Instead of passing it as an argument, JobBatch should be a return type; this is to keep it from cyclic dependancy
			{
				TextureGraphEngine::GetInstance()->GetScheduler()->GetObserverSource()->BatchJobsDone(LastBatch);
			})
			.then([this, LastBatch]()
			{
				TextureGraphEngine::GetInstance()->GetScheduler()->GetObserverSource()->BatchDone(LastBatch); // notify observer
				return std::make_shared<JobResult>();
			});
	}
	return cti::make_ready_continuable<JobResultPtr>(std::make_shared<JobResult>());
}

void HistogramService::Stop()
{
	check(IsInGameThread());
	Batch = nullptr;
}
 

void HistogramService::CaptureNextBatch()
{
	bCaptureNextBatch = true;
}
