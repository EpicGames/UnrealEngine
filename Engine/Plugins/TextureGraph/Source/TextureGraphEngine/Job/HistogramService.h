// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IdleService.h"
#include "Model/Mix/MixUpdateCycle.h"

#define UE_API TEXTUREGRAPHENGINE_API

class UModelObject;
typedef std::shared_ptr<class Job>		JobPtr;
typedef std::unique_ptr<class Job>		JobUPtr;
typedef std::weak_ptr<class Job>		JobPtrW;

class UMixInterface;

class HistogramService : public IdleService
{
private:
	JobBatchPtr Batch;
	bool bCaptureNextBatch = false;

public:
	UE_API HistogramService();
	UE_API virtual	~HistogramService() override;

	UE_API virtual AsyncJobResultPtr Tick() override;
	UE_API virtual void Stop() override;

	UE_API void AddHistogramJob(MixUpdateCyclePtr Cycle,JobUPtr JobToAdd, int32 TargetID, UMixInterface* Mix);
	UE_API JobBatchPtr GetOrCreateNewBatch(UMixInterface* mix);

	UE_API void CaptureNextBatch();
};

typedef std::shared_ptr<HistogramService>	HistogramServicePtr;
typedef std::weak_ptr<HistogramService>	HistogramServicePtrW;

#undef UE_API
