// Copyright Epic Games, Inc. All Rights Reserved.
#include "IdleService.h"

DEFINE_LOG_CATEGORY(LogIdle_Svc);

void IdleService::UpdateStats(double StartTime, double EndTime, bool DidOffendTimeLimit)
{
	FScopeLock Lock(&StatsMutex);

	StatsData.LastTick = EndTime;
	StatsData.LastTickDuration = EndTime - StartTime;
	StatsData.TotalTickDuration += StatsData.LastTickDuration;
	StatsData.NumTicks++;
	StatsData.AverageTickDuration = StatsData.TotalTickDuration / (double)StatsData.NumTicks;
	
	if (DidOffendTimeLimit)
		StatsData.NumTimeLimitOffenses++;
}
