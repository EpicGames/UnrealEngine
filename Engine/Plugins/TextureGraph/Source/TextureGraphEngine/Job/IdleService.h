// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "JobBatch.h"

#define UE_API TEXTUREGRAPHENGINE_API

DECLARE_LOG_CATEGORY_EXTERN(LogIdle_Svc, Log, Verbose);

class IdleService
{
public:
	struct Stats
	{
		double						LastTick = 0;			/// Time when this batch was last updated
		double						LastTickDuration = 0;	/// Time taken during the last tick 
		double						TotalTickDuration = 0;	/// Total tick duration
		double						AverageTickDuration = 0;/// Average duration of the tick
		size_t						NumTicks = 0;			/// The number of times this has been ticked
		size_t						NumTimeLimitOffenses = 0;/// How many times this batch offended the time limit
	};

protected:
	FString							Name;					/// The name of this idle batch. Used primarily for debugging purposes and logging

	FCriticalSection				StatsMutex;			/// Mutex for the stats object
	Stats							StatsData;					/// The stats for this idle batch

public:
	explicit						IdleService(const FString& InName) : Name(InName) {}
	virtual							~IdleService() = default;
	virtual AsyncJobResultPtr		Tick() = 0;
	virtual void					Stop() = 0;
	virtual bool					ShouldRunDuringBusyUpdate() const { return false; }

	UE_API void							UpdateStats(double StartTime, double EndTime, bool DidOffendTimeLimit);

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE const Stats&		GetStats() const { return StatsData; }
	FORCEINLINE const FString&		GetName() const { return Name; }
};

typedef std::shared_ptr<IdleService>	IdleServicePtr;

#undef UE_API
