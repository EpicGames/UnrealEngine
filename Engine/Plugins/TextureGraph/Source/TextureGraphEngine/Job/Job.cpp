// Copyright Epic Games, Inc. All Rights Reserved.
#include "Job.h"
#include "TextureGraphEngine.h"
#include "Data/Blob.h"
#include "Model/Mix/MixUpdateCycle.h"
#include "Device/FX/Device_FX.h"
#include "Device/Null/Device_Null.h"
#include "Profiling/StatGroup.h"
#include "TextureGraphEngineGameInstance.h"
#include "Data/Blobber.h"
#include "BlobHasherService.h"
#include "Model/Mix/MixSettings.h"
#include "Scheduler.h"
#include "2D/TargetTextureSet.h"
#include "JobArgs.h"
#include "Transform/Utility/T_FinaliseBlob.h"
#include "Transform/Utility/T_PrepareResources.h"
#include "Device/Mem/Device_Mem.h"
#include "Helper/Promise.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"

DEFINE_LOG_CATEGORY(LogJob);
DECLARE_CYCLE_STAT(TEXT("Job_PrepareResources"), STAT_Job_PrepareResources, STATGROUP_TextureGraphEngine);
DECLARE_CYCLE_STAT(TEXT("Job_PrepareTargets"), STAT_Job_PrepareTargets, STATGROUP_TextureGraphEngine);

std::shared_ptr<JobResult> JobResult::NullResult = std::make_shared<JobResult>(BlobRef(), nullptr);

JobResult::~JobResult()
{
}

//////////////////////////////////////////////////////////////////////////
Job::Job(int32 InTargetId, BlobTransformPtr InTransform, UObject* InErrorOwner /* = nullptr*/, uint16 InPriority /* = (uint16)JobPriority::kNormal */, uint64 InId /* = -1 */)
	: DeviceNativeTask(InPriority, InTransform ? InTransform->GetName() : TEXT(""))
	, MixObj(nullptr)
	, Id(InId)
	, Transform(InTransform)
	, ErrorOwner(InErrorOwner)
	, TileResults(0, 0)
	, TargetId(InTargetId)
	, TileInvalidationMatrix(0, 0)
{
}

Job::Job(UMixInterface* InMix, int32 InTargetId, BlobTransformPtr Transform, UObject* InErrorOwner /* = nullptr*/, uint16 priority /* = (uint16)JobPriority::kNormal */, uint64 id /* = -1 */)
	: Job(InTargetId, Transform, InErrorOwner, priority, id)
{
	check(InMix);
	MixObj = InMix;
}

Job::~Job()
{
	check(IsInGameThread());
	Args.Empty();
}

AsyncJobArgResultPtr Job::BindArgs(JobRunInfo InRunInfo, BlobTransformPtr TransformObj, int32 RowId, int32 ColId)
{
	if (!Args.Num())
		return cti::make_ready_continuable(std::make_shared<JobArgResult>());

	std::vector<std::decay_t<AsyncJobArgResultPtr>, std::allocator<std::decay_t<AsyncJobArgResultPtr>>> Promises;
	Promises.reserve(Args.Num());

	for (size_t ArgIndex = 0; ArgIndex < Args.Num(); ArgIndex++)
	{
		JobArgBindInfo ArgBindInfo = { this, InRunInfo.Batch, ArgIndex, RowId, ColId, InRunInfo.Dev, TransformObj, InRunInfo.Batch->GetCycle()->LODLevel() };

		if (!Args[ArgIndex]->Unbounded())
			Promises.emplace_back(std::forward<AsyncJobArgResultPtr>(Args[ArgIndex]->Bind(ArgBindInfo)));
		else
			Promises.emplace_back(std::forward<AsyncJobArgResultPtr>(cti::make_ready_continuable(std::make_shared<JobArgResult>())));
	}

	return cti::when_all(Promises.begin(), Promises.end()).then([this](std::vector<JobArgResultPtr> results) mutable
	{
		return std::make_shared<JobArgResult>();
	});
}

AsyncJobArgResultPtr Job::UnbindArgs(JobRunInfo InRunInfo, BlobTransformPtr TransformObj, int32 RowId, int32 ColId)
{
	if (!Args.Num())
		return cti::make_ready_continuable(std::make_shared<JobArgResult>());

	std::vector<std::decay_t<AsyncJobArgResultPtr>, std::allocator<std::decay_t<AsyncJobArgResultPtr>>> Promises;
	Promises.reserve(Args.Num());

	for (size_t ArgIndex = 0; ArgIndex < Args.Num(); ArgIndex++)
	{
		JobArgBindInfo ArgBindInfo = { this, InRunInfo.Batch, ArgIndex, RowId, ColId, InRunInfo.Dev, TransformObj, InRunInfo.Batch->GetCycle()->LODLevel() };

		if (!Args[ArgIndex]->Unbounded())
			Promises.emplace_back(std::forward<AsyncJobArgResultPtr>(Args[ArgIndex]->Unbind(ArgBindInfo)));
		else
			Promises.emplace_back(std::forward<AsyncJobArgResultPtr>(cti::make_ready_continuable(std::make_shared<JobArgResult>())));
	}

	return cti::when_all(Promises.begin(), Promises.end()).then([this](std::vector<JobArgResultPtr> results) mutable
	{
		return std::make_shared<JobArgResult>();
	});
}

AsyncInt Job::Bind_Or_Unbind_All_Generic(JobRunInfo InRunInfo, Bind_Unbind_Func BFunc)
{
	if (CanHandleTiles() && Transform->GeneratesData())
	{
		check(InRunInfo.Cycle->GetTarget(TargetId));

		std::vector<AsyncJobArgResultPtr> Promises;

		for (size_t RowId = 0; RowId < TileInvalidationMatrix.Rows(); RowId++)
		{
			for (size_t ColId = 0; ColId < TileInvalidationMatrix.Rows(); ColId++)
			{
				if (TileInvalidationMatrix[RowId][ColId])
				{
					Promises.push_back(BFunc(InRunInfo, TileResults[RowId][ColId]->Transform, RowId, ColId));
				}
			}
		}

		if (!Promises.empty())
		{
			auto FinalPromise(cti::when_all(Promises.begin(), Promises.end()));
			size_t Count = Promises.size();

			return cti::make_continuable<int32>([this, Count, FWD_PROMISE(FinalPromise), This = this](auto&& promise) mutable
			{
				/// Treat as a universal reference and change to l-value so that we can call .then(...) function
				(std::forward<decltype(FinalPromise)>(FinalPromise)).then([this, Count, FWD_PROMISE(promise), This](std::vector<JobArgResultPtr> results) mutable
				{
					UE_LOGF(LogJob, VeryVerbose, "Returning bind/unbind promise (multiple): %llu.%ls", This->RunInfo.Batch->BatchId, *This->Transform->GetName());
					promise.set_value((int32)Count);
				});
			});
		}
	}
	else
	{
		return cti::make_continuable<int32>([this, BFunc, This = this](auto&& promise)
		{
			BFunc(RunInfo, Transform, -1, -1).then([this, FWD_PROMISE(promise), This](JobArgResultPtr) mutable
			{
				UE_LOGF(LogJob, VeryVerbose, "Returning bind/unbind promise (single): %llu.%ls", This->RunInfo.Batch->BatchId, *This->Transform->GetName());
				promise.set_value(1);
			});
		});
	}

	UE_LOGF(LogJob, VeryVerbose, "Returning bind/unbind promise (none): %llu.%ls", InRunInfo.Batch->BatchId, *Transform->GetName());

	return cti::make_ready_continuable(0);
}

AsyncInt Job::BindArgs_All(JobRunInfo InRunInfo)
{
	using namespace std::placeholders;
	Bind_Unbind_Func BFunc = std::bind(&Job::BindArgs, this, _1, _2, _3, _4);
	return Bind_Or_Unbind_All_Generic(InRunInfo, BFunc);
}

AsyncInt Job::UnbindArgs_All(JobRunInfo InRunInfo)
{
	using namespace std::placeholders;
	Bind_Unbind_Func BFunc = std::bind(&Job::UnbindArgs, this, _1, _2, _3, _4);
	return Bind_Or_Unbind_All_Generic(InRunInfo, BFunc);
}

//////////////////////////////////////////////////////////////////////////

Job* Job::SetArgs(const TArray<JobArgPtr>& NewArgs)
{
	Args = NewArgs;
	return this;
}

Job* Job::AddArg(JobArgPtr Arg)
{
	Args.Add(Arg);
	return this;
}

uint32_t Job::NumArgs() const
{
	return Args.Num();
}

JobArgPtr Job::GetArg(uint32_t Index) const
{
	return Args[Index];
}

CHashPtr Job::Hash() const
{
	if (HashValue || !Transform)
		return HashValue;

	HashValue = TileHash(-1, -1);

	return HashValue;
}

CHashPtr Job::CalcTileHash(int32 RowId, int32 ColId) const
{
	CHashPtrVec ArgHashes;
	ArgHashes.reserve(Args.Num() + 1); /// +1 for the Transform

	for (size_t ArgIndex = 0; ArgIndex < Args.Num(); ArgIndex++)
	{
		JobArgPtr Arg = Args[ArgIndex];

		if (!Arg->IgnoreHash())
		{
			CHashPtr ArgHash;

			if (RowId >= 0 && ColId >= 0)
				ArgHash = Arg->TileHash(RowId, ColId);
			else
				ArgHash = Arg->Hash();

			if (!ArgHash)
				return nullptr;

			ArgHashes.push_back(ArgHash);
		}
	}

	ArgHashes.push_back(Transform->Hash());

	/// Sometimes, if the job has no real tiled blobs, the temp hashes of all the tiles
	/// and the full tiled BlobObj are going to be exactly the same.
	/// For the full job hash calculation we can append something to differentiate it from 
	/// any of the tiles. Otherwise we won't be able to differentiate between the two
	/// when someone queries it from the blobber. 
	if (RowId < 0 && ColId < 0)
	{
		ArgHashes.push_back(std::make_shared<CHash>(DataUtil::Hash_Int32(-1), true));
		ArgHashes.push_back(std::make_shared<CHash>(DataUtil::Hash_Int32(-1), true));
	}

	/// Add LOD-level to the hashing system
	int32 LODLevel = 0;

	if (RunInfo.Cycle)
		LODLevel = RunInfo.Cycle->LODLevel();

	ArgHashes.push_back(std::make_shared<CHash>(DataUtil::Hash_Int32(LODLevel), true));

	/// Must have either one of these available
	if (Result || DesiredResultDesc)
	{
		BufferDescriptor ResultDesc = Result ? GetResultDesc() : *DesiredResultDesc;
		int32 NumRows = MixObj->GetNumXTiles();
		int32 NumCols = MixObj->GetNumYTiles();

		ArgHashes.push_back(std::make_shared<CHash>(DataUtil::Hash_Int32(ResultDesc.Width), true));
		ArgHashes.push_back(std::make_shared<CHash>(DataUtil::Hash_Int32(ResultDesc.Height), true));
		ArgHashes.push_back(std::make_shared<CHash>(DataUtil::Hash_Int32(ResultDesc.ItemsPerPoint), true));
		ArgHashes.push_back(std::make_shared<CHash>(DataUtil::Hash_Int32((int32)ResultDesc.Format), true));
		ArgHashes.push_back(std::make_shared<CHash>(DataUtil::Hash_Int32((int32)ResultDesc.bIsSRGB), true));
		ArgHashes.push_back(std::make_shared<CHash>(DataUtil::Hash_Int32(NumRows), true));
		ArgHashes.push_back(std::make_shared<CHash>(DataUtil::Hash_Int32(NumCols), true));
	}

	return CHash::ConstructFromSources(ArgHashes);
}

CHashPtr Job::TileHash(int32 RowId, int32 ColId) const
{
	CHashPtr Hash = CalcTileHash(RowId, ColId);
	check(Hash);
	return TextureGraphEngine::GetBlobber()->AddGloballyUniqueHash(Hash);
}

BufferDescriptor Job::GetCombinedDesc(BufferDescriptor& ArgsDescCombined, size_t& Count) const
{
	std::vector<BufferDescriptor> Descs;
	Descs.reserve(Args.Num() + 1); /// +1 for the Transform

	Count = 0;

	for (size_t ArgIndex = 0; ArgIndex < Args.Num(); ArgIndex++)
	{
		JobArgPtr Arg = Args[ArgIndex];
		const BufferDescriptor* ArgDesc = Arg->GetDescriptor();

		if (ArgDesc && !Arg->IgnoreDesc())
		{
			Descs.push_back(*ArgDesc);
		}
	}

	BufferDescriptor Desc;

	if (!Descs.empty())
	{
		Count = Descs.size();
		if (Descs.size() >= 2)
		{
			Desc = BufferDescriptor::Combine(Descs[0], Descs[1]);

			for (size_t Index = 2; Index < Descs.size(); Index++)
			{
				Desc = BufferDescriptor::Combine(Desc, Descs[Index]);
			}
		}
		else
			Desc = Descs[0];
	}
	else
	{
		check(DesiredResultDesc);
		return *DesiredResultDesc.get();
	}

	ArgsDescCombined = Desc;

	if (DesiredResultDesc)
	{
		//if (DesiredResultDesc->IsLateBound())
		Desc = BufferDescriptor::CombineWithPreference(&Desc, DesiredResultDesc.get(), nullptr);
		Desc.Metadata = DesiredResultDesc->Metadata;
		Desc.Name = DesiredResultDesc->Name;
		Count++;
	}

	if (Result)
	{
		const BufferDescriptor& ExistingDesc = Result->GetDescriptor();
		Desc.Name = ExistingDesc.Name;
		Desc.Metadata = ExistingDesc.Metadata;
	}

	return Desc;
}

BufferDescriptor Job::GetResultDesc() const
{
	if (!Result)
		return BufferDescriptor();

	BufferDescriptor ResultDesc = Result->GetDescriptor();
	check(ResultDesc.Width > 0 && ResultDesc.Height > 0);
	return ResultDesc;
}

bool Job::CheckDefaultArgs() const
{
	for (size_t ArgIndex = 0; ArgIndex < Args.Num(); ArgIndex++)
	{
		JobArgPtr Arg = Args[ArgIndex];

		if (!Arg->IgnoreHash() && !Arg->IsDefault())
			return false;
	}

	return true;
}

bool Job::CanHandleTiles() const
{
	if (!bIsTiled) // || RunInfo.cycle->Details().IsDiscard())
		return false;

	if (!Transform->CanHandleTiles() || !Transform->GeneratesData())
		return false;

	for (JobArgPtr Arg : Args)
	{
		if (Arg->ForceNonTiledTransform())
			return false;
	}

	return true;
}

void Job::SetTileResult(int32 RowId, int32 ColId, JobResultPtr TileResult)
{
	check(TileResult->HashValue);

	if (RowId >= 0 && ColId >= 0)
		TileResults[RowId][ColId] = TileResult;
	else
		FinalJobResult = TileResult;
}

bool Job::CheckCached()
{
	CHashPtr TempHash = Hash();

	if (!TempHash)
		return false;

	TiledBlobRef ExistingResult = TextureGraphEngine::GetBlobber()->FindTiled(Hash()->Value());
	if (ExistingResult)
	{
		check(ExistingResult->IsTiled());
		BufferDescriptor Desc = ExistingResult->GetDescriptor();
		Desc.bIsTransient = IsDiscard();

		if (!ResultOrg)
			ResultOrg = std::make_shared<TiledBlob_Promise>(Desc, ExistingResult->Rows(), ExistingResult->Cols(), ExistingResult->Hash());

		ExistingResult->OnFinalise().then([ResultOrg = this->ResultOrg, ExistingResult]() mutable
		{
			//check(ExistingResult->IsFinalised());
			ResultOrg->FinaliseFrom(ExistingResult.get());
		});

		Result = TiledBlobRef(std::static_pointer_cast<TiledBlob>(ResultOrg), true, false);
		FinalJobResult = std::make_shared<JobResult>(GetResultRef(), nullptr);

		bIsCulled = true;

		return true;
	}

	return false;
}

bool Job::IsDiscard() const
{
	bool IsDiscardBatch = false;
	if (RunInfo.Batch)
		IsDiscardBatch = RunInfo.Batch->GetCycle()->GetDetails().IsDiscard();

	return IsDiscardBatch || (GetReplayCount() > 0);
}

TiledBlobPtr Job::InitResult(FString InNewName, const BufferDescriptor* InDesiredDesc, int32 NumTilesX /* = 0 */, int32 NumTilesY /* = 0 */)
{
	size_t NumInputBlobs = 0;
	BufferDescriptor ArgsDescCombined;
	BufferDescriptor CombinedDesc;
	BufferDescriptor ResultDesc;

	if (InDesiredDesc)
	{
		DesiredResultDesc = std::make_shared<BufferDescriptor>(*InDesiredDesc);
		CombinedDesc = GetCombinedDesc(ArgsDescCombined, NumInputBlobs);
	}

	if (!DesiredResultDesc || !DesiredResultDesc->IsFinal())
	{
		ResultDesc = CombinedDesc;

		if (DesiredResultDesc && NumInputBlobs && DesiredResultDesc->IsLateBound())
			DesiredResultDesc->Format = BufferFormat::Auto;
	}
	else if (DesiredResultDesc && DesiredResultDesc->IsFinal())
		ResultDesc = *DesiredResultDesc;

	/// Naming convention is all over the place right now resulting in confusing names of jobs and results.
	/// We ignore them for the time being and 
	DebugJobName = InNewName;
	FString NewName = Transform->GetName();

	if (!InNewName.IsEmpty())
		NewName = InNewName;

	ResultDesc.bIsTransient = IsDiscard();
	ResultDesc.Name = NewName;

	if (ResultDesc.Width < static_cast<uint32>(EResolution::Auto) || ResultDesc.Width > static_cast<uint32>(EResolution::Resolution8192))
		ResultDesc.Width = static_cast<uint32>(EResolution::Auto);

	if (ResultDesc.Height < static_cast<uint32>(EResolution::Auto) || ResultDesc.Height > static_cast<uint32>(EResolution::Resolution8192))
		ResultDesc.Height = static_cast<uint32>(EResolution::Auto);

	/// Update the job name as well
	Name = NewName;

	check(!ResultDesc.Name.IsEmpty());

	if (ResultDesc.IsLateBound())
	{
		return InitLateBoundResult(NewName, ResultDesc, NumInputBlobs);
	}
	else if (DesiredResultDesc && ArgsDescCombined.IsLateBound() && DesiredResultDesc->IsAutoSize() && ArgsDescCombined.IsAutoSize())
	{
		ResultDesc.Format = BufferFormat::LateBound;
		return InitLateBoundResult(NewName, ResultDesc, NumInputBlobs);
	}

	TargetTextureSetPtr& Target = MixObj->GetSettings()->Target(TargetId);
	if (ResultDesc.Width <= 0)
	{
		if(ArgsDescCombined.Width <= 0)
		{
			ResultDesc.Width = Target->GetWidth();
		}
		else
		{
			ResultDesc.Width = ArgsDescCombined.Width;			
		}
		
	}
	if (ResultDesc.Height <= 0)
	{
		if(ArgsDescCombined.Height <= 0)
		{
			ResultDesc.Height = Target->GetHeight();
		}
		else
		{
			ResultDesc.Height = ArgsDescCombined.Height;	
		}
	}

	if (!NumTilesX)
		NumTilesX = MixObj->GetNumXTiles();

	if (!NumTilesY)
		NumTilesY = MixObj->GetNumYTiles();

	check(NumTilesX > 0 && NumTilesY > 0);

	if (DesiredResultDesc)
		*DesiredResultDesc = ResultDesc;

	/// Now that we have a descriptor, check for cached result
	if (CheckCached())
		return Result;

	CHashPtr BlobHash = Hash();

	check(!ResultOrg);
	ResultOrg = std::make_shared<TiledBlob_Promise>(ResultDesc, NumTilesX, NumTilesY, BlobHash);
	Result = TiledBlobRef(std::static_pointer_cast<TiledBlob>(ResultOrg), true, false);

	check(Result->GetDescriptor().Width >= Result->Rows());
	check(Result->GetDescriptor().Height >= Result->Cols());

	return Result;
}

TiledBlobPtr Job::InitLateBoundResult(FString NewName, BufferDescriptor DesiredDesc, size_t NumInputBlobs)
{
	/// We can only check for a cached result for late bound blobs if there are no input blobs
	if (!NumInputBlobs && CheckCached())
	{
		return Result;
	}

	BufferDescriptor Desc = DesiredDesc;
	Desc.Name = NewName;
	Desc.Format = BufferFormat::LateBound;
	Desc.bIsTransient = IsDiscard();

	CHashPtr TempHash = Hash(); ///std::make_shared<CHash>(Desc.HashValue(), false);
	check(MixObj);

	int32 NumTilesX = MixObj->GetNumXTiles();
	int32 NumTilesY = MixObj->GetNumYTiles();

	ResultOrg = std::make_shared<TiledBlob_Promise>(Desc, NumTilesX, NumTilesY, TempHash);
	Result = TiledBlobRef(std::static_pointer_cast<TiledBlob>(ResultOrg), true, false);

	return Result;
}

AsyncJobResultPtr Job::FinaliseTiles(JobRunInfo InRunInfo)
{
	//std::vector<AsyncJobResultPtr> Promises;
	ResourceBindInfo ResBindInfo;

	std::vector<std::decay_t<AsyncBufferResultPtr>, std::allocator<std::decay_t<AsyncBufferResultPtr>>> Promises;

	bool bIsDiscard = RunInfo.Cycle->GetDetails().IsDiscard();

	/// only flush if these have not been marked to bIsDiscard
	AsyncJobResultPtr FinalPromise = cti::make_ready_continuable(FinalJobResult);
	double flushStartTime = Util::Time();

	check(Result->IsPromise());
	TiledBlob_PromisePtr result = GetResultPromise();

	if (!bIsDiscard)
	{
		if (CanHandleTiles())
		{
			for (size_t RowId = 0; RowId < TileResults.Rows(); RowId++)
			{
				for (size_t ColId = 0; ColId < TileResults.Rows(); ColId++)
				{
					JobResultPtr JobResult = TileResults[RowId][ColId];
					auto BlobObj = JobResult->BlobObj.lock();

					if (BlobObj)
					{
						UE_LOGF(LogJob, VeryVerbose, "FINISHED: %llu => %ls", JobResult->HashValue->Value(), *BlobObj->Name());
						result->SetTile(RowId, ColId, BlobObj);
					}
				}
			}
		}
	}

	return result->Finalise(false, 0)
		.then([this]() mutable
		{
			FinalJobResult = std::make_shared<JobResult>(GetResultRef(), nullptr);
			return FinalJobResult;
		});
}

bool Job::CheckCulled(JobRunInfo InRunInfo)
{
	check(IsInGameThread());

	/// If already has a result then job is already done
	if (FinalJobResult)
	{
		/// If this hits, then the job was fully culled already
		check(bIsCulled);
		check(Result); /// Must have a valid result pointer at least (the result itself might not be ready)
		return true;
	}

	/// If already culled (for whatever reason, or some custom implementation)
	/// then don't bother at all
	if (bIsCulled)
		return true;

	if (!CanHandleTiles())
		return false;

	/// Late bound cannot be culled early. We basically know nothing about this
	if (!Result || Result->IsLateBound())
		return false;

	/// There is a bug in this code. It basically (incorrectly) treats a full (non-tiled) blob and tries to 
	/// cull the job that generates it. Disabling for the time being as it's not a particular performance
	/// deterrant. But this needs to be tweaked for these kinds of jobs (like T_Transform for instance)
	check(Result->IsPromise());
	TiledBlob_PromisePtr ResultPromise = GetResultPromise();
	size_t NumRows = ResultPromise->Rows();
	size_t NumCols = ResultPromise->Cols();
	BlobPtrTiles Tiles(NumRows, NumCols);
	std::vector<AsyncBlobResultPtr> Promises;

	for (size_t RowId = 0; RowId < ResultPromise->Rows(); RowId++)
	{
		for (size_t ColId = 0; ColId < ResultPromise->Cols(); ColId++)
		{
			/// Check if any of the arguments are late bound
			for (const JobArgPtr& Arg : Args)
			{
				if (Arg->IsLateBound(RowId, ColId))
					return false;
			}

			CHashPtr ResultTileHash = TileHash(RowId, ColId);
			BlobRef TileBlob = TextureGraphEngine::GetBlobber()->FindSingle(ResultTileHash->Value());

			/// Couldn't find the result. Must run the job
			if (!TileBlob)
				return false;

			if (!TileBlob->IsFinalised())
				Promises.emplace_back(TileBlob->OnFinalise());

			UE_LOGF(LogJob, VeryVerbose, "[Job - %ls] matched tile: %d, %d", *GetName(), (int32)RowId, (int32)ColId);

			Tiles[RowId][ColId] = TileBlob;
		}
	}

	/// We get here then that means we have all the tiles
	ResultPromise->SetTiles(Tiles);

	if (!Promises.empty())
	{
		cti::when_all(Promises.begin(), Promises.end()).then([Result = this->Result]() mutable
		{
			/// Finalise the result if it hasn't been finalised yet
			if (Result && !Result->IsFinalised())
				Result->Finalise(true, 0);
		});
	}
	else
	{
		/// Finalise the result if it hasn't been finalised yet
		if (Result && !Result->IsFinalised())
			Result->Finalise(true, 0);
	}

	UE_LOGF(LogJob, Verbose, "[Job - %ls] All tiles found. Marking as culled!", *GetName());

	bIsCulled = true;

	/// Nothing to do over here. We just call the BeginNative and EndNative here
	/// so that the job collects timing information
	BeginNative(InRunInfo);
	EndNative();

	/// Make sure that the promise is resolved over here, if the job is culled
	SetPromise(0);

	return bIsCulled;
}

AsyncPrepareResult Job::PrepareTargets(JobBatch* Batch)
{
	check(IsInGameThread());

	Stats.TargetPrepStartTime = Stats.TargetPrepEndTime = Util::Time();
	Stats.TargetPrepWaitStartTime = Stats.TargetPrepWaitEndTime = Util::Time();

	SCOPE_CYCLE_COUNTER(STAT_Job_PrepareTargets)
	/// Don't wanna do anything for these devices
	if (Transform->NumTargetDevices() > 1)
		return cti::make_ready_continuable(0);

	/// If the job already has a result then that means that the job is already done
	/// we don't wanna do anything about it
	if (FinalJobResult || bIsCulled)
		return cti::make_ready_continuable(0);

	FinalJobResult = std::make_shared<JobResult>(GetResultRef(), Hash());
	FinalJobResult->Transform = Transform;

	if (!Transform->GeneratesData())
		return cti::make_ready_continuable(0);

	Device* TargetDevice = Transform->TargetDevice(0);
	ResourceBindInfo BindInfo;
	BindInfo.bWriteTarget = true;
	BindInfo.Dev = TargetDevice;

	/// We cannot prepare this right now
	if (Result && (!Result->GetDescriptor().IsValid() || Result->GetDescriptor().IsLateBound()))
	{
		size_t Count = 0;
		BufferDescriptor ArgDescCombined;
		BufferDescriptor CombinedDesc = GetCombinedDesc(ArgDescCombined, Count);
		check(Count > 0);
		check(!ArgDescCombined.IsLateBound());
		check(!CombinedDesc.IsAutoSize() && !ArgDescCombined.IsAutoSize());
		Result->ResolveLateBound(CombinedDesc);
	}

	check(!Result || Result->GetDescriptor().IsValid());
	check(Result->IsPromise());
	
	TiledBlob_PromisePtr ResultPromise = GetResultPromise();

	/// Resolve late bound blobs

	const BufferDescriptor& ResultDesc = ResultPromise->GetDescriptor();
	BufferDescriptor ResultTileDesc = ResultDesc;
	bool bIsPersistent = false;
	DeviceTransferChain TransferChain;

	if (ResultPromise->GetBufferRef())
	{
		TransferChain = ResultPromise->GetBufferRef()->GetDeviceTransferChain(&bIsPersistent);
	}

	ResultTileDesc.Width = ResultPromise->GetWidth() / ResultPromise->Rows();
	ResultTileDesc.Height = ResultPromise->GetHeight() / ResultPromise->Cols();

	check(ResultTileDesc.Width > 0 && ResultTileDesc.Height > 0);

	//Prepare tile buffers even if Target cant handle tiles
	//This is because in any case we have to do it, that is because we need these tiles to split our buffer on
	//So why not do it on prepare and let the tiles be ready by the time we finalize job

	bool bCanHandleTiles = CanHandleTiles();

	std::vector<std::decay_t<AsyncPrepareResult>, std::allocator<std::decay_t<AsyncPrepareResult>>> Promises;

	if (bCanHandleTiles)
	{
		ResultPromise->TiledTarget() = true;
	}
	else
	{
		ResultPromise->TiledTarget() = false;
		if (!ResultPromise->IsSingleBlob())
			Promises.push_back(ResultPromise->PrepareForWrite(BindInfo));
	}

	/// Keep this consistent across all the different types. Inconsistent sizes of these arrays
	/// was bowling over TextureGraph Insight. Consistency is better.
	TileResults.Resize(ResultPromise->Rows(), ResultPromise->Cols());
	TileInvalidationMatrix.Resize(ResultPromise->Rows(), ResultPromise->Cols());

	bool bIsDiscard = IsDiscard(); /// In the case we are replaying the job for debug purpose, do not cache the result
	std::unordered_map<HashType, BlobRef> DuplicatesThisJob;

	for (size_t RowId = 0; RowId < ResultPromise->Rows(); RowId++)
	{
		for (size_t ColId = 0; ColId < ResultPromise->Cols(); ColId++)
		{
			CHashPtr ResultTileHash = bCanHandleTiles ? TileHash(RowId, ColId) : TileHash(-1, -1);
			BlobRef TileBlob = !(Batch->IsNoCache() || bIsNoCache) ? TextureGraphEngine::GetBlobber()->FindSingle(ResultTileHash->Value()) : BlobRef();

			if (!TileBlob && DuplicatesThisJob.size())
			{
				auto iter = DuplicatesThisJob.find(ResultTileHash->Value());

				if (iter != DuplicatesThisJob.end())
					TileBlob = iter->second;
			}

			if (!TileBlob || !bCanHandleTiles)
			{
				CHashPtr TileHashCopy;
				ResultTileDesc.Name = TextureHelper::CreateTileName(ResultDesc.Name, RowId, ColId);
				ResultTileDesc.bIsTransient = bIsDiscard;

				TileHashCopy = std::make_shared<CHash>(ResultTileHash); /// Make a copy of the hash

				DeviceBufferRef TileBuffer = TargetDevice->Create(ResultTileDesc, TileHashCopy);

				BlobPtr TileBlobNew = std::make_shared<Blob>(TileBuffer);
				TileBlobNew->GetBufferRef()->SetDeviceTransferChain(TransferChain, bIsPersistent);
				Promises.push_back(TileBlobNew->PrepareForWrite(BindInfo));

				check(TileBlobNew->GetWidth() > 0 && TileBlobNew->GetHeight() > 0);

				TileInvalidationMatrix[RowId][ColId] = true;

				BlobCacheOptions CacheOpt;
				CacheOpt.Discard = bIsDiscard;
				CacheOpt.NoCacheBatch = Batch->IsNoCache();

				TileBlob = TextureGraphEngine::GetBlobber()->AddResult(TileHashCopy, TileBlobNew, CacheOpt);
			}
			else
			{
				UE_LOGF(LogJob, VeryVerbose, "Transform: %ls (Tile: %d, %d) => %ls", *Transform->GetName(), (int32)RowId, (int32)ColId, *TileBlob->Name());
				TileInvalidationMatrix[RowId][ColId] = false;
			}

			if (bCanHandleTiles)
			{
				JobResultPtr TileResult = std::make_shared<JobResult>(TileBlob, ResultTileHash);

				if (TileInvalidationMatrix[RowId][ColId])
				{
					FString TransformCloneName = FString::Printf(TEXT("%s-%llu,%llu"), *Transform->GetName(), RowId, ColId);
					BlobTransformPtr TransformClone = Transform->DuplicateInstance(TransformCloneName);

					if (TransformClone)
						TileResult->Transform = TransformClone;

					if (bIsDiscard)
						DuplicatesThisJob[ResultTileHash->Value()] = TileBlob;
				}

				TileResults[RowId][ColId] = TileResult;
			}

			ResultPromise->SetTile(RowId, ColId, TileBlob);
		}
	}

	/// Transient flag should've been set correctly earlier on
	if (bIsDiscard)
		Result->SetTransient();

	AddResultToBlobber();

	bIsNoCache = false;
	Stats.TargetPrepWaitEndTime = Util::Time();

	if (!Promises.empty())
	{
		Stats.TargetPrepWaitStartTime = Util::Time();

		return cti::when_all(Promises.begin(), Promises.end())
			.then([=]() mutable
			{
				return 0;
			});
	}

	return cti::make_ready_continuable(0);
}

AsyncPrepareResult Job::PrepareResources(JobBatch* Batch)
{
	check(IsInGameThread());

	SCOPE_CYCLE_COUNTER(STAT_Job_PrepareResources)
	UE_LOGF(LogJob, VeryVerbose, "Preparing resources for job: %ls", *Transform->GetName());

	/// Also allow the Transform to prepare itself from the game thread
	TransformArgs TransArgs;
	TransArgs.Dev = nullptr;
	TransArgs.JobObj = this;
	TransArgs.TargetId = TargetId;
	TransArgs.Cycle = Batch->GetCycle();
	

	UE_LOGF(LogJob, VeryVerbose, "Preparing resources for Transform: %ls", *Transform->GetName());
	return Transform->PrepareResources(TransArgs)
		.then([this, Batch](int32)
		{
			return PrepareTargets(Batch);
		});
	}

AsyncJobResultPtr Job::RunTile(JobRunInfo InRunInfo, int32 RowId, int32 ColId)
{
	if (RowId >= 0 && ColId >= 0)
	{
		if (!TileInvalidationMatrix[RowId][ColId])
			return cti::make_ready_continuable(TileResults[RowId][ColId]);
	}

	CHashPtr JobHash = FinalJobResult->HashValue;
	BlobTransformPtr ResultTransform = FinalJobResult->Transform;

	if (RowId >= 0 && ColId >= 0)
	{
		ResultTransform = TileResults[RowId][ColId]->Transform;
		JobHash = TileResults[RowId][ColId]->HashValue;
	}

	check(ResultTransform);

	UE_LOGF(LogJob, VeryVerbose, "No cached results for hash: %llu [Recalculating ...]", JobHash->Value());

	/// wait for the Transform to finish
	ExecTransform(InRunInfo, ResultTransform, RowId, ColId, JobHash).apply(cti::transforms::wait());

	if (ResultTransform->GeneratesData() && CanHandleTiles())
	{
		TileResults[RowId][ColId]->HashValue = JobHash;
		return cti::make_ready_continuable(TileResults[RowId][ColId]);
	}
	else
	{
		FinalJobResult = std::make_shared<JobResult>(GetResultRef(), JobHash);
		FinalJobResult->Transform = ResultTransform;
	}

	return cti::make_ready_continuable(FinalJobResult);
}

AsyncTransformResultPtr Job::ExecTransform(JobRunInfo InRunInfo, BlobTransformPtr TransformObj, int32 RowId, int32 ColId, CHashPtr JobHash)
{
	TransformArgs TransArgs;
	TransArgs.Dev = RunInfo.Dev;
	TransArgs.JobObj = this;
	TransArgs.TargetId = TargetId;
	TransArgs.Cycle = RunInfo.Cycle;
	TransArgs.Mesh = Mesh;

	if (TransformObj->GeneratesData())
	{
		TransArgs.Target = GetResultRef();

		if (RowId >= 0 && ColId >= 0 && Result->TiledTarget())
		{
			BlobRef TileResult = Result->GetTile(RowId, ColId);
			
			/// This should've been prepared in the PrepareResources function
			check(TileResult);

			/// If the tile result has already been finalised then we don't do anything to it
			if (TileResult->IsFinalised())
				return cti::make_ready_continuable(std::make_shared<TransformResult>());

			TransArgs.Target = TileResult.get();

			/// Set the job result preemptively
			JobResultPtr TileJobResult = std::make_shared<JobResult>(TransArgs.Target, JobHash);
			TileJobResult->Transform = TransformObj;
			TileJobResult->BlobObj = TileResult;

			SetTileResult(RowId, ColId, TileJobResult);
		}
	}

	return TransformObj->Exec(TransArgs);
}

AsyncJobResultPtr Job::RunSingle(JobRunInfo InRunInfo)
{
	check(!Result || (Result->GetBufferRef().IsValid() && !Result->GetBufferRef()->IsNull()));

	RunTile(InRunInfo, -1, -1).apply(cti::transforms::wait());

	Stats.EndRunTime = Util::Time();

	return cti::make_ready_continuable(FinalJobResult);
}

AsyncJobResultPtr Job::Run(JobRunInfo InRunInfo)
{
	/// Must have a transformation function
	check(Transform);
	check(RunInfo.Dev);

	auto ResultBlob = Result;

	/// Must have a valid result already setup if the Transform needs to generate data
	check(ResultBlob || !Transform->GeneratesData());

	UE_LOGF(LogJob, VeryVerbose, "Starting job: %llu.%d.%llu [Transform: %ls]", RunInfo.Batch->GetBatchId(), QueueId, Id, *Transform->GetName());

	Stats.BeginRunTime = Util::Time();

	/// This is the most common case
	if (CanHandleTiles() && Transform->GeneratesData())
	{
		/// Must have a valid result object created by now!
		check(ResultBlob && ResultBlob->Rows() && ResultBlob->Cols());

		check(RunInfo.Cycle);
		check(RunInfo.Cycle->GetTarget(TargetId));

		const TileInvalidateMatrix& InvalidationMatrix = RunInfo.Cycle->GetTarget(TargetId)->GetInvalidationMatrix();
		
		if (!InvalidationMatrix.Rows() || !InvalidationMatrix.Cols())
			return cti::make_ready_continuable(std::make_shared<JobResult>(GetResultRef(), nullptr));

		TileInvalidateMatrix LocalMatrix(Result->Rows(), Result->Cols());
		if (LocalMatrix.Rows() == InvalidationMatrix.Rows() && LocalMatrix.Cols() == InvalidationMatrix.Cols())
		{
			LocalMatrix = InvalidationMatrix;
		}
		else
		{
			for (int32 RowId = 0; RowId < (int32)LocalMatrix.Rows(); RowId++)
			{
				for (int32 ColId = 0; ColId < (int32)LocalMatrix.Cols(); ColId++)
				{
					LocalMatrix(RowId, ColId) = true;
				}
			}
		}

		std::vector<std::decay_t<AsyncJobResultPtr>, std::allocator<std::decay_t<AsyncJobResultPtr>>> Promises;
		for (int32 RowId = 0; RowId < (int32)LocalMatrix.Rows(); RowId++)
		{
			for (int32 ColId = 0; ColId < (int32)LocalMatrix.Cols(); ColId++)
			{
				int32 InvalidateThisTile = LocalMatrix[RowId][ColId];
				if (InvalidateThisTile != 0)
				{
					Promises.reserve(1 + Promises.size());
					Promises.emplace_back(std::forward<AsyncJobResultPtr>(RunTile(InRunInfo, RowId, ColId)));
				}
				else
				{
					/// TODO: use _defaultSource to copy over to the _result
				}
			}
		}

		if (!Promises.size())
		{
			Stats.EndRunTime = Util::Time();
			return cti::make_ready_continuable(FinalJobResult);
		}

		/// Just wait for all to finish
		cti::when_all(Promises.begin(), Promises.end()).apply(cti::transforms::wait());

		Stats.EndRunTime = Util::Time();
		return cti::make_ready_continuable(FinalJobResult);
	}
	else
		return RunSingle(RunInfo);
}

//////////////////////////////////////////////////////////////////////////
AsyncInt Job::BeginNative(JobRunInfo InRunInfo)
{
	RunInfo = InRunInfo;

	UE_LOGF(LogJob, VeryVerbose, "Job::BeginNative: %llu.%llu.%ls", RunInfo.Batch->GetBatchId(), Id, *Transform->GetName());

	Stats.BeginNativeTime = Util::Time();

	if (bIsCulled)
		return cti::make_ready_continuable(0);

	return PrepareResources(RunInfo.Batch)
		.then([this](int32 result)
		{
			return BindArgs_All(RunInfo);
		});
}

void Job::MarkJobDone()
{
	Stats.EndNativeTime = Util::Time();

	check((!Result || Result->IsFinalised()) && (!ResultOrg || ResultOrg->IsFinalised()));
	if (RunInfo.Batch)
	{
		RunInfo.Batch->OnJobDone(this, GetJobId());
		
		UE_LOGF(LogJob, VeryVerbose, "Job::Done: %llu.%lld.%ls", RunInfo.Batch->GetBatchId(), GetJobId(), *Transform->GetName());
	}

	DeviceNativeTask::bIsDone = true;

	Prev.clear();
}

void Job::AddResultToBlobber()
{
	if (ResultOrg && !IsCulled())
	{
		/// Add hashes to the results into the blobber
		auto TempHash = Hash();
		check(TempHash->IsValid());

		/// Blob must have a valid hash (even if its a temp one) by now
		check(ResultOrg->Hash());

		auto ResultHash = ResultOrg->Hash();

		/// We just peg it against temp hash. Blobber automatically adds for ResultOrg own hash anyway
		BlobCacheOptions CacheOpt;

		CacheOpt.Discard = IsDiscard();
		CacheOpt.NoCacheBatch = RunInfo.Batch->IsNoCache();

		/// TODO: A bit of juggling because of previous tech debt. Clean up later
		check(ResultOrg.get());

#if DEBUG_BLOB_REF_KEEPING == 1
		check(!TextureGraphEngine::Blobber()->IsBlobReferenced(ResultOrg.get()));
#endif 

		Result = TextureGraphEngine::GetBlobber()->AddTiledResult(TempHash, ResultOrg, CacheOpt);
		check(Result && Result->IsTiled());
	}
}

AsyncJobResultPtr Job::EndNative()
{
	Stats.EndNativeTime = Util::Time();

	AddResultToBlobber();

	/// Nothing to do over here
	if (bIsCulled)
		return cti::make_ready_continuable(FinalJobResult);

	return UnbindArgs_All(RunInfo)
		.then([this](int32) 
		{
			if (Result && !Result->IsFinalised())
				return Result->Finalise(true, nullptr);
			return static_cast<AsyncBufferResultPtr>(cti::make_ready_continuable<BufferResultPtr>(std::make_shared<BufferResult>()));
		})
		.then([this](BufferResultPtr) 
		{
			MarkJobDone();
			return FinalJobResult;
		});
}

//////////////////////////////////////////////////////////////////////////
Device* Job::GetTargetDevice() const
{
	return RunInfo.Dev ? RunInfo.Dev : Transform->TargetDevice(0);
}

AsyncInt Job::PreExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type ReturnThread)
{
	UE_LOGF(LogJob, VeryVerbose, "PreExec::%ls", *Transform->GetName());
	//DeviceNativeTask::PreExec();
	return cti::make_ready_continuable(0);
}

FString Job::GetDebugName() const
{
	return FString::Printf(TEXT("%s [%" INT64_FMT ", %d, ID: %" INT64_FMT "]"), *GetName(), GetTaskId(), GetPriority(), Id);
}

FString Job::GetName() const
{
	if (!Name.IsEmpty())
		return Name;
	return Transform->GetName();
}

bool Job::DebugCompleteCheck()
{
	DeviceNativeTask::DebugCompleteCheck();
 	check(Stats.EndNativeTime >= Stats.BeginNativeTime);
	return true;
}

FString Job::GetRunTimings(double BatchStartTime) const 
{
	float beginNativeTime = Stats.BeginNativeTime > 0 ? (float)Stats.BeginNativeTime - BatchStartTime : 0.0f;
	float endNativeTime = Stats.EndNativeTime > 0 ? (float)Stats.EndNativeTime - BatchStartTime : 0.0f;
	float nativeDelta = Stats.EndNativeTime > Stats.BeginNativeTime ? endNativeTime - beginNativeTime : 0.0f;
	float beginRunTime = Stats.BeginRunTime > 0 ? (float)Stats.BeginRunTime - BatchStartTime : 0.0f;
	float endRunTime = Stats.EndRunTime > 0 ? (float)Stats.EndRunTime - BatchStartTime : 0.0f;
	float runDelta = Stats.EndRunTime > Stats.BeginRunTime ? endRunTime - beginRunTime : 0.0f;

	FString main = FString::Printf(TEXT("Native: %0.2f - %0.2f [Diff: %0.2f], RunTime: %0.2f - %0.2f [Diff: %0.2f]"),
		beginNativeTime, endNativeTime, nativeDelta, beginRunTime, endRunTime, runDelta);

	if (Stats.TargetPrepStartTime > 0)
	{
		float targetPrepStartTime = Stats.TargetPrepStartTime > 0 ? (float)Stats.TargetPrepStartTime - BatchStartTime : 0.0f;
		float targetPrepEndTime = Stats.TargetPrepEndTime > 0 ? (float)Stats.TargetPrepEndTime - BatchStartTime : 0.0f;
		float targetPrepWaitStartTime = Stats.TargetPrepWaitStartTime > 0 ? (float)Stats.TargetPrepWaitStartTime - BatchStartTime : 0.0f;
		float targetPrepWaitEndTime = Stats.TargetPrepWaitEndTime > 0 ? (float)Stats.TargetPrepWaitEndTime - BatchStartTime : 0.0f;

		float prepTimeDelta = Stats.TargetPrepEndTime > Stats.TargetPrepStartTime ? targetPrepEndTime - targetPrepStartTime : 0.0f;
		float prepWaitTimeDelta = Stats.TargetPrepWaitEndTime > Stats.TargetPrepWaitStartTime ? targetPrepWaitEndTime - targetPrepWaitStartTime : 0.0f;

		FString suffix = FString::Printf(TEXT(", Prepare: %0.2f - %0.2f [Diff: %0.2f], Prepare-Wait: %0.2f - %0.2f [Diff: %0.2f]"),
			targetPrepStartTime, targetPrepEndTime, prepTimeDelta, targetPrepWaitStartTime, targetPrepWaitEndTime, prepWaitTimeDelta);

		return main + suffix;
	}

	return main;
}

int32 Job::Exec()
{
	UE_LOGF(LogJob, VeryVerbose, "Exec::%ls", *Transform->GetName());

	Stats.BeginNativeTime = Util::Time();
	Run(RunInfo).apply(cti::transforms::wait());
	Stats.EndNativeTime = Util::Time();

	SetPromise(0);

	return 0;
}

JobPtrVec Job::GetArgDependencies()
{
	JobPtrVec ArgDeps;

	/// Ok, go through the arguments and gather all the dependencies
	for (int32 ArgIndex = 0; ArgIndex < Args.Num(); ArgIndex++)
	{
		JobArgPtr Arg = Args[ArgIndex];
		JobPtrW GeneratingJobW = Arg->GeneratingJob();

		if (!GeneratingJobW.expired())
		{
			JobPtr GeneratingJob = GeneratingJobW.lock();
			TiledBlobPtr generatingJobResult = GeneratingJob->GetResult();

			ArgDeps.push_back(GeneratingJob);
		}
	}

	return ArgDeps;
}

void Job::GetDependencies(JobPtrVec& Prior, JobPtrVec& After, JobRunInfo InRunInfo)
{
	RunInfo = InRunInfo;

	if (bIsCulled)
		return;

	JobPtrVec ArgDeps = GetArgDependencies();

	JobRunInfo RunInfoCopy = RunInfo;

	/// Add a finalise 
	JobPtr ThisJob = RunInfo.ThisJob.lock();
	check(ThisJob);

	JobPtr PrepareJob = T_PrepareResources::Create(RunInfo.Cycle, ThisJob);
	RunInfoCopy.ThisJob = PrepareJob;
	PrepareJob->RunInfo = RunInfoCopy;
	PrepareJob->Generator = ThisJob;

	JobPtr FinaliseJob = T_FinaliseBlob::Create(RunInfo.Cycle, ThisJob);
	RunInfoCopy.ThisJob = FinaliseJob;
	FinaliseJob->RunInfo = RunInfoCopy;
	FinaliseJob->Generator = ThisJob;

	JobRunInfo PrepareRunInfo = RunInfo;
	PrepareRunInfo.Dev = PrepareJob->GetTransform()->TargetDevice(0);
	PrepareJob->GetDependencies(Prior, After, PrepareRunInfo);

	JobRunInfo AfterRunInfo = RunInfo;
	AfterRunInfo.Dev = FinaliseJob->GetTransform()->TargetDevice(0);
	FinaliseJob->GetDependencies(Prior, After, AfterRunInfo);

	JobsGeneratedPrior.push_back(PrepareJob);
	JobsGeneratedAfter.push_back(FinaliseJob);

	/// If this job is late bound then the prepare phase cannot start until the 
	/// previous job has totally completed
	for (auto PrevJob : Prev)
	{
		if (!PrevJob->IsDone())
			PrepareJob->AddPrev(Prev);
	}

	Prev.clear();

	for (JobPtr ArgDep : ArgDeps)
	{
		if (!ArgDep->IsDone())
		{
			PrepareJob->AddPrev(std::static_pointer_cast<DeviceNativeTask>(ArgDep));
		}
	}

	ThisJob->AddPrev(PrepareJob);

	/// Make sure the dependency chain is correct
	FinaliseJob->AddPrev(ThisJob);

	Prior = JobsGeneratedPrior;
	After = JobsGeneratedAfter;
}

void Job::PostExec()
{
	UE_LOGF(LogJob, VeryVerbose, "PostExec::%ls", *Transform->GetName());
	DeviceNativeTask::PostExec();
}

AsyncInt Job::ExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type ReturnThread)
{
	ThreadId = Util::GetCurrentThreadId();

	int32 RetVal = Exec();
	return cti::make_ready_continuable(RetVal);
}

ENamedThreads::Type Job::GetExecutionThread() const
{
	return Transform->ExecutionThread();
}

bool Job::IsAsync() const
{
	return Transform->IsAsync() && DeviceNativeTask::IsAsync();
}

void Job::ResetForReplay(bool noCache)
{
	ReplayCount++;

	bIsNoCache = noCache;
	DeviceNativeTask::Reset();

	if (Result)
	{
		check(Result->IsPromise());
		TiledBlob_PromisePtr ResultPromise = GetResultPromise();
		ResultPromise->ResetForReplay();
	}
}

bool Job::GetTileInvalidation(int32 RowId, int32 ColId) const
{
	if (RowId >= 0 && ColId >= 0)
	{
		check(RowId < (int32)TileInvalidationMatrix.Rows());
		check(ColId < (int32)TileInvalidationMatrix.Cols());

		return TileInvalidationMatrix[RowId][ColId] != 0;
	}

	return TileInvalidationMatrix(0, 0) != 0;
}

