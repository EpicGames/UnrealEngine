// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Helper/Promise.h"
#include "Device/DeviceNativeTask.h"
#include "JobCommon.h"
#include "JobArgs.h"
#include "Data/TiledBlob.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Job;
class Scheduler;
class JobBatch;
class MixUpdateCycle;

DECLARE_LOG_CATEGORY_EXTERN(LogJob, Log, All);

class RenderMesh;

//////////////////////////////////////////////////////////////////////////
class Job : public DeviceNativeTask
{
public:
	struct JobStats
	{
		double						BeginNativeTime = 0.0;		/// The timestamp [ms] when this job started
		double						EndNativeTime = 0.0;		/// The timestamp [ms] when this job ended
		double						BeginRunTime = 0.0;			/// The timestamp [ms] when this job RUN is started	
		double						EndRunTime = 0.0;			/// The timestamp [ms] when this job RUN has ended

		double						TargetPrepStartTime = 0;	/// The timestamp [ms] when this job PrepareTargets was run
		double						TargetPrepEndTime = 0;		/// The timestamp [ms] when this job PrepareTargets has ended
		double						TargetPrepWaitStartTime = 0;/// The timestamp [ms] when this job started waiting for targets to be prepared (all promises to return from Device)
		double						TargetPrepWaitEndTime = 0;	/// The timestamp [ms] when this job when the waiting of promises has finished
	};

protected:
	typedef T_Tiles<JobResultPtr>	JobResultPtrTiles;

	UMixInterface*					MixObj = nullptr;	/// The mix that this job belongs to
	int64							Id = -1;			/// Unique ID for the job [-1 means it was unassigned]
	int32							QueueId = -1;		/// ID of the queue that this job belongs to
	TArray<JobArgPtr>				Args;				/// The arguments for the job
	BlobTransformPtr				Transform;			/// The transformation that we'll be doing for the job
	FWeakObjectPtr					ErrorOwner;			/// Error owner is an object associated with any error reported to the TextureGraphEngine::ErrorReporter during this job execution
	mutable CHashPtr				HashValue = nullptr;/// What is the hash of this job
	TiledBlob_PromisePtr			ResultOrg;			/// The original result that was created
	TiledBlobRef					Result;				/// The rendering result of this job
	BufferDescriptorPtr				DesiredResultDesc;	/// Desired descriptor for the result
	JobResultPtrTiles				TileResults;		/// The results for all the tiles
	JobResultPtr					FinalJobResult;		/// The overall result info for this job

	JobPArray						Parents;			/// [upstream] Jobs that this job is dependent on
	JobPtrArray						Children;			/// [downstream] Jobs that are dependent on this job

	int32							TargetId = -1;		/// What is our target

	TileInvalidateMatrix			TileInvalidationMatrix; /// The tiles that were actually invalidated

	BufferDescriptor				TileDesc;			/// The descriptor for each tile

	FCriticalSection				Mutex;				/// For preventing shared table writes
	JobRunInfo						RunInfo;			/// The run info that was saved when it was passed as argument

	JobStats						Stats;				/// Job timing stats collected during execution

	const RenderMesh*				Mesh = nullptr;	/// The mesh that we're targeting 

	int32							ReplayCount = 0;	/// Counting the number of debug replay of this job
	bool							bIsNoCache = false;	/// Whether we should ignore the cache
	bool							bIsTiled = true;	/// Whether the job runs in tiled mode or not. You can override this and force the job to run in non-tiled mode

	JobPtrVec						JobsGeneratedPrior;	/// [Prior] The jobs that were generated from this job
	JobPtrVec						JobsGeneratedAfter;	/// [After] The jobs that were generated from this job

	JobPtrW							Generator;			/// The job that generated this job. This is useful in certain situations
	FString							DebugJobName;		/// The name of the job

	//std::vector<std::shared_ptr<cti::promise<TiledBlobPtr>>> _blobPreparedPromises;

	typedef std::pair<int32, int32>	IndexPair;
	typedef TMap<HashType, IndexPair> JobLUT;

	/// Binds/Unbinds args for individual tiles or the entire blob if its a non-tiled job
	UE_API virtual AsyncJobArgResultPtr	BindArgs(JobRunInfo NewRunInfo, BlobTransformPtr TransformObj, int32 RowId, int32 ColId);
	UE_API virtual AsyncJobArgResultPtr	UnbindArgs(JobRunInfo NewRunInfo, BlobTransformPtr TransformObj, int32 RowId, int32 ColId);

	/// Binds/Unbinds args for all the tiles at once and returns a promise
	UE_API virtual AsyncInt				BindArgs_All(JobRunInfo RunInfo);
	UE_API virtual AsyncInt				UnbindArgs_All(JobRunInfo RunInfo);

	typedef std::function			<AsyncJobArgResultPtr(JobRunInfo, BlobTransformPtr, int32, int32)> Bind_Unbind_Func;
	UE_API virtual AsyncInt				Bind_Or_Unbind_All_Generic(JobRunInfo NewRunInfo, Bind_Unbind_Func BFunc);

	UE_API virtual AsyncJobResultPtr		RunTile(JobRunInfo RunInfo, int32 RowId, int32 ColId);
	UE_API virtual AsyncJobResultPtr		RunSingle(JobRunInfo RunInfo);
	UE_API virtual AsyncJobResultPtr		FinaliseTiles(JobRunInfo RunInfo);
	UE_API void							SetTileResult(int32 RowId, int32 ColId, JobResultPtr TileResult);
	UE_API virtual AsyncPrepareResult		PrepareTargets(JobBatch* batch);

	UE_API AsyncTransformResultPtr			ExecTransform(JobRunInfo RunInfo, BlobTransformPtr TransformObj , int32 RowId, int32 ColId, CHashPtr jobHash);
	UE_API virtual void					MarkJobDone();
	UE_API virtual bool					IsDiscard() const;

	//////////////////////////////////////////////////////////////////////////
	/// These methods can be used to execute a Job then and there
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual AsyncJobResultPtr		Run(JobRunInfo RunInfo);
	UE_API virtual AsyncPrepareResult		PrepareResources(JobBatch* batch);

	UE_API virtual bool					CheckCached();
	UE_API BufferDescriptor				GetResultDesc() const;
	UE_API BufferDescriptor				GetCombinedDesc(BufferDescriptor& ArgsDescCombined, size_t& Count) const;

public:
									UE_API Job(int32 TargetId, BlobTransformPtr InTransform, UObject* ErrorOwner = nullptr, uint16 InPriority = (uint16)E_Priority::kNormal, uint64 InId = -1);
									UE_API Job(UMixInterface* InMix, int32 TargetId, BlobTransformPtr Transform, UObject* ErrorOwner = nullptr, uint16 priority = (uint16)E_Priority::kNormal, uint64 id = -1);
	UE_API virtual							~Job() override;


	UE_API virtual Job*					SetArgs(const TArray<JobArgPtr>& NewArgs);
	UE_API virtual Job*					AddArg(JobArgPtr Arg);
	UE_API uint32_t						NumArgs() const;
	UE_API JobArgPtr						GetArg(uint32_t Index) const;
	UE_API virtual void					AddResultToBlobber();

	UE_API virtual CHashPtr				Hash() const;
	UE_API virtual CHashPtr				TileHash(int32 RowId, int32 ColId) const;
	UE_API virtual CHashPtr				CalcTileHash(int32 RowId, int32 ColId) const;
	UE_API virtual bool					CanHandleTiles() const;
	UE_API virtual bool					CheckDefaultArgs() const;

	UE_API virtual TiledBlobPtr			InitResult(FString NewName, const BufferDescriptor* InDesiredDesc, int32 NumTilesX = 0, int32 NumTilesY = 0);

	UE_API void							ResetForReplay(bool noCache); /// For debug purpose, reset the job so it can be replayed

	//////////////////////////////////////////////////////////////////////////
	/// These methods are used to queue up the job as a DeviceNativeTask.
	/// You cannot call the DeviceNativeTask implementation directly. That is
	/// done by the Device itself.
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual AsyncInt				BeginNative(JobRunInfo RunInfo);
	UE_API virtual AsyncJobResultPtr		EndNative();

	//////////////////////////////////////////////////////////////////////////
	/// BEGIN: DeviceNativeTask implementation
	//////////////////////////////////////////////////////////////////////////
protected:
	UE_API virtual TiledBlobPtr			InitLateBoundResult(FString NewName, BufferDescriptor DesiredDesc, size_t NumInputBlobs);

	UE_API virtual AsyncInt				PreExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type ReturnThread) override;
	UE_API virtual int32					Exec() override;
	UE_API virtual void					PostExec() override;
	UE_API virtual AsyncInt				ExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type ReturnThread) override;
	UE_API virtual JobPtrVec				GetArgDependencies();

public:
	UE_API virtual Device*					GetTargetDevice() const override;
	UE_API virtual ENamedThreads::Type		GetExecutionThread() const override;
	UE_API virtual bool					IsAsync() const override;
	UE_API bool							GetTileInvalidation(int32 RowId, int32 ColId) const;

	UE_API virtual void					GetDependencies(JobPtrVec& Prior, JobPtrVec& After, JobRunInfo RunInfo);

	UE_API virtual FString					GetName() const override;

	UE_API virtual bool					DebugCompleteCheck() override;

	//////////////////////////////////////////////////////////////////////////
	/// END: DeviceNativeTask implementation
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual bool					CheckCulled(JobRunInfo RunInfo);
	UE_API virtual FString					GetRunTimings(double BatchStartTime) const;
	UE_API virtual FString					GetDebugName() const override;

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE int64				GetJobId() const { return Id; }
	FORCEINLINE void				SetJobId(int64 NewJobId) { Id = NewJobId; }
	FORCEINLINE int32				GetQueueId() const { return QueueId; }
	FORCEINLINE void				SetQueueId(int32 NewQueueId) { QueueId = NewQueueId; }
	FORCEINLINE BlobTransformPtr	GetTransform() const { return Transform; }
	FORCEINLINE TiledBlobRef		GetResult() const { return Result; }

	FORCEINLINE bool				GetTiled() const { return bIsTiled; }
	FORCEINLINE Job*				SetTiled(bool bTiled) { bIsTiled = bTiled; return this; }

	FORCEINLINE JobStats			GetStats() const { return Stats; }

	FORCEINLINE const JobRunInfo&	GetRunInfo() const { return RunInfo; }
	FORCEINLINE const RenderMesh*	GetMesh() const { return Mesh; }
	FORCEINLINE void				SetMesh(RenderMesh* NewMesh) { Mesh = NewMesh; }
	FORCEINLINE JobResultPtr		GetTileResult(int32 RowId, int32 ColId) const
	{
		if (RowId >= 0 && ColId >= 0)
			return TileResults[RowId][ColId];
		return FinalJobResult;
	}

	FORCEINLINE int32				GetReplayCount() const { return ReplayCount; } /// For debug purpose, report the replay number
	FORCEINLINE int32				GetTargetId() const { return TargetId; }

	FORCEINLINE JobPtrVec&			GeneratedPriorJobs() { return JobsGeneratedPrior; }
	FORCEINLINE JobPtrVec&			GeneratedAfterJobs() { return JobsGeneratedAfter; }
	FORCEINLINE const JobPtrW&		GeneratorJob() const { return Generator; }

	FORCEINLINE UMixInterface*		GetMix() const { return MixObj; }
	FORCEINLINE void				SetMix(UMixInterface* mix) { MixObj = mix; }

	FORCEINLINE BlobRef				GetResultRef() const { return BlobRef(std::static_pointer_cast<Blob>(Result.get()), false); }
	FORCEINLINE TiledBlob_PromisePtr GetResultPromise() const { return std::static_pointer_cast<TiledBlob_Promise>(Result.get()); }

	FORCEINLINE UObject*			GetErrorOwner() const { return ErrorOwner.Get(); }
};

//////////////////////////////////////////////////////////////////////////

#undef UE_API
