// Copyright Epic Games, Inc. All Rights Reserved.
#include "JobArgs.h"
#include "Job.h"
#include "TextureGraphEngine.h"
#include "Model/Mix/MixSettings.h"
#include "Data/Blob.h"
#include "JobBatch.h"
#include "2D/TextureHelper.h"
#include "Device/FX/Device_FX.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include "FxMat/RenderMaterial_FX.h"
#include "FxMat/RenderMaterial_FX_Combined.h"
#include "FxMat/RenderMaterial_BP.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "FxMat/RenderMaterial_FX_Combined.h"
#include "3D/RenderMesh.h"
#include "Data/Blobber.h"

DECLARE_CYCLE_STAT(TEXT("JobArg_Combined_Bind"), STAT_JobArg_Combined_Bind, STATGROUP_TextureGraphEngine);
DECLARE_CYCLE_STAT(TEXT("JobArg_Combined_UnBind"), STAT_JobArg_Combined_UnBind, STATGROUP_TextureGraphEngine);
DECLARE_CYCLE_STAT(TEXT("JobArg_Blob_Bind"), STAT_JobArg_Blob_Bind, STATGROUP_TextureGraphEngine);
DECLARE_CYCLE_STAT(TEXT("JobArg_Blob_UnBind"), STAT_JobArg_Blob_UnBind, STATGROUP_TextureGraphEngine);
DECLARE_CYCLE_STAT(TEXT("JobArg_Blob_Hash"), STAT_JobArg_Blob_Hash, STATGROUP_TextureGraphEngine);


JobArg_Blob::JobArg_Blob(TiledBlobPtr BlobObj, const ResourceBindInfo& BindInfo) 
	: JobArg_Resource(BindInfo)
	, BlobObjRef(BlobObj)
{
	check(BlobObjRef.get());

	/// Also check whether it's in the Blobber or not
	//auto CachedBlob = TextureGraphEngine::Blobber()->Find(BlobObj.GetHash()->Value());
	//check(CachedBlob || BlobObj.IsKeepStrong());

#if DEBUG_BLOB_REF_KEEPING == 1
	/// Check whether all the tiles are valid and available
	//for (size_t RowId = 0; RowId < BlobObj->Rows(); RowId++)
	//{
	//	for (size_t ColId = 0; ColId < BlobObj->Cols(); ColId++) 
	//	{
	//		BlobRef Tile = BlobObj->Tile(RowId, ColId);
	//		check(!Tile.expired());
	//	}
	//}

	TextureGraphEngine::Blobber()->AddReferencedBlob((Blob*)BlobObjRef.get().get(), this);
#endif 
}

JobArg_Blob::JobArg_Blob(TiledBlobPtr BlobObj, const char* TargetName) : JobArg_Blob(BlobObj, ResourceBindInfo({ FString(TargetName) })) 
{ 
}

JobArg_Blob::~JobArg_Blob()
{
#if DEBUG_BLOB_REF_KEEPING == 1
	TextureGraphEngine::Blobber()->RemoveReferencedBlob((Blob*)BlobObjRef.get().get(), this);
#endif 
}

bool JobArg::CanHandleTiles() const
{
	return true;
}

bool JobArg::ForceNonTiledTransform() const
{
	return false;
}

void JobArg_Blob::SetHandleTiles(bool bInCanHandleTiles)
{
	bCanHandleTiles = bInCanHandleTiles;
}

bool JobArg_Blob::CanHandleTiles() const
{
	return bCanHandleTiles;
}

JobArg_Blob& JobArg_Blob::WithNotHandleTiles()
{
	bCanHandleTiles = false;
	return (*this);
}

void JobArg_Blob::SetForceNonTiledTransform(bool bInForceNonTiledTransform)
{
	bForceNonTiledTransform = bInForceNonTiledTransform;
}

bool JobArg_Blob::ForceNonTiledTransform() const
{
	return bForceNonTiledTransform;
}

JobArg_Blob& JobArg_Blob::WithDownsampled4To1()
{
	bBindDownsampled4To1 = true;
	return (*this);
}

bool JobArg_Blob::IsDownsampled4To1() const
{
	return bBindDownsampled4To1;
}

JobArg_Blob& JobArg_Blob::WithNeighborTiles()
{
	bBindNeighborTiles = true;
	return (*this);
}

bool JobArg_Blob::IsNeighborTiles() const 
{
	return bBindNeighborTiles;
}

JobArg_Blob& JobArg_Blob::WithArrayOfTiles()
{
	bBindArrayOfTiles = true;
	return (*this);
}

bool JobArg_Blob::IsArrayOfTiles() const
{
	return bBindArrayOfTiles;
}

//////////////////////////////////////////////////////////////////////////
TiledBlobPtr JobArg_Blob::GetRootBlob(JobArgBindInfo JobBindInfo) const
{
	TiledBlobPtr RootBlob = BlobObjRef.get();

	if (JobBindInfo.LODLevel != 0 &&					/// LOD is enabled in this cycle
		!RootBlob->IsLODLevel() &&						/// Don't wanna LOD an existing LOD BlobObj
		RootBlob->HasLODLevels() &&						/// Check whether it has LOD levels
		RootBlob->HasLODLevel(JobBindInfo.LODLevel))	/// Ensure that it has a valid object in the LOD that we need
	{
		RootBlob = std::static_pointer_cast<TiledBlob>(RootBlob->GetLODLevel(JobBindInfo.LODLevel).lock());
		check(RootBlob);
	}

	return RootBlob;
}

AsyncJobArgResultPtr JobArg_Blob::Bind(JobArgBindInfo JobBindInfo)
{
	check(!bUnbound);

	SCOPE_CYCLE_COUNTER(STAT_JobArg_Blob_Bind)
	const Job* CurrentJob = JobBindInfo.JobObj;
	check(CurrentJob);

	BlobTransformPtr transform = JobBindInfo.Transform;

	ResourceBindInfo BindInfo = ArgBindInfo;
	BindInfo.Dev = JobBindInfo.Dev;
	check(BindInfo.Dev);

	TiledBlobPtr RootBlob = GetRootBlob(JobBindInfo);

	if (IsDownsampled4To1())
	{
		return RootBlob->TransferTo(BindInfo.Dev)
			.then([RootBlob, BindInfo, JobBindInfo](auto)
			{
				// Collect the individual tile textures from the root tiled BlobObj in an array
				std::vector<TexPtr> TileTexObjs;
				for (int RowId = 0; RowId < 2; RowId++)
				{
					for (int ColId = 0; ColId < 2; ColId++)
					{
						if (RootBlob->IsValidTileIndex(RowId + 2 * JobBindInfo.RowId, ColId + 2 * JobBindInfo.ColId))
						{
							auto Tile = RootBlob->GetTile(RowId + 2 * JobBindInfo.RowId, ColId + 2 * JobBindInfo.ColId);
							auto TileBuffer = std::static_pointer_cast<DeviceBuffer_FX>(Tile->GetBufferRef().GetPtr());
							TileTexObjs.push_back(TileBuffer->GetTexture());
						}
						else
						{
							auto Black = TextureHelper::GetBlack()->GetTile(0, 0);
							auto BlackBuffer = std::static_pointer_cast<DeviceBuffer_FX> (Black->GetBufferRef().GetPtr());
							TileTexObjs.push_back(BlackBuffer->GetTexture());
						}
					}
				}

				// True binding of the array of textures to the Material
				const RenderMaterial* Material = static_cast<const RenderMaterial*>(JobBindInfo.Transform.get());
				check(Material);
				Material->SetArrayTexture(*BindInfo.Target, TileTexObjs);

				return cti::make_ready_continuable(std::make_shared<JobArgResult>());
			});
	}
	else if (IsNeighborTiles())
	{
		return RootBlob->TransferTo(BindInfo.Dev)
			.then([RootBlob, BindInfo, JobBindInfo](auto)
			{
				// Collect the individual tile textures from the root tiled BlobObj in an array
				std::vector<TexPtr> Textures;
				auto MainRow = JobBindInfo.RowId;
				auto MainCol = JobBindInfo.ColId;
				auto GridRow = RootBlob->Rows();
				auto GridCol = RootBlob->Cols();

				for (int RowId = 0; RowId < 3; RowId++)
				{
					for (int ColId = 0; ColId < 3; ColId++)
					{
						// Compute the true tile index
						auto InnerRowId = (MainRow - 1 + RowId);
						auto InnerColId = (MainCol - 1 + ColId);
						if (InnerRowId < 0)
							InnerRowId = GridRow - 1;
						if (InnerRowId >= GridRow)
							InnerRowId = 0;
						if (InnerColId < 0)
							InnerColId = GridCol - 1;
						if (InnerColId >= GridCol)
							InnerColId = 0;

						if (RootBlob->IsValidTileIndex(InnerRowId, InnerColId))
						{
							auto Tile = RootBlob->GetTile(InnerRowId, InnerColId);
							auto TileBuffer = std::static_pointer_cast<DeviceBuffer_FX>(Tile->GetBufferRef().GetPtr());
							Textures.push_back(TileBuffer->GetTexture());
						}
						else
						{
							auto Black = TextureHelper::GetBlack()->GetTile(0, 0);
							auto BlackBuffer = std::static_pointer_cast<DeviceBuffer_FX> (Black->GetBufferRef().GetPtr());
							Textures.push_back(BlackBuffer->GetTexture());
						}
					}
				}

				// True binding of the array of textures to the Material
				const RenderMaterial* Material = static_cast<const RenderMaterial*>(JobBindInfo.Transform.get());
				check(Material);
				Material->SetArrayTexture(*BindInfo.Target, Textures);

				return cti::make_ready_continuable(std::make_shared<JobArgResult>());
			});
	}
	else if (IsArrayOfTiles())
	{
		return RootBlob->TransferTo(BindInfo.Dev)
			.then([RootBlob, BindInfo, JobBindInfo](auto)
				{
					// Collect the individual tile textures from the root tiled BlobObj in an array
					std::vector<TexPtr> Textures;
					int NumX = RootBlob->Cols();
					int NumY = RootBlob->Rows();

					for (int Y = 0; Y < NumY; ++Y)
					{
						for (int X = 0; X < NumX; ++X)
						{
							if (RootBlob->IsValidTileIndex(X, Y))
							{
								auto Tile = RootBlob->GetTile(X, Y);
								auto TileBuffer = std::static_pointer_cast<DeviceBuffer_FX>(Tile->GetBufferRef().GetPtr());
								Textures.push_back(TileBuffer->GetTexture());
							}
							else
							{
								auto Black = TextureHelper::GetBlack()->GetTile(0, 0);
								auto BlackBuffer = std::static_pointer_cast<DeviceBuffer_FX> (Black->GetBufferRef().GetPtr());
								Textures.push_back(BlackBuffer->GetTexture());
							}
						}
					}

					// True binding of the array of textures to the Material
					const RenderMaterial* Material = static_cast<const RenderMaterial*>(JobBindInfo.Transform.get());
					check(Material);
					Material->SetArrayTexture(*BindInfo.Target, Textures);

					return cti::make_ready_continuable(std::make_shared<JobArgResult>());
				});
	}
	else
	{
		BlobPtr BlobToBind = RootBlob;
		check(BlobToBind);

		bool bArgCanHandleTiles = CanHandleTiles();
		if (bArgCanHandleTiles && transform->CanHandleTiles() && JobBindInfo.RowId >= 0 && JobBindInfo.ColId >= 0)
			BlobToBind = BlobObjRef.get()->GetTile(JobBindInfo.RowId, JobBindInfo.ColId).lock();

		FString TransformName = CurrentJob->GetTransform()->GetName();

		//UE_LOGF(LogJob, Log, "[%ls] Binding BlobObj arg: %ls => %ls [Tile: %ls]", *transformName, *BlobObj->Name(), *BindInfo.TargetName, *blobToBind->Name());
	
		BindInfo.bIsCombined = !bArgCanHandleTiles;
		return BlobToBind->Bind(transform.get(), BindInfo)
			.then([this]() mutable
			{
				return std::make_shared<JobArgResult>();
			});
	}
}

AsyncJobArgResultPtr JobArg_Blob::Unbind(JobArgBindInfo JobBindInfo)
{
	check(!bUnbound);

	SCOPE_CYCLE_COUNTER(STAT_JobArg_Blob_UnBind)
	if (BlobObjRef.expired()) // No references remain , this cause may arise if BlobObj is cleared through sharedPtrs before unbind
		return cti::make_ready_continuable(std::make_shared<JobArgResult>());

	if (IsDownsampled4To1())
	{
		// IN this case the arg is only used to read from, nothing to do to unbind
		return cti::make_ready_continuable(std::make_shared<JobArgResult>());
	}
	else if (IsNeighborTiles())
	{
		// IN this case the arg is only used to read from, nothing to do to unbind
		return cti::make_ready_continuable(std::make_shared<JobArgResult>());
	}
	else
	{
		check(JobBindInfo.JobObj);

		BlobTransformPtr transform = JobBindInfo.Transform;

		BlobPtr blobToBind = GetRootBlob(JobBindInfo);

		bool bCanArgHandleTiles = CanHandleTiles();
		if (bCanArgHandleTiles && transform->CanHandleTiles() && JobBindInfo.RowId >= 0 && JobBindInfo.ColId >= 0)
		{
			blobToBind = BlobObjRef.get()->GetTile(JobBindInfo.RowId, JobBindInfo.ColId).lock();
		}

		ArgBindInfo.BatchId = JobBindInfo.Batch->GetBatchId();

		ArgBindInfo.bIsCombined = !bCanArgHandleTiles;
		return blobToBind->Unbind(transform.get(), ArgBindInfo)
			.then([=]() mutable
			{
				return std::make_shared<JobArgResult>();
			});
	}
}

bool JobArg_Blob::IsLateBound(uint32 RowId, uint32 ColId) const
{
	TiledBlobPtr BlobObj = BlobObjRef.get();
	BlobRef TileXY = BlobObj->GetTile(RowId, ColId);

	/// If the tile isn't even there then it's definitely late bound
	if (TileXY.IsNull())
		return true;

	BlobPtr TileXYPtr = TileXY.lock();
	return BlobObj->IsLateBound() || !TileXYPtr || TileXYPtr->IsNull() || TileXYPtr->IsLateBound();
}

const BufferDescriptor* JobArg_Blob::GetDescriptor() const 
{
	return BlobObjRef ? &BlobObjRef->GetDescriptor() : nullptr;
}

CHashPtr JobArg_Blob::TileHash(uint32 RowId, uint32 ColId) const
{
	TiledBlobPtr BlobObj = BlobObjRef.get();
	return BlobObj->GetTile(RowId, ColId)->Hash();
}

CHashPtr JobArg_Blob::Hash() const
{
	SCOPE_CYCLE_COUNTER(STAT_JobArg_Blob_Hash);
	check(BlobObjRef.get());
	return BlobObjRef.GetHash();
}

JobPtrW JobArg_Blob::GeneratingJob() const
{
	return BlobObjRef.get() ? BlobObjRef.get()->Job() : JobPtrW();
}

//////////////////////////////////////////////////////////////////////////

AsyncJobArgResultPtr JobArg_Mesh::Bind(JobArgBindInfo JobBindInfo) 
{
	JobBindInfo.JobObj->SetMesh(Mesh.get());
	return cti::make_ready_continuable(std::make_shared<JobArgResult>());
}

AsyncJobArgResultPtr JobArg_Mesh::Unbind(JobArgBindInfo JobBindInfo) 
{
	return cti::make_ready_continuable(std::make_shared<JobArgResult>());
}

CHashPtr JobArg_Mesh::Hash() const 
{ 
	return Mesh->Hash(); 
}

CHashPtr JobArg_Mesh::TileHash(uint32 RowId, uint32 ColId) const
{
	return Mesh->Hash();
}

//////////////////////////////////////////////////////////////////////////
AsyncJobArgResultPtr JobArg_TileInfo::Bind(JobArgBindInfo JobBindInfo)
{
	const Job* JobObj = JobBindInfo.JobObj;
	check(JobObj);
	BlobTransformPtr Transform = JobBindInfo.Transform;

	TiledBlobPtr Result = JobObj->GetResult();
	float TileWidth = Result->GetWidth() / Result->Cols();
	float TileHeight = Result->GetHeight() / Result->Rows();

	Value.TileX = JobBindInfo.ColId;
	Value.TileCountX = Result->Cols();
	Value.TileWidth = TileWidth;
	
	Value.TileY = JobBindInfo.RowId;
	Value.TileCountY = Result->Rows();
	Value.TileHeight = TileHeight;

	Transform->BindStruct<FTileInfo>(Value, ArgBindInfo);
	
	return cti::make_ready_continuable(std::make_shared<JobArgResult>());
}

AsyncJobArgResultPtr JobArg_TileInfo::Unbind(JobArgBindInfo JobBindInfo)
{
	return cti::make_ready_continuable(std::make_shared<JobArgResult>());
}

CHashPtr JobArg_TileInfo::Hash() const
{
	if (!HashValue)
		HashValue = TileHash(-1, -1);
	return HashValue;
}

CHashPtr JobArg_TileInfo::TileHash(uint32 RowId, uint32 ColId) const
{
	HashTypeVec StructHash =
	{
		MX_HASH_VAL_DEF(RowId),
		MX_HASH_VAL_DEF(Value.TileCountX),
		MX_HASH_VAL_DEF(Value.TileWidth),
		MX_HASH_VAL_DEF(ColId),
		
		MX_HASH_VAL_DEF(Value.TileCountY),
		MX_HASH_VAL_DEF(Value.TileHeight),
		MX_HASH_VAL_DEF(sizeof(Value)),
	};

	return std::make_shared<CHash>(DataUtil::Hash(StructHash), true);
}

//////////////////////////////////////////////////////////////////////////

CHashPtr JobArg_ForceTiling::Hash() const
{
	return TileHash(-1, -1);
}

CHashPtr JobArg_ForceTiling::TileHash(uint32 RowId, uint32 ColId) const
{
	HashTypeVec StructHash =
	{
		MX_HASH_VAL_DEF(RowId),
		MX_HASH_VAL_DEF(ColId),
	};
	
	return std::make_shared<CHash>(DataUtil::Hash(StructHash), true);
}
