// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Helper/Promise.h"
#include "Transform/BlobTransform.h"
#include "FxMat/RenderMaterial_FX.h"
#include "FxMat/TileInfo_FX.h"
#include "Data/TiledBlob.h"
#include "Profiling/StatGroup.h"

#define UE_API TEXTUREGRAPHENGINE_API

class Job;
class JobBatch;
class Scheduler;

DECLARE_CYCLE_STAT(TEXT("JobArg_SimpleType_Bind"), STAT_JobArg_SimpleType_Bind, STATGROUP_TextureGraphEngine);
DECLARE_CYCLE_STAT(TEXT("JobArg_SimpleType_UnBind"), STAT_JobArg_SimpleType_UnBind, STATGROUP_TextureGraphEngine);
//////////////////////////////////////////////////////////////////////////
struct JobArgResult
{
	std::exception_ptr				ExInner;			/// Original exception that was raised by the action
	int32							ErrorCode = 0;		/// What is the error code
};

typedef std::shared_ptr<JobArgResult> JobArgResultPtr;
typedef cti::continuable<JobArgResultPtr> AsyncJobArgResultPtr;

//////////////////////////////////////////////////////////////////////////
struct JobArgBindInfo
{
	Job*							JobObj = nullptr;	/// The job that this is associated with
	const JobBatch*					Batch = nullptr;	/// What is the batch that we're going to be part of (can be nullptr)
	size_t							JobIndex = 0;		/// The index of this job
	int32							RowId = -1;			/// The row index that we're binding for or -1 if it needs to be bound as one
	int32							ColId = -1;			/// The col index tile that we're binding for or -1 if it needs to be bound as one
	Device*							Dev = nullptr;		/// The device to which this is going to be bound to
	BlobTransformPtr				Transform;			/// The transformation function to which the argument will be bound
	uint32							LODLevel = 0;		/// What is the lod level that we'll be binding this to
};

//////////////////////////////////////////////////////////////////////////
class JobArg
{
protected:
	bool							bIgnoreDesc = false;/// Whether to ignore the descriptor for this argument
														/// when combining descriptors during merge
	bool							bIgnoreHash = false;/// Whether to ignore this argument in hash calculation or not
	bool							bUnbound = false;	/// Sometimes you need to add an argument for hashing calculation 
														/// and not necessarily binding to a BlobTransform

public:
	virtual							~JobArg() = default;
	virtual AsyncJobArgResultPtr	Bind(JobArgBindInfo JobBindInfo) = 0;
	virtual AsyncJobArgResultPtr	Unbind(JobArgBindInfo JobBindInfo) = 0;
	UE_API virtual bool					CanHandleTiles() const;
	UE_API virtual bool					ForceNonTiledTransform() const;

	virtual CHashPtr				Hash() const = 0;
	virtual CHashPtr				TileHash(uint32 TileX, uint32 TileY) const = 0;

	virtual CHashPtr				Hash_Default() const { return Hash(); }
	virtual CHashPtr				TileHash_Default(uint32 TileX, uint32 TileY) const { return TileHash(TileX, TileY); }
	virtual bool					IsDefault() const { return true; }

	virtual JobPtrW					GeneratingJob() const { return JobPtrW(); }
	virtual const BufferDescriptor* GetDescriptor() const { return nullptr; }
	virtual bool					IsLateBound(uint32 TileX, uint32 TileY) const { return false; }
	virtual bool					IsBlob() const { return false; }

	//////////////////////////////////////////////////////////////////////////	
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE bool				IgnoreHash() const { return bIgnoreHash; }
	FORCEINLINE void 				WithIgnoreHash(bool ignoreHash) { bIgnoreHash = ignoreHash; }

	FORCEINLINE bool				Unbounded() const { return bUnbound; }
	FORCEINLINE void 				WithUnbounded(bool unbounded) { bUnbound = unbounded; }

	FORCEINLINE bool				IgnoreDesc() const { return bIgnoreDesc; }
	FORCEINLINE void 				WithIgnoreDesc(bool ignoreDesc) { bIgnoreDesc = ignoreDesc; }
};

typedef std::shared_ptr<JobArg>		JobArgPtr;

//////////////////////////////////////////////////////////////////////////	
/// Generic CPU/GPU resource e.g. textures, samplers etc.
//////////////////////////////////////////////////////////////////////////
class JobArg_Resource : public JobArg
{
protected:
	ResourceBindInfo				ArgBindInfo;			/// Binding information for this resource

public:
									JobArg_Resource(const ResourceBindInfo& BindInfo) : ArgBindInfo(BindInfo) {}
									JobArg_Resource(const char* TargetName) : ArgBindInfo({ FString(TargetName) }) {}
};

//////////////////////////////////////////////////////////////////////////
class JobArg_Blob : public JobArg_Resource
{
protected:
	mutable TiledBlobRef			BlobObjRef;							/// Blob Cached between Bind and Unbind calls only
	bool							bCanHandleTiles = true;				/// Instead of providing a tile for associated blob, whole blob will be fed in the parameter
	bool							bForceNonTiledTransform = false;	/// Will force turn associated job to use Non Tiled mode (previously this was being done through canHandleTiles)

	bool							bBindDownsampled4To1 = false;		/// Bind the arg raster tile(s) in order to achieve the 4 to 1 downsampling per invocation
																		/// So for 1 invocation producing 1 destination tile, 4 arg source tiles are bound

	bool							bBindNeighborTiles = false;			/// Bind the tile and the rign of neighbors in order to be able to go fetch border information OUT of the current tile bound

	bool							bBindArrayOfTiles = false;			/// Bind ALL the tiles contained in this arg, filling the array of UTextures before execution of the job.


	UE_API TiledBlobPtr					GetRootBlob(JobArgBindInfo JobBindInfo) const;

public:
									UE_API JobArg_Blob(TiledBlobPtr blob, const ResourceBindInfo& BindInfo);
									UE_API JobArg_Blob(TiledBlobPtr BlobObj, const char* TargetName);

	UE_API virtual							~JobArg_Blob() override;
	UE_API virtual void					SetHandleTiles(bool bInCanHandleTiles);
	UE_API virtual bool					CanHandleTiles() const override;
	UE_API JobArg_Blob&					WithNotHandleTiles(); // equivalent to call  SetHandleTiles(false)


	UE_API virtual void					SetForceNonTiledTransform(bool bInForceNonTiledTransform);
	UE_API virtual bool					ForceNonTiledTransform() const override;

	UE_API JobArg_Blob&					WithDownsampled4To1();
	UE_API bool							IsDownsampled4To1() const;

	UE_API JobArg_Blob&					WithNeighborTiles();
	UE_API bool							IsNeighborTiles() const;

	UE_API JobArg_Blob&					WithArrayOfTiles();
	UE_API bool							IsArrayOfTiles() const;

	UE_API virtual AsyncJobArgResultPtr	Bind(JobArgBindInfo JobBindInfo) override;
	UE_API virtual AsyncJobArgResultPtr	Unbind(JobArgBindInfo JobBindInfo) override;

	UE_API virtual CHashPtr				Hash() const override;
	UE_API virtual CHashPtr				TileHash(uint32 TileX, uint32 TileY) const override;

	UE_API virtual bool					IsLateBound(uint32 TileX, uint32 TileY) const override;
	virtual bool					IsBlob() const override { return true; }

	UE_API virtual JobPtrW					GeneratingJob() const override;
	UE_API virtual const BufferDescriptor* GetDescriptor() const override;

	FORCEINLINE FString				Target() const { return ArgBindInfo.Target; }
	FORCEINLINE TiledBlobRef		GetBlob() const { return BlobObjRef; }
};


//////////////////////////////////////////////////////////////////////////

template <typename SimpleType>
class JobArg_SimpleType : public JobArg_Resource
{
protected:
	SimpleType						Value;				/// The underlying InValue to bind
	mutable CHashPtr				HashValue;				/// The InValue of the hash
	mutable CHashPtr				DefaultHash;		/// The default hash that we have

public:
									JobArg_SimpleType(SimpleType InValue, const ResourceBindInfo& BindInfo) : JobArg_Resource(BindInfo), Value(InValue) {}
									JobArg_SimpleType(SimpleType InValue, const char* TargetName, CHashPtr defaultHash = nullptr) : JobArg_Resource(TargetName)
										, Value(InValue), DefaultHash(defaultHash) {}

	virtual AsyncJobArgResultPtr	Bind(JobArgBindInfo JobBindInfo) override
	{
		check(!bUnbound);

		SCOPE_CYCLE_COUNTER(STAT_JobArg_SimpleType_Bind)
		const Job* job = JobBindInfo.JobObj;
		check(job);
		BlobTransformPtr transform = JobBindInfo.Transform;
		transform->Bind(Value, ArgBindInfo);
		return cti::make_ready_continuable(std::make_shared<JobArgResult>());
	}

	virtual AsyncJobArgResultPtr	Unbind(JobArgBindInfo JobBindInfo) override
	{
		check(!bUnbound);

		SCOPE_CYCLE_COUNTER(STAT_JobArg_SimpleType_UnBind)
		const Job* job = JobBindInfo.JobObj;
		check(job);
		BlobTransformPtr transform = JobBindInfo.Transform;
		transform->Unbind(Value, ArgBindInfo);
		return cti::make_ready_continuable(std::make_shared<JobArgResult>());
	}

	virtual CHashPtr				Hash() const override
	{
		if (!HashValue)
			HashValue = std::make_shared<CHash>(DataUtil::Hash((const uint8*)&Value, sizeof(Value)), true);

		return HashValue;
	}

	virtual CHashPtr				TileHash(uint32 TileX, uint32 TileY) const override
	{
		return Hash();
	}

	virtual bool					IsDefault() const override { return DefaultHash ? Hash()->Value() == DefaultHash->Value() : false; }
	virtual CHashPtr				Hash_Default() const override { return DefaultHash; }
	virtual CHashPtr				TileHash_Default(uint32 TileX, uint32 TileY) const override { return Hash_Default(); }
};

//////////////////////////////////////////////////////////////////////////

class JobArg_String : public JobArg_SimpleType<FString>
{
public:
									JobArg_String(FString InValue, const ResourceBindInfo& BindInfo) : JobArg_SimpleType<FString>(InValue, BindInfo) {}
									JobArg_String(FString InValue, const char* TargetName, CHashPtr defaultHash = nullptr) 
										: JobArg_SimpleType<FString>(InValue, TargetName, defaultHash) {}

	virtual CHashPtr				Hash() const override
	{
		if (!HashValue)
			HashValue = std::make_shared<CHash>(DataUtil::Hash_Simple(Value), true);
		return HashValue;
	}
};

//////////////////////////////////////////////////////////////////////////
class RenderMesh;
class JobArg_Mesh : public JobArg
{
private:
	std::shared_ptr<RenderMesh>		Mesh;			/// The mesh that we're binding
	int32							_targetId = -1;	/// What is the TargetName ID

public:
									JobArg_Mesh(std::shared_ptr<RenderMesh> mesh, int32 targetId) : Mesh(mesh), _targetId(targetId) { check(Mesh); }

	UE_API virtual AsyncJobArgResultPtr	Bind(JobArgBindInfo JobBindInfo) override;
	UE_API virtual AsyncJobArgResultPtr	Unbind(JobArgBindInfo JobBindInfo) override;
	UE_API virtual CHashPtr				Hash() const override;
	UE_API virtual CHashPtr				TileHash(uint32 TileX, uint32 TileY) const override;
};

//////////////////////////////////////////////////////////////////////////
template <typename Type>
class JobArg_Array : public JobArg_Resource
{
protected:
	TArray<Type>					Value;				/// The underlying InValue to bind

public:
	JobArg_Array() {};
	JobArg_Array(TArray<Type>	InValue, const ResourceBindInfo& BindInfo) : JobArg_Resource(BindInfo), Value(InValue) {}
	JobArg_Array(TArray<Type>	InValue, const char* TargetName) : JobArg_Resource(TargetName), Value(InValue) {}

	virtual AsyncJobArgResultPtr	Bind(JobArgBindInfo JobBindInfo) override
	{
		const Job* job = JobBindInfo.JobObj;
		check(job);
		auto transform = JobBindInfo.Transform;
		transform->BindScalarArray(Value, ArgBindInfo);
		return cti::make_ready_continuable(std::make_shared<JobArgResult>());
	}

	virtual AsyncJobArgResultPtr	Unbind(JobArgBindInfo JobBindInfo) override
	{
		const Job* job = JobBindInfo.JobObj;
		check(job);

		RenderMaterial_FXPtr transform = std::static_pointer_cast<RenderMaterial_FX>(JobBindInfo.Transform);

		return cti::make_ready_continuable(std::make_shared<JobArgResult>());
	}

	virtual CHashPtr				Hash() const override
	{
		return std::make_shared<CHash>(DataUtil::Hash((const uint8*)&Value, sizeof(Value)), true);
	}

	virtual CHashPtr				TileHash(uint32 TileX, uint32 TileY) const override
	{
		return Value.Num() ? Hash() : std::make_shared<CHash>(DataUtil::Hash_Simple(FVector2D(TileX, TileY)), false);
	}
};


//////////////////////////////////////////////////////////////////////////

template <typename StructType>
class JobArg_Struct: public JobArg_Resource
{

protected:
	StructType						Value;				/// The underlying struct to bind
	mutable CHashPtr				HashValue;				/// The InValue of the hash

public:
	JobArg_Struct(StructType InValue, const ResourceBindInfo& BindInfo) : JobArg_Resource(BindInfo), Value(InValue) {}
	JobArg_Struct(StructType InValue, const char* TargetName) : JobArg_Resource(TargetName), Value(InValue) {}

	virtual AsyncJobArgResultPtr	Bind(JobArgBindInfo JobBindInfo) override
	{
	
		const Job* job = JobBindInfo.JobObj;
		check(job);
		BlobTransformPtr transform = JobBindInfo.Transform;
		transform->BindStruct<StructType>(Value, ArgBindInfo);
		return cti::make_ready_continuable(std::make_shared<JobArgResult>());
	}

	virtual AsyncJobArgResultPtr	Unbind(JobArgBindInfo JobBindInfo) override
	{
		const Job* job = JobBindInfo.JobObj;
		check(job);
		BlobTransformPtr transform = JobBindInfo.Transform;
		//transform->UnbindStruct<StructType>(_value, _bindInfo);
		return cti::make_ready_continuable(std::make_shared<JobArgResult>());
	}

	virtual CHashPtr				Hash() const override
	{
		if (!HashValue)
			HashValue = std::make_shared<CHash>(DataUtil::Hash((const uint8*)&Value, sizeof(Value)), true);
		return HashValue;
	}

	virtual CHashPtr				TileHash(uint32 TileX, uint32 TileY) const override
	{
		return Hash();
	}
};

class JobArg_TileInfo : public JobArg_Struct<FTileInfo>
{

public:
	JobArg_TileInfo(FTileInfo InValue, const ResourceBindInfo& BindInfo) : JobArg_Struct<FTileInfo>(InValue, BindInfo) {}
	JobArg_TileInfo(FTileInfo InValue, const char* TargetName) : JobArg_Struct<FTileInfo>(InValue, TargetName) {}

	UE_API virtual AsyncJobArgResultPtr	Bind(JobArgBindInfo JobBindInfo) override;

	UE_API virtual AsyncJobArgResultPtr	Unbind(JobArgBindInfo JobBindInfo) override;

	UE_API virtual CHashPtr				Hash() const override;

	UE_API virtual CHashPtr				TileHash(uint32 TileX, uint32 TileY) const override;
};

//////////////////////////////////////////////////////////////////////////
typedef JobArg_SimpleType<float>		JobArg_Float;
typedef JobArg_SimpleType<int32>		JobArg_Int;
typedef JobArg_SimpleType<bool>			JobArg_Bool;
typedef JobArg_SimpleType<FLinearColor>	JobArg_LinearColor;
typedef JobArg_SimpleType<FMatrix>		JobArg_Matrix;

//////////////////////////////////////////////////////////////////////////

class JobArg_ForceTiling : public JobArg
{
protected:
public:
	virtual							~JobArg_ForceTiling() override = default;

	virtual AsyncJobArgResultPtr	Bind(JobArgBindInfo JobBindInfo) override {
		return cti::make_ready_continuable(std::make_shared<JobArgResult>());
	};

	virtual AsyncJobArgResultPtr	Unbind(JobArgBindInfo JobBindInfo) override  {
		return cti::make_ready_continuable(std::make_shared<JobArgResult>());
	};

	UE_API virtual CHashPtr				Hash() const override;
	UE_API virtual CHashPtr				TileHash(uint32 TileX, uint32 TileY) const override;
};
//////////////////////////////////////////////////////////////////////////




#define ARG_MATRIX(v, n)			std::make_shared<JobArg_Matrix>(v, n)
#define ARG_TILEINFO(v, n)			std::make_shared<JobArg_TileInfo>(v, n)
#define ARG_FLOAT(v, n)				std::make_shared<JobArg_Float>(v, n)
#define ARG_FLOAT_DEF(v, n, hv)		std::make_shared<JobArg_Float>(v, n, std::make_shared<CHash>(hv, true))
#define ARG_ARRAY(Type,v, n)		std::make_shared<JobArg_Array<Type>>(v, n)
#define ARG_VECTOR(v, n)			std::make_shared<JobArg_LinearColor>(v, n)
#define ARG_VECTOR_DEF(v, n, hv)	std::make_shared<JobArg_LinearColor>(v, n, std::make_shared<CHash>(hv, true))
#define ARG_VECTOR2(v, n)			std::make_shared<JobArg_LinearColor>(GraphicsUtil::Vec2AsLinearColor(v), n)
#define ARG_VECTOR3(v, n)			std::make_shared<JobArg_LinearColor>(GraphicsUtil::Vec3AsLinearColor(v), n)
#define ARG_TANGENT(v, n)			std::make_shared<JobArg_LinearColor>(GraphicsUtil::TangentAsLinearColor(v), n)
#define ARG_LINEAR_COLOR(v, n)		std::make_shared<JobArg_LinearColor>(v, n)
#define ARG_INT(v, n)				std::make_shared<JobArg_Int>(v, n)
#define ARG_BOOL(v, n)				std::make_shared<JobArg_Bool>(v, n)
#define ARG_STRING(v, n)			std::make_shared<JobArg_String>(v, n)
#define ARG_STRING_DEF(v, n, hv)	std::make_shared<JobArg_String>(v, n, std::make_shared<CHash>(hv, true))
#define ARG_STRUCT(s, v, n)			std::make_shared<JobArg_Struct<s>>(v, n)
//Single tile blob belonging to larger Blob fed into the shader. This argument not have any information of its surrounding to be used in shader
#define ARG_BLOB(v, n)				std::make_shared<JobArg_Blob>(v, n)
#define ARG_MESH(m, t)				std::make_shared<JobArg_Mesh>(m, t)
//All tiles of blob combined in a single blob (through SRV in shader)
//Blob can be fetched in HLSL using GetFullBlob(inout float 4) method. See AdjustUVGeneric.usf and TiledFetch_Combined.ush for help
#define ARG_COMBINEDBLOB(v, n)				std::make_shared<JobArg_Blob_Combined>(v, n)	//With Custom SRV

FORCEINLINE JobArgPtr				WithIgnoreDesc(JobArgPtr Arg, bool bIgnoreDesc = true)
{
	Arg->WithIgnoreDesc(bIgnoreDesc);
	return Arg;
}

FORCEINLINE JobArgPtr				WithIgnoreHash(JobArgPtr Arg, bool bIgnoreHash = true)
{
	Arg->WithIgnoreHash(bIgnoreHash);
	return Arg;
}

FORCEINLINE JobArgPtr				WithUnbounded(JobArgPtr Arg, bool bUnbounded = true)
{
	Arg->WithUnbounded(bUnbounded);
	return Arg;
}

#undef UE_API
