// Copyright Epic Games, Inc. All Rights Reserved.
#include "JobBatch.h"
#include "TextureGraphEngine.h"
#include "Model/Mix/MixUpdateCycle.h"
#include "Device/FX/Device_FX.h"
#include "Job.h"
#include "Transform/Utility/T_RenderCapture.h"

DEFINE_LOG_CATEGORY(LogBatch);
DECLARE_CYCLE_STAT(TEXT("JobBatch_Exec"), STAT_JobBatch_Exec, STATGROUP_TextureGraphEngine);

uint64 JobBatch::GBatchId = 0;

std::shared_ptr<JobBatch> JobBatch::Create(const FInvalidationDetails& details)
{
	auto Batch = std::make_shared<JobBatch>(details);
	Batch->GetCycle()->SetBatch(Batch);
	return Batch;
}

//////////////////////////////////////////////////////////////////////////

JobBatch::JobBatch(const FInvalidationDetails& Details) 
	: FrameId(TextureGraphEngine::GetFrameId())
	, BatchId(++JobBatch::GBatchId)
	, Cycle(std::make_shared<MixUpdateCycle>(Details))
{
}

JobBatch::~JobBatch()
{
	AllJobs.clear();

	//_cycle->End();
	Cycle = nullptr;

	UE_LOGF(LogBatch, VeryVerbose, "DELETING Batch: %llu", BatchId);
}

AsyncJobResultPtr JobBatch::BeginQueue_Native(JobPriorityQueue& InQueue)
{
	{
		FScopeLock Lock(&Mutex);

		if (!InQueue.size())
		{
			EndBatch();

			return cti::make_ready_continuable<JobResultPtr>(std::make_shared<JobResult>());
		}
	}

	JobRunInfo RunInfoDefault;
	RunInfoDefault.JobScheduler = TextureGraphEngine::GetScheduler();
	RunInfoDefault.Dev = Device_FX::Get();
	RunInfoDefault.Batch = this;
	RunInfoDefault.Cycle = Cycle;

	std::vector<JobPtr> Jobs = InQueue.to_vector();

	{
		FScopeLock lock(&Mutex);

		UE_LOGF(LogBatch, VeryVerbose, "Preparing phase for Batch: %llu, Num Jobs: %llu", BatchId, Jobs.size());

		size_t Index = 0;
		while (InQueue.size())
		{
			JobPtr JobObj = InQueue.top();
			InQueue.pop();
			Jobs[Index++] = JobObj;
		}
	}

	return cti::make_continuable<JobResultPtr>([this, Jobs, RunInfoDefault](auto&& promise) mutable
		{
			UE_LOGF(LogBatch, VeryVerbose, "Beginning Batch: %llu, Num Jobs: %llu", BatchId, Jobs.size());

			auto StartIter = Jobs.begin();
			auto EndIter = Jobs.end();

			int64 NumItems = 0;

			/// First of all we create all the additional dependencies that might be need for each JobObj. 
			/// These need to be done completely first, because they may have complicated chains that 
			/// may require an elevation of the priority. If those Jobs have already been added to the priority
			/// queue on the Device, then fixing the priority is not going to work, because of the way the 
			/// std::priority_queue works. 
			/// Therefore, we must know beforehand, ALL the Jobs and their respective priorities against
			/// each other. Only then, will be be able to ensure that things run smoothly
			for (auto Iter = StartIter; Iter < EndIter; Iter++)
			{
				JobPtr JobObj = *Iter;

				if (!JobObj->IsDone())
				{
					UE_LOGF(LogJob, VeryVerbose, "Batch processing: %ls", *JobObj->GetTransform()->GetName());

					/// Copy the default 
					JobRunInfo RunInfo = RunInfoDefault;
					RunInfo.Dev = JobObj->GetTransform()->TargetDevice(0);
					RunInfo.ThisJob = JobObj;

					JobPtrVec Prior, After;

					JobObj->GetDependencies(Prior, After, RunInfo);

					NumItems += (Prior.size() + After.size()) + 1; /// +1 for the main JobObj
				}
			}

			AllJobs.reserve(NumItems);
			JobsFinished.reserve(NumItems);

			NumJobsRunning = NumItems;
			AllJobs.clear();

			int64 JobId = 0;

			/// Queue all the Jobs
			for (auto iter = StartIter; iter < EndIter; iter++)
			{
				JobPtr JobObj = *iter;

				check(!JobObj->IsDone());

				UE_LOGF(LogJob, VeryVerbose, "Batch processing: %ls", *JobObj->GetTransform()->GetName());

				JobPtrVec& Prior = JobObj->GeneratedPriorJobs();
				JobPtrVec& After = JobObj->GeneratedAfterJobs();

				for (JobPtr PriorJob : Prior)
				{
					PriorJob->SetJobId(JobId++);
					AddNativeJob_Now(PriorJob, true);
				}

 				JobObj->SetJobId(JobId++);
				AddNativeJob_Now(JobObj, true);

				for (JobPtr AfterJob : After)
				{
					AfterJob->SetJobId(JobId++);
					AddNativeJob_Now(AfterJob, true);
				}

				Prior.clear();
				After.clear();
			}

			check(JobId == NumItems);

			promise.set_value(std::make_shared<JobResult>());
		});
}

void JobBatch::EndBatch()
{
	EndTime = Util::Time();
	bIsFinished = true;

	PrintJobTimings();

	if (!OnAllJobsDoneCallbacks.empty())
	{
		/// This can potentially be called from any thread. Callback resolves automatically
		/// if we're already on the game thread
		Util::OnThread(ENamedThreads::GameThread,
			[this]() mutable
			{
				for (size_t CallbackIndex = 0; CallbackIndex < OnAllJobsDoneCallbacks.size(); CallbackIndex++)
				{
					OnAllJobsDoneCallback Callback = std::move(OnAllJobsDoneCallbacks[CallbackIndex]);

					if (Callback)
						Callback(this);
				}

				OnAllJobsDoneCallbacks.clear();

				Cycle->End();
			});
	}
	else
	{
		/// this is where the cycle truly ends
		Cycle->End();
	}

}

void JobBatch::AddNativeJob_Now(JobPtr JobObj, bool AddToAllJobs)
{
	check(IsInGameThread());

	JobObj->SetBatchId(BatchId);

	if (!JobObj->IsDone())
	{
		JobObj->FixPriorities();

		JobObj->SetIsAsync(bIsAsync);
		JobObj->GetRunInfo().Dev->AddNativeTask(JobObj);
		JobsFinished.push_back(false);

		if (JobObj->GetRunInfo().Batch == this)
		{
			if (AddToAllJobs)
				AllJobs.push_back(JobObj);
		}
	}
	else
		JobsFinished.push_back(true);
}

void JobBatch::SetCaptureRenderDoc(bool CaptureRenderDoc /* = true */)
{
	bCaptureRenderDoc = CaptureRenderDoc;
}

AsyncJobResultPtr JobBatch::Exec(OnAllJobsDoneCallback callback)
{
	check(IsInGameThread());

	/// This Batch is starting to execute ... it will be locked forever (until reset for TextureGraph Insight)
	bIsLocked = true;

	SCOPE_CYCLE_COUNTER(STAT_JobBatch_Exec);

	StartTime = -1; /// Util::Time();
	
	Cycle->Begin();

	/// Load the counter of JobObj running to the actual number of Jobs in this Batch
	NumJobsRunning = 0;
	check(!bIsFinished);

	if (callback)
		OnAllJobsDoneCallbacks.push_back(std::move(callback));


	if (IsCaptureRenderDoc())
	{
		JobUPtr BeginJob = T_BeginRenderCapture::CreateJob(Cycle, 0);
		BeginJob->SetPriority((int32)E_Priority::kHighest, false);
		JobUPtr EndJob = T_EndRenderCapture::CreateJob(Cycle, 0);

		Queue.addBeginAndEnd(std::move(BeginJob), std::move(EndJob));
	}

	AllJobs = Queue.to_vector();


	UE_LOGF(LogBatch, VeryVerbose, "Running Batch: %llu [Count: %llu]", BatchId, Queue.size());

	/// We really need these to be in order. The reason being that the Job::Prepare phase (which happens inside
	/// Job::BeginNative is async and can result in some finishing later than others, even if they're ahead of 
	/// another JobObj in the queue.
	/// This has a knock-on effect where the _prev dependencies of DeviceNativeTask hasn't been met, since 
	/// the Device native queue is not in sync anymore (not the same order as the order of the Jobs). 
	/// This results in DeviceNativeTask::Wait() hanging indefinitely.
	return BeginQueue_Native(Queue)
		.then([this](JobResultPtr Result) 
		{
			UE_LOGF(LogBatch, VeryVerbose, "All Jobs queued and prepared for Batch: %llu", BatchId);
			return Result;
		})
		.fail([this](std::exception_ptr e)
		{
			UE_LOGF(LogBatch, Error, "Exception while running Batch: %llu", BatchId);
		});
			
}

bool JobBatch::IsLocked() const
{
	return bIsLocked.load();
}

void JobBatch::Terminate()
{
	for (size_t ji = 0; ji < JobsFinished.size(); ji++)
	{
		if (!JobsFinished[ji])
		{
			JobPtr JobObj = AllJobs[ji];
			JobObj->Terminate();
		}
	}
}

JobPtrW JobBatch::AddJob(JobUPtr job_)
{
	check(IsInGameThread());

	check(!bIsLocked);

	JobPtr JobObj = std::move(job_);
	JobObj->SetQueueId(0);
	JobObj->SetJobId(Queue.size());

	if (!JobObj->GetMix())
		JobObj->SetMix(Cycle->GetMix());

	UE_LOGF(LogBatch, Verbose, "Job added: [Job ID: %lld, Batch Id: %llu, Name: %ls]", JobObj->GetJobId(), BatchId, * JobObj->GetName());

	Queue.add(JobObj);

	return JobPtrW(JobObj);
}

void JobBatch::RemoveJob(JobPtrW JobObjW)
{
	/// If the JobObj has already expired then it cannot possibly be in the queue as that's a strong pointer
	if (JobObjW.expired())
		return;
	
	JobPtr JobObj = JobObjW.lock();
	UE_LOGF(LogBatch, Verbose, "Job removed: [Job ID: %lld, Batch Id: %llu, Name: %ls]", JobObj->GetJobId(), BatchId, * JobObj->GetName());
	Queue.remove(JobObj);
}

void JobBatch::DebugDumpUnfinishedJobs()
{
	if (NumJobsRunning <= 0)
		return;

	UE_LOGF(LogBatch, Verbose, "=== Offending Jobs Batch: %llu ===", BatchId);

	for (size_t ji = 0; ji < JobsFinished.size(); ji++)
	{
		if (!JobsFinished[ji])
		{
			JobPtr JobObj = AllJobs[ji];
			UE_LOGF(LogBatch, Verbose, "    - Job [IsDone: %ls, IsCulled: %ls, HasStarted: %ls] : %ls", 
				(JobObj->IsDone() ? TEXT("Yes") : TEXT(" No")),
				(JobObj->IsCulled() ? TEXT("Yes") : TEXT(" No")),
				(JobObj->GetStats().BeginNativeTime > 0 ? TEXT("Yes") : TEXT(" No")), 
				*JobObj->GetName()
			);
		}
	}
}

void JobBatch::PrintJobTimings() const
{
	UE_LOGF(LogBatch, VeryVerbose, "----------- BEGIN TIMINGS Batch: %llu -----------", BatchId);

	for (size_t JobIndex = 0; JobIndex < AllJobs.size(); JobIndex++)
	{
		JobPtr JobObj = AllJobs[JobIndex];

		/// Only do it for non-culled Jobs
		if (!JobObj->IsCulled())
		{
			FString timingStr = JobObj->GetRunTimings(StartTime);
			UE_LOGF(LogBatch, VeryVerbose, "    - Job Timings: %ls [%ls]", *timingStr, *JobObj->GetName());
		}
	}
	 
	UE_LOGF(LogBatch, VeryVerbose, "------------ END TIMINGS Batch: %llu ------------", BatchId);
}

void JobBatch::OnJobDone(Job* JobObj, int64 JobId)
{
	if (StartTime < 0)
		StartTime = Util::Time();

	if (JobId < 0 || JobId > (int64)JobsFinished.size())
		return;

	check(!JobObj || JobObj->GetRunInfo().Batch == this);

	UE_LOGF(LogBatch, VeryVerbose, "Finished JobObj id: %lld [Total: %d]", JobId, (int32)JobsFinished.size());

	if (JobsFinished[JobId] == false) 
	{
		auto NumJobsAlive = --NumJobsRunning; /// Decrement and then catch the current value
		check(JobId < (int64)NumJobs());
		check(NumJobsAlive >= 0);

		UE_LOGF(LogBatch, VeryVerbose, "Job done: %lld [Num left: %d, Batch Id: %llu]", JobId, (uint32)NumJobsAlive, BatchId);

		JobsFinished[JobId] = true;

		/// when all more Jobs are done then notify
		if (NumJobsAlive <= 0)
		{
			UE_LOGF(LogBatch, VeryVerbose, "All Jobs done for Batch: %llu", BatchId);
			EndBatch();
		}
	}
}

void JobBatch::OnDone(OnAllJobsDoneCallback Callback)
{
	check(IsInGameThread());
	checkf(Callback, TEXT("Attempting to call an unbound TFuncti----------on!"));
	OnAllJobsDoneCallbacks.push_back(std::move(Callback));
}

void JobBatch::ResetForReplay(int32 JobId)
{
	FScopeLock lock(&Mutex);

	/// unlock it
	bIsLocked = false;
	bIsFinished = false;
	ReplayCount++;

	if (JobId < 0)
	{
		bNoCache = true;
	}
	else
	{
		bNoCache = false;
	}

	// Resetting all the Jobs for replay
	for (auto JobObj : AllJobs)
	{
		JobObj->ResetForReplay(JobId == JobObj->GetJobId());
	}
}
