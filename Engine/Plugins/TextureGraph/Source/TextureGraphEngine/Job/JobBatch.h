// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include <vector>
#include "FxMat/RenderMaterial.h"
#include "Device/Device.h"
#include "JobCommon.h"

#define UE_API TEXTUREGRAPHENGINE_API

class										MixUpdateCycle;
class										Job;
typedef std::unique_ptr<Job>				JobUPtr;
typedef std::shared_ptr<Job>				JobPtr;
typedef std::weak_ptr<Job>					JobPtrW;
typedef cti::continuable<bool>				AsyncBool;
typedef std::shared_ptr<JobBatch>			JobBatchPtr;
typedef TUniqueFunction<void(JobBatch* )>	OnAllJobsDoneCallback;
typedef std::vector<OnAllJobsDoneCallback>	OnAllJobsDoneCallbackVec;
typedef GenericTaskPriorityQueue<std::shared_ptr<Job>, DeviceNativeTask_CompareStrong> JobPriorityQueue;

struct FInvalidationDetails;

DECLARE_LOG_CATEGORY_EXTERN(LogBatch, Log, All);

//////////////////////////////////////////////////////////////////////////
/// JobBatch: A series of jobs put together that need to execute at the 
/// same time
//////////////////////////////////////////////////////////////////////////
class JobBatch
{
public:
	static UE_API uint64					GBatchId;				/// For easier debugging
protected:

	mutable FCriticalSection		Mutex;						/// Mutex for the JobObj queue
	std::atomic<bool>				bIsLocked = false;			/// Whether the queue is now locked from any further changes

	std::vector<JobPtr>				AllJobs;					/// All the jobs within the system

	JobPriorityQueue				Queue;						/// The JobObj queue
	JobPtr							CurrJob;					/// The current JobObj ... we keep it to prevent it from going out of scope
	JobPtr							PrevJob;					/// The previous JobObj that we ran. Mostly for debugging purposes

	double							StartTime = 0;				/// Time when the batch was started
	double							EndTime = 0;				/// Time when the batch has ended

	uint64							FrameId = 0;				/// What is the frame id for this
	size_t							BatchId = 0;				/// What is the index of this batch

	std::shared_ptr<MixUpdateCycle> Cycle;						/// Scene update cycle information
	int32							ReplayCount = 0;			/// Counting the number of debug replay of this batch

	bool							bCaptureRenderDoc = false;	/// flag to record a RenderDoc capture for debugging the batch
	bool							bNoCache = false;			/// Whether we should ignore the cache
	bool							bIsIdle = false;			/// Is the batch generated by an idle service or not
	bool							bIsFinished = false;		/// Whether the batch has been finished or not
	bool							bIsAsync = true;			/// Whether the job batch is asynchronous or not

	UE_API AsyncJobResultPtr				BeginQueue_Native(JobPriorityQueue& InQueue);

	UE_API void							EndBatch();	

	friend class Job;
	UE_API void							OnJobDone(Job* JobObj, int64 JobId);	/// Once a JobObj is TRULY done then it reports to the batch,
	UE_API void							AddNativeJob_Now(JobPtr JobObj, bool AddToAllJobs);

	std::atomic<uint32>				NumJobsRunning = 0;		/// A count down of the jobs still running, decremented by OnJobDone
	OnAllJobsDoneCallbackVec		OnAllJobsDoneCallbacks;		/// If the call back has been assigned, that 's what will get called when all the jobs are done.

	std::vector<bool>				JobsFinished;				/// Jobs that have been finished

	UE_API void							PrintJobTimings() const;

public:
	UE_API explicit						JobBatch(const FInvalidationDetails& Details);
	UE_API virtual							~JobBatch();

	UE_API virtual AsyncJobResultPtr		Exec(OnAllJobsDoneCallback Callback = nullptr);
	
	UE_API virtual JobPtrW					AddJob(JobUPtr JobObj);
	UE_API virtual void					RemoveJob(JobPtrW JobObj);

	UE_API bool							IsLocked() const;

	UE_API void							ResetForReplay(int32 JobId = -1); /// For debug purpose we can reset the state of a batch for a replay
																	  /// A JobObj index can be specified to only reset that JobObj during replay

	UE_API void							SetCaptureRenderDoc(bool CaptureRenderDoc = true);
	UE_API void							DebugDumpUnfinishedJobs();

	UE_API void							Terminate();
	UE_API void							OnDone(OnAllJobsDoneCallback Callback);

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE bool				IsCaptureRenderDoc() const { return bCaptureRenderDoc; }
	FORCEINLINE void				SetNoCache(bool bInNoCache) { bNoCache = bInNoCache; }
	FORCEINLINE bool				IsNoCache() const { return bNoCache; }
	FORCEINLINE std::shared_ptr<MixUpdateCycle> GetCycle() const { return Cycle; }
	FORCEINLINE size_t				NumJobs() const { return AllJobs.size(); }
	FORCEINLINE uint64				GetFrameId() const { return FrameId; }
	FORCEINLINE uint64				GetBatchId() const { return BatchId; }

	FORCEINLINE JobPtrW				GetJobAt(uint64_t jobNum) const { return (AllJobs.size() > jobNum ? AllJobs[jobNum] : JobPtrW()); }
	FORCEINLINE double				GetStartTime() const { return StartTime; }
	FORCEINLINE double				GetEndTime() const { return EndTime; }
	FORCEINLINE int32				GetReplayCount() const { return ReplayCount; } /// For debug purpose, report the replay number
	FORCEINLINE uint32				GetNumJobsRunning() const { return NumJobsRunning; }

	FORCEINLINE bool				WasGeneratedFromIdleService() const { return bIsIdle; }
	FORCEINLINE bool&				IsIdle() { return bIsIdle; }
	FORCEINLINE bool				IsFinished() const { return bIsFinished; }
	FORCEINLINE void				SetAsync(bool bIsAsyncIn) { bIsAsync = bIsAsyncIn; }
	FORCEINLINE bool				IsAsync() const { return bIsAsync; }

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static FORCEINLINE uint64		CurrentBatchId() { return GBatchId; }
	static UE_API std::shared_ptr<JobBatch> Create(const FInvalidationDetails& details);
};


#undef UE_API
