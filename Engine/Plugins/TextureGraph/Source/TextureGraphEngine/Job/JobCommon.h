// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Helper/DataUtil.h"
#include "Data/Blobber.h"

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
class Device;
class Scheduler;
class JobBatch;
class MixUpdateCycle;

class Blob;
typedef std::weak_ptr<Blob>			BlobPtrW;
typedef std::shared_ptr<Blob>		BlobPtr;

class TiledBlob;
typedef std::shared_ptr<TiledBlob>	TiledBlobPtr;;

class BlobTransform;
typedef std::shared_ptr<BlobTransform> BlobTransformPtr;

class Job;

struct JobRunInfo
{
	Scheduler*						JobScheduler = nullptr;	/// The scheduler instance that's scheduled this job
	Device*							Dev = nullptr;			/// The device on which the job must be executed
	JobBatch*						Batch = nullptr;		/// The batch that this job belongs to
	std::shared_ptr<MixUpdateCycle> Cycle = nullptr;		/// The mix update cycle
	std::weak_ptr<Job>				ThisJob;				/// Shared ptr (weak) for this job. Needed in various places that need smart ptr
};

//////////////////////////////////////////////////////////////////////////
struct JobResult
{
	static UE_API std::shared_ptr<JobResult> NullResult;		/// Null result

	std::exception_ptr				ExInner;			/// Original exception that was raised by the action
	int32							ErrorCode = 0;		/// What is the error code
	BlobRef							BlobObj;			/// The resultant blob for this job
	CHashPtr						HashValue;			/// The has for the job
	BlobTransformPtr				Transform;			/// The transformation function (or a clone of it) that was used for this job

	JobResult() : BlobObj(BlobRef()) {}
	JobResult(BlobRef NewBlobObj, CHashPtr NewHashValue) : BlobObj(NewBlobObj), HashValue(NewHashValue) {}
	UE_API ~JobResult();
};

typedef std::shared_ptr<JobResult>	JobResultPtr;
typedef cti::continuable<JobResultPtr> AsyncJobResultPtr;

class Job;
typedef std::shared_ptr<Job>		JobPtr;
typedef std::vector<JobPtr>			JobPtrVec;
typedef TArray<JobPtr>				JobPtrArray;
typedef TArray<Job*>				JobPArray;

#undef UE_API
