// Copyright Epic Games, Inc. All Rights Reserved.
#include "MinMaxService.h"
#include "../TextureGraphEngine.h"
#include "Scheduler.h"
#include "Data/TiledBlob.h" 
#include "Transform/Utility/T_MinMax.h"
#include "Model/Mix/Mix.h"

MinMaxService::MinMaxService() : BlobHelperService(TEXT("MinMax"))
{
}

MinMaxService::~MinMaxService()
{
}

AsyncJobResultPtr MinMaxService::Tick()
{
	UE_LOGF(LogIdle_Svc, VeryVerbose, "Svc_MinMax::Tick");

	static constexpr size_t MaxBlobs = 8;
	BlobPtr BlobObjs[MaxBlobs] = { 0 };
	size_t NumBlobs = 0;

	auto Iter = Blobs.begin();
	auto StartIter = Iter;

	while (!TextureGraphEngine::IsDestroying() && NumBlobs < MaxBlobs && Blobs.size() > 0 && Iter != Blobs.end())
	{
		BlobPtr BlobObj = *Iter;

		if (BlobObj)
		{
			/// Must've resolved by now
			check(!BlobObj->IsLateBound());

			BlobObjs[NumBlobs] = BlobObj;
			NumBlobs++;
		}

		Iter++;
	}

	auto EndIter = Iter;

	/// Erase elements
	if (EndIter != StartIter)
		Blobs.erase(StartIter, EndIter);

	if (NumBlobs > 0)
	{
		/// Create a new batch
		FInvalidationDetails Details(UMix::NullMix());
		Details.All();

		JobBatchPtr Batch = JobBatch::Create(Details);
		Batch->IsIdle() = true;

		SceneTargetUpdatePtr Target = std::make_shared<MixTargetUpdate>(Details.Mix, 0);
		Batch->GetCycle()->AddTarget(Target);

		/// Then add all the T_MipMap jobs to it
		for (size_t BlobIndex = 0; BlobIndex < NumBlobs; BlobIndex++)
		{
			TiledBlobPtr BlobObj = TiledBlob::AsTiledBlob(BlobObjs[BlobIndex]);
			T_MinMax::Create(Batch->GetCycle(), BlobObj, 0, false);
		}

		/// Once the job has been constructed, just put it in the scheduler
		TextureGraphEngine::GetScheduler()->AddBatch(Batch);
	}

	return cti::make_ready_continuable<JobResultPtr>(std::make_shared<JobResult>());
}
 