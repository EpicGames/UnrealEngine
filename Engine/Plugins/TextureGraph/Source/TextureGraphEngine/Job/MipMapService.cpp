// Copyright Epic Games, Inc. All Rights Reserved.
#include "MipMapService.h"
#include "../TextureGraphEngine.h"
#include "Scheduler.h"
#include "Data/Blobber.h"
#include "Transform/Utility/T_MipMap.h"
#include "Model/Mix/Mix.h"

MipMapService::MipMapService() : BlobHelperService(TEXT("MipMap_Gen"))
{
}

MipMapService::~MipMapService()
{
}

AsyncJobResultPtr MipMapService::Tick()
{
	UE_LOGF(LogIdle_Svc, VeryVerbose, "Svc_MipMap::Tick");

	static constexpr size_t MaxBlobs = 8;
	BlobPtr BlobObjs[MaxBlobs] = { 0 };
	size_t NumBlobs = 0;

	auto Iter = Blobs.begin();
	auto StartIter = Iter;

	while (!TextureGraphEngine::IsDestroying() && NumBlobs < MaxBlobs && Blobs.size() > 0 && Iter != Blobs.end())
	{
		BlobPtr BlobObj = *Iter;

		if (BlobObj)
		{
			/// Must've resolved by now
			check(!BlobObj->IsLateBound());

			BlobObjs[NumBlobs] = BlobObj;
			NumBlobs++;
		}

		Iter++;
	}

	auto EndIter = Iter;

	/// Erase elements
	if (EndIter != StartIter)
		Blobs.erase(StartIter, EndIter);

	if (NumBlobs > 0)
	{
		/// Create a new batch
		FInvalidationDetails Details(UMix::NullMix());
		Details.All();

		JobBatchPtr Batch = JobBatch::Create(Details);
		Batch->IsIdle() = true;

		SceneTargetUpdatePtr target = std::make_shared<MixTargetUpdate>(Details.Mix, 0);
		Batch->GetCycle()->AddTarget(target);

		/// Then add all the T_MipMap jobs to it
		for (size_t BlobIndex = 0; BlobIndex < NumBlobs; BlobIndex++)
		{
			TiledBlobPtr BlobObj = TiledBlob::AsTiledBlob(BlobObjs[BlobIndex]);
			T_MipMap::Create(Batch->GetCycle(), BlobObj, 0);
		}

		/// Once the job has been constructed, just put it in the scheduler
		TextureGraphEngine::GetScheduler()->AddBatch(Batch);
	}

	return cti::make_ready_continuable<JobResultPtr>(std::make_shared<JobResult>());
}
