// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "BlobHelperService.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <vector>
#include <unordered_map>

#define UE_API TEXTUREGRAPHENGINE_API

class UModelObject;
typedef std::shared_ptr<class Job>		JobPtr;
typedef std::weak_ptr<class Job>		JobPtrW;

class UMixInterface;

class MipMapService : public BlobHelperService
{
public:
									UE_API MipMapService();
	UE_API virtual							~MipMapService() override;

	UE_API virtual AsyncJobResultPtr		Tick() override;
};

typedef std::shared_ptr<MipMapService>	MipMapServicePtr;
typedef std::weak_ptr<MipMapService>	MipMapServicePtrW;

#undef UE_API
