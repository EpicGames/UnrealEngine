// Copyright Epic Games, Inc. All Rights Reserved.
#include "Scheduler.h"
#include "Job.h"
#include "TempHashService.h"
#include "TextureGraphEngine.h"
#include "Device/DeviceManager.h"
#include "Device/Device.h"

DECLARE_CYCLE_STAT(TEXT("Scheduler_Update"), STAT_Scheduler_Update, STATGROUP_TextureGraphEngine);

const double Scheduler::IdleTimeInterval = 500.0;
const double Scheduler::IdleBusyTimeInterval = 1000.0;
const double Scheduler::IdleBatchTimeLimit = 500.0;
const double Scheduler::CurrentBatchWarningLimit = 2000.0;

Scheduler::Scheduler()
{
	ObserverSource = std::make_shared<SchedulerObserverSource>();

	TempHashServicePtr TempHashSPtr = std::make_shared<TempHashService>();
	BlobHasherServicePtr BlobHasherSPtr = std::make_shared<BlobHasherService>();
	DeviceTransferServicePtr DeviceTransferSPtr = std::make_shared<DeviceTransferService>();
	ThumbnailsServicePtr ThumbnailsSPtr = std::make_shared<ThumbnailsService>();
	MipMapServicePtr MipmapSPtr = std::make_shared<MipMapService>();
	MinMaxServicePtr MinmaxSPtr = std::make_shared<MinMaxService>();
	HistogramServicePtr HistogramSPtr = std::make_shared<HistogramService>();

	BlobHasherServiceObj = BlobHasherSPtr;
	DeviceTransferServiceObj = DeviceTransferSPtr;
	ThumbnailsServiceObj = ThumbnailsSPtr;
	MipMapServiceObj = MipmapSPtr;
	MinMaxServiceObj = MinmaxSPtr;
	HistogramServiceObj = HistogramSPtr;

	AddIdleService(BlobHasherSPtr);
	//AddIdleService(DeviceTransferSPtr);
	AddIdleService(ThumbnailsSPtr);
	AddIdleService(MipmapSPtr);
	AddIdleService(MinmaxSPtr);
	AddIdleService(HistogramSPtr);
}

Scheduler::~Scheduler()
{
	bIsRunning = false;
	StopServices();

	/// Add a bit of a delay to allow proper clean-up
	std::this_thread::sleep_for(std::chrono::milliseconds((int)IdleBatchTimeLimit * 2));

	Stop();
}

void Scheduler::Update(float dt)
{
	SCOPE_CYCLE_COUNTER(STAT_Scheduler_Update);
	check(IsInGameThread());

	/// Make sure value is initialised correct
	if (TimeSinceIdle < 0.001)
		TimeSinceIdle = Util::Time();

	if (TimeSinceIdleBatchRan < 0.001)
		TimeSinceIdleBatchRan = Util::Time();

	if (!bIsRunning)
		Start();

	double Delta = Util::TimeDelta(TimeSinceIdleBatchRan);
	if (Delta > IdleBusyTimeInterval)
	{
		UpdateIdle(true, true);
		ObserverSource->UpdateIdle();
	}

	{
		FScopeLock Lock(&CurrentBatchMutex);
		if (CurrentBatch)
		{
			double currentTime = Util::Time();
			double delta = currentTime - CurrentBatchStartTime;
			double timeoutLimit = !TextureGraphEngine::IsTestMode() ? CurrentBatchWarningLimit : CurrentBatchWarningLimit * 4.0;

			if (delta > timeoutLimit)
			{
				UE_LOGF(LogBatch, Verbose, "Current Batch: %llu been running for %f ms [Max threshold: %f, Jobs: %u/%u]!", 
					CurrentBatch->GetBatchId(), (float)delta, (float)CurrentBatchWarningLimit, CurrentBatch->GetNumJobsRunning(), 
					(uint32)CurrentBatch->NumJobs());
				CurrentBatch->DebugDumpUnfinishedJobs();
				//_currentBatch->Terminate();
			}

			return;
		}
	}

	JobBatchPtr BatchToRun = nullptr;
	{
		FScopeLock Lock(&BatchMutex);
		if (Batches.size())
		{
			BatchToRun = Batches.front();
			Batches.pop_front();

			/// We merge batches if we're lagging behind ...
			if (!TextureGraphEngine::IsTestMode() && Batches.size() >= 8)
			{
				/// OK here, we're going to merge batches belonging to the same mix together
				
				// figure out the prioritized mix from the Batch
				BatchToRun = Batches.front();
				Batches.pop_front();
				UMixInterface* CurrentBatchMix = BatchToRun->GetCycle()->GetMix();

				// find the latest Batch of required mix
				auto Iter = Batches.begin();
				JobBatchPtr LastMixBatch = BatchToRun;

				while (Iter != Batches.end())
				{
					JobBatchPtr FutureBatch = *Iter;

					if (FutureBatch->GetCycle()->GetMix() == CurrentBatchMix)
					{
						LastMixBatch = *Iter;
					}

					Iter++;
				}

				// we get latest Batch to run for the mix
				BatchToRun = LastMixBatch;

				// Merge invalidation details of the mix from multiple batches.
				// Remove the batches from the list after merging.
				Iter = Batches.begin();
				while (Iter != Batches.end())
				{
					JobBatchPtr FutureBatch = *Iter;

					if (FutureBatch->GetCycle()->GetMix() == CurrentBatchMix)
					{
						BatchToRun->GetCycle()->MergeDetails(FutureBatch->GetCycle()->GetDetails());
						Iter = Batches.erase(Iter);
					}
					else
						Iter++;
				}
			}
		}
	}

	if (BatchToRun)
	{
		FScopeLock Lock(&CurrentBatchMutex);
		CurrentBatch = BatchToRun;
		CurrentBatchStartTime = Util::Time();

		//UE_LOGF(LogBatch, Log, "Next Batch: %llu [Prev: %llu]", CurrentBatch->GetBatchId(), PreviousBatch ? PreviousBatch->GetBatchId() : 0);

		if (!CurrentBatch->WasGeneratedFromIdleService())
			TimeSinceIdle = Util::Time();

		if (bCaptureNextBatch)
		{
			CurrentBatch->SetCaptureRenderDoc(true);
			bCaptureNextBatch = false;
		}

		CurrentBatch->Exec([this] (JobBatch*)	/// Instead of passing it as an argument, JobBatch should be a return type; this is to keep it from cyclic dependancy
			{
				/// This closure will be called when all the jobs execution is really over
				/// tell the world through the observer
				/// Clear out the current Batch
				FScopeLock Lock(&CurrentBatchMutex);
				PreviousBatch = CurrentBatch;
				CurrentBatch = nullptr;

				if (PreviousBatch && !PreviousBatch->WasGeneratedFromIdleService())
					TimeSinceIdle = Util::Time();

				UE_LOGF(LogBatch, Verbose, "Scheduler, Batch fully queued: %llu. Triggering Observer::BatchJobsDone ...", PreviousBatch ? PreviousBatch->GetBatchId() : -1);

				ObserverSource->BatchJobsDone(PreviousBatch);

				/// Mark the current batch as done
				PreviousBatch->GetCycle()->GetDetails().BroadcastOnDone();

				UE_LOGF(LogBatch, Verbose, "Scheduler Observer::BatchJobsDone finished for Batch: %llu", PreviousBatch ? PreviousBatch->GetBatchId() : -1);
				PreviousBatch = nullptr;
			})
			.then([this]()
				{
					if (PreviousBatch && !PreviousBatch->WasGeneratedFromIdleService())
						TimeSinceIdle = Util::Time();

					UE_LOGF(LogBatch, Verbose, "Scheduler triggering Observer::BatchDone for Batch: %llu ...", PreviousBatch ? PreviousBatch->GetBatchId() : -1);

					ObserverSource->BatchDone(CurrentBatch);

					UE_LOGF(LogBatch, Verbose, "Scheduler Observer::BatchDone finished for Batch: %llu", PreviousBatch ? PreviousBatch->GetBatchId() : -1);
				});
	}
	else
	{
		Delta = Util::TimeDelta(TimeSinceIdle);
		if (Delta > IdleTimeInterval)
		{
			UpdateIdle(false);

			// Since we have time, notify the observer update
			ObserverSource->UpdateIdle();
		}
	}
}

void Scheduler::SetCaptureRenderDocNextBatch(bool bCapture /* = true */)
{
	bCaptureNextBatch = bCapture;
}

AsyncJobResultPtr Scheduler::UpdateIdleBatch(size_t Index, bool bIsBusyUpdate)
{
	check(Index < IdleServices.size());
	IdleServicePtr Batch = IdleServices[Index];
	double StartTime = Util::Time();
	if (bIsBusyUpdate && !Batch->ShouldRunDuringBusyUpdate())
		return cti::make_ready_continuable(std::make_shared<JobResult>());

	return Batch->Tick().then([this, Batch, StartTime](JobResultPtr result)
		{
			double EndTime = Util::Time();
			double Duration = EndTime - StartTime;
			bool bDidOffendTimeLimit = false;

			if (Duration > IdleBatchTimeLimit)
			{
				bDidOffendTimeLimit = true;
				UE_LOGF(LogBatch, Verbose, "Idle Batch [%ls] time limit offense @ %0.2f ms [Max: %0.2f ms]", *Batch->GetName(), (float)Duration, IdleBatchTimeLimit);
			}

			Batch->UpdateStats(EndTime, StartTime, bDidOffendTimeLimit);

			return result;
		});
}

void Scheduler::UpdateIdle(bool bForce, bool bIsBusyUpdate)
{
	if (bIsDisableIdle)
		return;

	if (!bForce)
	{
		FScopeLock Lock(&CurrentBatchMutex);

		/// If there's an active batch then we don't run this at all
		if (CurrentBatch)
			return;
	}

	/// Currently we don't do the idle update in test mode
	if (TextureGraphEngine::IsTestMode())
		return;

	// if no Batch to run, check if we have an idle Batch we should run
	if (!bForce)
	{
		/// Shouldn't run idle services if a Batch is still being processed
		FScopeLock Lock(&CurrentBatchMutex);
		if (CurrentBatch)
			return;
	}

	if (!bForce)
	{
		/// Shouldn't run idle services if we still have batches to process
		FScopeLock Lock(&BatchMutex);
		if (Batches.size())
			return;
	}

	UE_LOGF(LogIdle_Svc, VeryVerbose, "Scheduler::UpdateIdle");

	if (TextureGraphEngine::IsDestroying())
		return;

	size_t NumIdleBatches = 0;
	TimeSinceIdleBatchRan = Util::Time();

	{
		FScopeLock Lock(&IdlServiceMutex);
		NumIdleBatches = IdleServices.size();
	}

	std::vector<AsyncJobResultPtr> Promises;

	if (!bIsBusyUpdate)
		Promises.emplace_back(TextureGraphEngine::GetBlobber()->UpdateIdle());

	/// Idle update for all the devices
	for (size_t DeviceIndex = 0; DeviceIndex < TextureGraphEngine::GetDeviceManager()->GetNumDevices(); DeviceIndex++)
	{
		Device* Dev = TextureGraphEngine::GetDeviceManager()->GetDevice(DeviceIndex);
		if (Dev)
			Promises.emplace_back(Dev->UpdateIdle());
	}

	for (size_t IdleBatchIndex = 0; IdleBatchIndex < NumIdleBatches; IdleBatchIndex++)
	{
		auto Promise = UpdateIdleBatch(IdleBatchIndex, bIsBusyUpdate);
		Promises.push_back(std::move(Promise));
	}

	if (Promises.size())
	{
		cti::when_all(Promises.begin(), Promises.end()).then([this]()
			{
				/// TODO:
			});
	}
}

void Scheduler::AddIdleService(IdleServicePtr Service)
{
	FScopeLock Lock(&IdlServiceMutex);
	IdleServices.push_back(Service);
}

void Scheduler::CaptureRenderDocLastRunBatch()
{
}

void Scheduler::ClearCache()
{
	check(IsInGameThread() && TextureGraphEngine::IsTestMode());

	check(!CurrentBatch);
	check(!PreviousBatch || PreviousBatch->IsFinished());
	check(Batches.empty());

	PreviousBatch = nullptr;
}

void Scheduler::AddBatch(JobBatchPtr Batch)
{
	check(IsInGameThread());

	FScopeLock Lock(&BatchMutex);

#if 0
	if (Batches.size())
	{
		check(Batches.back()->GetFrameId() <= Batch->GetFrameId());
	}

	check(Batch->GetFrameId())
		UE_LOGF(LogBatch, Verbose, "Adding Batch: %llu [FrameId: %llu, Num batches: %llu]", Batch->GetBatchId(), Batch->GetFrameId(), Batches.size());
#endif 

	Batches.push_back(Batch);
	ObserverSource->BatchAdded(Batch); // notify observer
	TimeSinceIdle = Util::Time();
	Batch->GetCycle()->GetDetails().BroadcastOnQueued();
}

void Scheduler::Start()
{
	check(IsInGameThread());
	bIsRunning = true;

	if (ObserverSource)
		ObserverSource->Start();
}

void Scheduler::StopServices()
{
	check(IsInGameThread());
	for (auto Service : IdleServices)
		Service->Stop();

	IdleServices.clear();
}

void Scheduler::Stop()
{
	check(IsInGameThread());
	bIsRunning = false;

	if (ObserverSource)
		ObserverSource->Stop();
}

void Scheduler::RegisterObserverSource(const SchedulerObserverSourcePtr& InObserverSource)
{
	if (InObserverSource)
	{
		ObserverSource = InObserverSource;
	}
	else
	{
		ObserverSource = std::make_shared<SchedulerObserverSource>();
	}
}