// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IdleService.h"
#include "TempHashService.h"
#include "BlobHasherService.h"
#include "DeviceTransferService.h"
#include "ThumbnailsService.h"
#include "MipMapService.h"
#include "MinMaxService.h"
#include "HistogramService.h"
#include <queue>
#include <list>
#include "Profiling/StatGroup.h"

#define UE_API TEXTUREGRAPHENGINE_API

class JobBatch;
typedef std::shared_ptr<JobBatch>		JobBatchPtr;
typedef std::weak_ptr<JobBatch>			JobBatchPtrW;

class SchedulerObserverSource
{
protected:
	/// Protected interface of emitters called by the scheduler to notify the observers
	friend class Scheduler;
	friend class ThumbnailsService; /// ThumbnailsService is also authorized to notify its batch activity
	friend class HistogramService;

	virtual void						Start() {}
	virtual void						UpdateIdle() {}
	virtual void						Stop() {}
	virtual void						BatchAdded(JobBatchPtr Batch) {}
	virtual void						BatchDone(JobBatchPtr Batch) {}
	virtual void						BatchJobsDone(JobBatchPtr Batch) {}

public:
	SchedulerObserverSource() = default;
	virtual ~SchedulerObserverSource() = default;
};
typedef std::shared_ptr<SchedulerObserverSource> SchedulerObserverSourcePtr;

class Scheduler
{
private:
	static UE_API const double					IdleTimeInterval;				/// Start an Idle Batch after <this many> milliseconds
	static UE_API const double					IdleBusyTimeInterval;			/// Start an Idle Batch (during busy times) after <this many> milliseconds
	static UE_API const double					IdleBatchTimeLimit;				/// Start an Idle Batch after <this many> milliseconds
	static UE_API const double					CurrentBatchWarningLimit;		/// Scheduler starts spewing out warnings if the current batch doesn't finish ater <this many> milliseconds

	bool								bIsRunning = false;				/// Whether the scheduler is running or not

	typedef std::list<JobBatchPtr>		JobBatchPtrList;
	typedef std::vector<IdleServicePtr>	Svc_IdlePtrVec;

	SchedulerObserverSourcePtr			ObserverSource;					/// Scheduler observer source interface where public signals will be emitted from. 
																		/// Default observerSource is the default class implementation which is a no-op

	//////////////////////////////////////////////////////////////////////////
	/// Idle batches
	//////////////////////////////////////////////////////////////////////////
	mutable FCriticalSection			IdlServiceMutex;				/// Mutex for the idle batches
	Svc_IdlePtrVec						IdleServices;					/// The batches that we can run when the Scheduler is idle for s_idleTimeInterval

	BlobHasherServicePtrW				BlobHasherServiceObj;			/// Blob hashing service
	DeviceTransferServicePtrW			DeviceTransferServiceObj;		/// Device transfer idle service
	ThumbnailsServicePtrW				ThumbnailsServiceObj;			/// Thumbnail service
	MipMapServicePtrW					MipMapServiceObj;				/// Mip mapping service
	MinMaxServicePtrW					MinMaxServiceObj;				/// Min/max calculation service
	HistogramServicePtrW				HistogramServiceObj;			/// Histogram service

	//////////////////////////////////////////////////////////////////////////
	/// Normal batches
	//////////////////////////////////////////////////////////////////////////
	mutable FCriticalSection			BatchMutex;						/// Mutex for the job queue
	mutable FCriticalSection			CurrentBatchMutex;				/// Mutex for the job queue

	JobBatchPtr							CurrentBatch;					/// Current batch that we are executing
	double								CurrentBatchStartTime = 0;		/// The time when the current batch started. This is used to check and give out 
																		/// warnings, if the current batch is taking too long to process

	JobBatchPtr							PreviousBatch;					/// The previous batch that we rendered
	JobBatchPtrList						Batches;						/// The batches that we have to run
	JobBatchPtr							CurrentHighPriorityBatch;		/// The current high priority batch job

	double								TimeSinceIdle = 0;				/// time since scheduler has been idle
	double								TimeSinceIdleBatchRan = 0;		/// time since scheduler has been idle

	bool								bCaptureNextBatch = false;		/// set the next incoming batch to get captured by RenderDoc
	bool								bIsDisableIdle = false;			/// Disable idle services or not

	uint64								BatchIndex = 0;					/// Index of the batches

	UE_API void								Start();
	UE_API void								Stop();
	UE_API void								StopServices();

	UE_API AsyncJobResultPtr					UpdateIdleBatch(size_t index, bool bIsBusyUpdate);
	UE_API void								UpdateIdle(bool bForce, bool bIsBusyUpdate = false);

public:
										UE_API Scheduler();
										UE_API ~Scheduler();

	UE_API void								Update(float Delta);
	UE_API void								AddBatch(JobBatchPtr Batch);
	UE_API void								AddIdleService(IdleServicePtr batch);
	
	UE_API void								CaptureRenderDocLastRunBatch();

	UE_API void								RegisterObserverSource(const SchedulerObserverSourcePtr& observerSource); ///
	UE_API void								SetCaptureRenderDocNextBatch(bool capture = true);
	UE_API void								ClearCache();

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions 
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE BlobHasherServicePtrW	GetBlobHasherService() const { return BlobHasherServiceObj; }
	FORCEINLINE DeviceTransferServicePtrW GetDeviceTransferService() const { return DeviceTransferServiceObj; }
	FORCEINLINE ThumbnailsServicePtrW	GetThumbnailsService() const { return ThumbnailsServiceObj; }
	FORCEINLINE MipMapServicePtrW		GetMipMapService() const { return MipMapServiceObj; }
	FORCEINLINE MinMaxServicePtrW		GetMinMaxService() const { return MinMaxServiceObj; }
	FORCEINLINE HistogramServicePtrW	GetHistogramService() const { return HistogramServiceObj; }
	FORCEINLINE void					SetDisableIdle(bool bIsDisableIdleIn) { bIsDisableIdle = bIsDisableIdleIn; }
	FORCEINLINE bool					IsDisableIdle() const { return bIsDisableIdle; }

	FORCEINLINE	size_t					NumBatches() const
	{
		FScopeLock lock(&BatchMutex);
		size_t numBatches = Batches.size();
		return numBatches;
	}

	FORCEINLINE size_t					GetBatchIndex() const
	{
		FScopeLock lock(&BatchMutex);
		return BatchIndex;
	}

	FORCEINLINE SchedulerObserverSourcePtr GetObserverSource() const { return ObserverSource; }
};

typedef std::unique_ptr<Scheduler>		SchedulerPtr;

#undef UE_API
