// Copyright Epic Games, Inc. All Rights Reserved.
#include "TempHashService.h"
#include "TextureGraphEngine.h"
#include "Data/Blobber.h"

TempHashService::TempHashService() : IdleService(TEXT("TempHashResolver"))
{
	Hashes.reserve(1024);
}

TempHashService::~TempHashService()
{
}

AsyncJobResultPtr TempHashService::Tick()
{
	UE_LOGF(LogIdle_Svc, VeryVerbose, "Svc_BlobHasher::Tick");

	check(IsInGameThread());

	CHashPtr NextHash = nullptr;
	bool DidResolve = false;

	{
		FScopeLock Lock(&HashesMutex);
		if (!Hashes.empty())
			NextHash = Hashes.front();
	}

	if (NextHash)
	{
		HashType OldHash = NextHash->Value();
		bool DidUpdate = NextHash->TryFinalise();

		if (DidUpdate)
		{
			/// Update in the Blobber here
			TextureGraphEngine::GetBlobber()->UpdateHash(OldHash, NextHash);
		}

		if (NextHash->IsFinal())
		{
			FScopeLock Lock(&HashesMutex);
			Hashes.erase(Hashes.begin());
		}
	}

	return cti::make_ready_continuable<JobResultPtr>(std::make_shared<JobResult>());
}

void TempHashService::Stop()
{
	FScopeLock Lock(&HashesMutex);
	Hashes.clear();
}

void TempHashService::Add(CHashPtr HashValue)
{
	check(HashValue->IsTemp());

	FScopeLock Lock(&HashesMutex);
	Hashes.push_back(HashValue);
}
