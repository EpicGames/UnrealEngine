// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IdleService.h"
#include <vector>

#define UE_API TEXTUREGRAPHENGINE_API

class TempHashService : public IdleService
{
protected:
	FCriticalSection				HashesMutex;			/// Mutex for the hashes
	std::vector<CHashPtr>			Hashes;					/// The hashes that we're waiting to resolve

public:
									UE_API TempHashService();
	UE_API virtual							~TempHashService() override;

	UE_API virtual AsyncJobResultPtr		Tick() override;
	UE_API virtual void					Stop() override;

	UE_API void							Add(CHashPtr HashValue);
};

typedef std::shared_ptr<TempHashService>	TempHashServicePtr;
typedef std::weak_ptr<TempHashService>		TempHashServicePtrW;

#undef UE_API
