// Copyright Epic Games, Inc. All Rights Reserved.
#include "ThumbnailsService.h"
#include "../TextureGraphEngine.h"
#include "Scheduler.h" 
#include "Job/Job.h"
#include "Model/Mix/MixInterface.h"

ThumbnailsService::ThumbnailsService() : IdleService(TEXT("Thumbnail_Renderer"))
{
}

ThumbnailsService::~ThumbnailsService()
{
}

AsyncJobResultPtr ThumbnailsService::Tick()
{
	check(IsInGameThread());
 
 	/// if no batch or the batch has no jobs then just don't do anything
 	if (!Batch)
 		return cti::make_ready_continuable<JobResultPtr>(std::make_shared<JobResult>());
 
 	UE_LOGF(LogIdle_Svc, VeryVerbose, "ThumbnailsService::Tick");
 
 	/// Move over to the next cycle
 	JobBatchPtr CurrentBatch = GetNextUpdateCycle();
 
 	if (!CurrentBatch)
 		return cti::make_ready_continuable<JobResultPtr>(std::make_shared<JobResult>());
 
	TextureGraphEngine::GetInstance()->GetScheduler()->GetObserverSource()->BatchAdded(CurrentBatch); // notify observer
 
	return CurrentBatch->Exec([=, this](JobBatch*)	/// Instead of passing it as an argument, JobBatch should be a return type; this is to keep it from cyclic dependancy
	{
		OnUpdateThumbnailDelegate.Broadcast(CurrentBatch);
		UE_LOGF(LogIdle_Svc, Verbose, "ThumbnailsService, Batch fully queued: %llu. Triggering Observer::BatchJobsDone ...", CurrentBatch ? CurrentBatch->GetBatchId() : -1);

		TextureGraphEngine::GetInstance()->GetScheduler()->GetObserverSource()->BatchJobsDone(CurrentBatch);

		UE_LOGF(LogIdle_Svc, Verbose, "ThumbnailsService Observer::BatchJobsDone finished for Batch: %llu", CurrentBatch ? CurrentBatch->GetBatchId() : -1);
		
	})
 	.then([this, CurrentBatch]()
 	{
 		UE_LOGF(LogBatch, Verbose, "ThumbnailsService triggering Observer::BatchDone for Batch: %llu ...", CurrentBatch ? CurrentBatch->GetBatchId() : -1);

 		TextureGraphEngine::GetInstance()->GetScheduler()->GetObserverSource()->BatchDone(CurrentBatch); // notify observer

 		UE_LOGF(LogBatch, Verbose, "ThumbnailsService Observer::BatchDone finished for Batch: %llu", CurrentBatch ? CurrentBatch->GetBatchId() : -1);
			
 		return std::make_shared<JobResult>();
 	});
}

void ThumbnailsService::Stop()
{
	check(IsInGameThread());
	Batch = nullptr;
	Handled.clear();
}


JobBatchPtr ThumbnailsService::GetNextUpdateCycle()
{
	if (!Batch)
	{
		Handled.clear();
		return nullptr; // We dont want previous to be null
	}

	PrevBatch = Batch;	
	Batch = nullptr;

	Handled.clear();

	return PrevBatch;
}

JobBatchPtr ThumbnailsService::CreateNewUpdateCycle(UMixInterface* Mix)
{
	check(IsInGameThread());
	check(Mix);

	FInvalidationDetails Details(Mix);
	Details.All();

	JobBatchPtr NewBatch = JobBatch::Create(Details);

	SceneTargetUpdatePtr Target = std::make_shared<MixTargetUpdate>(Mix, 0);
	Target->InvalidateAllTiles();
	NewBatch->GetCycle()->AddTarget(Target);
	// NewBatch->SetCaptureRenderDoc(true);
	return NewBatch;
}

void ThumbnailsService::AddUniqueJobToCycle(UObject* CreatorComponent, UMixInterface* Mix, JobUPtr ThumbnailJob, int32 TargetId, bool bForceExec /* = false */)
{
	if (!Batch)
	{
		Batch = CreateNewUpdateCycle(Mix);
	}

	auto LastBatch = Batch; 
	auto Iter = Handled.end();

	if (!bForceExec)
	{
		Iter = Handled.find(CreatorComponent);
		if (Iter != Handled.end())
		{
			LastBatch = Iter->second.AssociatedBatch;
			LastBatch->RemoveJob(Iter->second.ThumbnailJob);
		}
	}

	JobPtrW AddedThumbnailJob = LastBatch->GetCycle()->AddJob(TargetId, std::move(ThumbnailJob));

	if (!bForceExec)
	{
		if (Iter != Handled.end())
			Iter->second.ThumbnailJob = AddedThumbnailJob;
		else
			Handled[CreatorComponent] = AddedJobs{ LastBatch, AddedThumbnailJob };
	}

}
