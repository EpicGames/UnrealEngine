// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IdleService.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <vector>
#include <unordered_map>

#define UE_API TEXTUREGRAPHENGINE_API

class UModelObject;
typedef std::shared_ptr<class Job>		JobPtr;
typedef std::unique_ptr<class Job>		JobUPtr;
typedef std::weak_ptr<class Job>		JobPtrW;

class UMixInterface;

class ThumbnailsService : public IdleService
{
protected:
	JobBatchPtr						Batch;					/// The thumbnail rendering batch
	JobBatchPtr						PrevBatch;				/// Last rendered batch
	MixUpdateCyclePtr				Cycle;					/// The cycle that we'll be using for the thumbnail updates

	struct AddedJobs
	{
		JobBatchPtr					AssociatedBatch;		/// Batch holding the following job
		JobPtrW						ThumbnailJob;			/// Thumbnail updating job assigned 
	};

	std::unordered_map<UObject*, AddedJobs> Handled;		/// The components that we have already added jobs for

	UE_API JobBatchPtr						CreateNewUpdateCycle(UMixInterface* Mix);
	UE_API JobBatchPtr						GetNextUpdateCycle();

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnThumbnailBatchDoneDelegate, JobBatchPtr Batch)
	FOnThumbnailBatchDoneDelegate	OnUpdateThumbnailDelegate;
									UE_API ThumbnailsService();
	UE_API virtual							~ThumbnailsService() override;

	UE_API virtual AsyncJobResultPtr		Tick() override;
	UE_API virtual void					Stop() override;
	UE_API void							AddUniqueJobToCycle(UObject* CreatorComponent, UMixInterface* Mix, JobUPtr ThumbnailJob, int32 TargetId, bool bForceExec = false);	//Use this to always generate thumbnail from last update cycle

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	JobBatchPtr						GetCurrentBatch() const { return Batch; }
};

typedef std::shared_ptr<ThumbnailsService>	ThumbnailsServicePtr;
typedef std::weak_ptr<ThumbnailsService>	ThumbnailsServicePtrW;

#undef UE_API
