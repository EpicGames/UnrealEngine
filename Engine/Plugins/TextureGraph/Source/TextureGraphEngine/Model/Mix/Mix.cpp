// Copyright Epic Games, Inc. All Rights Reserved.
#include "Mix.h"
#include "Helper/Util.h"
#include "Model/Mix/MixSettings.h"
#include "XmlFile.h"
#include "Profiling/StatGroup.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Mix)

UMix*				UMix::GNullMix = nullptr;
const FString		UMix::GRootSectionName = "Mix";

UMix* UMix::NullMix()
{
	check(IsInGameThread());

	if (GNullMix)
		return GNullMix;

	/// If there's no null mix, then we create a new one
	GNullMix = NewObject<UMix>(Util::GetModelPackage(), NAME_None, RF_NoFlags);
	GNullMix->GetSettings()->InitTargets(1);

	TargetTextureSetPtr Target = std::make_unique<TargetTextureSet>(
		0,
		TEXT("Null_T0"),
		nullptr,
		TextureSet::GDefaultTexSize,
		TextureSet::GDefaultTexSize
	);

	Target->Init();

	GNullMix->GetSettings()->SetTarget(0, Target);

	return GNullMix;
}

//////////////////////////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY(LogMix);
DECLARE_CYCLE_STAT(TEXT("Mix_Update"), STAT_Mix_Update, STATGROUP_TextureGraphEngine);

UMix::~UMix()
{
}

void UMix::Update(MixUpdateCyclePtr Cycle)
{
	Super::Update(Cycle);
}
