// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h" 
#include "MixInterface.h"

#include <memory>
#include <vector> 

#include "Mix.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

DECLARE_LOG_CATEGORY_EXTERN(LogMix, All, All);

class UMixParameters;

class RenderMesh;
typedef std::shared_ptr<RenderMesh>	RenderMeshPtr;

UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class UMix : public UMixInterface
{
	GENERATED_BODY()

public:
	

private:
	static UE_API UMix*						GNullMix;					/// This is a generic mix that is used in various places and rendering actions that
																	/// often require a Mix object. This is initialised in the first call to the public 
																	/// static function below. 
	static UE_API const FString				GRootSectionName;

public:
	static UE_API UMix*						NullMix();

public:
	UE_API virtual								~UMix() override;
	UE_API virtual void						Update(MixUpdateCyclePtr Cycle) override;
};

#undef UE_API
