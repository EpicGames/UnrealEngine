// Copyright Epic Games, Inc. All Rights Reserved.
#include "MixInterface.h"
#include "Model/Mix/MixUpdateCycle.h"
#include "MixSettings.h"
#include "MixManager.h"
#include <GameFramework/Actor.h>

#include UE_INLINE_GENERATED_CPP_BY_NAME(MixInterface)

const TMap<FString, FString> UMixInterface::s_uriAlias = UMixInterface::InitURIAlias();
const TMap<FString, FString> UMixInterface::s_uriAliasDecryptor = UMixInterface::InitURIAliasDecryptor();

TMap<FString, FString> UMixInterface::InitURIAlias()
{
	TMap<FString, FString> alias;

	alias.Add(TEXT("_layerStack"), TEXT("0"));
	alias.Add(TEXT("LayerSet"), TEXT("sets"));

	// for now, setting the same name here. 
	alias.Add(TEXT("Layer_Solid"), TEXT("L"));
	alias.Add(TEXT("Layer_Surface"), TEXT("L"));

	return alias;
}

TMap<FString, FString> UMixInterface::InitURIAliasDecryptor()
{
	TMap<FString, FString> aliasDecryptor;

	aliasDecryptor.Add(TEXT("0"), TEXT("_layerStack"));
	aliasDecryptor.Add(TEXT("sets"), TEXT("LayerSet"));

	return aliasDecryptor;
}

FString UMixInterface::GetObjectAlias(FString name)
{
	if (s_uriAlias.Contains(name))
	{
		return s_uriAlias[name];
	}

	return name;
}

void UMixInterface::Invalidate(FModelInvalidateInfo InvalidateInfo)
{
}

void UMixInterface::PostMeshLoad()
{
	FModelInvalidateInfo finfo;
	Invalidate(finfo);
}

FString UMixInterface::GetAliasToObject(FString Alias)
{
	if (s_uriAliasDecryptor.Contains(Alias))
	{
		return s_uriAliasDecryptor[Alias];
	}

	return Alias;
}

UMixInterface::~UMixInterface()
{
}

RenderMeshPtr UMixInterface::GetMesh() const
{
	return Settings->GetMesh();
}

bool UMixInterface::IsHigherPriorityThan(const UMixInterface* RHS) const
{
	return Priority > RHS->Priority;
} 

int32 UMixInterface::Width() const
{
	return GetSettings()->GetWidth();
}

int32 UMixInterface::Height() const
{
	return GetSettings()->GetHeight();
}

int32 UMixInterface::GetNumXTiles() const
{
	return GetSettings()->GetXTiles();
}

int32 UMixInterface::GetNumYTiles() const
{
	return GetSettings()->GetYTiles();
}

void UMixInterface::BroadcastOnRenderingDone(const FInvalidationDetails* Details)
{
	OnRenderDone.ExecuteIfBound(this, Details);
}

void UMixInterface::BroadcastOnBatchQueued(const FInvalidationDetails* Details)
{
	OnBatchQueued.ExecuteIfBound(this, Details);
}

UMixSettings* UMixInterface::GetSettings() const 
{ 
	check(Settings);
	return Settings; 
}

void UMixInterface::SetMesh(RenderMeshPtr MeshObj, int MeshType, FVector Scale, FVector2D Dimension)
{
	GetSettings()->SetMesh(MeshObj);
	InvalidateAll();
}

#if WITH_EDITOR
AsyncActionResultPtr UMixInterface::SetEditorMesh(AActor* Actor)
{
	return GetSettings()->SetEditorMesh(Actor)
		.then([this](ActionResultPtr Result) mutable
		{
			InvalidateAll();
			return Result;
		});
}

AsyncActionResultPtr UMixInterface::SetEditorMesh(UStaticMeshComponent* MeshComponent, UWorld* World)
{
	return GetSettings()->SetEditorMesh(MeshComponent, World)
		.then([this](ActionResultPtr Result) mutable
			{
				InvalidateAll();
				return Result;
			});
}
#endif

void UMixInterface::Update(MixUpdateCyclePtr cycle)
{
	UpdatedFrameId = TextureGraphEngine::GetFrameId();
}

void UMixInterface::InvalidateWithDetails(const FInvalidationDetails& InDetails)
{
	FInvalidationDetails Details = InDetails;
	Details.Mix = this;

	InvalidationFrameId = TextureGraphEngine::GetFrameId();

	TextureGraphEngine::GetMixManager()->InvalidateMix(this, Details);
}

void UMixInterface::InvalidateAll()
{
	FInvalidationDetails Details = FInvalidationDetails().All();
	Details.Mix = this;

	InvalidationFrameId = TextureGraphEngine::GetFrameId();

	TextureGraphEngine::GetMixManager()->InvalidateMix(this, Details);
}
