// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h" 
#include "Data/TiledBlob.h"
#include "Model/Mix/MixUpdateCycle.h"

#include "MixInterface.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

class UMix;
class ULayerStack;
class UMixSettings;
class AActor;

class RenderMesh;
typedef std::shared_ptr<RenderMesh>		RenderMeshPtr;

class MeshAsset;
typedef std::shared_ptr<MeshAsset>		MeshAssetPtr;

//////////////////////////////////////////////////////////////////////////
/// Mix can now be saved as Mix or MixInstance.
/// To support this functionality, we introduced the new class named MixInterface. 
/// This will be the base class for both Mix and MixInstance.
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI, Abstract)
class UMixInterface : public UModelObject
{
	GENERATED_BODY()

private:
	static UE_API const TMap<FString, FString> s_uriAlias;						/// Contains the list of URI aliases map to keep the data meaningful, short and precise.
	static UE_API const TMap<FString, FString> s_uriAliasDecryptor;			/// We need alias decryptor to convert it so we can extract the object from the data. 

	static UE_API TMap<FString, FString>		InitURIAlias();
	static UE_API TMap<FString, FString>		InitURIAliasDecryptor();

	UE_API FString								GetObjectAlias(FString object);
	UE_API FString								GetAliasToObject(FString Alias);

protected:
	UPROPERTY()
	int32								Priority;			/// The priority of this mix in terms of update cycle

	UPROPERTY()
	TObjectPtr<UMixSettings>			Settings = nullptr;/// The settings for this mix interface

	UPROPERTY()
	bool								bInvalidateTextures;/// Invalidate the scene textures or not

	std::atomic_int64_t					InvalidationFrameId;/// When was the mix last invalidated
	std::atomic_int64_t					UpdatedFrameId;		/// When was the mix actually last updated

	bool								bEnableLOD = true;	/// Enable LOD-ing in the system

	virtual UModelObject*				RootURI() { checkNoEntry() return nullptr; };
	UE_API virtual void						Invalidate(FModelInvalidateInfo InvalidateInfo);

public:
	UE_API virtual								~UMixInterface() override;



	DECLARE_DELEGATE_TwoParams(FOnBatchQueued, UMixInterface*, const FInvalidationDetails*);
	FOnBatchQueued						OnBatchQueued;

	DECLARE_DELEGATE_TwoParams(FOnRenderDone, UMixInterface*, const FInvalidationDetails*);
	FOnRenderDone						OnRenderDone;
	UE_API virtual void						PostMeshLoad();

	virtual bool						CanEdit() { return false; }

	UE_API virtual void						SetMesh(RenderMeshPtr MeshObj, int MeshType, FVector Scale, FVector2D Dimension);

#if WITH_EDITOR
	UE_API virtual AsyncActionResultPtr		SetEditorMesh(AActor* Actor);
	UE_API virtual AsyncActionResultPtr		SetEditorMesh(UStaticMeshComponent* MeshComponent, UWorld* World);
#endif
	
	UE_API virtual void						Update(MixUpdateCyclePtr cycle);	

	UE_API virtual RenderMeshPtr				GetMesh() const; // Get the mesh assigned in Settings

	UE_API virtual int32						Width() const; 
	UE_API virtual int32						Height() const;
	
	UE_API virtual int32						GetNumXTiles() const;
	UE_API virtual int32						GetNumYTiles()const;
	
	UE_API virtual UMixSettings*				GetSettings() const;

	UE_API bool								IsHigherPriorityThan(const UMixInterface* RHS) const;

	UE_API virtual void						InvalidateWithDetails(const FInvalidationDetails& Details);
	UE_API virtual void						InvalidateAll();

	UE_API virtual void						BroadcastOnRenderingDone(const FInvalidationDetails* Details);
	UE_API virtual void						BroadcastOnBatchQueued(const FInvalidationDetails* Details);
	
	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE int32					GetPriority() const { return Priority; }

	FORCEINLINE int64					GetInvalidationFrameId() const { return InvalidationFrameId; }
	FORCEINLINE int64					GetUpdateFrameId() const { return UpdatedFrameId; }
};


#undef UE_API
