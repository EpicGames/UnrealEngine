// Copyright Epic Games, Inc. All Rights Reserved.
#include "MixManager.h"
#include "MixInterface.h"
#include "MixSettings.h"
#include "Job/JobBatch.h"
#include "TextureGraphEngine.h"
#include "Job/Scheduler.h"

DEFINE_LOG_CATEGORY(LogMixManager);


//////////////////////////////////////////////////////////////////////////
bool MixInterface_ComparePriority::operator()(const MixInvalidateInfo& LHS, const MixInvalidateInfo& RHS)
{
	if (!LHS.MixObj || !RHS.MixObj)
		return false;

	/// This is the most immediate priority
	if (LHS.Priority != RHS.Priority)
		return LHS.Priority > RHS.Priority;

	/// Fallback to mix priority if immediate priority isn't satisfied
	return !LHS.MixObj->IsHigherPriorityThan(RHS.MixObj);
}

MixManager::MixManager()
{
}

MixManager::~MixManager()
{
	/// Intentionally empty. Exit() drains the queue while engine subsystems are still alive.
}

void MixManager::FlushMix(UMixInterface* MixObj)
{
	check(IsInGameThread());
	MixesToFlush.push_back(MixObj);
}

void MixManager::Update(float Delta)
{
	/// Mix manager has been suspended for the time being. 
	/// Do not update any mixes
	if (bIsSuspended)
		return;

	std::vector<MixInvalidateInfo> MixInfos = Queue.to_vector_and_clear();

	if (!MixInfos.size())
	{
		// if no invalidations queued, we can clear out MixesToFlush as well,
		// this avoids a bug where the next time this same mix is opened it is immediately flushed and doesn't trigger an update
		MixesToFlush.clear();
		return;
	}

	if (!MixesToFlush.empty())
	{
		for (UMixInterface* MixToFlush : MixesToFlush)
		{
			for (auto Iter = MixInfos.begin(); Iter != MixInfos.end();)
			{
				MixInvalidateInfo& MixInfo = *Iter;
				UMixInterface* MixObj = MixInfo.MixObj;

				if (MixObj == MixToFlush)
					Iter = MixInfos.erase(Iter);
				else
					Iter++;
			}
		}

		MixesToFlush.clear();
	}

	std::vector<MixInvalidateInfo> BucketedMixInfos;
	
	// First merge invalidation touching on the same mix
	// High priority mixInfo are last in list; this is by design of the queue
	// We are reversing that order as we produce the bucketed list
	for (auto Iter = MixInfos.rbegin(); Iter != MixInfos.rend(); ++Iter)
	{
		MixInvalidateInfo& MixInfo = *Iter;

		if (!MixInfo.MixObj->IsValidLowLevel())
			continue;

		UMixInterface* MixObj = MixInfo.MixObj;
		auto RIter = std::find_if(BucketedMixInfos.begin(), BucketedMixInfos.end(),
			[MixObj](const MixInvalidateInfo& Info)
			{
				return MixObj == Info.MixObj;
			});
		if (RIter == BucketedMixInfos.end()) // New mix encoutered
		{
			BucketedMixInfos.emplace_back(MixInfo); // Push the new mixInfo at the index
		}
		else
		{
			int32 Index = RIter - BucketedMixInfos.begin();

			// Merge the new mixInfo invalidation over the one already present
			BucketedMixInfos[Index].InvalidationDetails.Merge(MixInfo.InvalidationDetails);

			// The first priority bucketed should be the highest, no need to worry about priority anymore
		}
	}

	// Second, execute the bucketedMixInfos commands
	for (auto& MixInfo : BucketedMixInfos)
	{
		UE_LOGF(LogMixManager, Log, "Rendering mix: %ls [Invalidation FrameId: %llu, Update FrameId: %llu, Discard: %ls", 
			*MixInfo.MixObj->GetName(), MixInfo.MixObj->GetInvalidationFrameId(), MixInfo.MixObj->GetUpdateFrameId(),
			(MixInfo.InvalidationDetails.IsDiscard() ? TEXT("Yes") : TEXT("No")));

		UMixSettings* MixSettings = MixInfo.MixObj->GetSettings();
		FInvalidationDetails& Details = MixInfo.InvalidationDetails;

		if (!Details.Mix.IsValid())
			Details.Mix = MixInfo.MixObj;

		JobBatchPtr Batch = JobBatch::Create(Details);
		MixUpdateCyclePtr Cycle = Batch->GetCycle();

		// Clear the existing errors for the mix before updating.
		// Update will generate new errors.
		TextureGraphEngine::GetErrorReporter(MixInfo.MixObj)->Clear();
		MixInfo.MixObj->Update(Cycle);

		TextureGraphEngine::GetScheduler()->AddBatch(Batch);
	}
	
}

void MixManager::Suspend()
{
	bIsSuspended = true;
}

void MixManager::Resume()
{
	bIsSuspended = false;
}

void MixManager::Exit()
{
	UE_LOGF(LogMixManager, Log, "Begin MixManager::Suspend");

	FRenderCommandFence SuspendFence;

	SuspendFence.BeginFence();
	SuspendFence.Wait();

	Suspend();

	/// Drain any pending invalidations now, while engine subsystems are
	/// still alive. After Suspend(), bIsSuspended is true so InvalidateMix
	/// early-returns and no new items can be queued between here and
	/// ~MixManager.
	Queue.clear();
	MixesToFlush.clear();

	UE_LOGF(LogMixManager, Log, "End MixManager::Suspend");
}

void MixManager::InvalidateMix(UMixInterface* MixObj, const FInvalidationDetails &Details, int32 Priority /* = (int32)E_Priority::kNormal */)
{
	check(IsInGameThread());

	/// Mix manager has been suspended for the time being. 
	/// We dont want to add it in invalidation queue.
	if(bIsSuspended)
		return;

	Queue.add(MixInvalidateInfo
		{
			MixObj,
			Details,
			Priority
		});
}
