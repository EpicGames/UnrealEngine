// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Device/TaskPriorityQueue.h"
#include "Helper/Util.h"
#include "Model/ModelObject.h"

#define UE_API TEXTUREGRAPHENGINE_API

class UMixInterface;
class UActionManager;

DECLARE_LOG_CATEGORY_EXTERN(LogMixManager, All, All);

struct MixInvalidateInfo
{
	UMixInterface					*MixObj;							/// The mix that needs to be updated
	FInvalidationDetails			InvalidationDetails;			/// Invalidation details
	int32							Priority = (int32)E_Priority::kNormal; /// The pririty of this invalidation

	/// Global comparison between a MixInvalidateInfo internal mix and an external mix
	FORCEINLINE friend bool			operator==(MixInvalidateInfo const& LHS, const UMixInterface* RHS) { return LHS.MixObj == RHS; }
};

//////////////////////////////////////////////////////////////////////////
struct MixInterface_ComparePriority
{
	bool operator()(const MixInvalidateInfo& LHS, const MixInvalidateInfo& RHS);
};

typedef GenericTaskPriorityQueue<MixInvalidateInfo, MixInterface_ComparePriority> MixPriorityQueue;

//////////////////////////////////////////////////////////////////////////
/// MixManager - Manages mixes with priority updates. This just 
/// makes sure mix update is called and then that updates the 
/// data in the scheduler. This is here to ensure that mixes are updated
/// in the correct priority with actively visible mixes getting top priority
/// compared with background mixes, that might still be updating. 
//////////////////////////////////////////////////////////////////////////
class MixManager
{
private:
	std::atomic_bool				bIsSuspended = false;			/// Whether the mix manager is suspended or not
	MixPriorityQueue				Queue;							/// Mix queue
	std::vector<UMixInterface*>		MixesToFlush;						/// The mixes to flush
	
public:
									UE_API MixManager();
	UE_API virtual							~MixManager();

	UE_API void							Update(float Delta);
	UE_API void							Suspend();
	UE_API void							Resume();
	UE_API void							Exit();

	UE_API void							InvalidateMix(UMixInterface* MixObj, const FInvalidationDetails &Details, int32 Priority = (int32)E_Priority::kNormal);
	UE_API void							FlushMix(UMixInterface* MixObj);
	
	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
};

typedef std::unique_ptr<MixManager>		MixManagerUPtr;

#undef UE_API
