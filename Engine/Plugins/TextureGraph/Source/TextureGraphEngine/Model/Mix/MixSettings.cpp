// Copyright Epic Games, Inc. All Rights Reserved.
#include "MixSettings.h"

#include "ViewportSettings.h"
#include "3D/RenderMesh_Editor.h"
#include "Model/Mix/MixInterface.h"
#include "Helper/Util.h"
#include "FxMat/MaterialManager.h"
#include "Job/Scheduler.h"
#include "2D/Tex.h"
#include "3D/MeshInfo.h"
#include "3D/RenderMesh_EditorScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MixSettings)

UMixSettings::~UMixSettings()
{
	FreeTargets();
}

void UMixSettings::FreeTargets()
{
	if (_targets)
	{
		_targets->clear();
		delete _targets;
		_targets = nullptr;
	}
}

void UMixSettings::Free()
{
	FreeTargets();
}

void UMixSettings::InitTargets(size_t count)
{
	verify(_targets == nullptr);

	_targets = new TargetTextureSetPtrVec(count);
}

void UMixSettings::SetTarget(size_t index, TargetTextureSetPtr& target)
{
	verify(index < (*_targets).size());
	(*_targets)[index] = std::move(target);
}

#if WITH_EDITOR
AsyncActionResultPtr UMixSettings::SetEditorMesh(AActor* actor)
{
	return cti::make_continuable<ActionResultPtr>([this, actor](auto&& promise) mutable
		{
			RenderMesh_EditorScenePtr editorMesh(new RenderMesh_EditorScene(actor));

			editorMesh->Load()
				.then([this, FWD_PROMISE(promise), editorMesh](ActionResultPtr fbxResult) mutable
			{
			Util::OnGameThread([this, FWD_PROMISE(promise), editorMesh]() mutable
				{
					SetMeshInternal<RenderMesh_EditorScene>(editorMesh, (int)MeshType::Editor, FVector::OneVector, FVector2D::UnitVector);
					promise.set_value(std::make_shared<ActionResult>(nullptr));
				});
			});
		});

}

AsyncActionResultPtr UMixSettings::SetEditorMesh(UStaticMeshComponent* meshComponent, UWorld* world)
{
	return cti::make_continuable<ActionResultPtr>([this, meshComponent, world](auto&& promise) mutable
		{
			RenderMesh_EditorPtr editorMesh(new RenderMesh_Editor(meshComponent, world));

			editorMesh->Load()
				.then([this, FWD_PROMISE(promise), editorMesh](ActionResultPtr fbxResult) mutable
			{
				Util::OnGameThread([this, FWD_PROMISE(promise), editorMesh]() mutable
				{
					SetMeshInternal<RenderMesh_Editor>(editorMesh, (int)MeshType::Editor, FVector::OneVector, FVector2D::UnitVector);
					promise.set_value(std::make_shared<ActionResult>(nullptr));
				});
			});
		});

}
#endif

RenderMeshPtr UMixSettings::GetMesh()
{
	//this is never supposed to be null
	return _mesh;
}

void UMixSettings::SetMesh(RenderMeshPtr mesh)
{
	_mesh = mesh;
}

#if WITH_EDITOR
void UMixSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName Name = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	const FName MemberPropertyName = (PropertyChangedEvent.MemberProperty != NULL) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	if (!MemberPropertyName.IsNone())
	{
		if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMixSettings, XTiles) && !bAllowRectangularResolution && XTiles != YTiles)
		{
			YTiles = XTiles;
		}
		else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMixSettings, YTiles) && !bAllowRectangularResolution && XTiles != YTiles)
		{
			XTiles = YTiles;
		}
		else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMixSettings, Width) && !bAllowRectangularResolution && Width != Height)
		{
			Height = Width;
		}
		else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMixSettings, Height) && !bAllowRectangularResolution && Width != Height)
		{
			Width = Height;
		}
	}

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMixSettings, ViewportSettings))
	{
		if(Name == GET_MEMBER_NAME_CHECKED(FViewportSettings, Material))
		{
			ViewportSettings.OnMaterialUpdate();	
		}
	}
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMixSettings, PreviewMesh))
	{
		OnPreviewMeshChangedEvent.Broadcast();
	}
	
}

void UMixSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	UObject::PostEditChangeChainProperty(PropertyChangedEvent);

	FProperty* Property = PropertyChangedEvent.Property;
	FProperty* MemberProperty = nullptr;

	if (PropertyChangedEvent.PropertyChain.GetActiveMemberNode())
	{
		MemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
	}

	if (MemberProperty && Property)
	{
		if (MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMixSettings, ViewportSettings))
		{
			FName Name = Property->GetFName();
			
			if(Name == "Target")
			{
				ViewportSettings.OnMaterialMappingChangedEvent.Broadcast();
			}
		}
	}
}

#endif

UMixInterface* UMixSettings::Mix() const
{	
	return GetOuter() ? dynamic_cast<UMixInterface*>(GetOuter()) : nullptr;
}

FViewportSettings& UMixSettings::GetViewportSettings()
{
	return ViewportSettings;
}

template<typename RenderMeshClass>
void UMixSettings::SetMeshInternal(RenderMeshPtr mesh, int meshType, FVector scale, FVector2D dimension)
{
	_mesh = mesh;

	UWorld* world = Util::GetGameWorld();

	bIsPlane = meshType == (int)MeshType::Plane;

	if (IsPlane())
	{
		PlaneDimensions = dimension;
	}

	for (int i = 0; i < _sceneMeshes.Num(); i++)
	{
		_sceneMeshes[i]->Clear();
	}

	/// Clear out the meshes
	_sceneMeshes.Empty();

	/// Clear the scene targets
	FreeTargets();

	TArray<MeshInfoPtr>& subMeshes = _mesh->Meshes();

	if (_mesh->GetMeshSplitType() == MeshSplitType::Single)
	{
		_sceneMeshes.Add(_mesh);
	}
	else
	{
		TMap<int, TArray<int>> materialsForTextureSets;

		for (int meshIndex = 0; meshIndex < _mesh->Meshes().Num(); meshIndex++)
		{
			MeshInfoPtr meshInfo = _mesh->Meshes()[meshIndex];

			if (!materialsForTextureSets.Contains(meshInfo->GetMaterialIndex()))
			{
				materialsForTextureSets.Add(meshInfo->GetMaterialIndex(), TArray<int>());
			}

			materialsForTextureSets[meshInfo->GetMaterialIndex()].Add(meshIndex);
		}

		for (auto pair : materialsForTextureSets)
		{
			UE_LOGF(LogMesh, Log, "Mesh Key : %d : Count : %d", pair.Key, pair.Value.Num());
		}

		TArray<int> textureSets;
		materialsForTextureSets.GetKeys(textureSets); // Get Texture sets from json

		_sceneMeshes.SetNum(textureSets.Num());

		int textureSetIndex = 0;

		for (int materialIndex : textureSets)
		{
			TArray<int> meshesForTextureSet = materialsForTextureSets[materialIndex]; // meshes For texture sets
																					  //RenderMeshClassPtr customMesh = std::make_shared<RenderMesh_Custom>();

			TArray<MeshInfoPtr> meshInfos;

			MaterialInfoPtr matInfo = _mesh->CurrentMaterials()[materialIndex];

			for (int meshIndex = 0; meshIndex < _mesh->Meshes().Num(); meshIndex++)
			{
				auto subMesh = _mesh->Meshes()[meshIndex];

				if (subMesh->GetMaterialIndex() == materialIndex)
				{
					meshInfos.Add(subMesh);
				}
			}

			std::shared_ptr<RenderMeshClass> sceneMesh = std::make_shared<RenderMeshClass>(_mesh.get(), meshInfos, matInfo);
			_sceneMeshes[materialIndex] = sceneMesh;
		}
	}

	InitTargets(_sceneMeshes.Num());

	/// Now we add these to the scene
	for (size_t smi = 0; smi < _sceneMeshes.Num(); smi++)
	{
		_sceneMeshes[smi]->PrepareForRendering(world, scale);

		TargetTextureSetPtr target = std::make_unique<TargetTextureSet>
			(
				(int32)smi,
				_sceneMeshes[smi]->GetMaterialName(),
				_sceneMeshes[smi],
				Util::GDefaultWidth,
				Util::GDefaultHeight
			);

		target->Init();

		SetTarget(smi, target);
	}
}
