// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h" 
#include "ViewportSettings.h"
#include "2D/TargetTextureSet.h"
#include "2D/TextureHelper.h"
#include "Helper/Promise.h"
#include "MixSettings.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

UENUM()
enum class PBRWorkflowMode
{
	Specular = 0,
	Metalness = 1,
	Both = 2
};

UENUM()
enum class ETileSize 
{
	Tile1 = 0	UMETA(DisplayName = "1"),
	Tile2 = 2	UMETA(DisplayName = "2"),
	Tile4 = 4	UMETA(DisplayName = "4"),
	Tile8 = 8	UMETA(DisplayName = "8"),
	Tile16 = 16	UMETA(DisplayName = "16")
};

UENUM()
enum class EResolution
{
	Auto		  = 0		UMETA(DisplayName = "Auto"),
	Resolution8   = 8		UMETA(DisplayName = "8"),
	Resolution16  = 16		UMETA(DisplayName = "16"),
	Resolution32  = 32		UMETA(DisplayName = "32"),
	Resolution64  = 64		UMETA(DisplayName = "64"),
	Resolution128 = 128		UMETA(DisplayName = "128"),
	Resolution256 = 256		UMETA(DisplayName = "256"),
	Resolution512 = 512		UMETA(DisplayName = "512"),
	Resolution1024 = 1024	UMETA(DisplayName = "1024"),
	Resolution2048 = 2048	UMETA(DisplayName = "2048"),
	Resolution4096 = 4096	UMETA(DisplayName = "4096"),
	Resolution8192 = 8192	UMETA(DisplayName = "8192")
};

UENUM()
enum class UE_DEPRECATED(5.3, "You should use ETG_TextureFormat instead") ETSBufferFormat
{
	Auto = -1 				UMETA(DisplayName = "Auto"), // Auto buffer format is deduced automatically based on other textures within the graph 
	Byte = 0  				UMETA(DisplayName = "Byte"), // One byte of data per channel 
	Half = 1  				UMETA(DisplayName = "Half"), // Two bytes (half float) of data per channel
	Float = 2 				UMETA(DisplayName = "Float") // Four bytes (full float) of data per channel
};

UENUM()
enum class UE_DEPRECATED(5.3, "You should use ETG_TextureFormat instead") ETSBufferChannels 
{
	Auto = 0 				UMETA(DisplayName = "Auto"), // Auto number of channels are deduced automatically based on other channels within the graph
	One = 1  				UMETA(DisplayName = "R"),	 // One channel per pixel (e.g. Red only)
	Two = 2  				UMETA(DisplayName = "RG"),	 // Two channels per pixel (e.g. RG = Red and Green only)
	Three = 3 				UMETA(DisplayName = "RGB"),	 // Three channels per pixel (e.g. RGB)
	Four = 4 				UMETA(DisplayName = "RGBA")	 // Four channels per pixel (e.g. RGBA)
};

UENUM(BlueprintType)
enum class ETG_TextureFormat : uint8
{
	Auto		UMETA(DisplayName = "Auto"), // Auto number of channels are deduced automatically based on other channels within the graph /*UMETA(Hidden) = -1,*/
	G8			UMETA(DisplayName = "8-bit Grayscale"),
	BGRA8		UMETA(DisplayName = "8-bit per-pixel RGBA"),
	R16F		UMETA(DisplayName = "16-bit (half) Single Channel (Red)"),
	RGBA16F		UMETA(DisplayName = "16-bit (half) per-pixel RGBA"),
	R32F		UMETA(DisplayName = "32-bit (float) Single Channel (Red)"),
	RGBA32F		UMETA(DisplayName = "32-bit (float) per-pixel RGBA")
};

UENUM(BlueprintType)
enum class ETG_TexturePresetType : uint8
{
	None		UMETA(DisplayName = "None"), // None exposes the other settings like srgb, compression and Lod Texture group */
	Diffuse		UMETA(DisplayName = "Diffuse"),
	Emissive	UMETA(DisplayName = "Emissive"),
	FX			UMETA(DisplayName = "FX"),
	Normal		UMETA(DisplayName = "Normal"),
	MaskComp	UMETA(DisplayName = "Mask Comp"),
	Specular	UMETA(DisplayName = "Specular"),
	Tangent		UMETA(DisplayName = "Tangent")
};

class UMixInterface;

class RenderMesh;
typedef std::shared_ptr<RenderMesh>			RenderMeshPtr;

class RenderMesh_Editor;
typedef std::shared_ptr<RenderMesh_Editor>	RenderMesh_EditorPtr;

class RenderMaterial_BP;
typedef std::shared_ptr<RenderMaterial_BP>	RenderMaterial_BPPtr;

class UMixInterface;
class UStaticMesh;
//////////////////////////////////////////////////////////////////////////
/// MixSettings: Collectively represents the settings of a particular
/// mix. These can be specified on a per mix basis. However, they can 
/// be over-ridden on a per mix instance basis as well
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class UMixSettings : public UObject
{
	GENERATED_BODY()

	friend class UMixInterface;

private:


	UPROPERTY()
	bool							bIsPlane;									/// Is this a plane scene

	UPROPERTY()
	FVector2D						PlaneDimensions;							/// Default dimensions for plane in the scene
	
	UPROPERTY()
	float							StandardHeight = 0.1f;						/// 10 cm for 0-1 range displacements
	
	UPROPERTY()
	float							MidPoint = 0.5f;							/// Midpoint for displacement. Default is gray.
	
	UPROPERTY()
	PBRWorkflowMode					_workflow = PBRWorkflowMode::Metalness;		/// The workflow for the scene right now

	UPROPERTY()
	EResolution						Width = EResolution::Resolution2048;		// Default width of the target(s). Auto will take the width from Input texture(s)
	
	UPROPERTY()
	EResolution						Height = EResolution::Resolution2048;		// Default height of the target(s). Auto will take the height from Input texture(s)

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Texture Format instead"))
	ETSBufferChannels				Channels_DEPRECATED = ETSBufferChannels::Auto;			// How many channels per pixel.  Auto will take the number of channels from Input texture(s)

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Texture Format instead"))
	ETSBufferFormat					Format_DEPRECATED = ETSBufferFormat::Auto;				// Per channel size/format.  Auto will take the format from Input texture(s)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	UPROPERTY()
	ETG_TextureFormat				TextureFormat = ETG_TextureFormat::Auto;

	UPROPERTY()
	ETileSize						XTiles = ETileSize::Tile8;					// How many horizontal tiles to split the inputs into for the tiled processing pipeline
	
	UPROPERTY()
	ETileSize						YTiles = ETileSize::Tile8;					// How many vertical tiles to split the inputs into for the tiled processing pipeline
	
	UPROPERTY()
	bool							bAllowRectangularResolution = false;		// Allow rectangular outputs

	UPROPERTY(EditAnywhere, Category = "Viewport Material", meta = (NoResetToDefault, ShowOnlyInnerProperties))
	FViewportSettings				ViewportSettings;							// Viewport settings
	
	UPROPERTY(EditAnywhere, Category = "Viewport Mesh", DisplayName = "Static Mesh", NoClear)
	TObjectPtr<UStaticMesh>			PreviewMesh;

private:
	/// Mesh related data 
	RenderMeshPtr					_mesh;										/// The full mesh (along with all its parts) that was loaded 
	TArray<RenderMeshPtr>			_sceneMeshes;								/// All the different submeshes that were added to the scene
	RenderMaterial_BPPtr			_currentMat = nullptr;						/// What is the active material for rendering the mesh

	/// Mesh's target texture sets
	TargetTextureSetPtrVec*			_targets = nullptr;							/// The target textures sets

	template<typename RenderMeshClass>
	void							SetMeshInternal(RenderMeshPtr mesh, int meshType, FVector scale, FVector2D dimension);
	UE_API void							SetMesh(RenderMeshPtr mesh);
	
protected:

#if WITH_EDITOR
	UE_API virtual void					PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void					PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override; // For TArray type properties in the component.
#endif

public:
	UE_API virtual							~UMixSettings() override;

	UE_API void							FreeTargets();
	UE_API void							InitTargets(size_t count);
	UE_API void							SetTarget(size_t index, TargetTextureSetPtr& target);
	UE_API virtual void					Free();
	
#if WITH_EDITOR
	UE_API AsyncActionResultPtr			SetEditorMesh(AActor* actor);
	UE_API AsyncActionResultPtr			SetEditorMesh(UStaticMeshComponent* meshComponent,UWorld* world);
#endif

	UE_API RenderMeshPtr					GetMesh();
	UE_API UMixInterface*					Mix() const;
	
	UE_API FViewportSettings&				GetViewportSettings();

	DECLARE_MULTICAST_DELEGATE(FPreviewMeshUpdateEvent)
	FPreviewMeshUpdateEvent OnPreviewMeshChangedEvent;
	
	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	[[nodiscard]] TObjectPtr<UStaticMesh> GetPreviewMesh() const{	return PreviewMesh; }
	FORCEINLINE void				SetPreviewMesh(const TObjectPtr<UStaticMesh>& InPreviewMesh)	{	this->PreviewMesh = InPreviewMesh; }
	FORCEINLINE bool				IsPlane() const { return bIsPlane; }
	FORCEINLINE FVector2D			GetPlaneDimensions() const { return PlaneDimensions; }
	FORCEINLINE const TargetTextureSetPtr& Target(size_t index) const { verify(index < _targets->size()); return (*_targets)[index]; }
	FORCEINLINE TargetTextureSetPtr& Target(size_t index) { verify(index < _targets->size()); return (*_targets)[index]; }
	FORCEINLINE size_t				NumTargets() const { return _targets ? static_cast<int32>(_targets->size()) : 0; }

	FORCEINLINE PBRWorkflowMode		Workflow() const { return _workflow; }
	FORCEINLINE PBRWorkflowMode&	Workflow() { return _workflow; }
	FORCEINLINE void				SetWidth(EResolution InWidth) { Width = InWidth; }
	FORCEINLINE void				SetHeight(EResolution InHeight) { Height = InHeight; }
	FORCEINLINE void				SetTextureFormat(ETG_TextureFormat InFormat) { TextureFormat = InFormat; }
	FORCEINLINE float				GetMidPoint() const { return MidPoint; }
	FORCEINLINE float				GetStandardHeight() const { return StandardHeight; }
	FORCEINLINE int32				GetWidth()  const { return static_cast<int32>(Width); } // 2^7 is 128 and Width starts with 1. So 256, 1k, 2k, 4k and 8k
	FORCEINLINE int32				GetHeight() const { return static_cast<int32>(Height); }
	FORCEINLINE int32				GetXTiles() const { return static_cast<int32>(XTiles); }
	FORCEINLINE int32				GetYTiles() const { return static_cast<int32>(YTiles); }
	FORCEINLINE int32				GetXTileSize() const { return GetWidth() / GetXTiles(); }
	FORCEINLINE int32				GetYTileSize() const { return GetHeight() / GetYTiles(); }
	FORCEINLINE ETG_TextureFormat	GetTextureFormat() const { return TextureFormat; }
	
	BufferDescriptor	GetDescriptor() const
	{
		uint32 NumChannels = 0;
		BufferFormat Format = BufferFormat::Auto;
		TextureHelper::GetBufferFormatAndChannelsFromTGTextureFormat(TextureFormat, Format, NumChannels);
		return BufferDescriptor(
			GetWidth(),
			GetHeight(),
			NumChannels,
			Format,
			FLinearColor::Black
		);
	}
};

#undef UE_API
