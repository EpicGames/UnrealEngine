// Copyright Epic Games, Inc. All Rights Reserved.
#include "MixUpdateCycle.h"
#include "Model/Mix/Mix.h"
#include "TextureGraphEngine.h"
#include "Job/Job.h"
#include "Job/JobBatch.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"

MixTargetUpdate::MixTargetUpdate(TWeakObjectPtr<UMixInterface> InMixObj, int32 InTargetId) 
	: MixObj(InMixObj)
	, InvalidationMatrix(InMixObj->GetNumXTiles(), InMixObj->GetNumYTiles())
	, TargetId(InTargetId)
{
	check(TargetId >= 0);
}

MixTargetUpdate::MixTargetUpdate(TWeakObjectPtr<UMixInterface> InMixObj, const TileInvalidateMatrix& InInvalidationMatrix, int32 InTargetId)
	: MixTargetUpdate(InMixObj, InTargetId)
{
	InvalidationMatrix = InInvalidationMatrix;
}

MixTargetUpdate::~MixTargetUpdate()
{
}

void MixTargetUpdate::InvalidateTile(int32 Row, int32 Col)
{
	check(CheckIsValid(Row, Col));
	InvalidationMatrix[Row][Col] = true;
}

void MixTargetUpdate::InvalidateAllTiles()
{
	for (size_t Index = 0; Index < InvalidationMatrix.Length(); Index++)
		InvalidationMatrix.Tiles()[Index] = 1;
}

void MixTargetUpdate::InvalidateNoTiles()
{
	for (size_t Index = 0; Index < InvalidationMatrix.Length(); Index++)
		InvalidationMatrix.Tiles()[Index] = 0;
}

//////////////////////////////////////////////////////////////////////////
MixUpdateCycle::MixUpdateCycle(const FInvalidationDetails& InDetails) 
	: MixObj(InDetails.Mix)
	, Details(InDetails)
{
	check(MixObj.Get());

	size_t NumTargets = MixObj->GetSettings()->NumTargets();
	Targets.resize(NumTargets);
}

MixUpdateCycle::~MixUpdateCycle()
{
}

void MixUpdateCycle::MergeDetails(const FInvalidationDetails& InDetails)
{
	Details.Merge(InDetails);
}

void MixUpdateCycle::SetBatch(std::shared_ptr<JobBatch> InBatch)
{
	check(!Batch && InBatch);
	Batch = InBatch;
}

bool MixUpdateCycle::NoCache() const 
{
	return Details.IsDiscard() || Batch->IsNoCache();
}

void MixUpdateCycle::Begin()
{
}

void MixUpdateCycle::End()
{
	check(IsInGameThread());

	if (Batch)
	{
		UE_LOGF(LogBatch, Log, "ENDING scene cycle for batch: %llu", Batch->GetBatchId());
		Batch = nullptr;
	}
}

void MixUpdateCycle::PushMix(UMixInterface* mix)
{
	ActiveMixStack.Push(mix);
}

void MixUpdateCycle::PopMix()
{
	ActiveMixStack.Pop();
}

UMixInterface* MixUpdateCycle::TopMix()
{
	return ActiveMixStack.Top();
}

bool MixUpdateCycle::ContainsMix(UMixInterface* InMixObj) const
{
	return ActiveMixStack.Contains(InMixObj);
}

JobPtrW MixUpdateCycle::LastAddedJob(int32 InTargetId) const
{
	check(InTargetId >= 0 && InTargetId < (int32)Targets.size());
	return Targets[InTargetId]->GetLastAddedJob();
}

JobPtrW MixUpdateCycle::AddJob(int32 InTargetId, JobUPtr JobObj)
{
	check(InTargetId >= 0 && InTargetId < (int32)Targets.size());

	JobPtrW JobW = Batch->AddJob(std::move(JobObj));

	JobRunInfo RunInfo;
	RunInfo.JobScheduler = TextureGraphEngine::GetScheduler();
	RunInfo.Batch = Batch.get();
	RunInfo.Cycle = Batch->GetCycle();
	RunInfo.ThisJob = JobW;

	JobPtr JobS = JobW.lock();

	/// If the job is fully culled, then we don't need to do anything
	if (JobS->CheckCulled(RunInfo))
	{
		UE_LOGF(LogBatch, VeryVerbose, "MixUpdateCycle::AddJob  Job [%llu]: IsCulled", Batch->GetBatchId());
	}

	SceneTargetUpdatePtr Target = Targets[InTargetId];

	if (JobS->GetResult())
	{
		JobS->GetResult()->Job() = JobW;
		if (Details.IsDiscard())
			JobS->GetResult()->SetTransient();
	}

	/// If the job is culled, then we remove it
	if (JobS->IsCulled())
		Batch->RemoveJob(JobW);

	return JobW;
}

void MixUpdateCycle::AddTarget(SceneTargetUpdatePtr Target)
{
	check(Target);

	int32 TargetId = Target->GetTargetId();
	check((size_t)TargetId < Targets.size());

	check(!Targets[TargetId]);
	Targets[TargetId] = Target;
}

uint32 MixUpdateCycle::LODLevel() const
{
	return 0;
	/// If we're not discarding the current update cycle, then we keep it at highes LOD level
	//if (!_details.IsDiscard())
	//	return 0;

	//return 4;

#if 0 /// TODO: Make this better, but hardcoded for the time being
	static const uint32 maxLODSize = 1024;

	uint32 mixSize = (uint32)std::max(_mix->Width(), _mix->Height());
	uint32 lod = 0;

	if (mixSize > maxLODSize)
		lod = mixSize / maxLODSize

	return lod; 
#endif /// 0
}

void MixUpdateCycle::AddReferencedObjects(FReferenceCollector& collector)
{
}

FString MixUpdateCycle::GetReferencerName() const
{
	return TEXT("MixUpdateCycle");
}

