// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "2D/TextureSet.h"
#include <vector>
#include "Job/JobCommon.h"
#include "Model/ModelObject.h"
#include "UObject/GCObject.h"

#define UE_API TEXTUREGRAPHENGINE_API

class JobBatch;

class Job;
typedef std::shared_ptr<Job>		JobPtr;
typedef std::unique_ptr<Job>		JobUPtr;
typedef std::weak_ptr<Job>			JobPtrW;

class UMixInterface;

//////////////////////////////////////////////////////////////////////////
/// This class doesn't need to create or manage textures. 
/// its here to refer to textures created by other layers. 
//////////////////////////////////////////////////////////////////////////
class MixTextureSet : public TextureSet
{
public:
	MixTextureSet() : TextureSet(1, 1) {}
};

typedef std::vector<MixTextureSet>	SceneTextureSetVec;

//////////////////////////////////////////////////////////////////////////
class MixTargetUpdate
{
private:
	TWeakObjectPtr<UMixInterface>	MixObj;							/// What is the mix that we're dealing with
	TileInvalidateMatrix			InvalidationMatrix;				/// Mix invalidation matrix
	int32							TargetId = -1;					/// What is the target oo this update cycle
	MixTextureSet					LastRender;						/// The last renders in this target. This is just a dummy placeholder 

	JobPtrW							LastAddedJob;					/// Last job that was added

public:
									UE_API MixTargetUpdate(TWeakObjectPtr<UMixInterface> InMixObj, int32 InTargetId);
									UE_API MixTargetUpdate(TWeakObjectPtr<UMixInterface> InMixObj, const TileInvalidateMatrix& InInvalidationMatrix, int32 InTargetId);
									UE_API ~MixTargetUpdate();

	UE_API void							InvalidateTile(int32 Row, int32 Col);
	UE_API void							InvalidateAllTiles();
	UE_API void							InvalidateNoTiles();

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE const 
		TileInvalidateMatrix&		GetInvalidationMatrix() const { return InvalidationMatrix; }
	FORCEINLINE bool				CheckIsValid(int32 Row, int32 Col) const { return (size_t)Row < InvalidationMatrix.Rows() && (size_t)Col < InvalidationMatrix.Cols(); }
	FORCEINLINE bool				IsInvalidated(int32 Row, int32 Col) const { check(CheckIsValid(Row, Col)); return InvalidationMatrix(Row, Col) != 0; }
	FORCEINLINE MixTextureSet&		GetLastRender() { return LastRender; }
	FORCEINLINE int32				GetTargetId() const { return TargetId; }
	FORCEINLINE JobPtrW				GetLastAddedJob() const { return LastAddedJob; }
};

typedef std::shared_ptr<MixTargetUpdate>	SceneTargetUpdatePtr;
typedef std::vector<SceneTargetUpdatePtr>	SceneTargetUpdatePtrVec;

//////////////////////////////////////////////////////////////////////////
class ULayerSet;
class ULayerComponent;

class UMaskStack;

class MixUpdateCycle : public FGCObject
{
private:
	const TWeakObjectPtr<UMixInterface>	MixObj;				/// What is the mix that we're dealing with

	std::shared_ptr<JobBatch>		Batch = nullptr;				/// The batch that we have

	TArray<UMixInterface*>			ActiveMixStack;/// Maintains the stack of mixes that are currently being updated. Mix remains in the list as long as evaluation is happening.
																	/// Used mainly to figure out recursion in subgraph's.

	/// TODO: Pointers inside _details get GC'd. This needs fixing.
	FInvalidationDetails			Details;						/// What are the details of the invalidation

	SceneTargetUpdatePtrVec			Targets;						/// The targets that we're going to be working on in this update cycle
																	/// into the actual LayerComponent [_dynamic] textures.
																	/// 
	//////////////////////////////////////////////////////////////////////////
	/// FGCObject overrides
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual void					AddReferencedObjects(FReferenceCollector& collector) override;
	UE_API virtual FString					GetReferencerName() const override;

public:
	UE_API explicit						MixUpdateCycle(const FInvalidationDetails& InDetails);
	UE_API virtual							~MixUpdateCycle() override;

	UE_API JobPtrW							AddJob(int32 InTargetId, JobUPtr JobObj);
	UE_API void							AddTarget(SceneTargetUpdatePtr Target);

	UE_API void							Begin();
	UE_API void							End();
	
	UE_API void							PushMix(UMixInterface* mix);
	UE_API void							PopMix();
	UE_API UMixInterface*					TopMix();
	UE_API bool							ContainsMix(UMixInterface* InMixObj) const;
	
	UE_API JobPtrW							LastAddedJob(int32 InTargetId) const; 
	UE_API void							SetBatch(std::shared_ptr<JobBatch> InBatch);
	UE_API uint32							LODLevel() const;
	UE_API bool							NoCache() const;
	UE_API void							MergeDetails(const FInvalidationDetails& InDetails);
	SceneTargetUpdatePtr			GetTarget(size_t targetId) { check(targetId < Targets.size()); return Targets[targetId]; }

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE const 
		FInvalidationDetails&		GetDetails() const { return Details; }
	FORCEINLINE const JobBatch*		GetBatch() const { return Batch.get(); }
	FORCEINLINE size_t				GetNumTargets() const { return Targets.size(); }
	FORCEINLINE UMixInterface*		GetMix() { return MixObj.IsValid() ? MixObj.Get() : nullptr; }
	FORCEINLINE bool				IsTweaking() const { return Details.bTweaking; }
};

typedef std::shared_ptr<MixUpdateCycle>		MixUpdateCyclePtr;
typedef std::weak_ptr<MixUpdateCycle>		SceneUpdateCyclePtrW;

#undef UE_API
