// Copyright Epic Games, Inc. All Rights Reserved.
#include "ViewportSettings.h"

#include "MixSettings.h"
#include "FxMat/MaterialManager.h"
#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ViewportSettings)

FName FViewportSettings::GetMaterialMappingInfo(const FName MaterialInput)
{
	for(const FMaterialMappingInfo& MappingInfo : MaterialMappingInfos)
	{
		if(MappingInfo.MaterialInput == MaterialInput)
		{
			return MappingInfo.Target;
		}
	}

	return FName();
}

bool FViewportSettings::ContainsMaterialMappingInfo(const FName InMaterialInput)
{
	if (MaterialMappingInfos.Num() > 0)
	{
		const FMaterialMappingInfo& MappingInfo = MaterialMappingInfos[0];
		return MappingInfo.MaterialInput == InMaterialInput;
	}

	return false;
}

void FViewportSettings::SetDefaultTarget(FName DefaultTargetName)
{
	if(MaterialMappingInfos.Num() > 0)
	{
		MaterialMappingInfos[0].Target = DefaultTargetName;
	}
}

UMaterial* FViewportSettings::GetDefaultMaterial()
{
	UMaterial* LoadedMaterial = TextureGraphEngine::GetMaterialManager()->LoadMaterial(TEXT("Scene/M_BasicMaterial"));

	check(LoadedMaterial)

	return LoadedMaterial;
}

FName FViewportSettings::GetMaterialName() const
{
	if(Material)
	{
		return Material->GetFName();
	}

	return NAME_None;
}

void FViewportSettings::InitDefaultSettings(FName InitialTargetName)
{
	UMaterial* LoadedMaterial = GetDefaultMaterial();

	if(LoadedMaterial)
	{
		Material = LoadedMaterial;

		OnMaterialUpdate();
		SetDefaultTarget(InitialTargetName);
	}

}

bool FViewportSettings::RemoveMaterialMappingForTarget(const FName OutputNode)
{
	for(FMaterialMappingInfo& MappingInfo : MaterialMappingInfos)
	{
		if(MappingInfo.Target == OutputNode)
		{
			MappingInfo.Target = FName();

			OnMaterialMappingChangedEvent.Broadcast();
			return true;
		}
	}

	return false;
}

void FViewportSettings::OnMaterialUpdate()
{
	if(!Material)
	{
		Material = GetDefaultMaterial();
	}
	
	if(Material)
	{
		TArray<FMaterialParameterInfo> OutParameterInfo;
		TArray<FGuid> OutParameterIds;

		Material->GetAllTextureParameterInfo(OutParameterInfo, OutParameterIds);

		MaterialMappingInfos.Empty();
	
		for (FMaterialParameterInfo Param : OutParameterInfo)
		{
			FMaterialMappingInfo MappingInfo;
			MappingInfo.MaterialInput = Param.Name;
		
			if(!ContainsMaterialMappingInfo(Param.Name))
			{
				MaterialMappingInfos.Add(MappingInfo);	
			}
		}

		// In case of multiple inputs, sort the inputs in ascending order.
		if(MaterialMappingInfos.Num() > 1)
		{
			MaterialMappingInfos.Sort([](const FMaterialMappingInfo& A, const FMaterialMappingInfo& B)
			{
				return A.MaterialInput.LexicalLess(B.MaterialInput);
			});
		}

		OnViewportMaterialChangedEvent.Broadcast();
	}
}

void FViewportSettings::OnTargetRename(const FName OldName,const FName NewName)
{
	bool bTargetUpdated = false;
	
	for(FMaterialMappingInfo& MappingInfo : MaterialMappingInfos)
	{
		if(MappingInfo.HasTarget() && MappingInfo.Target == OldName)
		{
			MappingInfo.Target = NewName;
			bTargetUpdated = true;
		}
	}

	if(bTargetUpdated)
	{
		OnMaterialMappingChangedEvent.Broadcast();
	}
}

int FViewportSettings::NumAssignedTargets()
{
	int ValidTargets = 0;
	for(const FMaterialMappingInfo& MappingInfo : MaterialMappingInfos)
	{
		if(MappingInfo.HasTarget())
		{
			ValidTargets++;
		}
	}

	return ValidTargets;
}
