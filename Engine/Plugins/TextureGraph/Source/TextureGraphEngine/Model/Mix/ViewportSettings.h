// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h" 
#include "ViewportSettings.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

class UMaterial;

USTRUCT()
struct FMaterialMappingInfo
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Viewport Settings")
	FName						MaterialInput;

	UPROPERTY(EditAnywhere, Category = "Viewport Settings")
	FName						Target;

	bool						HasTarget() const { return !Target.IsNone(); }
};

USTRUCT()
struct FViewportSettings
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Viewport Material", NoClear)
	TObjectPtr<UMaterial> Material;
	
	UPROPERTY(EditAnywhere, EditFixedSize, Category = "Viewport Material")
	TArray<FMaterialMappingInfo> MaterialMappingInfos;
	
	UE_API void InitDefaultSettings(FName InitialTargetName);
	UE_API void SetDefaultTarget(FName DefaultTargetName);
	
	UE_API UMaterial* GetDefaultMaterial();
	
	UE_API FName GetMaterialName() const;
	UE_API FName GetMaterialMappingInfo(const FName MaterialInput);
	UE_API bool ContainsMaterialMappingInfo(const FName InMaterialInput);

	UE_API bool RemoveMaterialMappingForTarget(FName OutputNode);
	UE_API void OnMaterialUpdate();
	UE_API void OnTargetRename(const FName OldName, const FName NewName);

	UE_API int  NumAssignedTargets();
	
	DECLARE_MULTICAST_DELEGATE(FViewportSettingsUpdateEvent)
	FViewportSettingsUpdateEvent OnViewportMaterialChangedEvent;

	DECLARE_MULTICAST_DELEGATE(FMaterialMappingChangedEvent)
	FMaterialMappingChangedEvent OnMaterialMappingChangedEvent;
};

#undef UE_API
