// Copyright Epic Games, Inc. All Rights Reserved.
#include "ModelObject.h"
#include "TextureGraphEngine.h"
#include "Model/Mix/MixInterface.h"
#include "Misc/Paths.h"
#include "Model/Mix/MixManager.h"

//////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelObject)
FInvalidationDetails::FInvalidationDetails(const TWeakObjectPtr<UMixInterface> mix_) : Mix(mix_)
{
}

FInvalidationDetails& FInvalidationDetails::All()
{
	auto OnDoneCurrent = OnDone;
	*this = FInvalidationDetails(Mix);
	if (OnDone.IsBound())
	{
		/// Create a fresh copy
		OnDone = OnDoneCurrent;
	}

	bRender = true;

	bSelective = false;

	return *this;
}

FInvalidationDetails& FInvalidationDetails::None()
{
	*this = FInvalidationDetails(Mix);
	return *this;
}

CHashPtr FInvalidationDetails::Hash() const
{
	if (HashValue)
		return HashValue;

	HashTypeVec Hashes =
	{
		MX_HASH_VAL_DEF(bReload),
		MX_HASH_VAL_DEF(bSelective),
		MX_HASH_VAL_DEF(bForceInvalidateParent),
		MX_HASH_VAL_DEF(bRender),
	};
	
	HashType FinalHashValue = DataUtil::Hash(Hashes);
	HashValue = std::make_shared<CHash>(FinalHashValue, true);

	return HashValue;
}

FInvalidationDetails& FInvalidationDetails::Merge(const FInvalidationDetails& Details)
{
	FInvalidationDetails CurrentDetails = *this;
	*this = Details;

	// Current contains all the accumulated delegates
	// the parameter Details is expected to NOT contain any delegates in _onDoneMerged
	check(!OnDoneMergedInternal.IsBound());
	OnDoneMergedInternal = CurrentDetails.OnDoneMergedInternal;
	// Accumulate potentially CurrentDetails.OnDone
	if (CurrentDetails.OnDone.IsBound())
		OnDoneMergedInternal.Add(CurrentDetails.OnDone);
	// At this point this contains the delegate OnDone from Details
	// And accumulated in _internal_onDoneMerged all the previously accumulated delegates in this

	bTweaking = CurrentDetails.bTweaking && Details.bTweaking;
	bRender = CurrentDetails.bRender || Details.bRender;

	bSelective &= CurrentDetails.bSelective;

	// reset hash to recalculate with updated values when requested
	HashValue = nullptr;

	return *this;
}

void FInvalidationDetails::BroadcastOnDone() const
{
	OnDone.ExecuteIfBound(this);
	OnDoneMergedInternal.Broadcast(this);
	check(Mix.Get());
	Mix->BroadcastOnRenderingDone(this);
}

void FInvalidationDetails::BroadcastOnQueued() const
{
	OnQueued.ExecuteIfBound(this);
	Mix->BroadcastOnBatchQueued(this);
}

bool FInvalidationDetails::IsDiscard() const
{
	return bTweaking;
}

//////////////////////////////////////////////////////////////////////////


CHashPtr FModelInvalidateInfo::Hash() const
{
	if (HashValue)
		return HashValue;

	HashTypeVec Hashes = 
	{
		Details.Hash()->Value(),
		(Trigger ? Trigger->HashValue_Simple() : 0),
	};

	HashType FinalHashValue = DataUtil::Hash(Hashes);
	HashValue = std::make_shared<CHash>(FinalHashValue, true);

	return HashValue;
}


//////////////////////////////////////////////////////////////////////////
UModelObject::~UModelObject()
{
}

HashType UModelObject::HashValue_Simple() const
{
	return DataUtil::Hash_GenericString_Name(ToString());
}

FString UModelObject::ToString() const
{
	return GetName();
}

UObject* UModelObject::LoadObjectFromPath(const FString& distilledPath)
{
	static FString ContentRoot = FModuleManager::Get().GetModuleChecked<FTextureGraphEngineModule>("TextureGraphEngine").GetParentPluginName();
	FString FullPath = "/" + FPaths::Combine(ContentRoot, distilledPath);
	FSoftObjectPath ObjRef(FullPath);
	UObject* Obj = Cast<UObject>(ObjRef.TryLoad());
	return Obj;
}

