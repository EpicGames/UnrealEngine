// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <functional>
#include <cmath>
#include "CoreMinimal.h"
#include "Helper/Util.h"
#include "Containers/UnrealString.h"
#include "2D/TextureType.h"
#include "2D/TextureHelper.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "ModelObject.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

class UMixInterface;

//////////////////////////////////////////////////////////////////////////
USTRUCT(BlueprintType)
struct FInvalidationDetails
{
	GENERATED_BODY()

	TWeakObjectPtr<UMixInterface>		Mix = nullptr;								/// The mix that is being invalidated
	
	DECLARE_DELEGATE_OneParam(FOnDone, const FInvalidationDetails*);
	FOnDone								OnDone;										/// Delegate allowing to provide a user callback once the invalidation is executed

	DECLARE_DELEGATE_OneParam(FOnQueued, const FInvalidationDetails*);
	FOnDone								OnQueued;									/// Delegate allowing to provide a user callback once the invalidation is executed

	bool								bReload = false;							/// Whether to reload the sources or not
	bool								bSelective = true;							/// Selective invalidation [tile based]
	bool								bTweaking = false;							/// Whether being tweaked or not
	bool								bExporting = false;							/// Should only be true in case of exporting.
	
	/// The list of things that have been invalidated
	bool								bForceInvalidateParent = false;				/// Force invalidates the parent
	bool								bRender = false;							/// Re-render

	mutable CHashPtr					HashValue = nullptr;								/// The hash for this invalidation 
	
	UE_API FInvalidationDetails&				All();
	UE_API FInvalidationDetails&				None();
	UE_API FInvalidationDetails&				Merge(const FInvalidationDetails& other); /// Merge the other details with this one,

	/// also merge the other.OnDone delegate, grouped in this._internal_onDoneMerged

	UE_API CHashPtr							Hash() const;
	UE_API bool								IsDiscard() const;

	UE_API void								BroadcastOnDone() const;	/// Trigger the broadcast of the OnDone delegate AND all 
																	/// the other accumulated delegates in _internal_onDoneMerged

	UE_API void								BroadcastOnQueued() const;

	FInvalidationDetails() : Mix(nullptr) {}
	UE_API explicit FInvalidationDetails(const TWeakObjectPtr<UMixInterface> InMixObj);


	template <typename ListType>
	static TArray<ListType*>			MergeList(const TArray<ListType*>& List1, const TArray<ListType*>& List2)
	{
		TArray<ListType*> MergedList;

		if (!List1.IsEmpty() && !List2.IsEmpty())
		{
			MergedList.Append(List1);

			for (ListType* Item : List2)
			{
				if (Item == nullptr)
				{
					// if there is an empty item in the list, we set the list empty so it overrides
					// and invalidates as a whole
					MergedList.Empty();
					break;
				}
				MergedList.AddUnique(Item);
			}
		}

		return MergedList;
	}

private:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDoneMulti, const FInvalidationDetails*);
	FOnDoneMulti				OnDoneMergedInternal; /// "Private" multicast delegate where we merge OnDone delegates for merged details
};

//////////////////////////////////////////////////////////////////////////
USTRUCT(Blueprintable, BlueprintType)
struct FModelInvalidateInfo
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category = "Trigger Info") 
	TObjectPtr<UModelObject>	Trigger = nullptr;				/// This is the model object that triggered the chain of ModelInvalidate calls

	FInvalidationDetails		Details;						/// details about the invalidation
	mutable CHashPtr			HashValue = nullptr;					/// The hash for this invalidation 

	UE_API CHashPtr					Hash() const;

	FORCEINLINE UMixInterface*	GetMix() const
	{
		if(Details.Mix.IsValid())
		{
			return Details.Mix.Get();
		}
		
		return nullptr;
	}
};

//////////////////////////////////////////////////////////////////////////
/// This is the base class for all models that need to be loaded from or
/// saved to the mix
//////////////////////////////////////////////////////////////////////////
UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class UModelObject : public UObject
{
	GENERATED_BODY()

public:
	UE_API virtual						~UModelObject() override;
	UE_API virtual HashType			HashValue_Simple() const;
	UE_API virtual FString				ToString() const;
	
	//////////////////////////////////////////////////////////////////////////
	// Static methods 
	//////////////////////////////////////////////////////////////////////////
	static UE_API UObject*				LoadObjectFromPath(const FString& distilledPath);

	template <typename ModelClass>
	static ModelClass* CreateNew(EObjectFlags Flags = RF_NoFlags)
	{
		return NewObject<ModelClass>(Util::GetModelPackage(), NAME_None, Flags);
	}
};

#undef UE_API
