// Copyright Epic Games, Inc. All Rights Reserved.
#include "StaticImageResource.h"

#include "2D/Tex.h"
#include "Job/JobArgs.h"
#include "Mix/MixInterface.h"
#include "Transform/Utility/T_LoadStaticResource.h"
#include "Misc/PackageName.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticImageResource)

UStaticImageResource::~UStaticImageResource()
{
}

AsyncTiledBlobRef UStaticImageResource::Load(MixUpdateCyclePtr Cycle)
{
	const FString& FileName = AssetUUID;

	// Asset path could be empty in case of a new asset channel source, we fallback to a default flat load
	if (FileName.IsEmpty() || FileName == TEXT("None"))
		return cti::make_ready_continuable(TiledBlobRef(TextureHelper::GBlack));
	
	// if the path is a valid package path (content browser asset)
	check(bIsFilesystem || FPackageName::IsValidPath(FileName));

	// we load the asset -> UTexture2d -> Tex -> Blob and create a SourceAsset from that
	return PromiseUtil::OnGameThread().then([=, this](int32) mutable
	{
		Tex* TexObj = nullptr;

		// we load the asset -> UTexture2d -> Tex -> Blob and create a SourceAsset from that
		TexObj = new Tex();
		bool bDidLoad = false;

		try
		{
			if (!bIsFilesystem)
			{
				FSoftObjectPath SoftPath(FileName);
				bDidLoad = TexObj->LoadAsset(SoftPath);
			}
			else
				bDidLoad = TexObj->LoadFile(FileName);
		}
		catch (std::exception e)
		{
		}

		if (bDidLoad)
		{
			int32 NumXTiles = Cycle->GetMix()->GetNumXTiles();
			int32 NumYTiles = Cycle->GetMix()->GetNumYTiles();

			return TexObj->ToBlob(NumXTiles, NumYTiles, 0, 0, false);
		}

		BlobObj = TextureHelper::GetMagenta();
		return (AsyncTiledBlobRef)cti::make_ready_continuable<TiledBlobRef>(TiledBlobRef(BlobObj));
	})
	.then([=](TiledBlobRef LoadedBlob) mutable
	{
		return (AsyncTiledBlobRef)PromiseUtil::OnGameThread().then([=]() { return LoadedBlob; });
	});
}


TiledBlobPtr UStaticImageResource::GetBlob(MixUpdateCyclePtr Cycle, BufferDescriptor* DesiredDesc, int32 TargetId)
{
	if (BlobObj)
		return BlobObj;

	FDateTime LastModified = GetAssetTimeStamp();

	auto JobObj = std::make_unique<Job_LoadStaticImageResource>(Cycle->GetMix(), this, TargetId);

	JobObj->AddArg(ARG_STRING(AssetUUID, "AssetUUID")); /// Assett UUID contains the Name of the asset megascan surface AND the Name of channel
	JobObj->AddArg(ARG_STRING(LastModified.ToString(), "TimeStamp"));

	FString Name = "[StaticImageResource]-" + AssetUUID;
	BufferDescriptor Desc;

	if (DesiredDesc)
		Desc = *DesiredDesc;

	Desc.Format = BufferFormat::LateBound;
	BlobObj = JobObj->InitResult(Name, &Desc);

	Cycle->AddJob(TargetId, std::move(JobObj));

	return BlobObj;
}

FDateTime UStaticImageResource::GetAssetTimeStamp()
{
	FString FullPath = AssetUUID;

	if (!IsFileSystem())
	{
		const FString PackageName = FPackageName::ObjectPathToPackageName(AssetUUID);
		const FString FileName = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());
		FullPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FileName);
	}
	
	FDateTime LastModified;
	IFileManager& FileManager = IFileManager::Get();

	if (FileManager.FileExists(*FullPath))
	{
		LastModified = FileManager.GetTimeStamp(*FullPath);
	}

	return LastModified;
}
