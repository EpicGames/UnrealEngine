// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ModelObject.h"
#include "Device/FX/Device_FX.h"
#include "Mix/MixUpdateCycle.h"
#include "StaticImageResource.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Base static image resource class
////////////////////////////////////////////////////////////////////////// 
UCLASS(MinimalAPI, Blueprintable, BlueprintType)
class UStaticImageResource : public UModelObject
{
public:
	GENERATED_BODY()
	
private:
	friend class Job_LoadStaticImageResource;
	
	/*Unique id for the asset within the entire system*/
	UPROPERTY()
	FString AssetUUID;

	/*The blob that represents the data for this source*/
	TiledBlobPtr BlobObj;

	/*Is loading directly from the filesystem*/
	bool bIsFilesystem = false;

	UE_API virtual AsyncTiledBlobRef Load(MixUpdateCyclePtr Cycle);

	UE_API FDateTime GetAssetTimeStamp();
	
public:
	UE_API virtual ~UStaticImageResource() override;
	
	UE_API virtual TiledBlobPtr GetBlob(MixUpdateCyclePtr Cycle, BufferDescriptor* DesiredDesc, int32 TargetId);

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE const FString& GetAssetUUID() const { return AssetUUID; }
	FORCEINLINE void SetAssetUUID(const FString& UUID) { AssetUUID = UUID; }
	FORCEINLINE void SetIsFileSystem(bool bInIsFileSystem) { bIsFilesystem = bInIsFileSystem; }
	FORCEINLINE bool IsFileSystem() const { return bIsFilesystem; }
};

#undef UE_API
