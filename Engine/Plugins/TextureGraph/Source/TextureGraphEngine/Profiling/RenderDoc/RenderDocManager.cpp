// Copyright Epic Games, Inc. All Rights Reserved.
#include "RenderDocManager.h"

#include "Misc/AutomationTest.h"
#include "Job/Scheduler.h"

#include "TextureGraphEngine.h"
#include "RenderingThread.h"

DEFINE_LOG_CATEGORY(LogRenderDocTextureGraph);

/**
 * Enable (or disable) concretely renderdoc capture in texture graph module without changing the api
 * DO NOT FORGET to run the project with the command line -AttachRenderDoc OR enable auto attach in the renderdoc plugin
 */
#if !defined(TEXTUREGRAPH_RENDERDOC_ENABLED)
#define TEXTUREGRAPH_RENDERDOC_ENABLED 1
#endif


#if TEXTUREGRAPH_RENDERDOC_ENABLED
#include <IRenderDocPlugin.h>
#endif

namespace TextureGraphEditor
{


	///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Public Interface
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	RenderDocManager::RenderDocManager()
	{
#if TEXTUREGRAPH_RENDERDOC_ENABLED
		static FAutoConsoleCommand CCmdRenderDocCaptureFrame = FAutoConsoleCommand(
			TEXT("renderdoc.TextureGraph_CaptureNextBatch"),
			TEXT("Captures the next Job Batch and launches RenderDoc"),
			FConsoleCommandDelegate::CreateRaw(this, &RenderDocManager::CaptureNextBatch)
		);

		static FAutoConsoleCommand CCmdRenderDocCapturePIE = FAutoConsoleCommand(
			TEXT("renderdoc.TextureGraph_CapturePrevBatch"),
			TEXT("Captures the previous Job Batch and launches RenderDoc"),
			FConsoleCommandDelegate::CreateRaw(this, &RenderDocManager::CapturePreviousBatch)
		);

		static FAutoConsoleCommand CCmdRenderDocCaptureHistogram = FAutoConsoleCommand(
			TEXT("renderdoc.TextureGraph_CaptureNextBatchHistogram"),
			TEXT("Captures the next Job Batch producing histogram and launches RenderDoc"),
			FConsoleCommandDelegate::CreateRaw(this, &RenderDocManager::CaptureNextBatchHistogram)
		);
#endif 
	}

	RenderDocManager::~RenderDocManager()
	{
	}


	void RenderDocManager::CaptureNextBatch()
	{
#if TEXTUREGRAPH_RENDERDOC_ENABLED
		TextureGraphEngine::GetScheduler()->SetCaptureRenderDocNextBatch(true);
#endif 
	}

	void RenderDocManager::CapturePreviousBatch()
	{
#if TEXTUREGRAPH_RENDERDOC_ENABLED
		TextureGraphEngine::GetScheduler()->CaptureRenderDocLastRunBatch();
#endif 
	}

	void RenderDocManager::CaptureNextBatchHistogram()
	{
#if TEXTUREGRAPH_RENDERDOC_ENABLED
		HistogramServicePtr HistoService = TextureGraphEngine::GetScheduler()->GetHistogramService().lock();
		if (HistoService)
		{
			HistoService->CaptureNextBatch();
		}
#endif 
	}

	void RenderDocManager::BeginCapture()
	{
#if TEXTUREGRAPH_RENDERDOC_ENABLED
		ENQUEUE_RENDER_COMMAND(BeginCaptureCommand)([this](FRHICommandListImmediate& RHICommandList)
			{
				IRenderDocPlugin& PluginModule = FModuleManager::GetModuleChecked<IRenderDocPlugin>("RenderDocPlugin");
				PluginModule.BeginCapture(&RHICommandList, IRenderCaptureProvider::ECaptureFlags_Launch, FString("TextureGraph"));
			});
#endif 
	}

	void RenderDocManager::EndCapture()
	{
#if TEXTUREGRAPH_RENDERDOC_ENABLED
		ENQUEUE_RENDER_COMMAND(EndnCaptureCommand)([this](FRHICommandListImmediate& RHICommandList)
			{
				IRenderDocPlugin& PluginModule = FModuleManager::GetModuleChecked<IRenderDocPlugin>("RenderDocPlugin");
				PluginModule.EndCapture(&RHICommandList);
			});
#endif 
	}
}