// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include <memory>

#define UE_API TEXTUREGRAPHENGINE_API

DECLARE_LOG_CATEGORY_EXTERN(LogRenderDocTextureGraph, Log, All);

namespace TextureGraphEditor
{
	class RenderDocManager
	{

	public:
		UE_API RenderDocManager();
		UE_API ~RenderDocManager();
		UE_API void												Initialize();

		UE_API void												CaptureNextBatch();
		UE_API void												CapturePreviousBatch();
		UE_API void												BeginCapture();
		UE_API void												EndCapture();
		UE_API void												CaptureNextBatchHistogram();

};
	typedef std::unique_ptr<RenderDocManager>				RenderDocManagerPtr;
}

#undef UE_API
