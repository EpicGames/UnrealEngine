// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"


DECLARE_STATS_GROUP(TEXT("TextureGraphEngine"), STATGROUP_TextureGraphEngine, STATCAT_Advanced)

//To be Used in CPP
//Every CPP can have more than one Cycle stat declared
//DECLARE_CYCLE_STAT(TEXT("Mix_Update"), STAT_Mix_Update, STATGROUP_TextureGraphEngine);
//Args = DisplayName,GroupName,Category

//To be used in method
//SCOPE_CYCLE_COUNTER(STAT_Mix_Update);
//Args = GroupName