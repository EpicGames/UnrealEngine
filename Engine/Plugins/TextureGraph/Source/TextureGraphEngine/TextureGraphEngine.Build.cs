// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;
using System;
using System.IO;


public class TextureGraphEngine : ModuleRules
{
    private void AddDefaultIncludePaths()
    {

        // Add all the public directories
        PublicIncludePaths.Add(ModuleDirectory);

        //Add Public dir to 
        string PublicDirectory = Path.Combine(ModuleDirectory, "Public");
        if (Directory.Exists(PublicDirectory))
        {
            PublicIncludePaths.Add(PublicDirectory);
        }

        // Add the base private directory for this module
        string PrivateDirectory = Path.Combine(ModuleDirectory, "Private");
        if (Directory.Exists(PrivateDirectory))
        {
            PrivateIncludePaths.Add(PrivateDirectory);
        }
	}


    public TextureGraphEngine(ReadOnlyTargetRules Target) : base(Target)
	{
        PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
        bEnableExceptions = true;
		bDisableAutoRTFMInstrumentation = true;

		// Disable Clang analysis due to a crash in clang-cl
		if (Target.StaticAnalyzer != StaticAnalyzer.None
			&& Target.Platform.IsInGroup(UnrealPlatformGroup.Microsoft)
			&& Target.WindowsPlatform.Compiler.IsClang())
		{
			// https://developercommunity.visualstudio.com/t/clang-cl---analyze-crash-with-thread-i/10746623
			bDisableStaticAnalysis = true;
		}

		//PublicDefinitions.Add("WITH_MALLOC_STOMP=1");

		PrivateIncludePaths.AddRange(
			new string[] {
				Path.Combine(ModuleDirectory, "../../../../Source/Runtime/Engine/Private")
			}
		);

		PrivateDependencyModuleNames.AddRange(new string[] 
		{ 
			"Slate", 
			"SlateCore",
			"UMG", 
			"MutableRuntime"
		});

        PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
			"InputCore",
			"XmlParser" ,
			"ProceduralMeshComponent",
			"Renderer",
			"RenderCore",
			"RHI",
			"HTTP",
			"Json",
			"JsonUtilities",
			"ImageCore",
			"ImageWrapper",
			"Projects",
			"ImageWriteQueue",
			"LibJpegTurbo",
			"Function2",
			"Continuable",
			"Sockets",
			"Networking",
			"DeveloperSettings",
		});

		if (Target.bBuildDeveloperTools && (Target.Platform == UnrealTargetPlatform.Win64 || Target.Platform == UnrealTargetPlatform.Linux))
		{
			PublicDependencyModuleNames.Add("RenderDocPlugin");
			PublicDefinitions.Add("TEXTUREGRAPH_RENDERDOC_ENABLED=1");
		}
		else
		{
			PublicDefinitions.Add("TEXTUREGRAPH_RENDERDOC_ENABLED=0");
		}
		
		if (Target.bBuildEditor == true)
		{
			//reference the module "MyModule"
			PublicDependencyModuleNames.AddRange(new string[]
			{
				"UnrealEd",
				"AssetTools"
			});
		}

		AddDefaultIncludePaths();

		if (!ModuleDirectory.StartsWith(EngineDirectory))
			throw new BuildException("TextureGraphEngine module directory must be under engine");

		string ModuleRelativeToEngineDir = ModuleDirectory.Substring(EngineDirectory.Length + 1);

		string ModDirLiteral = ModuleRelativeToEngineDir.Replace('\\', '/');
		string defModuleName = "MODULE_DIR \"" + ModDirLiteral + "\"=";
		PrivateDefinitions.Add(defModuleName);
	}

}
