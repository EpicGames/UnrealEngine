// Copyright Epic Games, Inc. All Rights Reserved.
#include "TextureGraphEngine.h"
#include "Modules/ModuleManager.h"
#include "Stats/Stats.h"
#include "Model/Mix/MixManager.h"
#include "Job/Scheduler.h"

#include "Model/Mix/MixSettings.h"
#include "Model/Mix/MixManager.h"
#include "Data/Blobber.h"
#include "FxMat/MaterialManager.h"
#include "Device/DeviceManager.h"
#include "Profiling/RenderDoc/RenderDocManager.h"

#include <Interfaces/IPluginManager.h>
#include "Model/Mix/MixInterface.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include "CoreGlobals.h"
#include "Framework/Application/SlateApplication.h"

DECLARE_CYCLE_STAT(TEXT("TextureGraphEngine-Update"), STAT_TextureGraphEngine_Update, STATGROUP_TextureGraphEngine);
DEFINE_LOG_CATEGORY(LogTextureGraphEngine);


static TAutoConsoleVariable<bool> CVarAllowTextureGraphInCommandlets(
	TEXT("TextureGraph.AllowCommandlets"),
	false,
	TEXT("Allow TextureGraph engine to be initialized when running via Commandlets")
	);

void FTextureGraphEngineModule::StartupModule()
{
	FDefaultGameModuleImpl::StartupModule();

	MapShaders();
}

bool FTextureGraphEngineModule::MapShaders()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	FString fullModuleDir = FPaths::Combine(FPaths::EngineDir(), MODULE_DIR);
	bool added = false;

	FString rightSanitized;
	FString shaderDirectory;
	FString realPluginName;

	fullModuleDir.Split(TEXT("Plugins/"), nullptr, &rightSanitized);	//We don't know plugin name, it should work in any folder regardless of the plugin
	realPluginName = rightSanitized.Replace(TEXT("/Source/TextureGraphEngine"), TEXT(""));
	_pluginName = realPluginName.Replace(TEXT("Experimental/"), TEXT(""));

	//Try for developer plugin first
	const FString virtualShaderDir = "/Plugin/" + _pluginName;
	if (!AllShaderSourceDirectoryMappings().Contains(virtualShaderDir))  // Two modules try to map this, so we have to check if its already set.
	{
		shaderDirectory = FPaths::Combine(FPaths::ProjectPluginsDir(), realPluginName, TEXT("Shaders"));

		if (!FPaths::DirectoryExists(shaderDirectory))
			shaderDirectory = FPaths::Combine(FPaths::EnginePluginsDir(), realPluginName, TEXT("Shaders"));	//If not part of project then maybe its part of engine

		if (FPaths::DirectoryExists(shaderDirectory))
		{
			AddShaderSourceDirectoryMapping(virtualShaderDir, shaderDirectory);
			added = true;
		}
	}
	else 
	{
		added = true;
	}

	//Plugin is versioned.
	if (!added)
	{
		IPluginManager& pluginManager = IPluginManager::Get();
		TArray<TSharedRef<IPlugin>> plugins = pluginManager.GetDiscoveredPlugins();

		for(auto plugin: plugins) {
			if (plugin->GetName().Contains(_pluginName)) {
				shaderDirectory = FPaths::Combine(plugin->GetBaseDir(), TEXT("Shaders"));
				_pluginName = plugin->GetName();
				FString versionedShaderPath = "/Plugin/" + _pluginName;
				AddShaderSourceDirectoryMapping(virtualShaderDir, shaderDirectory);
				added = true;
				break;
			}
		}
	}

	return added;
}

FString FTextureGraphEngineModule::GetParentPluginName()
{
	return _pluginName;
}

IMPLEMENT_MODULE(FTextureGraphEngineModule, TextureGraphEngine)

//////////////////////////////////////////////////////////////////////////

TextureGraphEngine* TextureGraphEngine::GInstance = nullptr;
std::atomic<bool> TextureGraphEngine::bGIsEngineDestroying(false);

/// Initialize the observer to default no-op
EngineObserverSourcePtr TextureGraphEngine::GObserverSource = std::make_shared<EngineObserverSource>();

TextureGraphEngine::TextureGraphEngine(bool bInIsTestMode) : 
	SchedulerObj(nullptr), 
	DeviceManagerObj(nullptr), 
	BlobberObj(nullptr),
	RenderDocMgrObj(nullptr),
	bIsTestMode(bInIsTestMode)
{
	UE_LOGF(LogTextureGraphEngine, Log, "Initialising the TextureGraphEngine!");

	if (bIsTestMode)
	{
		LockWaitMutex = new std::mutex();
		LockWaitCVar = new std::condition_variable();

		/// In test mode, the engine is always running
		bRunEngine = true;
	}
}

void TextureGraphEngine::InitEngineInternal()
{
	check(IsInGameThread());
	check(!bIsEngineInitDone);
	check(!bGIsEngineDestroying);

	// Currently we're only supporting editor only 
#if WITH_EDITOR
	// Early out if commandlet execution is not allowed
	if (TextureGraphEngine::BlockCommandlets())
		return;

	DeviceManagerObj = std::make_unique<DeviceManager>();
	BlobberObj = std::make_unique<::Blobber>();

	/// Init some of the stock textures that we keep
	//TextureHelper::InitStockTextures();

	// Allocate the render doc manager (wether or not render doc define is enabled)
	RenderDocMgrObj = std::make_unique<TextureGraphEditor::RenderDocManager>();

	MixMgrObj = std::make_unique<::MixManager>();

	SchedulerObj = std::make_unique<::Scheduler>();

	if (bIsTestMode)
		FirstRunInit();

	ErrorReporters.Add(nullptr, std::make_shared<FTextureGraphErrorReporter>());

	bIsEngineInitDone = true;

	/// Notify the world engine is created
	GObserverSource->Created();
#endif // WITH_EDITOR
}

TextureGraphEngine::~TextureGraphEngine()
{
	check(IsInGameThread());

	// Currently we're only supporting editor only 
#if WITH_EDITOR
	/// We set this before the lock check so that anyone with a lock to the engine
	/// can release it if someone is waiting for it to be destroyed
	bGIsEngineDestroying = true;

	/// If the engine is locked then wait for the lock to be released
	if (bIsTestMode && bIsLocked)
	{
		std::unique_lock<std::mutex> lock(*LockWaitMutex);
		LockWaitCVar->wait(lock);
	}

	check(!bIsTestMode || !bIsLocked);

	delete LockWaitMutex;
	LockWaitMutex = nullptr;

	delete LockWaitCVar;
	LockWaitCVar = nullptr;

	/// This is to control the order of destruction
	TextureHelper::FreeStockTextures();

	// No more render doc manager
	RenderDocMgrObj = nullptr;

	MaterialMgrObj = nullptr;
	MixMgrObj = nullptr;
	SchedulerObj = nullptr;

	BlobberObj = nullptr;

	// CollectGarbage(RF_NoFlags, true);

	DeviceManagerObj = nullptr;

	UE_LOGF(LogTextureGraphEngine, Log, "Destroying the TextureGraphEngine!");
	
	/// Notify the world engine is destroyed
	if (GObserverSource)
		GObserverSource->Destroyed();
#endif // WITH_EDITOR
}

void TextureGraphEngine::InitTests()
{
	check(IsInGameThread());

	// Currently we're only supporting editor only 
#if WITH_EDITOR
	if (!GEditor)
		return;
		
	if (!GInstance)
		Create();

	// explicitly adding check to avoid Warning C6011 - Dereferencing NULL pointer when running Static Analysis Tests
	if (GInstance)
	{
		GInstance->bIsTestMode = true;
		GInstance->LockWaitMutex = new std::mutex();
		GInstance->LockWaitCVar = new std::condition_variable();

		GetMixManager()->Suspend();	//We do not want RenderScene to have default Update behavior - We will update mix manually
	}
#endif // WITH_EDITOR
}

void TextureGraphEngine::DoneTests()
{
	check(IsInGameThread());

	// Currently we're only supporting editor only 
#if WITH_EDITOR
	if (!GEditor || !GInstance || !GInstance->bIsEngineInitDone)
		return;
		
	bGIsEngineDestroying = true;

	GInstance->SchedulerObj->ClearCache();
	GInstance->BlobberObj->ClearCache();

	delete GInstance->LockWaitMutex;
	GInstance->LockWaitMutex = nullptr;

	delete GInstance->LockWaitCVar;
	GInstance->LockWaitCVar = nullptr;

	GInstance->bIsTestMode = false;

	bGIsEngineDestroying = false;

	GInstance->GetMixManager()->Resume();
#endif // WITH_EDITOR
}

void TextureGraphEngine::Lock()
{
	/// Only allowed in test mode
	check(GInstance->bIsTestMode);
	GInstance->bIsLocked = true;
}

void TextureGraphEngine::Unlock()
{
	/// Only allowed in test mode
	check(GInstance->bIsTestMode);
	GInstance->bIsLocked = false;
	GInstance->LockWaitCVar->notify_one();
}

void TextureGraphEngine::SetRunEngine()
{
	if (bGIsEngineDestroying)
		return;

	if (GInstance && !GInstance->bRunEngine)
	{
		GInstance->FirstRunInit();
	}
}

bool TextureGraphEngine::BlockCommandlets()
{
	return (IsRunningCookCommandlet() || IsRunningDLCCookCommandlet() || !CVarAllowTextureGraphInCommandlets.GetValueOnAnyThread())
	#if WITH_EDITOR
		&& !(GEditor && FSlateApplication::IsInitialized() && FApp::CanEverRender())
	#endif
	;
}

void TextureGraphEngine::FirstRunInit()
{
	check(IsInGameThread());
	UE_LOGF(LogTextureGraphEngine, Log, "Setting TextureGraphEngine to run mode = true!");
	
	bRunEngine = true;
	
	MaterialMgrObj = TStrongObjectPtr<UMaterialManager>(UMaterialManager::CreateNew<UMaterialManager>());
	
#if WITH_EDITOR
	// Early out if commandlet execution is not allowed
	if (TextureGraphEngine::BlockCommandlets())
	{
		return;
	}

	// If the engine is not set to run, then we don't do anything here. Not even initialise it (to void any 
	// initialisation issues that we might get with people who don't really use the TextureGraph)
	TextureHelper::InitStockTextures();
#endif /// WITH_EDITOR
}


void TextureGraphEngine::Create(bool bIsTestMode /* = false */)
{
	check(IsInGameThread());

	/// Cannot create a new instance before destroying the old one
	check(!GInstance);

	GInstance = new TextureGraphEngine(bIsTestMode);
	bGIsEngineDestroying = false;
	GInstance->InitEngineInternal();
}

void TextureGraphEngine::Destroy()
{
	check(IsInGameThread());

	if (GInstance && GInstance->bIsEngineInitDone && GetMixManager())
	{
		GetMixManager()->Exit();
	}

	delete GInstance;
	GInstance = nullptr;
}

void TextureGraphEngine::Update(float dt)
{
	check(IsInGameThread());

	// Currently we're only supporting editor only 
#if WITH_EDITOR
	// Early out if commandlet execution is not allowed
	if (TextureGraphEngine::BlockCommandlets())
		return;

	// If engine is destroying then we don't do anything
	if (bGIsEngineDestroying)
		return;

	verify(GInstance);

	// If the engine has not initialised then we need to do that first
	if (!GInstance->bIsEngineInitDone)
	{
		// Once we init, then we don't do anything in that frame and do the updates in the next frames
		GInstance->InitEngineInternal();
		return;
	}

	// If the engine isn't running then we don't update anything
	if (!GInstance->bRunEngine)
		return;
		
	SCOPE_CYCLE_COUNTER(STAT_TextureGraphEngine_Update);
	GInstance->FrameId++;

	GInstance->DeviceManagerObj->Update(dt);
	GInstance->BlobberObj->Update(dt);
	GInstance->MixMgrObj->Update(dt);
	GInstance->SchedulerObj->Update(dt);
#endif // WITH_EDITOR
}

void TextureGraphEngine::RegisterObserverSource(const EngineObserverSourcePtr& observerSource)
{
	if (GObserverSource)
		GObserverSource->Destroyed();

	if (observerSource)
	{
		GObserverSource = observerSource;
		if (GInstance) // Engine does exist!, so notify a first created()
		{
			GObserverSource->Created();
		}
	}
	else
	{
		GObserverSource = std::make_shared<EngineObserverSource>();
	}
}

void TextureGraphEngine::RegisterErrorReporter(const UMixInterface* MixKey, const ErrorReporterPtr& InErrorReporterPtr)
{
	if (GInstance)
	{
		if (GInstance->ErrorReporters.Contains(MixKey))
		{
			GInstance->ErrorReporters[MixKey] = InErrorReporterPtr;
		}
		else
		{
			GInstance->ErrorReporters.Add(MixKey, InErrorReporterPtr);
		}
	}
}