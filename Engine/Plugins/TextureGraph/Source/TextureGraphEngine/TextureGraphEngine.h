// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Profiling/StatGroup.h"
#include "UObject/StrongObjectPtr.h"
#include "HAL/IConsoleManager.h"

#include <memory>
#include <mutex>

#include "Modules/ModuleManager.h"
#include "TextureGraphErrorReporter.h"

#define UE_API TEXTUREGRAPHENGINE_API

DECLARE_LOG_CATEGORY_EXTERN(LogTextureGraphEngine, Log, All);
class UMixInterface;
class UAppState;
class ULayerStack;

class MixManager;
typedef std::unique_ptr<MixManager>						MixManagerUPtr;

class AssetsManager;
typedef std::unique_ptr<AssetsManager>					AssetsManagerPtr;

class DeviceManager;
typedef std::unique_ptr<DeviceManager>					DeviceManagerPtr;

class UMaterialManager;

class Blobber;
typedef std::unique_ptr<Blobber>						BlobberPtr;

class Scheduler;
typedef std::unique_ptr<Scheduler>						SchedulerPtr;

namespace TextureGraphEditor 
{
	class RenderDocManager;
	typedef std::unique_ptr<RenderDocManager>			RenderDocManagerPtr;
}

//////////////////////////////////////////////////////////////////////////
class FTextureGraphEngineModule : public FDefaultGameModuleImpl
{
private:
	FString												_libPath;
	FString												_pluginName;
	bool												MapShaders();
public:
	virtual void										StartupModule() override;
	virtual FString										GetParentPluginName();
};


class EngineObserverSource
{
protected:
	/// Protected interface of emitters called by the engine to notify the observers
	friend class TextureGraphEngine;

	virtual void										Created() {}
	virtual void										Destroyed() {}

public:
														EngineObserverSource() = default;
	virtual												~EngineObserverSource() = default;
};
typedef std::shared_ptr<EngineObserverSource>			EngineObserverSourcePtr;
typedef std::shared_ptr<FTextureGraphErrorReporter>			ErrorReporterPtr;

//////////////////////////////////////////////////////////////////////////
/// The Engine is supposed to maintain a global state for the application
/// and giving access to various shared data structures.
/// This is just a place for accessing the state of the application
/// and NOT meant to be a place for doing a lot of the things.
//////////////////////////////////////////////////////////////////////////
class TextureGraphEngine
{
private:
	static UE_API TextureGraphEngine*							GInstance;					/// Static instance. Use Create to create a new instance. Destroy the current instance first
	static UE_API std::atomic<bool>							bGIsEngineDestroying;		/// Whether the Engine is in a state of being destroyed
	
	bool												bIsEngineInitDone = false;	/// Whether the Engine init is actually done. This is to peg it to the editor

	UE_API explicit											TextureGraphEngine(bool bInIsTestMode);

	UE_API void												InitEngineInternal();		/// This is to allow modules of Engine to access s_instance

														UE_API ~TextureGraphEngine();

	static UE_API EngineObserverSourcePtr						GObserverSource;			/// Engine observer source interface where public signals will be emitted from. 
																					/// Default observerSource is the default class implementation which is a no-op
	
	TStrongObjectPtr<UMaterialManager>					MaterialMgrObj;				/// The material manager
	MixManagerUPtr										MixMgrObj;					/// Mix manager
	SchedulerPtr										SchedulerObj;				/// The job scheduler
	DeviceManagerPtr									DeviceManagerObj;			/// The device manager
	BlobberPtr											BlobberObj;					/// The blobber that is responsible for managing all the blobs and their transfers across devices
	TextureGraphEditor::RenderDocManagerPtr				RenderDocMgrObj;			/// The RenderDoc manager that we have
	TMap<const UMixInterface*, ErrorReporterPtr>		ErrorReporters;				/// Error reporters mapped by Mix (each mix has their own reporter to allow
																					/// managing errors (report/resolve) per mix
	uint64												FrameId = 1;				/// Frame ID that we keep

	/// Test related functionality
	bool												bIsTestMode = false;		/// Whether the engine is running in unit test mode or not
	std::atomic_bool									bRunEngine = false;			/// Whether to run the engine or not. If no one is using TextureGraph, 
																					/// then the engine shouldn't be running in the background

	std::atomic_bool									bIsLocked = false;			/// Whether the engine is locked. This is only available in test mode.
																					/// This is currently being used to detect and identify hard to find 
																					/// race condition(s) where engine may start getting destroyed
																					/// while the tests are still running.

	std::mutex*											LockWaitMutex = nullptr;	/// Mutex used to wait if the engine is locked and asked to be destroyed
	std::condition_variable*							LockWaitCVar = nullptr;		/// Conditional variable to be used in conjunction with the mutex

	UE_API void												FirstRunInit();

public:
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	/// Init can be called again and again to reset the engine
	/// and recreate the data structures
	static UE_API void											Create(bool bIsTestMode = false);

	/// Destroy the current instance
	static UE_API void											Destroy();
	
	static UE_API void											InitTests();
	static UE_API void											DoneTests();

	/// Update the components in the engine
	static UE_API void											Update(float dt);

	/// Register the observer source (only one active)
	static UE_API void											RegisterObserverSource(const EngineObserverSourcePtr& observerSource);

	/// Register the Error Reporter
	static UE_API void											RegisterErrorReporter(const UMixInterface* MixKey, const ErrorReporterPtr& ErrorReporter);

	static UE_API void											Lock();
	static UE_API void											Unlock();
	static UE_API void											SetRunEngine();

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	/// The if check is temporary over here since no one is initialising this right now
	FORCEINLINE static TextureGraphEngine*				GetInstance() { return GInstance; }
	FORCEINLINE static MixManager*						GetMixManager() { return GInstance ? GInstance->MixMgrObj.get() : nullptr; }
	FORCEINLINE static UMaterialManager*				GetMaterialManager() { return GInstance ? GInstance->MaterialMgrObj.Get() : nullptr; }
	FORCEINLINE static Scheduler*						GetScheduler() { return GInstance ? GInstance->SchedulerObj.get() : nullptr; }
	FORCEINLINE static DeviceManager*					GetDeviceManager() { return GInstance ? GInstance->DeviceManagerObj.get() : nullptr; }
	FORCEINLINE static Blobber*							GetBlobber() { return GInstance ? GInstance->BlobberObj.get() : nullptr; }
	FORCEINLINE static int64							GetFrameId() { return GInstance ? GInstance->FrameId : 0; }
	FORCEINLINE static TextureGraphEditor::RenderDocManager* GetRenderDocManager() { return GInstance ? GInstance->RenderDocMgrObj.get() : nullptr; }

	FORCEINLINE static bool								IsTestMode() { return GInstance ? GInstance->bIsTestMode : false; }

	FORCEINLINE static bool								IsDestroying() { return bGIsEngineDestroying; }
	FORCEINLINE static EngineObserverSourcePtr			GetObserverSource() { return GObserverSource; }
	FORCEINLINE static FTextureGraphErrorReporter*		GetErrorReporter(UMixInterface* MixKey) { return GInstance ? GInstance->ErrorReporters.Find(MixKey)->get() : nullptr; }

	FORCEINLINE static bool								IsRunning() { return GInstance ? GInstance->bRunEngine.load() : false; }
	static bool											BlockCommandlets();
};


#undef UE_API
