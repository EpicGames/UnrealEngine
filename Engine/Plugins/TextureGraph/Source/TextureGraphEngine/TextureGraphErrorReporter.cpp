// Copyright Epic Games, Inc. All Rights Reserved.
#include "TextureGraphErrorReporter.h"

#include "TextureGraphEngine.h"
#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY(LogTextureGraphError);

FTextureGraphErrorReport FTextureGraphErrorReporter::ReportLog(int32 ErrorId, const FString& ErrorMsg, UObject* ReferenceObj /*= nullptr*/)
{
	FTextureGraphErrorReport Report{ErrorId, ErrorMsg, ReferenceObj};
	UE_LOGF(LogTextureGraphError, Log, "ErrorReporter: %ls", *Report.GetFormattedMessage());
	return Report;
}

FTextureGraphErrorReport FTextureGraphErrorReporter::ReportWarning(int32 ErrorId, const FString& ErrorMsg, UObject* ReferenceObj /*= nullptr*/)
{
	FTextureGraphErrorReport Report{ErrorId, ErrorMsg, ReferenceObj};
	// Don't throw Warning assert when in test mode
	if (!TextureGraphEngine::IsTestMode())
	{
		UE_LOGF(LogTextureGraphError, Warning, "ErrorReporter: %ls", *Report.GetFormattedMessage());
	}
	return Report; 
}

FTextureGraphErrorReport FTextureGraphErrorReporter::ReportError(int32 ErrorId, const FString& ErrorMsg, UObject* ReferenceObj /*= nullptr*/)
{
	FTextureGraphErrorReport Report{ErrorId, ErrorMsg, ReferenceObj};
	// Don't throw Error assert when in test mode
	if (!TextureGraphEngine::IsTestMode())
	{
		UE_LOGF(LogTextureGraphError, Error, "ErrorReporter: %ls", *Report.GetFormattedMessage());
	}
	return Report; 
}


