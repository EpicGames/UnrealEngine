// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"

#define UE_API TEXTUREGRAPHENGINE_API

DECLARE_LOG_CATEGORY_EXTERN(LogTextureGraphError, Log, All);
struct FTextureGraphErrorReport
{
	int32 ErrorId;
	FString ErrorMsg;
	UObject* ReferenceObj = nullptr;
	
	FString GetFormattedMessage()
	{
		return FString::Format(TEXT("({0}) {1}"), { ErrorId, *ErrorMsg });
	}
};
using ErrorReportMap = TMap<int32, TArray<FTextureGraphErrorReport>>;
enum class ETextureGraphErrorType: int32
{
	UNSUPPORTED_TYPE,
	MISSING_REQUIRED_INPUT,
	RECURSIVE_CALL,
	UNSUPPORTED_MATERIAL,
	SUBGRAPH_INTERNAL_ERROR,
	NODE_WARNING,
	INPUT_ARRAY_WARNING
};
class FTextureGraphErrorReporter
{
public:
	FTextureGraphErrorReporter()
	{
	}
	
	virtual ~FTextureGraphErrorReporter() = default;
	
	UE_API virtual FTextureGraphErrorReport ReportLog(int32 ErrorId, const FString& ErrorMsg, UObject* ReferenceObj = nullptr);

	UE_API virtual FTextureGraphErrorReport ReportWarning(int32 ErrorId, const FString& ErrorMsg, UObject* ReferenceObj = nullptr);

	UE_API virtual FTextureGraphErrorReport ReportError(int32 ErrorId, const FString& ErrorMsg, UObject* ReferenceObj = nullptr);
	
	virtual void Clear()
	{
		CompilationErrors.Empty();
	}
protected:
	ErrorReportMap CompilationErrors;

public:
	ErrorReportMap GetCompilationErrors() const
	{
		return CompilationErrors;
	}
};

#undef UE_API
