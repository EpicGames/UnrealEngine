// Copyright Epic Games, Inc. All Rights Reserved.
#include "BlobTransform.h"
#include "Data/Blob.h"
#include "Device/Device.h"
#include "Job/Job.h"
#include "Helper/GraphicsUtil.h"


DEFINE_LOG_CATEGORY(LogBlobTransform);

BlobTransform::BlobTransform(FString Name_) 
	: Name(!Name_.IsEmpty() ? Name_ : FString("<Unknown>"))
	, HashValue(std::make_shared<CHash>(DataUtil::Hash_GenericString_Name(Name), true))
{
}

BlobTransform::~BlobTransform()
{
}

//BlobPtr BlobTransform::Target()
//{
//	return _target;
//}
//
//void BlobTransform::SetTarget(BlobPtr target)
//{
//	check(!_target || _target == _target);
//	_target = target;
//}

AsyncBufferResultPtr BlobTransform::Bind(BlobPtr BlobObj, const ResourceBindInfo& BindInfo)
{
	return BlobObj->Bind(this, BindInfo);
}

bool BlobTransform::GeneratesData() const
{
	return true;
}

bool BlobTransform::CanHandleTiles() const
{
	return true;
}

AsyncPrepareResult BlobTransform::PrepareResources(const TransformArgs& Args)
{
	return cti::make_ready_continuable(0);
}

AsyncBufferResultPtr BlobTransform::Unbind(BlobPtr BlobObj, const ResourceBindInfo& BindInfo)
{
	return BlobObj->Unbind(this, BindInfo);
}

CHashPtr BlobTransform::Hash() const
{
	if (HashValue)
		return HashValue;

	HashValue = std::make_shared<CHash>(DataUtil::Hash_GenericString_Name(Name), true);

	return HashValue;
}

ENamedThreads::Type BlobTransform::ExecutionThread() const
{
	return ENamedThreads::AnyThread;
}

bool BlobTransform::IsAsync() const
{
	return false;
}

void BlobTransform::Bind(const Vector3& InValue, const ResourceBindInfo& BindInfo)
{
	const FLinearColor Value(InValue.X, InValue.Y, InValue.Z, 0.0f);
	Bind(Value, BindInfo);
}

void BlobTransform::Unbind(const Vector3& InValue, const ResourceBindInfo& BindInfo)
{
	const FLinearColor Value(InValue.X, InValue.Y, InValue.Z, 0.0f);
	Bind(Value, BindInfo);
}

void BlobTransform::Bind(const Vector4& InValue, const ResourceBindInfo& BindInfo)
{
	const FLinearColor Value(InValue);
	Bind(Value, BindInfo);
}

void BlobTransform::Unbind(const Vector4& InValue, const ResourceBindInfo& BindInfo)
{
	const FLinearColor Value(InValue);
	Bind(Value, BindInfo);
}

void BlobTransform::Bind(const Tangent& InValue, const ResourceBindInfo& BindInfo)
{
	FLinearColor Value(GraphicsUtil::ConvertTangent(InValue));
	Bind(Value, BindInfo);
}

void BlobTransform::Unbind(const Tangent& InValue, const ResourceBindInfo& BindInfo)
{
	FLinearColor Value(GraphicsUtil::ConvertTangent(InValue));
	Bind(Value, BindInfo);
}

//////////////////////////////////////////////////////////////////////////
Null_Transform::Null_Transform(Device* Dev, FString Name, bool bIsAsync_ /* = false */, bool bGeneratesData_ /* = false */)
	: BlobTransform(Name)
	, Dev(Dev)
	, bIsAsync(bIsAsync_)
	, bGeneratesData(bGeneratesData_)
{
	verify(Dev);
}

Device* Null_Transform::TargetDevice(size_t index) const
{
	return Dev;
}

bool Null_Transform::IsAsync() const
{
	return bIsAsync;
}

bool Null_Transform::GeneratesData() const
{
	return bGeneratesData;
}

AsyncTransformResultPtr	Null_Transform::Exec(const TransformArgs& Args)
{
	return cti::make_ready_continuable(std::make_shared<TransformResult>());
}

std::shared_ptr<BlobTransform> Null_Transform::DuplicateInstance(FString TransformName)
{
	return std::make_shared<Null_Transform>(Dev, TransformName);
}

AsyncBufferResultPtr Null_Transform::Bind(BlobPtr BlobObj, const ResourceBindInfo& BindInfo)
{
	return cti::make_ready_continuable<BufferResultPtr>(std::make_shared<BufferResult>());
}

AsyncBufferResultPtr Null_Transform::Unbind(BlobPtr BlobObj, const ResourceBindInfo& BindInfo)
{
	return cti::make_ready_continuable<BufferResultPtr>(std::make_shared<BufferResult>());
}


