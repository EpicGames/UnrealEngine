// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Data/RawBuffer.h"
#include "Data/Blobber.h"
#include "GraphicsDefs.h"
#include <vector>

#define UE_API TEXTUREGRAPHENGINE_API

class Device;
class Blob;
class DeviceNativeTask;
typedef std::shared_ptr<DeviceNativeTask>	DeviceNativeTaskPtr;
typedef std::weak_ptr<DeviceNativeTask>		DeviceNativeTaskPtrW;
typedef std::vector<DeviceNativeTaskPtr>	DeviceNativeTaskPtrVec;

DECLARE_LOG_CATEGORY_EXTERN(LogBlobTransform, Log, Log);

//////////////////////////////////////////////////////////////////////////
/// Provides additional binding information for the resource 
//////////////////////////////////////////////////////////////////////////
struct ResourceBindInfo
{
	FString							Target;						/// Named target (for devices that support this)
	int32							TargetIndex = -1;			/// Indexed target (for devices that support this)
	Device*							Dev = nullptr;				/// The Dev to which the resource will be bound
	bool							bCalculateHash = false;		/// Calculate hash immediately
	bool							bWriteTarget = false;		/// Whether the resource is a write target or not
	bool							bStaticBuffer = false;		/// Whether its a static buffer or not. Only used with writeTarget == true
	bool							bIsCombined = false;		/// Used to transfer information to material if texture being registered to array parameter
	int32							NumTilesX = 0;				/// [Optional] Number of xtiles
	int32							NumTilesY = 0;				/// [Optional] Number of ytiles
	uint64							BatchId = 0;				/// [Optional] The batch id when this was last bound
};

//////////////////////////////////////////////////////////////////////////
class Job;
class MixUpdateCycle;
class RenderMesh;

struct TransformArgs
{
	int32							TargetId = -1;				/// The target id
	const Job*						JobObj = nullptr;			/// The job that this is being triggered from
	Device*							Dev = nullptr;				/// The Dev on which to execute the transform. 
																/// If this is NULL then execute on the default Dev

	BlobRef							Target;						/// The target that the BlobObj transform is going to have to render to.
																/// If this is not given then the transform must create an appropriate
																/// target to render to

	const RenderMesh*				Mesh = nullptr;				/// The mesh that we are targeting

	std::shared_ptr<MixUpdateCycle> Cycle;					/// The update cycle
};

//////////////////////////////////////////////////////////////////////////
struct TransformResult
{
	std::exception_ptr				ExInner;					/// Original exception that was raised by the action
	int32							ErrorCode = 0;				/// What is the error code
	BlobRef							Target;						/// The target that was passed in TransformArgs returned
};

typedef std::shared_ptr<TransformResult>	TransformResultPtr;
typedef cti::continuable<TransformResultPtr> AsyncTransformResultPtr;
typedef cti::continuable<int32>				AsyncPrepareResult;

//////////////////////////////////////////////////////////////////////////
/// BlobTransform: Abstract class
/// This transformation 'function' targets a specific Dev. It then
/// takes one or more datasets as inputs (think shader parameters or
/// function arguments) and then produces the result in a new BlobObj.
/// The new BlobObj is unique and may be kept for later by the caller.
/// The Exec function is deliberately synchronous because the job
/// of having asynchronous function is left upto the Job class, which 
/// is responsible for using the BlobTransform functions.
//////////////////////////////////////////////////////////////////////////
class BlobTransform
{
public:
	

protected:
	FString							Name;						/// User friendly Name of the transformation function
	mutable CHashPtr				HashValue;						/// The hash for this BlobObj transform

public:
									UE_API explicit BlobTransform(FString Name);
	UE_API virtual							~BlobTransform();

	virtual size_t					NumTargetDevices() const { return 1; }
	virtual Device*					TargetDevice(size_t Index) const = 0;
	virtual AsyncTransformResultPtr	Exec(const TransformArgs& Args) = 0;
	UE_API virtual CHashPtr				Hash() const;
	virtual std::shared_ptr<BlobTransform> DuplicateInstance(FString TransformName) { return nullptr; }

	//virtual BlobPtr					Target();
	//virtual void					SetTarget(BlobPtr target);

	UE_API virtual AsyncBufferResultPtr	Bind(BlobPtr BlobObj, const ResourceBindInfo& BindInfo);
	UE_API virtual AsyncBufferResultPtr	Unbind(BlobPtr BlobObj, const ResourceBindInfo& BindInfo);

	virtual void					Bind(int32 InValue, const ResourceBindInfo& BindInfo) {}
	virtual void					Bind(float InValue, const ResourceBindInfo& BindInfo) {}
	virtual void					Bind(const FLinearColor& InValue, const ResourceBindInfo& BindInfo) {}
	virtual void					Bind(const FIntVector4& InValue, const ResourceBindInfo& BindInfo) {}
	virtual void					Bind(const FMatrix& InValue, const ResourceBindInfo& BindInfo) {}
	virtual void					BindStruct(const char* ValueAddress, size_t StructSize, const ResourceBindInfo& BindInfo) {}
	virtual void					BindScalarArray(const char* StartingAddress, size_t TypeSize, size_t ArraySize, const ResourceBindInfo& BindInfo) {}

	template <typename Type>
	void							BindScalarArray(const TArray<Type>& InValue, const ResourceBindInfo& BindInfo) 
	{
		const char* valueAddress = reinterpret_cast<const char*>(&InValue[0]);
		BindScalarArray(valueAddress, sizeof(Type), InValue.Num(), BindInfo);
	}
	
	template<typename StructType>
	void							BindStruct(const StructType& InValue, const ResourceBindInfo& BindInfo)
	{
		const char* valueAddress = reinterpret_cast<const char*>(&InValue);
		BindStruct(valueAddress, sizeof(StructType), BindInfo);
	}
	UE_API virtual void					Bind(const Vector3& InValue, const ResourceBindInfo& BindInfo);
	UE_API virtual void					Bind(const Vector4& InValue, const ResourceBindInfo& BindInfo);
	UE_API virtual void					Bind(const Tangent& InValue, const ResourceBindInfo& BindInfo);
	virtual void					Bind(const FString& InValue, const ResourceBindInfo& BindInfo) {}

	UE_API virtual bool					GeneratesData() const;
	UE_API virtual bool					CanHandleTiles() const;

	UE_API virtual AsyncPrepareResult		PrepareResources(const TransformArgs& Args);


	/// No-Ops
	virtual void					Unbind(int32 InValue, const ResourceBindInfo& BindInfo) {}
	virtual void					Unbind(float InValue, const ResourceBindInfo& BindInfo) {}
	virtual void					Unbind(const FLinearColor& InValue, const ResourceBindInfo& BindInfo) {}
	virtual void					Unbind(const FIntVector4& InValue, const ResourceBindInfo& BindInfo) {}
	virtual void					Unbind(const FMatrix& InValue, const ResourceBindInfo& BindInfo) {}
	UE_API virtual void					Unbind(const Vector3& InValue, const ResourceBindInfo& BindInfo);
	UE_API virtual void					Unbind(const Vector4& InValue, const ResourceBindInfo& BindInfo);
	UE_API virtual void					Unbind(const Tangent& InValue, const ResourceBindInfo& BindInfo);
	virtual void					Unbind(const FString& InValue, const ResourceBindInfo& BindInfo) {}

	UE_API virtual ENamedThreads::Type		ExecutionThread() const;
	UE_API virtual bool					IsAsync() const;
	virtual FString					GetName() const { return Name; }


	/// During the execution of the transform, errors can be raised to the TextureGraphEngine::ErrorReporter
	/// and associated to the "ErrorOwner" object provided by this method.
	/// return null by default
	virtual UObject*				GetErrorOwner() const { return nullptr; }

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
};

//////////////////////////////////////////////////////////////////////////
/// A NULL_Transform that can be used for jobs that do not require
/// a transform. This is for completeness so that we don't have to 
/// put checks in places that are expecting a transform
//////////////////////////////////////////////////////////////////////////
class Null_Transform : public BlobTransform
{
private:
	Device*							Dev = nullptr;			/// The device that this transform belongs to
	bool							bIsAsync = false;		/// Whether the transform is async or not
	bool							bGeneratesData = false;	/// The null transform doesn't generate data by default

public:
									UE_API Null_Transform(Device* Dev, FString Name, bool bIsAsync_ = false, bool bGeneratesData_ = false);
	UE_API virtual AsyncBufferResultPtr	Bind(BlobPtr InValue, const ResourceBindInfo& BindInfo) override;
	UE_API virtual AsyncBufferResultPtr	Unbind(BlobPtr BlobObj, const ResourceBindInfo& BindInfo) override;
	UE_API virtual bool					IsAsync() const override;
	UE_API virtual bool					GeneratesData() const override;
	UE_API virtual Device*					TargetDevice(size_t Index) const override;
	UE_API virtual AsyncTransformResultPtr	Exec(const TransformArgs& Args) override;
	UE_API virtual std::shared_ptr<BlobTransform> DuplicateInstance(FString TransformName) override;
};

typedef std::shared_ptr<BlobTransform> BlobTransformPtr;

#undef UE_API
