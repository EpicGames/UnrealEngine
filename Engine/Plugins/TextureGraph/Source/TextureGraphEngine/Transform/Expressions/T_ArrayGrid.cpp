// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_ArrayGrid.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "Math/Vector.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"
#include "Transform/Utility/T_CombineTiledBlob.h"
#include "Transform/Utility/T_SplitToTiles.h"
#include "Model/Mix/MixInterface.h"
#include "Device/FX/Device_FX.h"

class TEXTUREGRAPHENGINE_API Job_ArrayGrid : public Job
{
private:
	TArray<TiledBlobPtr>			Inputs;			/// The inputs to this array grid
	int32							NumRows = 0;	/// Number of rows
	int32							NumCols = 0;	/// Number of columns
	uint32							MaxColWidth = 0;/// Maximum column width
	uint32							MaxRowHeight = 0;/// Maximum row height
	BufferDescriptor				InDesiredOutputDesc;	/// The desired input desc
	TiledBlob_PromisePtr			TiledInput;		/// The tiled input which is a combination of all the array tiles

public:
	Job_ArrayGrid(UMixInterface* InMix, int32 TargetId, TArray<TiledBlobPtr> InInputs, 
		int32 InNumRows, int32 InNumCols, uint32 InMaxColWidth, uint32 InMaxRowHeight, BufferDescriptor& InDesiredOutputDesc_, 
		TiledBlob_PromisePtr InTiledInput)
		: Job(InMix, TargetId, std::make_shared<Null_Transform>(Device_FX::Get(), TEXT("T_ArrayGrid"), true, true))
		, Inputs(InInputs)
		, NumRows(InNumRows)
		, NumCols(InNumCols)
		, MaxColWidth(InMaxColWidth)
		, MaxRowHeight(InMaxRowHeight)
		, InDesiredOutputDesc(InDesiredOutputDesc_)
		, TiledInput(InTiledInput)
	{
		Name = TEXT("T_ArrayGrid");
	}

	static BlobPtrTiles				GetTiles(TArray<TiledBlobPtr>& Inputs, int32 InNumRows, int32 InNumCols, 
		int32 NumRows, int32 NumCols, uint32* OutMaxRowHeight = nullptr, uint32* OutMaxColWidth = nullptr, 
		uint32* OutWidth = nullptr, uint32 *OutHeight = nullptr,
		bool* bIsWellDefinedOut = nullptr, CHashPtrVec* OutHashes = nullptr)
	{
		BlobPtrTiles ResultTiles(InNumRows * NumRows, InNumCols * NumCols); 
		CHashPtrVec InputHashes(NumRows * NumCols);
		bool bIsWellDefined = true;
		uint32 MaxRowHeight = 0;
		uint32 MaxColWidth = 0;
		uint32 Width = 0;
		uint32 Height = 0;
		size_t HashIndex = 0;

		/// First of all we combine all the tiles into a single result
		for (int32 ArrRowIter = 0; ArrRowIter < NumRows; ArrRowIter++)
		{
			int32 RowIndex = ArrRowIter * InNumCols;
			uint32 RowWidth = 0;
			for (int32 ArrColIter = 0; ArrColIter < NumCols; ArrColIter++)
			{
				int32 TileIndex = ArrRowIter * NumCols + ArrColIter;
				check(TileIndex < Inputs.Num());
				TiledBlobPtr Input = Inputs[TileIndex];
				int32 ColIndex = ArrColIter * InNumCols;

				MaxRowHeight = std::max(MaxRowHeight, Input->GetHeight());
				MaxColWidth = std::max(MaxColWidth, Input->GetWidth());
				RowWidth += Input->GetWidth();

				check(HashIndex < InputHashes.size());
				check(Input->Hash());
				InputHashes[HashIndex++] = Input->Hash();

				bIsWellDefined &= Input->IsWellDefined();

				/// Ok here we copy the tiles over to the result tiles. However, there can be situations where the 
				/// input has fewer tiles than the output (1x1 as opposed to 8x8 for instance). In this case
				/// we just copy clamped (RowId, ColId) over to the destination
				for (int32 RowId = 0; RowId < InNumRows; RowId++)
				{
					int32 InputRowId = std::min<int32>(RowId, (int32)Input->Rows() - 1);
					for (int32 ColId = 0; ColId < InNumCols; ColId++)
					{
						int32 InputColId = std::min<int32>(ColId, (int32)Input->Cols() - 1);
						BlobPtr Tile = Input->GetTile(InputRowId, InputColId);
						ResultTiles[RowIndex + RowId][ColIndex + ColId] = Tile;

						if (Tile)
							bIsWellDefined &= Tile->IsWellDefined();
						else
							bIsWellDefined = false;
					}
				}
			}

			Width = std::max(Width, RowWidth);
			Height += MaxRowHeight;
		}

		if (OutWidth)
			*OutWidth = Width;
		if (OutHeight)
			*OutHeight = Height;
		if (OutMaxRowHeight)
			*OutMaxRowHeight = MaxRowHeight;
		if (OutMaxColWidth)
			*OutMaxColWidth = MaxColWidth;
		if (bIsWellDefinedOut)
			*bIsWellDefinedOut = bIsWellDefined;
		if (OutHashes)
			*OutHashes = std::move(InputHashes);

		return ResultTiles;
	}

protected:
	virtual AsyncPrepareResult		PrepareTargets(JobBatch* Batch) override
	{
		int32 InNumRows = Inputs[0]->Rows();
		int32 InNumCols = Inputs[0]->Cols();
		uint32 Width = 0, Height = 0;
		BlobPtrTiles ResultTiles = GetTiles(Inputs, InNumRows, InNumCols, NumRows, NumCols, &MaxRowHeight, &MaxColWidth, &Width, &Height);

		/// The we combine those tiles into a single result
		const BufferDescriptor& InputDesc = Inputs[0]->GetDescriptor();
		BufferDescriptor DesiredOutputDesc = BufferDescriptor::Combine(InputDesc, InDesiredOutputDesc);

		if (DesiredOutputDesc.IsLateBound())
			DesiredOutputDesc.Format = InputDesc.Format;

		DesiredOutputDesc.Width = Width;
		DesiredOutputDesc.Height = Height;

		TiledBlobPtr CombinedResultTiled = TiledBlob::InitFromTiles(DesiredOutputDesc, ResultTiles);

		if (ResultOrg)
			ResultOrg->ResolveLateBound(DesiredOutputDesc, true);

		if (Result && Result.get().get() != ResultOrg.get())
			Result->ResolveLateBound(DesiredOutputDesc, true);

		std::shared_ptr<CombineTiledBlob_Transform> CombinedTiledBlobMat = std::make_shared<CombineTiledBlob_Transform>(Name, CombinedResultTiled);
		Job::Transform = CombinedTiledBlobMat;

		return Job::PrepareTargets(Batch);
	}
};

//////////////////////////////////////////////////////////////////////////
TiledBlobPtr T_ArrayGrid::Create(MixUpdateCyclePtr Cycle, BufferDescriptor& InDesiredOutputDesc, TArray<TiledBlobPtr> Inputs, 
	int32 NumRows, int32 NumCols, FLinearColor BackgroundColor, int32 TargetId)
{
	check(!Inputs.IsEmpty());

	int32 InNumRows = Cycle->GetMix()->GetNumXTiles();
	int32 InNumCols = Cycle->GetMix()->GetNumYTiles();

	CHashPtrVec InputHashes(Inputs.Num());
	bool bIsWellDefined = true;
	uint32 MaxRowHeight = 0;
	uint32 MaxColWidth = 0;
	uint32 Width = 0;
	uint32 Height = 0;

	BlobPtrTiles ResultTiles = Job_ArrayGrid::GetTiles(Inputs, InNumRows, InNumCols, NumRows, NumCols, 
		&MaxColWidth, &MaxRowHeight, &Width, &Height, 
		&bIsWellDefined, &InputHashes);
	TiledBlobPtr FinalResult;

	CombineSettings Settings
	{
		.bFixed = false,
		.bMaintainAspectRatio = true,
		.BackgroundColor = BackgroundColor
	};

	BufferDescriptor InputDesc = Inputs[0]->GetDescriptor();
	InputDesc.Width = Width;
	InputDesc.Height = Height;

	BufferDescriptor DesiredOutputDesc = BufferDescriptor::CombineWithPreference(&InputDesc, &InDesiredOutputDesc, nullptr);

	/// If the result is late bound then we have no option but to resize the result render target through a custom job
	if (!bIsWellDefined)
	{
		CHashPtr Hash = CHash::ConstructFromSources(InputHashes);
		BufferDescriptor Desc = InDesiredOutputDesc;
		Desc.Format = BufferFormat::LateBound;
		Desc.Width = Desc.Height = 0;
		TiledBlob_PromisePtr CombinedResultTiled = std::make_shared<TiledBlob_Promise>(Desc, InNumRows * NumRows, InNumCols * NumCols, Hash);
		JobUPtr JobObj = std::make_unique<Job_ArrayGrid>(Cycle->GetMix(), TargetId, Inputs, NumRows, NumCols, MaxColWidth, MaxRowHeight, InDesiredOutputDesc, CombinedResultTiled);

		for (int32 InputIndex = 0; InputIndex < Inputs.Num(); InputIndex++)
			JobObj->AddArg(WithUnbounded(ARG_BLOB(Inputs[InputIndex], "Input")));

		TiledBlobPtr CombinedResult = T_CombineTiledBlob::Create(Cycle, InputDesc, TargetId, CombinedResultTiled, std::move(JobObj), nullptr);
		TiledBlobPtr CombinedResultOne = CombinedResult;

		if (InDesiredOutputDesc.Width != 0 || InDesiredOutputDesc.Height != 0)
			CombinedResultOne = T_CombineTiledBlob::Create(Cycle, DesiredOutputDesc, TargetId, CombinedResult, nullptr, &Settings);

		FinalResult = T_SplitToTiles::Create(Cycle, TargetId, CombinedResultOne);
	}
	else
	{
		/// The we combine those tiles into a single result
		TiledBlobPtr CombinedResultTiled = TiledBlob::InitFromTiles(InputDesc, ResultTiles);
		TiledBlobPtr CombinedResultOne = T_CombineTiledBlob::Create(Cycle, DesiredOutputDesc, TargetId, CombinedResultTiled, nullptr, &Settings);

		if (DesiredOutputDesc.Width > 1 && DesiredOutputDesc.Height > 1)
			FinalResult = T_SplitToTiles::Create(Cycle, TargetId, CombinedResultOne);
	}

	return FinalResult;
}
