// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Math/Vector2D.h"

#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
class T_ArrayGrid
{
public:
	static UE_API TiledBlobPtr Create(MixUpdateCyclePtr Cycle, BufferDescriptor& DesiredOutputDesc, TArray<TiledBlobPtr> Inputs, 
		int32 NumRows, int32 NumCols, FLinearColor BackgroundColor, int32 TargetId = 0);
};

#undef UE_API
