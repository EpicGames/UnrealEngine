// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_Blend.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"

IMPLEMENT_GLOBAL_SHADER(FSH_BlendNormal		, "/Plugin/TextureGraph/Expressions/Expression_Blend.usf", "FSH_BlendNormal"	, SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_BlendAdd		, "/Plugin/TextureGraph/Expressions/Expression_Blend.usf", "FSH_BlendAdd"		, SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_BlendSubtract	, "/Plugin/TextureGraph/Expressions/Expression_Blend.usf", "FSH_BlendSubtract"	, SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_BlendMultiply	, "/Plugin/TextureGraph/Expressions/Expression_Blend.usf", "FSH_BlendMultiply"	, SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_BlendDivide		, "/Plugin/TextureGraph/Expressions/Expression_Blend.usf", "FSH_BlendDivide"	, SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_BlendDifference	, "/Plugin/TextureGraph/Expressions/Expression_Blend.usf", "FSH_BlendDifference", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_BlendMax		, "/Plugin/TextureGraph/Expressions/Expression_Blend.usf", "FSH_BlendMax"		, SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_BlendMin		, "/Plugin/TextureGraph/Expressions/Expression_Blend.usf", "FSH_BlendMin"		, SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_BlendStep		, "/Plugin/TextureGraph/Expressions/Expression_Blend.usf", "FSH_BlendStep"		, SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_BlendOverlay	, "/Plugin/TextureGraph/Expressions/Expression_Blend.usf", "FSH_BlendOverlay"	, SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_BlendDistort	, "/Plugin/TextureGraph/Expressions/Expression_Blend.usf", "FSH_BlendDistort"	, SF_Pixel);

T_Blend::T_Blend()
{
}

T_Blend::~T_Blend()
{
}

TiledBlobPtr T_Blend::Create(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, EBlendModes::Type InBlendMode, const FBlendSettings* InBlendSettings)
{
	switch(InBlendMode)
	{
		case EBlendModes::Normal:
			return CreateNormal(InCycle, DesiredDesc, InTargetId, InBlendSettings);
		case EBlendModes::Add:
			return CreateAdd(InCycle, DesiredDesc, InTargetId, InBlendSettings);
		case EBlendModes::Subtract:
			return CreateSubtract(InCycle, DesiredDesc, InTargetId, InBlendSettings);
		case EBlendModes::Multiply:
			return CreateMultiply(InCycle, DesiredDesc, InTargetId, InBlendSettings);
		case EBlendModes::Divide:
			return CreateDivide(InCycle, DesiredDesc, InTargetId, InBlendSettings);
		case EBlendModes::Difference:
			return CreateDifference(InCycle, DesiredDesc, InTargetId, InBlendSettings);
		case EBlendModes::Max:
			return CreateMax(InCycle, DesiredDesc, InTargetId, InBlendSettings);
		case EBlendModes::Min:
			return CreateMin(InCycle, DesiredDesc, InTargetId, InBlendSettings);
		case EBlendModes::Step:
			return CreateStep(InCycle, DesiredDesc, InTargetId, InBlendSettings);
		case EBlendModes::Overlay:
			return CreateOverlay(InCycle, DesiredDesc, InTargetId, InBlendSettings);
		// case EBlendModes::Distort:
		// 	return CreateDistort(InCycle, DesiredDesc, InBackgroundTexture, InForeGroundTexture, InMask, InOpacity, InTargetId);

	default:
		// Unhandled case
		checkNoEntry();
		return TextureHelper::GetBlack();
	}
}

template <typename FSH_Type>
TiledBlobPtr CreateGenericBlend(MixUpdateCyclePtr InCycle, int32 InTargetId, BufferDescriptor DesiredDesc,FString InTransformName, const T_Blend::FBlendSettings* InBlendSettings)
{
	FSH_BlendBase::FPermutationDomain PermutationVector;
	PermutationVector.Set<FSH_BlendBase::FIgnoreAlpha>(InBlendSettings->bIgnoreAlpha);
	PermutationVector.Set<FSH_BlendBase::FClamp>(InBlendSettings->bClamp);
	
	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_Type>(InTransformName, PermutationVector);
	check(RenderMaterial);

	TiledBlobPtr BackgroundTexture = InBlendSettings->BackgroundTexture; 
	TiledBlobPtr ForegroundTexture = InBlendSettings->ForegroundTexture; 
	TiledBlobPtr MaskTexture = InBlendSettings->Mask; 
	
	if (!BackgroundTexture)
	{
		BackgroundTexture = TextureHelper::GetBlack();
	}
	
	if (!ForegroundTexture)
	{
		ForegroundTexture = TextureHelper::GetBlack();
	}
	
	if(!MaskTexture)
	{
		MaskTexture = TextureHelper::GetWhite();
	}
	
	JobUPtr JobPtr = std::make_unique<Job>(InCycle->GetMix(), InTargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	
	JobPtr
		->AddArg(ARG_BLOB(BackgroundTexture, "BackgroundTexture"))
		->AddArg(ARG_BLOB(ForegroundTexture, "ForegroundTexture"))
		->AddArg(ARG_BLOB(MaskTexture, "MaskTexture"))
		->AddArg(ARG_FLOAT(InBlendSettings->Opacity, "Opacity"))
		->AddArg(WithUnbounded(ARG_BOOL(InBlendSettings->bIgnoreAlpha, "IgnoreAlpha")))
		->AddArg(WithUnbounded(ARG_BOOL(InBlendSettings->bClamp, "Clamp")))
		;

	const FString Name = FString::Printf(TEXT("[%llu] - Blend - %s"), InCycle->GetBatch()->GetBatchId(), *InTransformName);

	TiledBlobPtr Result = JobPtr->InitResult(Name, &DesiredDesc);

	InCycle->AddJob(InTargetId, std::move(JobPtr));

	return Result;
}

TiledBlobPtr T_Blend::CreateNormal(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings)
{
	return CreateGenericBlend<FSH_BlendNormal>(InCycle, InTargetId, DesiredDesc, "T_BlendNormal", InBlendSettings);
}

TiledBlobPtr T_Blend::CreateAdd(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings)
{
	return CreateGenericBlend<FSH_BlendAdd>(InCycle, InTargetId, DesiredDesc, "T_BlendAdd", InBlendSettings);
}

TiledBlobPtr T_Blend::CreateSubtract(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings)
{
	return CreateGenericBlend<FSH_BlendSubtract>(InCycle, InTargetId, DesiredDesc, "T_BlendSubtract", InBlendSettings);
}

TiledBlobPtr T_Blend::CreateMultiply(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings)
{
	return CreateGenericBlend<FSH_BlendMultiply>(InCycle, InTargetId, DesiredDesc, "T_BlendMultiply", InBlendSettings);
}

TiledBlobPtr T_Blend::CreateDivide(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings)
{
	return CreateGenericBlend<FSH_BlendDivide>(InCycle, InTargetId, DesiredDesc, "T_BlendDivide", InBlendSettings);
}

TiledBlobPtr T_Blend::CreateDifference(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings)
{
	return CreateGenericBlend<FSH_BlendDifference>(InCycle, InTargetId, DesiredDesc, "T_BlendDifference", InBlendSettings);
}

TiledBlobPtr T_Blend::CreateMax(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc,int InTargetId, const FBlendSettings* InBlendSettings)
{
	return CreateGenericBlend<FSH_BlendMax>(InCycle, InTargetId, DesiredDesc, "T_BlendMax", InBlendSettings);
}

TiledBlobPtr T_Blend::CreateMin(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc,int InTargetId, const FBlendSettings* InBlendSettings)
{
	return CreateGenericBlend<FSH_BlendMin>(InCycle, InTargetId, DesiredDesc, "T_BlendMin", InBlendSettings);
}

TiledBlobPtr T_Blend::CreateStep(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc,int InTargetId, const FBlendSettings* InBlendSettings)
{
	return CreateGenericBlend<FSH_BlendStep>(InCycle, InTargetId, DesiredDesc, "T_BlendStep", InBlendSettings);
}

TiledBlobPtr T_Blend::CreateOverlay(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc,int InTargetId, const FBlendSettings* InBlendSettings)
{
	return CreateGenericBlend<FSH_BlendOverlay>(InCycle, InTargetId, DesiredDesc, "T_BlendOverlay", InBlendSettings);
}

TiledBlobPtr T_Blend::CreateDistort(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings)
{
	return CreateGenericBlend<FSH_BlendDistort>(InCycle, InTargetId, DesiredDesc, "T_BlendDistort", InBlendSettings);
}