// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <2D/BlendModes.h>

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////

class FSH_BlendBase : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_BlendBase, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, BackgroundTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, ForegroundTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, MaskTexture)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER(float, Opacity)
	END_SHADER_PARAMETER_STRUCT()

	class FIgnoreAlpha : SHADER_PERMUTATION_BOOL("IGNORE_ALPHA");
	class FClamp : SHADER_PERMUTATION_BOOL("CLAMP");
	using FPermutationDomain = TShaderPermutationDomain<FIgnoreAlpha, FClamp>;

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParams, FShaderCompilerEnvironment& InEnv)
	{
	}
};

//////////////////////////////////////////////////////////////////////////
/// Normal
//////////////////////////////////////////////////////////////////////////
class FSH_BlendNormal : public FSH_BlendBase
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_BlendNormal, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_BlendNormal, FSH_BlendBase);
};

//////////////////////////////////////////////////////////////////////////
/// Add
//////////////////////////////////////////////////////////////////////////
class FSH_BlendAdd : public FSH_BlendBase
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_BlendAdd, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_BlendAdd, FSH_BlendBase);
};

//////////////////////////////////////////////////////////////////////////
/// Add
//////////////////////////////////////////////////////////////////////////
class FSH_BlendSubtract : public FSH_BlendBase
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_BlendSubtract, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_BlendSubtract, FSH_BlendBase);
};

//////////////////////////////////////////////////////////////////////////
/// Multiply
//////////////////////////////////////////////////////////////////////////
class FSH_BlendMultiply : public FSH_BlendBase
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_BlendMultiply, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_BlendMultiply, FSH_BlendBase);
};

//////////////////////////////////////////////////////////////////////////
/// Divide
//////////////////////////////////////////////////////////////////////////
class FSH_BlendDivide : public FSH_BlendBase
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_BlendDivide, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_BlendDivide, FSH_BlendBase);
};
//////////////////////////////////////////////////////////////////////////
/// Difference
//////////////////////////////////////////////////////////////////////////
class FSH_BlendDifference : public FSH_BlendBase
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_BlendDifference, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_BlendDifference, FSH_BlendBase);
};

//////////////////////////////////////////////////////////////////////////
/// Max
//////////////////////////////////////////////////////////////////////////
class FSH_BlendMax : public FSH_BlendBase
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_BlendMax, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_BlendMax, FSH_BlendBase);
};

//////////////////////////////////////////////////////////////////////////
/// Min
//////////////////////////////////////////////////////////////////////////
class FSH_BlendMin : public FSH_BlendBase
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_BlendMin, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_BlendMin, FSH_BlendBase);
};

//////////////////////////////////////////////////////////////////////////
/// Step
//////////////////////////////////////////////////////////////////////////
class FSH_BlendStep : public FSH_BlendBase
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_BlendStep, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_BlendStep, FSH_BlendBase);
};

//////////////////////////////////////////////////////////////////////////
/// Overlay
//////////////////////////////////////////////////////////////////////////
class FSH_BlendOverlay : public FSH_BlendBase
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_BlendOverlay, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_BlendOverlay, FSH_BlendBase);
};

//////////////////////////////////////////////////////////////////////////
/// Distort
//////////////////////////////////////////////////////////////////////////
class FSH_BlendDistort : public FSH_BlendBase
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_BlendDistort, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_BlendDistort, FSH_BlendBase);
};
/**
 * 
 */
class T_Blend
{
public:
	UE_API T_Blend();
	UE_API ~T_Blend();

	struct FBlendSettings
	{
		TiledBlobPtr BackgroundTexture;
		TiledBlobPtr ForegroundTexture;
		TiledBlobPtr Mask;
		float Opacity;
		bool bIgnoreAlpha;
		bool bClamp;
	};
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////

	static UE_API TiledBlobPtr				Create(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, EBlendModes::Type InBlendMode, const FBlendSettings* InBlendSettings);
	static UE_API TiledBlobPtr				CreateNormal(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings);
	static UE_API TiledBlobPtr				CreateAdd(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings);
	static UE_API TiledBlobPtr				CreateSubtract(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings);
	static UE_API TiledBlobPtr				CreateMultiply(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings);
	static UE_API TiledBlobPtr				CreateDivide(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings);
	static UE_API TiledBlobPtr				CreateDifference(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings);
	static UE_API TiledBlobPtr				CreateMax(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings);
	static UE_API TiledBlobPtr				CreateMin(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings);
	static UE_API TiledBlobPtr				CreateStep(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings);
	static UE_API TiledBlobPtr				CreateOverlay(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings);
	static UE_API TiledBlobPtr				CreateDistort(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, int InTargetId, const FBlendSettings* InBlendSettings);
};

#undef UE_API
