// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_Blur.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Utility/T_CombineTiledBlob.h"

IMPLEMENT_GLOBAL_SHADER(FSH_GaussianBlur, "/Plugin/TextureGraph/Expressions/Expression_GaussianBlur.usf", "FSH_GaussianBlur", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_DirectionalBlur, "/Plugin/TextureGraph/Expressions/Expression_DirectionalBlur.usf", "FSH_DirectionalBlur", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_RadialBlur, "/Plugin/TextureGraph/Expressions/Expression_RadialBlur.usf", "FSH_RadialBlur", SF_Pixel);

const FVector2f T_Blur::RadialBlurCenter = FVector2f(0.5, 0.5);

T_Blur::T_Blur()
{
}

T_Blur::~T_Blur()
{
}

TiledBlobPtr T_Blur::CreateGaussian(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, int32 InRadius, int InTargetId)
{
	if(SourceTexture)
	{
		RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_GaussianBlur>(TEXT("T_GaussianBlur"));
		check(RenderMaterial);
		
		// Combine the tiled texture
		TiledBlobPtr CombinedBlob = T_CombineTiledBlob::Create(InCycle, SourceTexture->GetDescriptor(), InTargetId, SourceTexture);

		FTileInfo TileInfo;

		JobUPtr RenderJob = std::make_unique<Job>(InCycle->GetMix(), InTargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
		RenderJob
			->AddArg(ARG_TILEINFO(TileInfo, "TileInfo"))
			->AddArg(ARG_BLOB(CombinedBlob, "SourceTexture"))
			->AddArg(ARG_INT(InRadius, "Radius"))
			;

		const FString Name = FString::Printf(TEXT("T_GaussianBlur.[%llu]"), InCycle->GetBatch()->GetBatchId());

		TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredDesc);
		InCycle->AddJob(InTargetId, std::move(RenderJob));

		return Result;
	}

	return TextureHelper::GetBlack();
}

TiledBlobPtr T_Blur::CreateDirectional(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, float InAngle, float InStrength, int InTargetId)
{
	if(SourceTexture)
	{
		RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_DirectionalBlur>(TEXT("T_DirectionalBlur"));
		check(RenderMaterial);
		
		// Combine the tiled texture
		TiledBlobPtr CombinedBlob = T_CombineTiledBlob::Create(InCycle, SourceTexture->GetDescriptor(), InTargetId, SourceTexture);

		FTileInfo TileInfo;

		JobUPtr RenderJob = std::make_unique<Job>(InCycle->GetMix(), InTargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
		RenderJob
			->AddArg(ARG_TILEINFO(TileInfo, "TileInfo"))
			->AddArg(ARG_BLOB(CombinedBlob, "SourceTexture"))
			->AddArg(ARG_FLOAT(InAngle, "Angle"))
			->AddArg(ARG_FLOAT(InStrength, "Strength"))
			->AddArg(ARG_FLOAT(50, "Steps"))
			->AddArg(ARG_FLOAT(2, "Sigma"))
			->AddArg(ARG_FLOAT(20, "StrengthMultiplier"))
			;

		const FString Name = FString::Printf(TEXT("T_DirectionalBlur.[%llu]"), InCycle->GetBatch()->GetBatchId());

		TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredDesc);
		InCycle->AddJob(InTargetId, std::move(RenderJob));

		return Result;
	}

	return TextureHelper::GetBlack();
}

TiledBlobPtr T_Blur::CreateRadial(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, float InRadius, float InStrength, int InTargetId)
{
	if(SourceTexture)
	{
		RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_RadialBlur>(TEXT("T_RadialBlur"));
		check(RenderMaterial);
		
		// Combine the tiled texture
		TiledBlobPtr CombinedBlob = T_CombineTiledBlob::Create(InCycle, SourceTexture->GetDescriptor(), InTargetId, SourceTexture);

		FTileInfo TileInfo;

		JobUPtr RenderJob = std::make_unique<Job>(InCycle->GetMix(), InTargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
		RenderJob
			->AddArg(ARG_TILEINFO(TileInfo, "TileInfo"))
			->AddArg(ARG_BLOB(CombinedBlob, "SourceTexture"))
			->AddArg(ARG_INT(InRadius, "Radius"))
			->AddArg(ARG_FLOAT(InStrength, "Strength"))
			->AddArg(ARG_FLOAT(RadialBlurCenter.X, "CenterX"))
			->AddArg(ARG_FLOAT(RadialBlurCenter.Y, "CenterY"))
			->AddArg(ARG_FLOAT(50, "Samples"))
			;

		const FString Name = FString::Printf(TEXT("T_RadialBlur.[%llu]"), InCycle->GetBatch()->GetBatchId());

		TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredDesc);
		InCycle->AddJob(InTargetId, std::move(RenderJob));

		return Result;
	}

	return TextureHelper::GetBlack();
}
