// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <2D/BlendModes.h>

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Gaussian
//////////////////////////////////////////////////////////////////////////
class FSH_GaussianBlur : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_GaussianBlur, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_GaussianBlur, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER(int32, Radius)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
};
//////////////////////////////////////////////////////////////////////////
/// Directional
//////////////////////////////////////////////////////////////////////////
class FSH_DirectionalBlur : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_DirectionalBlur, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_DirectionalBlur, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER(float, Angle)
		SHADER_PARAMETER(float, Strength)
		SHADER_PARAMETER(float, Steps)
		SHADER_PARAMETER(float, Sigma)
		SHADER_PARAMETER(float, StrengthMultiplier)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
};

//////////////////////////////////////////////////////////////////////////
/// Radial
//////////////////////////////////////////////////////////////////////////
class FSH_RadialBlur : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_RadialBlur, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_RadialBlur, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER(int32, Radius)
		SHADER_PARAMETER(float, Strength)
		SHADER_PARAMETER(float, CenterX)
		SHADER_PARAMETER(float, CenterY)
		SHADER_PARAMETER(float, Samples)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
};

/**
 * 
 */
class T_Blur
{
public:

	UE_API const static FVector2f RadialBlurCenter;
	
	UE_API T_Blur();
	UE_API ~T_Blur();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API TiledBlobPtr				CreateGaussian(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, int32 InRadius, int InTargetId);
	static UE_API TiledBlobPtr				CreateDirectional(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, float InAngle, float InStrength, int InTargetId);
	static UE_API TiledBlobPtr				CreateRadial(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, float InRadius, float InStrength, int InTargetId);
};

#undef UE_API
