// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_Brightness.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "Math/Vector.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"

IMPLEMENT_GLOBAL_SHADER(FSH_Brightness, "/Plugin/TextureGraph/Expressions/Expression_Brightness.usf", "FSH_Brightness", SF_Pixel);

TiledBlobPtr T_Brightness::Create(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Source, float Brightness, float Contrast, int32 TargetId)
{
	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_Brightness>(TEXT("T_Brightness"));

	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_BLOB(Source, "SourceTexture"))
		->AddArg(ARG_FLOAT(Brightness, "Brightness"))
		->AddArg(ARG_FLOAT(Contrast, "Contrast"))
		;

	FString Name = TEXT("Brightness"); // FString::Printf(TEXT("GaussianBlur.[%s].[%d].[%llu]"), *gmask->ID(), InContext->TargetId, InContext->Cycle->Batch()->BatchId());

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}
