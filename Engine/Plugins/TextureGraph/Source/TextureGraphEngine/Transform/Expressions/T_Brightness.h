// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////
class FSH_Brightness : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_Brightness, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_Brightness, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER(float, Brightness)
		SHADER_PARAMETER(float, Contrast)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool						ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
	static void						ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env) {}
};

class T_Brightness
{
public:
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API TiledBlobPtr				Create(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Source, float Brightness, float Contrast, int32 TargetId);
};

#undef UE_API
