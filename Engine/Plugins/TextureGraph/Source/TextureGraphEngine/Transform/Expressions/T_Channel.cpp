// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_Channel.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "Math/Vector.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(T_Channel)

IMPLEMENT_GLOBAL_SHADER(FSH_ChannelSplitter_Red, "/Plugin/TextureGraph/Expressions/Expression_ChannelSplitter.usf", "FSH_ChannelSplitter_Red", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_ChannelSplitter_Green, "/Plugin/TextureGraph/Expressions/Expression_ChannelSplitter.usf", "FSH_ChannelSplitter_Green", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_ChannelSplitter_Blue, "/Plugin/TextureGraph/Expressions/Expression_ChannelSplitter.usf", "FSH_ChannelSplitter_Blue", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_ChannelSplitter_Alpha, "/Plugin/TextureGraph/Expressions/Expression_ChannelSplitter.usf", "FSH_ChannelSplitter_Alpha", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(FSH_ChannelCombiner, "/Plugin/TextureGraph/Expressions/Expression_ChannelCombiner.usf", "FSH_ChannelCombiner", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(FSH_ChannelSwizzle, "/Plugin/TextureGraph/Expressions/Expression_ChannelSwizzle.usf", "FSH_ChannelSwizzle", SF_Pixel);
