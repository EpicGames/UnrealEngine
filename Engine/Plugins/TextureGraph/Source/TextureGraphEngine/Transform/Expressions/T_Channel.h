// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>
#include "T_Channel.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

UENUM(BlueprintType)
enum class EColorChannel : uint8
{
	Red		= 0		UMETA(DisplayName = "Red"),
	Green	= 1		UMETA(DisplayName = "Green"),
	Blue	= 2		UMETA(DisplayName = "Blue"),
	Alpha	= 3		UMETA(DisplayName = "Alpha"),
};

//////////////////////////////////////////////////////////////////////////
/// Basic Channel Op
//////////////////////////////////////////////////////////////////////////
class FSH_ChannelSplitter : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_ChannelSplitter, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
	END_SHADER_PARAMETER_STRUCT()

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Red
//////////////////////////////////////////////////////////////////////////
class FSH_ChannelSplitter_Red : public FSH_ChannelSplitter
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_ChannelSplitter_Red, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_ChannelSplitter_Red, FSH_ChannelSplitter);
};

//////////////////////////////////////////////////////////////////////////
/// Green
//////////////////////////////////////////////////////////////////////////
class FSH_ChannelSplitter_Green : public FSH_ChannelSplitter
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_ChannelSplitter_Green, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_ChannelSplitter_Green, FSH_ChannelSplitter);
};

//////////////////////////////////////////////////////////////////////////
/// Red
//////////////////////////////////////////////////////////////////////////
class FSH_ChannelSplitter_Blue : public FSH_ChannelSplitter
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_ChannelSplitter_Blue, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_ChannelSplitter_Blue, FSH_ChannelSplitter);
};

//////////////////////////////////////////////////////////////////////////
/// Red
//////////////////////////////////////////////////////////////////////////
class FSH_ChannelSplitter_Alpha : public FSH_ChannelSplitter
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_ChannelSplitter_Alpha, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_ChannelSplitter_Alpha, FSH_ChannelSplitter);
};

//////////////////////////////////////////////////////////////////////////
/// Channel Combiner
//////////////////////////////////////////////////////////////////////////
class FSH_ChannelCombiner : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_ChannelCombiner, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_ChannelCombiner, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceRed)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceGreen)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceBlue)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceAlpha)
	END_SHADER_PARAMETER_STRUCT()

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Channel Combiner
//////////////////////////////////////////////////////////////////////////
class FVar_SwizzleDstChannelRed : SHADER_PERMUTATION_INT("DST_CHANNEL_RED", 4);
class FVar_SwizzleDstChannelGreen : SHADER_PERMUTATION_INT("DST_CHANNEL_GREEN", 4);
class FVar_SwizzleDstChannelBlue : SHADER_PERMUTATION_INT("DST_CHANNEL_BLUE", 4);
class FVar_SwizzleDstChannelAlpha : SHADER_PERMUTATION_INT("DST_CHANNEL_ALPHA", 4);

class FSH_ChannelSwizzle : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_ChannelSwizzle, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_ChannelSwizzle, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<FVar_SwizzleDstChannelRed, FVar_SwizzleDstChannelGreen, FVar_SwizzleDstChannelBlue, FVar_SwizzleDstChannelAlpha>;
public:
	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Helper C++ class
//////////////////////////////////////////////////////////////////////////
class T_Channel
{
public:
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
