// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_Clamp.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "Math/Vector.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"

IMPLEMENT_GLOBAL_SHADER(FSH_Clamp, "/Plugin/TextureGraph/Expressions/Expression_Clamp.usf", "FSH_Clamp", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_SmoothStep, "/Plugin/TextureGraph/Expressions/Expression_Clamp.usf", "FSH_SmoothStep", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Min, "/Plugin/TextureGraph/Expressions/Expression_MinMax.usf", "FSH_Min", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Max, "/Plugin/TextureGraph/Expressions/Expression_MinMax.usf", "FSH_Max", SF_Pixel);

TiledBlobPtr T_Clamp::CreateClamp(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Source, TiledBlobPtr MinValue, TiledBlobPtr MaxValue, int32 TargetId)
{
	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_Clamp>(TEXT("T_Clamp"));

	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_BLOB(Source, "SourceTexture"))
		->AddArg(ARG_BLOB(MinValue, "MinValue"))
		->AddArg(ARG_BLOB(MaxValue, "MaxValue"))
		;

	FString Name = TEXT("Clamp"); // FString::Printf(TEXT("GaussianBlur.[%s].[%d].[%llu]"), *gmask->ID(), InContext->TargetId, InContext->Cycle->Batch()->BatchId());

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}

TiledBlobPtr T_Clamp::CreateSmoothStep(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Source, TiledBlobPtr MinValue, TiledBlobPtr MaxValue, int32 TargetId)
{
	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_SmoothStep>(TEXT("T_SmoothStep"));

	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_BLOB(Source, "SourceTexture"))
		->AddArg(ARG_BLOB(MinValue, "MinValue"))
		->AddArg(ARG_BLOB(MaxValue, "MaxValue"))
		;

	FString Name = TEXT("SmoothStep"); // FString::Printf(TEXT("GaussianBlur.[%s].[%d].[%llu]"), *gmask->ID(), InContext->TargetId, InContext->Cycle->Batch()->BatchId());

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}

TiledBlobPtr T_Clamp::CreateMin(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Input1, TiledBlobPtr Input2, int32 TargetId)
{
	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_Min>(TEXT("T_Min"));

	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_BLOB(Input1, "Input1"))
		->AddArg(ARG_BLOB(Input2, "Input2"))
		;

	FString Name = TEXT("Clamp"); // FString::Printf(TEXT("GaussianBlur.[%s].[%d].[%llu]"), *gmask->ID(), InContext->TargetId, InContext->Cycle->Batch()->BatchId());

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}

TiledBlobPtr T_Clamp::CreateMax(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Input1, TiledBlobPtr Input2, int32 TargetId)
{
	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_Max>(TEXT("T_Max"));

	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_BLOB(Input1, "Input1"))
		->AddArg(ARG_BLOB(Input2, "Input2"))
		;

	FString Name = TEXT("Clamp"); // FString::Printf(TEXT("GaussianBlur.[%s].[%d].[%llu]"), *gmask->ID(), InContext->TargetId, InContext->Cycle->Batch()->BatchId());

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}

