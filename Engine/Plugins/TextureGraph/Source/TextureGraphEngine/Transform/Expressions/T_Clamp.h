// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Clamp Shader
//////////////////////////////////////////////////////////////////////////
class FSH_Clamp : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_Clamp, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_Clamp, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, MinValue)
		SHADER_PARAMETER_TEXTURE(Texture2D, MaxValue)
	END_SHADER_PARAMETER_STRUCT()

public:

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Clamp Shader
//////////////////////////////////////////////////////////////////////////
class FSH_SmoothStep : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_SmoothStep, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_SmoothStep, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, MinValue)
		SHADER_PARAMETER_TEXTURE(Texture2D, MaxValue)
	END_SHADER_PARAMETER_STRUCT()

public:

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Min Shader
//////////////////////////////////////////////////////////////////////////
class FSH_Min : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_Min, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_Min, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, Input1)
		SHADER_PARAMETER_TEXTURE(Texture2D, Input2)
	END_SHADER_PARAMETER_STRUCT()

public:

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Max Shader
//////////////////////////////////////////////////////////////////////////
class FSH_Max : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_Max, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_Max, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, Input1)
		SHADER_PARAMETER_TEXTURE(Texture2D, Input2)
	END_SHADER_PARAMETER_STRUCT()

public:

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

class T_Clamp
{
public:
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API TiledBlobPtr				CreateClamp(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Source, TiledBlobPtr MinValue, TiledBlobPtr MaxValue, int32 TargetId);
	static UE_API TiledBlobPtr				CreateSmoothStep(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Source, TiledBlobPtr MinValue, TiledBlobPtr MaxValue, int32 TargetId);
	static UE_API TiledBlobPtr				CreateMin(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Input1, TiledBlobPtr Input2, int32 TargetId);
	static UE_API TiledBlobPtr				CreateMax(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Input1, TiledBlobPtr Input2, int32 TargetId);
};

#undef UE_API
