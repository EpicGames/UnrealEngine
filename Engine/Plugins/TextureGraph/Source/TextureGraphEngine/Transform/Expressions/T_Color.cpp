// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_Color.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "Math/Vector.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"

IMPLEMENT_GLOBAL_SHADER(FSH_Grayscale, "/Plugin/TextureGraph/Expressions/Expression_Grayscale.usf", "FSH_Grayscale", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Threshold, "/Plugin/TextureGraph/Expressions/Expression_Threshold.usf", "FSH_Threshold", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_HSV, "/Plugin/TextureGraph/Expressions/Expression_HSV.usf", "FSH_HSV", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_RGB2HSV, "/Plugin/TextureGraph/Expressions/Expression_HSV.usf", "FSH_RGB2HSV", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_HSV2RGB, "/Plugin/TextureGraph/Expressions/Expression_HSV.usf", "FSH_HSV2RGB", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_ColorCorrection, "/Plugin/TextureGraph/Expressions/Expression_ColorCorrection.usf", "FSH_ColorCorrection", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Premult, "/Plugin/TextureGraph/Expressions/Expression_Premult.usf", "FSH_Premult", SF_Pixel);
