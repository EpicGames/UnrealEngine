// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Simple grayscale shader
//////////////////////////////////////////////////////////////////////////
class FSH_Grayscale : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_Grayscale, FSH_Base);
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_Grayscale, UE_API);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
	END_SHADER_PARAMETER_STRUCT()

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Simple levels shader
//////////////////////////////////////////////////////////////////////////
class FSH_Threshold : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_Threshold, FSH_Base);
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_Threshold, UE_API);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, Input)
		SHADER_PARAMETER(float, Threshold)
	END_SHADER_PARAMETER_STRUCT()

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Simple HSV shader
//////////////////////////////////////////////////////////////////////////
class FSH_HSV : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_HSV, FSH_Base);
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_HSV, UE_API);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, Input)
		SHADER_PARAMETER(float, Hue)
		SHADER_PARAMETER(float, Saturation)
		SHADER_PARAMETER(float, Value)
	END_SHADER_PARAMETER_STRUCT()

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Simple RGB2HSV shader
//////////////////////////////////////////////////////////////////////////
class FSH_RGB2HSV : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_RGB2HSV, FSH_Base);
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_RGB2HSV, UE_API);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, Input)
	END_SHADER_PARAMETER_STRUCT()

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Simple HSV2RGB shader
//////////////////////////////////////////////////////////////////////////
class FSH_HSV2RGB : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_HSV2RGB, FSH_Base);
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_HSV2RGB, UE_API);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, Input)
	END_SHADER_PARAMETER_STRUCT()

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Color correction shader
//////////////////////////////////////////////////////////////////////////
class FSH_Premult : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_Premult, FSH_Base);
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_Premult, UE_API);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, Input)
	END_SHADER_PARAMETER_STRUCT()

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Color correction shader
//////////////////////////////////////////////////////////////////////////
class FSH_ColorCorrection : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_ColorCorrection, FSH_Base);
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_ColorCorrection, UE_API);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, Input)
		SHADER_PARAMETER(float, Brightness)
		SHADER_PARAMETER(float, Contrast)
		SHADER_PARAMETER(float, Gamma)
		SHADER_PARAMETER(float, Saturation)
		SHADER_PARAMETER(FLinearColor, TemperatureRGB)
		SHADER_PARAMETER(float, TemperatureStrength)
		SHADER_PARAMETER(float, TemperatureBrightnessNormalization)
	END_SHADER_PARAMETER_STRUCT()

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Helper C++ class
//////////////////////////////////////////////////////////////////////////
class T_Color
{
public:
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
