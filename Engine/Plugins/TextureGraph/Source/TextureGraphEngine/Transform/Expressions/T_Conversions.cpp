// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_Conversions.h"

#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Utility/T_CombineTiledBlob.h"

IMPLEMENT_GLOBAL_SHADER(FSH_NormalFromHeightMap, "/Plugin/TextureGraph/Expressions/Expression_NormalFromHeightMap.usf", "FSH_NormalFromHeightMap", SF_Pixel);

TiledBlobPtr T_Conversions::CreateNormalFromHeightMap(MixUpdateCyclePtr InCycle, TiledBlobPtr HeightMapTexture, float Offset, float Strength, int InTargetId, BufferDescriptor DesiredOutputDesc)
{
	if(HeightMapTexture)
	{
		RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_NormalFromHeightMap>(TEXT("T_NormalFromHeightMap"));
		check(RenderMaterial);
		
		// Combine the tiled texture
		TiledBlobPtr CombinedBlob = T_CombineTiledBlob::Create(InCycle, HeightMapTexture->GetDescriptor(), InTargetId, HeightMapTexture);

		FTileInfo TileInfo;

		JobUPtr RenderJob = std::make_unique<Job>(InCycle->GetMix(), InTargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
		RenderJob
			->AddArg(ARG_TILEINFO(TileInfo, "TileInfo"))
			->AddArg(ARG_BLOB(CombinedBlob, "SourceTexture"))
			->AddArg(ARG_FLOAT(Offset, "Offset"))
			->AddArg(ARG_FLOAT(Strength, "Strength"))
			;

		const FString Name = FString::Printf(TEXT("NormalFromHeightMap.[%llu]"), InCycle->GetBatch()->GetBatchId());

		TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
		InCycle->AddJob(InTargetId, std::move(RenderJob));

		return Result;
	}

	return TextureHelper::GetBlack();
}
