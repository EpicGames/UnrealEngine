// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>
#include "Data/TiledBlob.h"
#include "Data/RawBuffer.h"

#define UE_API TEXTUREGRAPHENGINE_API
//////////////////////////////////////////////////////////////////////////
/// Normal from height map shader
//////////////////////////////////////////////////////////////////////////
class FSH_NormalFromHeightMap : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_NormalFromHeightMap, FSH_Base);
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_NormalFromHeightMap, UE_API);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER(float, Strength)
		SHADER_PARAMETER(float, Offset)
	END_SHADER_PARAMETER_STRUCT()

	static bool						ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
	static void						ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Params, FShaderCompilerEnvironment& Env) {}
};

//////////////////////////////////////////////////////////////////////////
/// Helper C++ class
//////////////////////////////////////////////////////////////////////////
class T_Conversions
{
public:
	UE_API T_Conversions();
	UE_API ~T_Conversions();
	
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	///
	static UE_API TiledBlobPtr				CreateNormalFromHeightMap(MixUpdateCyclePtr InCycle, TiledBlobPtr HeightMapTexture, float Offset, float Strength, int InTargetId, BufferDescriptor DesiredOutputDesc);
};

#undef UE_API
