// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_ExtractMaterialIds.h"

#include "2D/Tex.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include "Device/Mem/Device_Mem.h"
#include "Transform/Utility/T_CombineTiledBlob.h"
#include "TextureResource.h"
#include "Device/DeviceManager.h"
#include "Helper/ColorUtil.h"
#include "Helper/MathUtils.h"
#include "Runtime/Core/Public/Async/ParallelFor.h"
#include "Data/RawBuffer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(T_ExtractMaterialIds)

static bool IsAlreadyInList(TArray<FLinearColor> UniqueColors, FLinearColor Color, FIntVector3 Hsv, HSVBucket* Bucket)
{
	int SrcBucket = (int)(Hsv.X / HSVBucket::HCount);

	for(auto UniqueColor : UniqueColors)
	{
		FLinearColor HSV = Color.LinearRGBToHSV();
		
		// Color.RGBToHSV(uclr, out float uh, out float uc, out float uv);
		FIntVector3 Uhsv = FIntVector3((int)(HSV.R), (int)(HSV.G * HSVBucket::SRange), (int)(HSV.B * HSVBucket::VRange));
		int CheckBucket = (int)(Uhsv.X / HSVBucket::HCount);

		if (/*GetSquaredDistance(c, uclr) < 0.01f ||*/
			SrcBucket == CheckBucket && FMath::Abs(HSV.G - Uhsv.Y) < 5 && FMath::Abs(HSV.B - Uhsv.Z) < 5)
		{
			return true;
		}

		//if value is greater than 90 and saturation < 10 then colour is almost white
		if (HSV.B >= 90 && HSV.G <= 10)
		{
			if (Uhsv.Z >= 90 && Uhsv.Y <= 10)
			{
				return true;
			}
		}

		//if value is lesser than 5 then colour is almost black
		if (HSV.B <= 5)
		{
			if (Uhsv.Z <= 5)
			{
				return true;
			}
		}
	}
	return false;
}

static bool CheckColorBucketMatch(FLinearColor Color1, FLinearColor Color2)
{
	int32 H1, S1, V1;
	int32 H2, S2, V2;
	auto Color1HSVBucket = HSVBucket::GetBucket(Color1, H1, S1, V1);
	auto Color2HSVBucket = HSVBucket::GetBucket(Color2, H2, S2, V2);

	if (Color1HSVBucket == Color2HSVBucket || ColorUtil::GetSquaredDistance(Color1, Color2) < 0.01f)
	{
		return true;
	}

	return false;
}

static Bucket3dArray GetHSVBuckets_Range(const TArray<FLinearColor> Pixels, int32 Width, int32 Height, int32 XStart, int32 XEnd, int32 YStart, int32 YEnd)
{
	Bucket3dArray Buckets;

	// Resize the array dimensions
	const int32 SizeX = HSVBucket::HBuckets;
	const int32 SizeY = HSVBucket::SBuckets;
	const int32 SizeZ = HSVBucket::VBuckets;
	
	Buckets.SetNum(SizeX);

	for (int32 X = 0; X < SizeX; X++)
	{
		Buckets[X].SetNum(SizeY);

		for (int32 Y = 0; Y < SizeY; Y++)
		{
			Buckets[X][Y].SetNum(SizeZ);
		}
	}

	auto Epsilon = std::numeric_limits<float>::epsilon();

	double StartTime = FPlatformTime::Seconds();

	for (int i = XStart; i < XEnd; i++)
	{
		for (int j = YStart; j < YEnd; j++)
		{
			int H, S, V;
			int PixelIndex = j * Width + i;
			
			auto Pixel = Pixels[PixelIndex];
	
			auto Bucket = HSVBucket::GetBucket(Pixel, H, S, V);
	
			//if (pixel.r * 255.0f > 250.0f && pixel.g * 255.0f > 250.0f && pixel.b * 255.0f > 250.0f)
			//    pixel = pixel;
	
			if (Bucket.X >= 0)
			{
				/// Check whether this pixel is some continuous block or not
				auto LeftPixelIndex = FMath::Max(j - 1, 0) * Width + i;
				auto RightPixelIndex = FMath::Min(j + 1, Height - 1) * Width + i;
				auto BottomPixelIndex = j * Width + FMath::Max(i - 1, 0);
				auto TopPixelIndex = j * Width + FMath::Min(i + 1, Width - 1);
	
				/// Ignore if there isn't one matching pixel in the neighbourhood
				if (ColorUtil::GetSquaredDistance(Pixel, Pixels[LeftPixelIndex]) < Epsilon ||
					ColorUtil::GetSquaredDistance(Pixel, Pixels[RightPixelIndex]) < Epsilon ||
					ColorUtil::GetSquaredDistance(Pixel, Pixels[BottomPixelIndex]) < Epsilon ||
					ColorUtil::GetSquaredDistance(Pixel, Pixels[TopPixelIndex]) < Epsilon)
				{
					if (Buckets[Bucket.X][Bucket.Y][Bucket.Z] == nullptr)
						Buckets[Bucket.X][Bucket.Y][Bucket.Z] = new HSVBucket(Bucket.X, Bucket.Y, Bucket.Z);
					
					
					Buckets[Bucket.X][Bucket.Y][Bucket.Z]->Add(Pixel, H, S, V, FVector2f((float)i, (float)j));
				}
			}
		}
	}

	const double EndTime = FPlatformTime::Seconds();
	const double ExecutionTime = EndTime - StartTime;

	return Buckets;
}

static Bucket3dArray GetHSVBuckets(const TArray<FLinearColor> Pixels, int32 Width, int32 Height)
{
	const int MaxThreads = FPlatformMisc::NumberOfCores() * 2;
	const double StartTime = FPlatformTime::Seconds();
					
	int DeltaX = FMath::Max(Width / MaxThreads, 512.0f);
	int DeltaY = FMath::Max(Height / MaxThreads, 512.0f);

	TArray<UE::Tasks::TTask<Bucket3dArray>> Tasks;
	
	int Xiter = 0;

	while (Xiter < Width)
	{
		int Xend = Xiter + FMath::Min(DeltaX, Width - Xiter);

		int Yiter = 0;

		while (Yiter < Height)
		{
			int Yend = Yiter + FMath::Min(DeltaY, Height - Yiter);

			UE::Tasks::TTask<Bucket3dArray> Task = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&Pixels, Width, Height, Xiter, Xend, Yiter, Yend]
			{
				return GetHSVBuckets_Range(Pixels, Width, Height, Xiter, Xend, Yiter, Yend);
			}, LowLevelTasks::ETaskPriority::BackgroundHigh);
			
			Tasks.Add(Task);
			Yiter = Yend;
		}

		Xiter = Xend;
	}

	UE::Tasks::Wait(Tasks);

	const double EndTime = FPlatformTime::Seconds();
	const double ExecutionTime = EndTime - StartTime;

	UE_LOGF(LogData, Log, "T_ExtractMaterialIds : GetHSVBuckets : Execution Time: %f seconds", ExecutionTime);

	Bucket3dArray Buckets;

	// Resize the array dimensions
	const int32 SizeX = HSVBucket::HBuckets;
	const int32 SizeY = HSVBucket::SBuckets;
	const int32 SizeZ = HSVBucket::VBuckets;
					
	Buckets.SetNum(SizeX);

	for (int32 X = 0; X < SizeX; X++)
	{
		Buckets[X].SetNum(SizeY);

		for (int32 Y = 0; Y < SizeY; Y++)
		{
			Buckets[X][Y].SetNum(SizeZ);
		}
	}
					
	for (auto Task : Tasks)
	{
		auto TaskBuckets = Task.GetResult();
		for (int hi = 0; hi < HSVBucket::HBuckets; hi++)
		{
			for (int si = 0; si < HSVBucket::SBuckets; si++)
			{
				for (int vi = 0; vi < HSVBucket::VBuckets; vi++)
				{
					if (TaskBuckets[hi][si][vi] != nullptr)
					{
						if (Buckets[hi][si][vi] == nullptr)
						{
							Buckets[hi][si][vi] = new HSVBucket(hi, si, vi);
						}				
                                        					
						Buckets[hi][si][vi]->Add(*TaskBuckets[hi][si][vi]);
					}
				}
			}
		}
	}
	
	return Buckets;
}

static TArray<FLinearColor> GetUniqueColors(const TArray<FLinearColor>& Pixels, const int32& Width, const int32& Height)
{
	const double StartTime = FPlatformTime::Seconds();
	
	Bucket3dArray Buckets = GetHSVBuckets(Pixels, Width, Height);

	TArray<FLinearColor> UniqueColors;
	
    for (int hi = 0; hi < HSVBucket::HBuckets; hi++)
    {
    	for (int si = 0; si < HSVBucket::SBuckets; si++)
    	{
    		for (int vi = 0; vi < HSVBucket::VBuckets; vi++)
    		{
    			if (Buckets[hi][si][vi] != nullptr)
    			{
    				FIntVector3 HsvVector;
    				FLinearColor UniqueColor = Buckets[hi][si][vi]->GetColor(HsvVector);
    				if (Buckets[hi][si][vi]->ShouldConsider(Width, Height))
    				{
    					if (!IsAlreadyInList(UniqueColors, UniqueColor, HsvVector, Buckets[hi][si][vi]))
    					{
    						UniqueColors.Add(UniqueColor);
    					}
    				}
    			}
    		}
    	}
    }

	const double EndTime = FPlatformTime::Seconds();
	const double ExecutionTime = EndTime - StartTime;

	UE_LOGF(LogData, Log, "T_ExtractMaterialIds : GetUniqueColors : Execution Time: %f seconds", ExecutionTime);

	return UniqueColors;
}
//////
HSVBucket::HSVBucket(int32 InHBucket, int32 InSBucket, int32 InVBucket)
{
	HSVStart = FIntVector3(InHBucket * HCount, InSBucket * SCount, InVBucket * VCount);
	MinPos = MathUtils::MaxFVector2();
	MaxPos = MathUtils::MinFVector2();
	MaxIndex = FIntVector3(-1,-1, -1);

	ColorCount.SetNum(HCount);

	for (int32 X = 0; X < HCount; X++)
	{
		ColorCount[X].SetNum(SCount);

		for (int32 Y = 0; Y < SCount; Y++)
		{
			ColorCount[X][Y].SetNum(VCount);
		}
	}
}

void HSVBucket::Add(const FLinearColor Color, int H, int S, int V, FVector2f Position)
{
	int32 Dh = H - HSVStart.X;
	int32 Ds = S - HSVStart.Y;
	int32 Dv = V - HSVStart.Z;

	ColorCount[Dh][Ds][Dv]++;

	if (ColorCount[Dh][Ds][Dv] > MaxIndividualCount || MaxIndex.X < 0)
	{
		MaxIndividualCount = ColorCount[Dh][Ds][Dv];
		MaxIndex = FIntVector3(Dh, Ds, Dv);
	}

	Positions.Add(Position);

	MinPos.X = FMath::Min(Position.X, MinPos.X);
	MinPos.Y = FMath::Min(Position.Y, MinPos.Y);

	MaxPos.X = FMath::Max(Position.X, MaxPos.X);
	MaxPos.Y = FMath::Max(Position.Y, MaxPos.Y);

	TotalCount++;
}

void HSVBucket::Add(HSVBucket Rhs)
{
	TotalCount += Rhs.TotalCount;

	for (int Hi = 0; Hi < HCount; Hi++)
	{
		for (int Si = 0; Si < SCount; Si++)
		{
			for (int Vi = 0; Vi < VCount; Vi++)
			{
				ColorCount[Hi][Si][Vi] += Rhs.ColorCount[Hi][Si][Vi];

				if (ColorCount[Hi][Si][Vi] > MaxIndividualCount)
				{
					MaxIndividualCount = ColorCount[Hi][Si][Vi];
					MaxIndex = FIntVector3(Hi, Si, Vi);
				}
			}
		}
	}

	MinPos.X = FMath::Min(Rhs.MinPos.X, MinPos.X);
	MinPos.Y = FMath::Min(Rhs.MinPos.Y, MinPos.Y);

	MaxPos.X = FMath::Max(Rhs.MaxPos.X, MaxPos.X);
	MaxPos.Y = FMath::Max(Rhs.MaxPos.Y, MaxPos.Y);

	Positions.Append(Rhs.Positions);
}

FIntVector3 HSVBucket::GetBucket(const FLinearColor Pixel, int& OutH, int& OutS, int& OutV)
{
	OutH = OutS = OutV = 0;

	/// Now we need to figure out the bucket for this pixel
	FLinearColor HSVColor = Pixel.LinearRGBToHSV();

	float Hue = HSVColor.R;
	float Sat = HSVColor.G;
	float Val = HSVColor.B;

	if (Pixel.A > 0)
	{
		OutH = (int)Hue;
		OutS = (int)(Sat * SRange);
		OutV = (int)(Val * VRange);
		
		int hbucket = (int)(OutH / (float)HCount);
		int sbucket = (int)(OutS / (float)SCount);
		int vbucket = (int)(OutV / (float)VCount);

		return FIntVector3(hbucket, sbucket, vbucket);
	}

	return FIntVector3(-1, -1, -1);
}

FLinearColor HSVBucket::GetColor(FIntVector3& OutHsv)
{
	OutHsv = HSVStart + MaxIndex;
	float H = OutHsv.X;
	float S = OutHsv.Y / SRange;
	float V = OutHsv.Z / VRange;

	FLinearColor HSV(H, S, V, 1);
	return HSV.HSVToLinearRGB();
}

bool HSVBucket::ShouldConsider(int32 Width, int32 Height)
{
	if (TotalCount <= MinPixelsCountThreshold || MaxIndividualCount <= MinPixelsCountThreshold)
		return false;

	float P1 = (TotalCount * 100.0f) / (Width * Height);
	float P2 = (MaxIndividualCount * 100.0f) / (Width * Height);

	if ((P1 > MinPixelsInTextureThreshold && P2 > MinPixelsInTextureThreshold) || P1 > MinPixelsOccurance)
		return true;

	return false;
}

//////////////////////////////////////////////////////////////////////////
//// Job ExtractMaterialIds
//////////////////////////////////////////////////////////////////////////

Job_ExtractMaterialIds::Job_ExtractMaterialIds(int32 targetId, TiledBlobRef InSourceTexture, FMaterialIDCollection& InMaterialIDInfoCollection, TArray<FMaterialIDMaskInfo>& InMaterialIDMaskInfo, int32& InActiveColorsCount,
	UObject* InErrorOwner /*= nullptr*/, uint16 priority /*= (uint16)E_Priority::kHigh*/, uint64 id /*= 0*/)
	: Job(targetId, std::make_shared<Null_Transform>(Device_Mem::Get(), TEXT("T_ExtractMaterialIds"), true, false), InErrorOwner, priority)
	, SourceTexture(InSourceTexture)
	, MaterialIDMaskInfo(InMaterialIDMaskInfo)
	, MaterialIDCollection(InMaterialIDInfoCollection)
	, ActiveColorsCount(InActiveColorsCount)
{
	DeviceNativeTask::Name = TEXT("ExtractMaterialIds");
}

cti::continuable<int32> Job_ExtractMaterialIds::PreExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type returnThread)
{
	// 1. First we try to find if we have a cached blob using a job hash.
	auto HashPtr = Job::Hash();
	BlobRef CachedResult = TextureGraphEngine::GetBlobber()->Find(HashPtr->Value());

	if (CachedResult)
	{
		// 2. If we have a cached blob we try to access its Raw data.
		UE_LOGF(LogData, Log, "[ExtractMaterialIds] Found cached result!");

		return CachedResult->GetBufferRef()->Raw().then([this](RawBufferPtr Raw)
		{
			auto ElementSize = sizeof(FMaterialIDInfo);
			
			check(Raw->GetDescriptor().Width == ElementSize)
			check(Raw->GetDescriptor().Size() == Raw->GetLength())
			check(Raw->HasData())
			
			int ElementNum = Raw->GetLength()/ElementSize;
			
			const uint8* RawData = Raw->GetData();
			
			if(RawData)
			{
				// 3. If we have cached data, we try to create Mat ID Collection
				FMaterialIDCollection CachedMaterialIDCollection;
				for(int i = 0; i < ElementNum; i++)
				{
					auto MatIdInfo = *(FMaterialIDInfo*)(RawData + (i * ElementSize));
					CachedMaterialIDCollection.Infos.Add(MatIdInfo);
				}
				
				// 4. There are two possible cases.
				// - Hash is same : We dont do anything
				// - Hash is different : We clear the existing collection and masking information
				// and replace it with cached collection.
				if(CachedMaterialIDCollection.HashValue() != MaterialIDCollection.HashValue())
				{
					MaterialIDCollection.Infos.Empty();
					MaterialIDMaskInfo.Empty();
					ActiveColorsCount = 0;
			
					for(int i = 0; i < CachedMaterialIDCollection.Infos.Num(); i++)
					{
						auto MatIdInfo = CachedMaterialIDCollection.Infos[i];
						MaterialIDCollection.Infos.Add(FMaterialIDInfo(MatIdInfo.Id, MatIdInfo.RGBColor, MatIdInfo.SrgbLinearColor));
			
						UE_LOGF(LogData, Log, "FoundColor R :%f, G :%f, B :%f, ColorName : %ls", MatIdInfo.SrgbLinearColor.R, MatIdInfo.SrgbLinearColor.G, MatIdInfo.SrgbLinearColor.B, *ColorUtil::GetColorName(MatIdInfo.SrgbLinearColor));
					}
				
					for(const FMaterialIDInfo& MatIdInfo : MaterialIDCollection.Infos)
					{
						MaterialIDMaskInfo.Add(FMaterialIDMaskInfo(MatIdInfo.Id, MatIdInfo.RGBColor));
					}
				}
				
				bIsCulled = true;
				bIsDone = true;
			
				MarkJobDone();
			}
			
			
			return cti::make_ready_continuable(0);
		});
	}

	return cti::make_ready_continuable(0);
}

cti::continuable<int32> Job_ExtractMaterialIds::ExecAsync(ENamedThreads::Type ExecThread, ENamedThreads::Type ReturnThread)
{
	UE_LOGF(LogData, Log, "Extracting unique colors: %ls", *SourceTexture->Name());

	if (IsDone())
	{
		MarkJobDone();
		return cti::make_ready_continuable(0);
	}

	// There are multiple steps required to extract the colors from the map.
			
	// 1. Get the source texture.
	return Job::BeginNative(RunInfo)
		.then([this]()
		{
			return PromiseUtil::OnGameThread();
		})
		.then([this]()
		{
			return SourceTexture->OnFinalise();
		})
		.then([this]()
        {
			auto Blob = SourceTexture->GetTile(0, 0);
			check(Blob);
			
			return Blob->Raw();

			// TODO: Combined Source should have raw data 
        	//return SourceTexture->Raw();
        })
		.then([this](RawBufferPtr BufferPtr)
		{
			TArray<FLinearColor> Pixels;
			BufferPtr->GetAsLinearColor(Pixels);

			if(Pixels.Num() > 0)
			{
				// 3. Get unique colors by converting colors into HSVBuckets
				// We are doing this asynchronously and on multiple threads. 
				TArray<FLinearColor> UniqueColors = GetUniqueColors(Pixels, SourceTexture->GetWidth(), SourceTexture->GetHeight());

				if(UniqueColors.Num() > 0)
				{
					// 4. Create Material ID collection from unique colors
					FMaterialIDCollection CachedMaterialIDCollection;

					UE_LOGF(LogData, Log, "T_ExtractMaterialIds : ExecAsyc : UniqueColors : Count : %i", UniqueColors.Num());
					
					for(int i = 0; i < UniqueColors.Num(); i++)
					{
						auto Color = UniqueColors[i];
						CachedMaterialIDCollection.Infos.Add(FMaterialIDInfo(i, Color.ToFColor(false) ,Color));
					}

					// 5. Convert the data in raw buffer.
					// So that we can cache it for the next time.
					const auto ElementSize = sizeof(FMaterialIDInfo);
					const int32 NumberOfElements = CachedMaterialIDCollection.Infos.Num();
					int32 DataSize = ElementSize * NumberOfElements;

					// Make a copy of the data.
					uint8* Buffer = new uint8[DataSize];
					FMemory::Memcpy(Buffer, CachedMaterialIDCollection.Infos.GetData(), DataSize );
					
					// 6. Cache the data by converting it to blob and passing it to our Blobber.
					BufferDescriptor Desc;
					Desc.Format = BufferFormat::Byte;
					Desc.Type = BufferType::Generic;
					Desc.Name = "MaterialIDInfos";
					Desc.Width = ElementSize;
					Desc.Height = NumberOfElements; 
					Desc.ItemsPerPoint = 1; 
					
					HashType FinalHash = DataUtil::Hash(Buffer, DataSize);
					auto Hash = std::make_shared<CHash>(FinalHash, true);
					
					RawBufferPtr RawBufferPtr = std::make_shared<RawBuffer>(Buffer, DataSize, Desc, Hash);
					
					Device* Device = TextureGraphEngine::GetDeviceManager()->GetDevice(DeviceType::Mem);
					
					auto Blob = TextureGraphEngine::GetBlobber()->Create(Device, RawBufferPtr);
					
					auto JobHash = Job::Hash();
					auto FinalBlob = TextureGraphEngine::GetBlobber()->AddResult(JobHash, Blob);
					
					// Record the loaded blob under the job JobHash so it is cached
					TextureGraphEngine::GetBlobber()->UpdateBlobHash(JobHash->Value(), Blob);

					// If existing collection is not same.
					if(CachedMaterialIDCollection.HashValue() != MaterialIDCollection.HashValue())
					{
						// 7. Clear the existing ids
						MaterialIDCollection.Infos.Empty();
                        MaterialIDMaskInfo.Empty();
                        ActiveColorsCount = 0;
						
						for(int i = 0; i < CachedMaterialIDCollection.Infos.Num(); i++)
                        {
                        	MaterialIDCollection.Infos.Add(CachedMaterialIDCollection.Infos[i]);
                        }
						
						// 8. Create Material ID Mask info used to active/deactive colors
	                    // Clear existing Ids
	                    for(const FMaterialIDInfo& MatIdInfo : MaterialIDCollection.Infos)
	                    {
                    		MaterialIDMaskInfo.Add(FMaterialIDMaskInfo(MatIdInfo.Id, MatIdInfo.RGBColor));
                    		
                    		UE_LOGF(LogData, Log, "FoundColor R :%f, G :%f, B :%f, ColorName : %ls", MatIdInfo.SrgbLinearColor.R, MatIdInfo.SrgbLinearColor.G, MatIdInfo.SrgbLinearColor.B, *ColorUtil::GetColorName(MatIdInfo.SrgbLinearColor));
	                    }
					}
					
				}
				else
				{
					MaterialIDCollection.Infos.Empty();
					MaterialIDMaskInfo.Empty();
					ActiveColorsCount = 0;
				}
			}
			
			EndNative();
			SetPromise(0);
			
			return 0;
		});
}

//////////////////////////////////////////////////////////////////////////


T_ExtractMaterialIds::T_ExtractMaterialIds()
{
}

T_ExtractMaterialIds::~T_ExtractMaterialIds()
{
}

TiledBlobPtr T_ExtractMaterialIds::Create(MixUpdateCyclePtr InCycle, TiledBlobPtr InMaterialIDTexture, FMaterialIDCollection& MaterialIDInfoCollection, TArray<FMaterialIDMaskInfo>& InMaterialIDMaskInfo, int32& ActiveColorsCount, int InTargetId)
{
	// Combine the tiled texture
	TiledBlobPtr CombinedBlob = T_CombineTiledBlob::Create(InCycle, InMaterialIDTexture->GetDescriptor(), InTargetId, InMaterialIDTexture);

	// Read the texture colors
	auto JobPtr = std::make_unique<Job_ExtractMaterialIds>(InTargetId, CombinedBlob, MaterialIDInfoCollection, InMaterialIDMaskInfo, ActiveColorsCount);
	JobPtr->AddArg(ARG_BLOB(InMaterialIDTexture, "SourceTexture"));
	
	InCycle->AddJob(InTargetId, std::move(JobPtr));
	
	return CombinedBlob;
}

