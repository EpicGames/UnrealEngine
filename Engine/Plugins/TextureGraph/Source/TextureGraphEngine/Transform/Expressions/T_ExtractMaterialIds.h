// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <2D/BlendModes.h>

#include "CoreMinimal.h"
#include "Helper/ColorUtil.h"
#include "Job/Job.h"
#include "Model/Mix/MixUpdateCycle.h"
#include "T_ExtractMaterialIds.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

typedef TArray<TArray<TArray<class HSVBucket*>>> Bucket3dArray;
typedef TArray<TArray<TArray<int32>>> Color3dArray;

USTRUCT()
struct FMaterialIDInfo
{
	GENERATED_BODY()

	FMaterialIDInfo() {}

	FMaterialIDInfo(int32 InId, FColor InSrgbColor, FLinearColor InSrgbLinearColor)
	{
		Id = InId;
		RGBColor = InSrgbColor;
		SrgbLinearColor = InSrgbLinearColor;
	}

	UPROPERTY()
	int32 Id = 0;
	
	UPROPERTY(VisibleAnywhere, Category = MaterialIDInfo)
	FColor RGBColor = FColor::Black;

	UPROPERTY()
	FLinearColor SrgbLinearColor = FLinearColor::Black;

	HashType HashValue() const
	{
		std::vector<HashType> Hashes =
		{
			DataUtil::Hash_Simple(Id),
			DataUtil::Hash_Simple(RGBColor),
			DataUtil::Hash_Simple(SrgbLinearColor)
		};
	
		return DataUtil::Hash(Hashes);
	}
};

USTRUCT()
struct FMaterialIDMaskInfo
{
	GENERATED_BODY()

	FMaterialIDMaskInfo(){}
	
	FMaterialIDMaskInfo(const int32 InId, FColor InDisplayColor)
	{
		MaterialIdReferenceId = InId;
		Color = InDisplayColor;
	}

	UPROPERTY()
	int32 MaterialIdReferenceId = 0;

	UPROPERTY(VisibleAnywhere, Category = MaterialIDInfo, meta = (NoResetToDefault))
	FColor Color = FColor::Black;

	UPROPERTY(EditAnywhere, Category = MaterialIDInfo)
	bool bIsEnabled = false;
};

USTRUCT()
struct FMaterialIDCollection
{
	GENERATED_BODY()

	FMaterialIDCollection() {}

	UPROPERTY()
	TArray<FMaterialIDInfo> Infos;

	int32 Num() const
	{
		return Infos.Num();
	}

	bool GetColor(const int32 Id, FLinearColor& OutColor)
	{
		for(const FMaterialIDInfo& MatIdInfo : Infos)
		{
			if(MatIdInfo.Id == Id)
			{
				OutColor = MatIdInfo.SrgbLinearColor;
				return true;
			}
		}

		return false;
	}

	HashType HashValue() const
	{
		if(Infos.Num() > 0)
		{
			std::vector<HashType> Hashes;

			for (const FMaterialIDInfo& MaterialIDInfo : Infos)
				Hashes.push_back(MaterialIDInfo.HashValue());

			return DataUtil::Hash(Hashes);
		}
		else
		{
			return DataUtil::GNullHash;
		}
	}

	FString GetColorName(const int32 Id)
	{
		FLinearColor FoundColor = FLinearColor::Black;
		
		GetColor(Id, FoundColor);

		return ColorUtil::GetColorName(FoundColor);
	}

	const FMaterialIDInfo* GetIdInfo(int32 Id)
	{
		const FMaterialIDInfo* MaterialIDInfo = nullptr;
		
		if(Infos.Num() > 0 && Infos.Num() < Id)
		{
			MaterialIDInfo = &Infos[Id];
		}

		return MaterialIDInfo;
	}
	
};

class HSVBucket
{
public:
	static const int HBuckets = 15;
	static const int SBuckets = 5;
	static const int VBuckets = 5;
	static const int HCount = 360 / HBuckets;
	static const int SCount = 100 / SBuckets;
	static const int VCount = 100 / VBuckets;

	static const int MinPixelsCountThreshold = 10;
	inline static const float MinPixelsInTextureThreshold = 0.01f;
	inline static const float MinPixelsOccurance = 0.3f;
	inline static const float HRange = 359.0f;// H Range is 0-360
	inline static const float SRange = 99.0f; // S Range is 0-100
	inline static const float VRange = 99.0f; // V Range is 0-100
	

public:
	int32				TotalCount = 0;                         /// Total count of all the pixels that fall into this bucket
	int32				MaxIndividualCount = 0;                 /// The max number of a pixel that has fallen into this bucket
	FIntVector3			MaxIndex;								/// Index of the colour with the max count
	Color3dArray		ColorCount;								/// Count of individual colours that fall into this bucket (in HSV space)
	FIntVector3			HSVStart;                               /// The starting value of the HSV colours
	TArray<FVector2f>	Positions;								/// Positions of each pixel
	FVector2f			MinPos;   								/// Min position (for calculating spread)
	FVector2f			MaxPos;   								/// Max position (for calculating spread)

	HSVBucket(int32 InHBucket, int32 InSBucket, int32 InVBucket);

	void Add(const FLinearColor Color, int H, int S, int V, FVector2f Position);
	void Add(HSVBucket Rhs);
	static FIntVector3 GetBucket(const FLinearColor Pixel, int& OutH, int& OutS, int& OutV);
	FLinearColor GetColor(FIntVector3& OutHsv);
	bool ShouldConsider(int32 Width, int32 Height);
};

//////////////////////////////////////////////////////////////////////////
class Job_ExtractMaterialIds : public Job
{
private:
	TiledBlobRef SourceTexture;
	TArray<FMaterialIDMaskInfo>& MaterialIDMaskInfo;
	FMaterialIDCollection& MaterialIDCollection;
	int32 ActiveColorsCount;

public:
	UE_API Job_ExtractMaterialIds(int32 targetId, TiledBlobRef InSourceTexture, FMaterialIDCollection& MaterialIDInfoCollection, TArray<FMaterialIDMaskInfo>& InMaterialIDMaskInfo, int32& ActiveColorsCount, UObject* InErrorOwner = nullptr, uint16 priority = (uint16)E_Priority::kHigh, uint64 id = 0);

protected:
	virtual void GetDependencies(JobPtrVec& prior, JobPtrVec& after, JobRunInfo runInfo) override { RunInfo = runInfo; Prev.clear(); }
	virtual AsyncPrepareResult PrepareTargets(JobBatch* batch) { return cti::make_ready_continuable(0); }
	virtual AsyncPrepareResult PrepareResources(JobBatch* batch) { return cti::make_ready_continuable(0); }

	virtual int32 Exec() override { return 0; }
	virtual cti::continuable<int32>	BindArgs_All(JobRunInfo runInfo) { return cti::make_ready_continuable(0); }
	virtual cti::continuable<int32>	UnbindArgs_All(JobRunInfo runInfo) { return cti::make_ready_continuable(0); }
	virtual bool CanHandleTiles() const { return false; }

	UE_API virtual cti::continuable<int32>	PreExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type returnThread) override;
	UE_API virtual cti::continuable<int32>	ExecAsync(ENamedThreads::Type ExecThread, ENamedThreads::Type ReturnThread) override;
};

/**
 * 
 */
class T_ExtractMaterialIds
{
public:
	UE_API T_ExtractMaterialIds();
	UE_API ~T_ExtractMaterialIds();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////

	static UE_API TiledBlobPtr Create(MixUpdateCyclePtr InCycle, TiledBlobPtr MaterialIDTexture, FMaterialIDCollection& MaterialIDCollection, TArray<FMaterialIDMaskInfo>& InMaterialIDMaskInfo, int32& ActiveColorsCount, int InTargetId);
};

#undef UE_API
