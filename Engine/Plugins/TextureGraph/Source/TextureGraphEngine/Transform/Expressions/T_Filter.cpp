// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_Filter.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Utility/T_CombineTiledBlob.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(T_Filter)

IMPLEMENT_GLOBAL_SHADER(FSH_EdgeDetect, "/Plugin/TextureGraph/Expressions/Expression_EdgeDetect.usf", "FSH_EdgeDetect", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_DirectionalWarp, "/Plugin/TextureGraph/Expressions/Expression_Warp.usf", "FSH_DirectionalWarp", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_NormalWarp, "/Plugin/TextureGraph/Expressions/Expression_Warp.usf", "FSH_NormalWarp", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_SineWarp, "/Plugin/TextureGraph/Expressions/Expression_Warp.usf", "FSH_SineWarp", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_ErodeDilate, "/Plugin/TextureGraph/Expressions/Expression_ErodeDilate.usf", "FSH_ErodeDilate", SF_Pixel);

TiledBlobPtr T_Filter::CreateEdgeDetect(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, float Thickness, int32 InTargetId)
{
	if (!SourceTexture)
		return TextureHelper::GetBlack();

	RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_EdgeDetect>(TEXT("T_EdgeDetect"));
	check(RenderMaterial);

	// Combine the tiled texture
	TiledBlobPtr CombinedBlob = T_CombineTiledBlob::Create(InCycle, SourceTexture->GetDescriptor(), InTargetId, SourceTexture);
	float StepX = Thickness / (float)SourceTexture->GetWidth();
	float StepY = Thickness / (float)SourceTexture->GetHeight();

	BufferDescriptor Desc = BufferDescriptor::Combine(DesiredDesc, SourceTexture->GetDescriptor());

	FTileInfo TileInfo;
	JobUPtr RenderJob = std::make_unique<Job>(InCycle->GetMix(), InTargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_TILEINFO(TileInfo, "TileInfo"))
		->AddArg(ARG_BLOB(CombinedBlob, "SourceTexture"))
		->AddArg(ARG_FLOAT(Thickness, "Thickness"))
		;

	const FString Name = FString::Printf(TEXT("T_EdgeDetect.[%llu]"), InCycle->GetBatch()->GetBatchId());

	TiledBlobPtr Result = RenderJob->InitResult(Name, &Desc);
	InCycle->AddJob(InTargetId, std::move(RenderJob));

	return Result;
}

template <typename FSH_Type>
static JobUPtr CreateWarp(FString Name, MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, TiledBlobPtr Mask, 
	float Intensity, int32 InTargetId, BufferDescriptor& Desc)
{
	/// If no mask is given then we just use a white mask
	if (!Mask)
		Mask = TextureHelper::GetWhite();

	RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_Type>(Name);
	check(RenderMaterial);

	// Combine the tiled texture
	TiledBlobPtr CombinedSourceTexture = T_CombineTiledBlob::Create(InCycle, SourceTexture->GetDescriptor(), InTargetId, SourceTexture);
	TiledBlobPtr CombinedMask = T_CombineTiledBlob::Create(InCycle, Mask->GetDescriptor(), InTargetId, Mask);
	BufferDescriptor CombinedDesc = BufferDescriptor::Combine(SourceTexture->GetDescriptor(), Mask->GetDescriptor());
	Desc = BufferDescriptor::Combine(DesiredDesc, CombinedDesc);

	FTileInfo TileInfo;
	JobUPtr RenderJob = std::make_unique<Job>(InCycle->GetMix(), InTargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_TILEINFO(TileInfo, "TileInfo"))
		->AddArg(ARG_BLOB(CombinedSourceTexture, "SourceTexture"))
		->AddArg(ARG_BLOB(CombinedMask, "Mask"))
		->AddArg(ARG_FLOAT(Intensity, "Intensity"))
		;

	return RenderJob;
}

TiledBlobPtr T_Filter::CreateDirectionalWarp(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, TiledBlobPtr Mask, float Intensity, float AngleRad, int32 InTargetId)
{
	if (!SourceTexture)
		return TextureHelper::GetBlack();

	BufferDescriptor Desc;
	JobUPtr RenderJob = CreateWarp<FSH_DirectionalWarp>(TEXT("T_DirectionalWarp"), InCycle, DesiredDesc, SourceTexture, Mask, Intensity, InTargetId, Desc);
	RenderJob
		->AddArg(ARG_FLOAT(AngleRad, "AngleRad"))
		;

	const FString Name = FString::Printf(TEXT("T_DirectionalWarp.[%llu]"), InCycle->GetBatch()->GetBatchId());

	TiledBlobPtr Result = RenderJob->InitResult(Name, &Desc);
	InCycle->AddJob(InTargetId, std::move(RenderJob));

	return Result;
}

TiledBlobPtr T_Filter::CreateNormalWarp(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, TiledBlobPtr Mask, float Intensity, int32 InTargetId)
{
	if (!SourceTexture)
		return TextureHelper::GetBlack();

	BufferDescriptor Desc;
	JobUPtr RenderJob = CreateWarp<FSH_NormalWarp>(TEXT("T_NormalWarp"), InCycle, DesiredDesc, SourceTexture, Mask, Intensity, InTargetId, Desc);
	const FString Name = FString::Printf(TEXT("T_NormalWarp.[%llu]"), InCycle->GetBatch()->GetBatchId());

	TiledBlobPtr Result = RenderJob->InitResult(Name, &Desc);
	InCycle->AddJob(InTargetId, std::move(RenderJob));

	return Result;
}

TiledBlobPtr T_Filter::CreateSineWarp(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, TiledBlobPtr Mask, float Intensity, float PhaseU, float PhaseV, int32 InTargetId)
{
	if (!SourceTexture)
		return TextureHelper::GetBlack();

	BufferDescriptor Desc;
	JobUPtr RenderJob = CreateWarp<FSH_SineWarp>(TEXT("T_SineWarp"), InCycle, DesiredDesc, SourceTexture, Mask, Intensity, InTargetId, Desc);
	RenderJob
		->AddArg(ARG_FLOAT(PhaseU, "PhaseU"))
		->AddArg(ARG_FLOAT(PhaseV, "PhaseV"))
		;

	const FString Name = FString::Printf(TEXT("T_SineWarp.[%llu]"), InCycle->GetBatch()->GetBatchId());

	TiledBlobPtr Result = RenderJob->InitResult(Name, &Desc);
	InCycle->AddJob(InTargetId, std::move(RenderJob));

	return Result;
}

