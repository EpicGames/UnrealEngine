// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <2D/BlendModes.h>

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>
#include "T_Filter.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Edge detection
//////////////////////////////////////////////////////////////////////////
class FSH_EdgeDetect : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_EdgeDetect, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_EdgeDetect, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER(float, Thickness)
	END_SHADER_PARAMETER_STRUCT()

public:
	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

UENUM(BlueprintType)
namespace EWarp
{
	enum Type : int
	{
		Directional = 0				UMETA(DisplayName = "Directional"),
		Normal = 1					UMETA(DisplayName = "Normal"),
		Sine = 2					UMETA(DisplayName = "Sine"),
	};
}

//////////////////////////////////////////////////////////////////////////
/// Directional warp
//////////////////////////////////////////////////////////////////////////
class FSH_DirectionalWarp : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_DirectionalWarp, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_DirectionalWarp, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, Mask)
		SHADER_PARAMETER(float, AngleRad)
		SHADER_PARAMETER(float, Intensity)
	END_SHADER_PARAMETER_STRUCT()

public:
	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Directional warp
//////////////////////////////////////////////////////////////////////////
class FSH_NormalWarp : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_NormalWarp, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_NormalWarp, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, Mask)
		SHADER_PARAMETER(float, Intensity)
	END_SHADER_PARAMETER_STRUCT()

public:
	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

//////////////////////////////////////////////////////////////////////////
/// Directional warp
//////////////////////////////////////////////////////////////////////////
class FSH_SineWarp : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_SineWarp, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_SineWarp, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, Mask)
		SHADER_PARAMETER(float, Intensity)
		SHADER_PARAMETER(float, PhaseU)
		SHADER_PARAMETER(float, PhaseV)
	END_SHADER_PARAMETER_STRUCT()

public:
	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

class FVar_ErodeDilate_Type : SHADER_PERMUTATION_INT("ED_TYPE", 2);
class FVar_ErodeDilate_Kernel : SHADER_PERMUTATION_INT("ED_KERNEL", 3);
class FVar_ErodeDilate_IsSingleChannel : SHADER_PERMUTATION_BOOL("ED_SINGLECHANNEL");
//////////////////////////////////////////////////////////////////////////
/// Erode/Dilate
//////////////////////////////////////////////////////////////////////////
class FSH_ErodeDilate : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_ErodeDilate, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_ErodeDilate, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, Input)
		SHADER_PARAMETER(int, Size)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<FVar_ErodeDilate_Type, FVar_ErodeDilate_Kernel, FVar_ErodeDilate_IsSingleChannel>;
public:
	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};


//////////////////////////////////////////////////////////////////////////
class T_Filter
{
public:
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API TiledBlobPtr				CreateEdgeDetect(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, float Thickness, int32 InTargetId);
	static UE_API TiledBlobPtr				CreateDirectionalWarp(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, TiledBlobPtr Mask, float Intensity, float AngleRad, int32 InTargetId);
	static UE_API TiledBlobPtr				CreateNormalWarp(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, TiledBlobPtr Mask, float Intensity, int32 InTargetId);
	static UE_API TiledBlobPtr				CreateSineWarp(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredDesc, TiledBlobPtr SourceTexture, TiledBlobPtr Mask, float Intensity, float PhaseU, float PhaseV, int32 InTargetId);
};

#undef UE_API
