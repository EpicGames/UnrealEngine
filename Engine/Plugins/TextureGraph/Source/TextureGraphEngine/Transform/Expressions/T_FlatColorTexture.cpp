// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_FlatColorTexture.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"
#include "2D/TargetTextureSet.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"

IMPLEMENT_GLOBAL_SHADER(FSH_FlatColorTexture, "/Plugin/TextureGraph/Layer/ChannelSourceFlat.usf", "FSH_FlatColorTexture", SF_Pixel);

BufferDescriptor T_FlatColorTexture::GetFlatColorDesc(FString name, BufferFormat InBufferFormat)
{
	BufferDescriptor Desc;
	
	Desc.Name = name;
	Desc.Format = InBufferFormat;
	Desc.ItemsPerPoint = 4;
	Desc.Width = 1;
	Desc.Height = 1;
	Desc.bMipMaps = false;
	
	return Desc;
}

TiledBlobPtr T_FlatColorTexture::Create(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredOutputDesc, FLinearColor Color, int InTargetId)
{
	const RenderMaterial_FXPtr SimpleColor = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_FlatColorTexture>(TEXT("T_FlatColorTexture"));

	check(SimpleColor);

	JobUPtr RenderJob = std::make_unique<Job>(InCycle->GetMix(), InTargetId, std::static_pointer_cast<BlobTransform>(SimpleColor));

	BufferDescriptor Desc;

	Desc.Width = InCycle->GetMix()->GetNumXTiles();
	Desc.Height = InCycle->GetMix()->GetNumYTiles();
	Desc.Format = BufferFormat::Byte;
	Desc.ItemsPerPoint = 4;
	Desc.bMipMaps = false;
	
	BufferDescriptor OutputDesc = BufferDescriptor::Combine(Desc, DesiredOutputDesc);
	if (OutputDesc.Format == BufferFormat::Byte) // Byte size automatically enable sRGB
	{
		OutputDesc.bIsSRGB = true;
	}

	OutputDesc.DefaultValue = Color;

	RenderJob->AddArg(ARG_LINEAR_COLOR(Color, "Color"))
		 ->AddArg(WithUnbounded(ARG_INT(OutputDesc.ItemsPerPoint,"ItemsPerPoint")))
		 ->AddArg(WithUnbounded(ARG_INT(OutputDesc.Height, "Height")))
		 ->AddArg(WithUnbounded(ARG_INT(OutputDesc.Width, "Width")))
		 ->AddArg(WithUnbounded(ARG_INT((int32)OutputDesc.Format, "Format")))
		 ->AddArg(WithUnbounded(ARG_INT((int32)OutputDesc.bMipMaps, "MipMaps")));

	auto JobResult = RenderJob->InitResult(SimpleColor->GetName(), &OutputDesc);
	
	InCycle->AddJob(InTargetId, std::move(RenderJob));

	return JobResult;
}
