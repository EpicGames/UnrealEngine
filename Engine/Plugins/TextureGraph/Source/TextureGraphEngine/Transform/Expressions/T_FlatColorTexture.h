// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////

class FSH_FlatColorTexture : public FSH_Base
{
public:

	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_FlatColorTexture, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_FlatColorTexture, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FLinearColor, Color)
		END_SHADER_PARAMETER_STRUCT()

};

typedef FxMaterial_Normal<VSH_Simple, FSH_FlatColorTexture>	Fx_FlatColorTexture;

/**
 * 
 */
class T_FlatColorTexture
{
public:
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API BufferDescriptor			GetFlatColorDesc(FString name, BufferFormat InBufferFormat = BufferFormat::Byte);
	static UE_API TiledBlobPtr				Create(MixUpdateCyclePtr InCycle, BufferDescriptor DesiredOutputDesc, FLinearColor Color, int InTargetId);
};

#undef UE_API
