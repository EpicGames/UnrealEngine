// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_Invert.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "Math/Vector.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"

IMPLEMENT_GLOBAL_SHADER(FSH_Invert, "/Plugin/TextureGraph/Expressions/Expression_Invert.usf", "FSH_Invert", SF_Pixel);

TiledBlobPtr T_Invert::Create(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Source, float MaxValue, int32 TargetId, bool bIncludeAlpha, bool bClamp)
{
	FSH_Invert::FPermutationDomain PermutationVector;
	PermutationVector.Set<FSH_Invert::FInvertIncludeAlpha>(bIncludeAlpha);
	PermutationVector.Set<FSH_Invert::FInvertClamp>(bClamp);

	RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_Invert>(TEXT("T_Invert"), PermutationVector);
	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_BLOB(Source, "SourceTexture"))
		->AddArg(ARG_FLOAT(MaxValue, "MaxValue"))
		->AddArg(WithUnbounded(ARG_BOOL(bIncludeAlpha, "IncludeAlpha")))
		->AddArg(WithUnbounded(ARG_BOOL(bClamp, "Clamp")))
		;

	FString Name = TEXT("Invert");

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}
