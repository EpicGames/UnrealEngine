// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////
class FSH_Invert : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_Invert, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_Invert, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER(float, MaxValue)
	END_SHADER_PARAMETER_STRUCT()

	class FInvertIncludeAlpha : SHADER_PERMUTATION_BOOL("INVERT_INCLUDE_ALPHA");
	class FInvertClamp : SHADER_PERMUTATION_BOOL("INVERT_CLAMP");
	using FPermutationDomain = TShaderPermutationDomain<FInvertIncludeAlpha, FInvertClamp>;

public:
	static bool						ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
	static void						ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env) {}
};

class T_Invert
{
public:
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API TiledBlobPtr				Create(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Source, float MaxValue, int32 TargetId, bool bIncludeAlpha, bool bClamp);
};

#undef UE_API
