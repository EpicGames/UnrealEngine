// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_Levels.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "Math/Vector.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"
#include "Transform/Utility/T_TextureHistogram.h"

IMPLEMENT_GLOBAL_SHADER(FSH_Levels, "/Plugin/TextureGraph/Expressions/Expression_Levels.usf", "FSH_Levels", SF_Pixel);

bool FLevels::SetHigh(float InValue)
{
	float NewValue = std::min(1.f, std::max(InValue, Low));
	if (NewValue != High)
	{
		float CurveExponent = EvalMidExponent();
		High = NewValue;
		return SetMidFromMidExponent(CurveExponent);
	}
	return false;
}

bool FLevels::SetLow(float InValue)
{
	float NewValue = std::max(0.f, std::min(InValue, High));
	if (NewValue != Low)
	{
		float CurveExponent = EvalMidExponent();
		Low = NewValue;
		return SetMidFromMidExponent(CurveExponent);
	}
	return false;
}

bool FLevels::SetMid(float InValue)
{
	float NewValue = std::max(Low, std::min(InValue, High));
	if (NewValue != Mid)
	{
		Mid = NewValue;
		return true;
	}
	return false;
}

float FLevels::EvalRange(float Val) const
{
	return std::max(0.f, std::min((Val - Low)/(High - Low), 1.f));
}
float FLevels::EvalRangeInv(float Val) const
{
	return Val * (High - Low) + Low;
}

float FLevels::EvalMidExponent() const
{
	// 0.5 = EvalRange(Mid) ^ Exponent
	// thus
	float MidRanged = EvalRange(Mid);
	MidRanged = std::max(0.001f, std::min(9.999f, MidRanged));

	return log(0.5) / log(MidRanged);
}

bool FLevels::SetMidFromMidExponent(float InExponent)
{
	// 0.5 = EvalRange(Mid) ^ Exponent
	// thus
	float NewValue = EvalRangeInv(pow(0.5, 1.0/InExponent));

	if (NewValue != Mid)
	{
		Mid = NewValue;
		return true;
	}
	return false;
}

void FLevels::InitFromLowMidHigh(float LowValue, float MidValue, float HighValue, float OutLowValue, float OutHighValue)
{
	Low = std::max(0.0f, LowValue);
	Mid = MidValue;
	High = std::min(1.0f, HighValue);

	OutLow = OutLowValue;
	OutHigh = OutHighValue;

	IsAutoLevels = false;
}

void FLevels::InitFromAutoLevels(float InMidPercentage)
{
	IsAutoLevels = true;
	MidPercentage = std::min(std::max(0.0f, InMidPercentage), 1.0f);
}

void FLevels::InitFromHistogramScan(float InPosition, float InContrast)
{
	// Histogram scan is affecting the input low and high and the mid.
	// InPosition determines the center of the remapped input range
	// Contrast controls the width of the range of remapping
	// Contrast is scaled depending on the InPosition
	// Contrast shrink to 0 when InPosition is at 0 or 1
	// Contrast is whole when InPosition is at 0.5

	float P = 1.0f - FMath::Clamp(InPosition, 0, 1);

	float C = (1.0 - FMath::Clamp(InContrast, 0, 1)) * 0.5f;
	float D = 1.0f - FMath::Abs(0.5f - P) * 2.0f;
	float ScaledC = C * D;

	Low = P - ScaledC;
	High = P + ScaledC;

	Mid = Low + (High - Low) * 0.5f;

	IsAutoLevels = false;
}

void FLevels::InitFromHistogramRange(float InPosition,  float InRange)
{
	// Histogram range is just affecting the output low and high

	float range = FMath::Clamp(InRange, 0.0f, 1.0f) * 0.5f;
	float pos = FMath::Clamp(InPosition, 0.0f, 1.0f);

	OutLow = FMath::Clamp(pos - range, 0.0f, 1.0f);
	OutHigh = FMath::Clamp(pos + range, 0.0f, 1.0f);

	IsAutoLevels = false;
}

class TEXTUREGRAPHENGINE_API RenderMaterial_FX_Levels : public RenderMaterial_FX
{
	using Super = RenderMaterial_FX;
private:
	FLevelsPtr Levels;

public:
	RenderMaterial_FX_Levels(FString InName, FxMaterialPtr InMaterial, const FLevelsPtr& InLevels);
	virtual ~RenderMaterial_FX_Levels() override;

	virtual AsyncPrepareResult		PrepareResources(const TransformArgs& Args) override;
	virtual BlobTransformPtr		DuplicateInstance(FString InName) override;

	virtual AsyncTransformResultPtr	Exec(const TransformArgs& Args) override;
};


RenderMaterial_FX_Levels::RenderMaterial_FX_Levels(FString InName, FxMaterialPtr InMaterial, const FLevelsPtr& InLevels) :
	RenderMaterial_FX(InName, InMaterial),
	Levels(InLevels)
{

}

RenderMaterial_FX_Levels::~RenderMaterial_FX_Levels()
{

}

AsyncPrepareResult		RenderMaterial_FX_Levels::PrepareResources(const TransformArgs& Args)
{
	return Super::PrepareResources(Args);
}


std::shared_ptr<BlobTransform> RenderMaterial_FX_Levels::DuplicateInstance(FString InName)
{
	if (InName.IsEmpty())
		InName = Name;

	check(FXMaterial);
	FxMaterialPtr Clone = FXMaterial->Clone();
	check(Clone);

	return std::static_pointer_cast<BlobTransform>(std::make_shared<RenderMaterial_FX_Levels>(InName, Clone, Levels));
}

AsyncTransformResultPtr	RenderMaterial_FX_Levels::Exec(const TransformArgs& Args)
{
	if (Levels->IsAutoLevels)
	{
		float Low = Levels->HistogramData.HistogramData[256 + 2].X;
		float High = Levels->HistogramData.HistogramData[256 + 3].X;
		float Mid = Levels->MidPercentage * (High - Low) + Low;

		// feedback the min max mid values in the Levels struct
		Levels->Low = Low;
		Levels->High = High;
		Levels->Mid = Mid;

		//Pass on uniform values to the shader
		SetFloat(TEXT("LowValue"), Low);
		SetFloat(TEXT("MidValue"), Mid);
		SetFloat(TEXT("HighValue"), High);

	}
	return Super::Exec(Args);
}


//////////////////////////////////////////////////////////////////////////
class TEXTUREGRAPHENGINE_API Job_Levels : public Job
{
private:
	TiledBlobRef Histogram;
	FLevelsPtr Levels;

public:
	Job_Levels(UMixInterface* InMix, int32 targetId, BlobTransformPtr InTransform, TiledBlobRef InHistogram, const FLevelsPtr& InLevels, UObject* InErrorOwner = nullptr, uint16 priority = (uint16)E_Priority::kHigh, uint64 id = 0);

protected:

	virtual cti::continuable<int32>	PreExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type returnThread) override;
};

Job_Levels::Job_Levels(UMixInterface* InMix, int32 targetId, BlobTransformPtr InTransform, TiledBlobRef InSourceHistogram, const FLevelsPtr& InLevels,
											   UObject* InErrorOwner /*= nullptr*/, uint16 priority /*= (uint16)E_Priority::kHigh*/, uint64 id /*= 0*/)
: Job(InMix, targetId, InTransform, InErrorOwner, priority)
, Histogram(InSourceHistogram)
, Levels(InLevels)
{
	DeviceNativeTask::Name = TEXT("Levels");
}

cti::continuable<int32> Job_Levels::PreExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type returnThread)
{
	//TODO: Need to cache the Result so that the job do not execute every time with matching arguments
/*	BlobPtr cachedResult = TextureGraphEngine::GetBlobber()->Find(Job::Hash()->Value());
	if (cachedResult)
	{
		bIsCulled = true;
		bIsDone = true;

		return cti::make_ready_continuable(0);
	}*/

	// Early exit if download not needed
	if (!Levels->IsAutoLevels)
	{
		return cti::make_ready_continuable(0);
	}

	// Now the job is really needed, let's go on with the Histogram download:
	BlobRef HistoBuffer = Histogram->GetTile(0,0);
	if (HistoBuffer)
	{
		/*
		* // TODO ITRied to use this path, but what happen when the download of the raw is in flight ?
		if (!HistoBuffer->GetBufferRef()->HasRaw())
		{
			/// If it's already fetching raw, then we don't want to do it again
			if (HistoBuffer->GetBufferRef()->IsFetchingRaw())
				return cti::make_ready_continuable(0);
				*/
			return HistoBuffer->GetBufferRef()->GetRawOrMaketIt().then([this](RawBufferPtr Raw)
			{
				// TODO: This should NOT happen ?
				if (!Raw)
				{
					check(Raw)
					return cti::make_ready_continuable(0);
				}
				check(Raw->GetDescriptor().Width == 256)
				check(Raw->GetDescriptor().Height == 2)
				check(Raw->GetDescriptor().Size() == Raw->GetLength())
				check(Raw->HasData())
	
				int32 NumBins = 256;
				int32 NumMetaBins = 5;
				const uint8* RawData = Raw->GetData();	
				if (RawData)
				{
					Levels->HistogramData.HistogramData.Append((FVector4f*)RawData, (NumBins + NumMetaBins));
				}	
				return cti::make_ready_continuable(0);
			});

		//}
	}

	return cti::make_ready_continuable(0);
}


TiledBlobPtr	T_Levels::Create(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Source, const FLevelsPtr& InLevels, int32 TargetId)
{
	bool NeedsConvertToGrayscale = (Source->GetDescriptor().ItemsPerPoint > 1) ;
	
	if (!Source)
	{
		return TextureHelper::GBlack;
	}

	bool bNeedHistogramRaw = false;
	TiledBlobPtr Histogram = TextureHelper::GBlack;

	if (InLevels->IsAutoLevels)
	{
		bNeedHistogramRaw = true;
		Histogram = T_TextureHistogram::Create(Cycle, Source, TargetId);
	}

	if (!Histogram)
	{
		Histogram = TextureHelper::GBlack;
		bNeedHistogramRaw = false;
	}
	
	FSH_Levels::FPermutationDomain PermutationVector;
	PermutationVector.Set<FSH_Levels::FConvertToGrayscale>(NeedsConvertToGrayscale);
	PermutationVector.Set<FSH_Levels::FIsAutoLevels>(InLevels->IsAutoLevels);

	std::shared_ptr<FxMaterial_Normal<VSH_Simple, FSH_Levels>> Mat = 
		std::make_shared<FxMaterial_Normal<VSH_Simple, FSH_Levels>>(typename VSH_Simple::FPermutationDomain(), PermutationVector);

	const RenderMaterial_FXPtr RenderMaterial = std::make_shared<RenderMaterial_FX_Levels>(TEXT("T_Levels"), Mat, InLevels);
	check(RenderMaterial);

	float OutputRange = FMath::Clamp(InLevels->OutHigh - InLevels->OutLow, 0.0f, 1.0f);

	JobUPtr RenderJob = std::make_unique<Job_Levels>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial), Histogram, InLevels);
	RenderJob
		->AddArg(ARG_BLOB(Source, "SourceTexture"))
		->AddArg(ARG_FLOAT(InLevels->Low, "LowValue"))
		->AddArg(ARG_FLOAT(InLevels->High, "HighValue"))
		->AddArg(ARG_FLOAT(InLevels->Mid, "MidValue"))
		->AddArg(ARG_FLOAT(float(InLevels->IsAutoLevels), "DoAutoLevel"))
		->AddArg(ARG_FLOAT(InLevels->MidPercentage, "MidPercentage"))
		->AddArg(ARG_FLOAT(InLevels->OutLow, "OutLow"))
		->AddArg(ARG_FLOAT(InLevels->OutHigh, "OutHigh"))
		->AddArg(ARG_FLOAT(OutputRange, "OutputRange"))
		->AddArg(WithIgnoreDesc(std::make_shared<JobArg_Blob>(JobArg_Blob(Histogram, "Histogram").WithNotHandleTiles())))
		;


	if (DesiredOutputDesc.Format == BufferFormat::Auto)
	{
		DesiredOutputDesc.Format = Source->GetDescriptor().Format;
	}

	TiledBlobPtr Result = RenderJob->InitResult(TEXT("Levels"), &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;

}
