// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API


//////////////////////////////////////////////////////////////////////////
/// Simple levels shader
//////////////////////////////////////////////////////////////////////////
class FSH_Levels : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_Levels, FSH_Base);
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_Levels, UE_API);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER(float, LowValue)
		SHADER_PARAMETER(float, HighValue)
		SHADER_PARAMETER(float, MidValue)
		SHADER_PARAMETER(float, DoAutoLevel)
		SHADER_PARAMETER(float, MidPercentage)
		SHADER_PARAMETER(float, OutLow)
		SHADER_PARAMETER(float, OutHigh)
		SHADER_PARAMETER(float, OutputRange)
		SHADER_PARAMETER_TEXTURE(Texture2D, Histogram)
	END_SHADER_PARAMETER_STRUCT()

	class FConvertToGrayscale : SHADER_PERMUTATION_BOOL("CONVERT_TO_GRAYSCALE");
	class FIsAutoLevels : SHADER_PERMUTATION_BOOL("AUTO_LEVELS");
	using FPermutationDomain = TShaderPermutationDomain<FConvertToGrayscale, FIsAutoLevels>;

	static bool						ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
	static void						ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env) {}
};


struct FLevelsHistogramData
{
	TArray<FVector4f> HistogramData;
};


struct FLevels
{

	// The Low value of the Levels adjustment, any pixel under that value is set to black. Default is 0.
	float								Low = 0;

	// The mid value of the Levels adjustment, must be in the range [Min, Max] and the Default is 0.5.
	// The mid value determine where the smoothing curve applying the midpoint filter is crossing 0.5.
	float								Mid = 0.5f; // Midtones

	// The High value of the Levels adjustment, any pixel above that value is set to white. Default is 1.
	float								High = 1;

	bool 								IsAutoLevels = false;
	float 								MidPercentage = 0.5f;

	// The black point of the output. Moving this will remap the dark point to this value. Default is 0
	float								OutLow = 0;

	// The white point of the output. Moving this will remap the white point to this value. Default is 1
	float								OutHigh = 1;

	UE_API bool SetLow(float InValue);
	UE_API bool SetMid(float InValue);
	UE_API bool SetHigh(float InValue);

	// Eval Low-High range mapping on value
	UE_API float EvalRange(float Val) const;
	// Eval reverse Low-High range mapping on value
	UE_API float EvalRangeInv(float Val) const;

	// Evaluate the MidExponent of the power curve applying the midpoint filter
	UE_API float EvalMidExponent() const;
	UE_API bool SetMidFromMidExponent(float InExponent);

	// various init from different parameter set
	UE_API void InitFromLowMidHigh(float LowValue, float MidValue, float HighValue, float OutLow, float OutHigh);
	UE_API void InitFromAutoLevels(float MidPercentage);
	UE_API void InitFromHistogramScan(float InPosition, float InContrast);
	UE_API void InitFromHistogramRange(float InPosition, float InRange);

	FLevelsHistogramData HistogramData;
};

using FLevelsPtr = TSharedPtr< FLevels>;

class T_Levels
{
public:
	static UE_API TiledBlobPtr				Create(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Source, const FLevelsPtr& Levels, int32 TargetId);
};

#undef UE_API
