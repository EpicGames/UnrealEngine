// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_MaterialIDMask.h"
#include "T_ExtractMaterialIds.h"
#include "FxMat/MaterialManager.h"

IMPLEMENT_GLOBAL_SHADER(FSH_MaterialIDMask, "/Plugin/TextureGraph/Expressions/Expression_MaterialID.usf", "FSH_MaterialIDMask", SF_Pixel);


T_MaterialIDMask::T_MaterialIDMask()
{
}

T_MaterialIDMask::~T_MaterialIDMask()
{
	
}

TiledBlobPtr T_MaterialIDMask::Create(MixUpdateCyclePtr InCycle, TiledBlobPtr InMaterialIDTexture, const TArray<FLinearColor>& InActiveColors, const int32& ActiveColorsCount, BufferDescriptor DesiredDesc, int InTargetId)
{
	TArray<FVector4f>Buckets;

	Buckets.SetNum(HSVBucket::HBuckets * HSVBucket::SBuckets * HSVBucket::VBuckets);

	for (int i = 0; i < Buckets.Num(); i++)
	{
		Buckets[i] = FVector4f::Zero();
	}
	
	for (int i = 0; i < ActiveColorsCount; i++)
	{
		int32 H, S, V;
		FIntVector3 Bucket = HSVBucket::GetBucket(InActiveColors[i], H, S, V);
	
		int BucketId = Bucket.X + Bucket.Y * HSVBucket::HBuckets + Bucket.Z * (HSVBucket::SBuckets * HSVBucket::HBuckets);
	
		check(BucketId >= 0 && BucketId < Buckets.Num());
	
		Buckets[BucketId] = FVector4f(1, 0 ,0 ,0);
	}

	const RenderMaterial_FXPtr MaterialIDMaskMat = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_MaterialIDMask>(TEXT("T_MaterialIDMask"));

	check(MaterialIDMaskMat);

	JobUPtr JobPtr = std::make_unique<Job>(InCycle->GetMix(), InTargetId, std::static_pointer_cast<BlobTransform>(MaterialIDMaskMat));
	
	JobPtr
		->AddArg(ARG_BLOB(InMaterialIDTexture, "MaterialIDTexture"))
		->AddArg(ARG_ARRAY(FVector4f, Buckets, "Buckets"))
		->AddArg(ARG_ARRAY(FLinearColor, InActiveColors, "ActiveColors"))
		->AddArg(ARG_INT(ActiveColorsCount, "ActiveColorsCount"));

	BufferDescriptor Desc = DesiredDesc;

	// Default format to RGBA8
	if (Desc.Format == BufferFormat::Auto)
		Desc.Format = BufferFormat::Byte;
	if (Desc.ItemsPerPoint == 0)
		Desc.ItemsPerPoint = 4;

	Desc.DefaultValue = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);

	if (Desc.Width <= 0) 
	{
		Desc.Width = InMaterialIDTexture->GetWidth();
	}

	if(Desc.Height <= 0)
	{
		Desc.Height = InMaterialIDTexture->GetHeight();
	}
	
	auto Result = JobPtr->InitResult(MaterialIDMaskMat->GetName(), &Desc);

	InCycle->AddJob(InTargetId, std::move(JobPtr));
	
	return Result;
}
