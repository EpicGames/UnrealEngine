// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <2D/BlendModes.h>

#include "CoreMinimal.h"
#include "T_ExtractMaterialIds.h"
#include "FxMat/FxMaterial.h"
#include "Job/Job.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////

class FSH_MaterialIDMask : public FSH_Base
{
public:

	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_MaterialIDMask, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_MaterialIDMask, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, MaterialIDTexture)
		SHADER_PARAMETER_ARRAY(FVector4f, Buckets, [HSVBucket::HBuckets * HSVBucket::SBuckets * HSVBucket::VBuckets])
		SHADER_PARAMETER_ARRAY(FVector4f, ActiveColors, [128])
		SHADER_PARAMETER(int, ActiveColorsCount)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
	END_SHADER_PARAMETER_STRUCT()

	static bool	ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParams, FShaderCompilerEnvironment& InEnv)
	{
	}

};

/**
 * 
 */
class T_MaterialIDMask
{
public:
	UE_API T_MaterialIDMask();
	UE_API ~T_MaterialIDMask();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////

	static UE_API TiledBlobPtr				Create(MixUpdateCyclePtr InCycle, TiledBlobPtr InMaterialIDTexture, const TArray<FLinearColor>& InActiveColors, const int32& ActiveColorsCount, BufferDescriptor DesiredDesc, int InTargetId);
};

#undef UE_API
