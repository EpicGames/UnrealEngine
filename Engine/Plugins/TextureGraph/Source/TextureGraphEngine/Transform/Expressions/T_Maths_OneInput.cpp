// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_Maths_OneInput.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "Math/Vector.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(T_Maths_OneInput)

IMPLEMENT_GLOBAL_SHADER(FSH_Sin, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Sin", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Cos, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Cos", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Tan, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Tan", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_ASin, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_ASin", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_ACos, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_ACos", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_ATan, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_ATan", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(FSH_ToRadians, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_ToRadians", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_ToDegrees, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_ToDegrees", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(FSH_Abs, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Abs", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Sqrt, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Sqrt", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Square, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Square", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Cube, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Cube", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Cbrt, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Cbrt", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Exp, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Exp", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Log2, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Log2", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Log10, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Log10", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Log, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Log", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Floor, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Floor", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Ceil, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Ceil", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Round, "/Plugin/TextureGraph/Expressions/Expression_Maths_OneInput.usf", "FSH_Round", SF_Pixel);

template <typename FSH_Type>
TiledBlobPtr CreateGenericOneInput(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input, FString TransformName)
{	
	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_Type>(TransformName);

	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_BLOB(Input, "Input"))
		;

	const FString Name = FString::Printf(TEXT("Maths_OneInput_%s"), *TransformName);

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}


TiledBlobPtr T_Maths_OneInput::CreateTrigonometry(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input, ETrigFunction Function)
{	
	switch (Function)
	{
	case ETrigFunction::Sin:
		return CreateGenericOneInput<FSH_Sin>(Cycle, DesiredOutputDesc, TargetId, Input, "T_Sin");
	case ETrigFunction::Cos:
		return CreateGenericOneInput<FSH_Cos>(Cycle, DesiredOutputDesc, TargetId, Input, "T_Cos");
	case ETrigFunction::Tan:
		return CreateGenericOneInput<FSH_Tan>(Cycle, DesiredOutputDesc, TargetId, Input, "T_Tan");
	case ETrigFunction::ASin:
		return CreateGenericOneInput<FSH_ASin>(Cycle, DesiredOutputDesc, TargetId, Input, "T_ASin");
	case ETrigFunction::ACos:
		return CreateGenericOneInput<FSH_ACos>(Cycle, DesiredOutputDesc, TargetId, Input, "T_ACos");
	case ETrigFunction::ATan:
		return CreateGenericOneInput<FSH_ATan>(Cycle, DesiredOutputDesc, TargetId, Input, "T_ATan");
	default:
		check(false);
		break;
	}

	return nullptr;
}

#define IMPL_SINGLE_ARG_SHADER(Name) TiledBlobPtr T_Maths_OneInput::Create##Name(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input) \
{ \
	FString TransformName = FString(TEXT("T_")) + FString(#Name); \
	return CreateGenericOneInput<FSH_##Name>(Cycle, DesiredOutputDesc, TargetId, Input, TransformName); \
}

IMPL_SINGLE_ARG_SHADER(Abs);
IMPL_SINGLE_ARG_SHADER(Sqrt);
IMPL_SINGLE_ARG_SHADER(Square);
IMPL_SINGLE_ARG_SHADER(Cube);
IMPL_SINGLE_ARG_SHADER(Cbrt);
IMPL_SINGLE_ARG_SHADER(Exp);
IMPL_SINGLE_ARG_SHADER(Log2);
IMPL_SINGLE_ARG_SHADER(Log10);
IMPL_SINGLE_ARG_SHADER(Log);
IMPL_SINGLE_ARG_SHADER(Floor);
IMPL_SINGLE_ARG_SHADER(Ceil);
IMPL_SINGLE_ARG_SHADER(Round);

#undef IMPL_SINGLE_ARG_SHADER
