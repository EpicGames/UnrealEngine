// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#include "T_Maths_OneInput.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Basic one input maths op
//////////////////////////////////////////////////////////////////////////
class FSH_MathsOp_OneInput : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_MathsOp_OneInput, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, Input)
	END_SHADER_PARAMETER_STRUCT()

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Sin, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Cos, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Tan, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_ASin, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_ACos, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_ATan, FSH_MathsOp_OneInput);

DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_ToRadians, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_ToDegrees, FSH_MathsOp_OneInput);

DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Abs, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Sqrt, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Square, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Cube, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Cbrt, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Exp, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Log2, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Log10, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Log, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Floor, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Ceil, FSH_MathsOp_OneInput);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Round, FSH_MathsOp_OneInput);


//////////////////////////////////////////////////////////////////////////
/// Trigonometric functions
//////////////////////////////////////////////////////////////////////////
UENUM(BlueprintType)
enum class ETrigFunction : uint8
{
	Sin							UMETA(DisplayName = "Sine"),
	Cos							UMETA(DisplayName = "Cosine"),
	Tan							UMETA(DisplayName = "Tangent"),
	ASin						UMETA(DisplayName = "ArcSine"),
	ACos						UMETA(DisplayName = "ArcCosine"),
	ATan						UMETA(DisplayName = "ArcTangent"),
};

//////////////////////////////////////////////////////////////////////////
/// Helper C++ class
//////////////////////////////////////////////////////////////////////////
class T_Maths_OneInput
{
public:
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API TiledBlobPtr			CreateTrigonometry(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input, ETrigFunction Function);
	static UE_API TiledBlobPtr			CreateToRadians(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input);
	static UE_API TiledBlobPtr			CreateToDegrees(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input);
	static UE_API TiledBlobPtr			CreateAbs(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input);
	static UE_API TiledBlobPtr			CreateSqrt(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input);
	static UE_API TiledBlobPtr			CreateSquare(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input);
	static UE_API TiledBlobPtr			CreateCbrt(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input);
	static UE_API TiledBlobPtr			CreateCube(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input);
	static UE_API TiledBlobPtr			CreateExp(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input);
	static UE_API TiledBlobPtr			CreateLog2(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input);
	static UE_API TiledBlobPtr			CreateLog10(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input);
	static UE_API TiledBlobPtr			CreateLog(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input);
	static UE_API TiledBlobPtr			CreateFloor(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input);
	static UE_API TiledBlobPtr			CreateCeil(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input);
	static UE_API TiledBlobPtr			CreateRound(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Input);
};

#undef UE_API
