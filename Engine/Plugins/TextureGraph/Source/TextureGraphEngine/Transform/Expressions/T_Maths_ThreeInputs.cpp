// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_Maths_ThreeInputs.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "Math/Vector.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"

IMPLEMENT_GLOBAL_SHADER(FSH_Mad, "/Plugin/TextureGraph/Expressions/Expression_Maths_ThreeInputs.usf", "FSH_Mad", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Lerp, "/Plugin/TextureGraph/Expressions/Expression_Maths_ThreeInputs.usf", "FSH_Lerp", SF_Pixel);

template <typename FSH_Type>
static TiledBlobPtr CreateGenericThreeInput(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId,
	TiledBlobPtr Operand1, TiledBlobPtr Operand2, TiledBlobPtr Operand3, FString TransformName)
{
	// assign default values if any of the 3 operands are not specified to obtain the same result as combining a Multiply and a Add expressions
	if (!Operand1 && !Operand2)
	{
		Operand1 = Operand2 = TextureHelper::GBlack;
	}
	else
	{
		if (!Operand1)
		{
			Operand1 = TextureHelper::GWhite;
		}
		if (!Operand2)
		{
			Operand2 = TextureHelper::GWhite;
		}
	}
	if (!Operand3)
	{
		Operand3 = TextureHelper::GBlack;
	}

	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_Type>(TransformName);

	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_BLOB(Operand1, "Operand1"))
		->AddArg(ARG_BLOB(Operand2, "Operand2"))
		->AddArg(ARG_BLOB(Operand3, "Operand3"))
		;

	const FString Name = FString(TEXT("BasicMathOp_T_")) + TransformName;

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}


TiledBlobPtr T_Maths_ThreeInputs::CreateMad(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2, TiledBlobPtr Operand3)
{
	return CreateGenericThreeInput<FSH_Mad>(Cycle, DesiredOutputDesc, TargetId, Operand1, Operand2, Operand3, "T_Mad");
}

TiledBlobPtr T_Maths_ThreeInputs::CreateLerp(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2, TiledBlobPtr Operand3)
{
	return CreateGenericThreeInput<FSH_Lerp>(Cycle, DesiredOutputDesc, TargetId, Operand1, Operand2, Operand3, "T_Lerp");
}
