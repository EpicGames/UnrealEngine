// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//#include "T_Maths_ThreeInputs.generated.h"

//////////////////////////////////////////////////////////////////////////
/// MAD
//////////////////////////////////////////////////////////////////////////
class FSH_ThreeInputs : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_ThreeInputs, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, Operand1)
		SHADER_PARAMETER_TEXTURE(Texture2D, Operand2)
		SHADER_PARAMETER_TEXTURE(Texture2D, Operand3)
	END_SHADER_PARAMETER_STRUCT()

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Mad, FSH_ThreeInputs);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Lerp, FSH_ThreeInputs);

//////////////////////////////////////////////////////////////////////////
/// Helper C++ class
//////////////////////////////////////////////////////////////////////////
class T_Maths_ThreeInputs
{
public:
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API TiledBlobPtr				CreateMad(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2, TiledBlobPtr Operand3);
	static UE_API TiledBlobPtr				CreateLerp(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2, TiledBlobPtr Operand3);
};

#undef UE_API
