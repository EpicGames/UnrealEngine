// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_Maths_TwoInputs.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "Math/Vector.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(T_Maths_TwoInputs)

IMPLEMENT_GLOBAL_SHADER(FSH_Multiply, "/Plugin/TextureGraph/Expressions/Expression_Maths_TwoInputs.usf", "FSH_Multiply", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Divide, "/Plugin/TextureGraph/Expressions/Expression_Maths_TwoInputs.usf", "FSH_Divide", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Add, "/Plugin/TextureGraph/Expressions/Expression_Maths_TwoInputs.usf", "FSH_Add", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Subtract, "/Plugin/TextureGraph/Expressions/Expression_Maths_TwoInputs.usf", "FSH_Subtract", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(FSH_Dot, "/Plugin/TextureGraph/Expressions/Expression_Maths_TwoInputs.usf", "FSH_Dot", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Cross, "/Plugin/TextureGraph/Expressions/Expression_Maths_TwoInputs.usf", "FSH_Cross", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Pow, "/Plugin/TextureGraph/Expressions/Expression_Maths_TwoInputs.usf", "FSH_Pow", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_Step, "/Plugin/TextureGraph/Expressions/Expression_Maths_TwoInputs.usf", "FSH_Step", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_GT_Component", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_GT_Component, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_GT_Component", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_GT_All, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_GT_All", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_GT_Grayscale, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_GT_Grayscale", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_GTE_Component, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_GTE_Component", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_GTE_All, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_GTE_All", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_GTE_Grayscale, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_GTE_Grayscale", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_LT_Component, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_LT_Component", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_LT_All, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_LT_All", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_LT_Grayscale, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_LT_Grayscale", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_LTE_Component, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_LTE_Component", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_LTE_All, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_LTE_All", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_LTE_Grayscale, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_LTE_Grayscale", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_EQ_Component, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_EQ_Component", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_EQ_All, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_EQ_All", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_EQ_Grayscale, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_EQ_Grayscale", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_NEQ_Component, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_NEQ_Component", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_NEQ_All, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_NEQ_All", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_IfThenElse_NEQ_Grayscale, "/Plugin/TextureGraph/Expressions/Expression_IfThenElse.usf", "FSH_IfThenElse_NEQ_Grayscale", SF_Pixel);

template <typename FSH_Type>
TiledBlobPtr CreateGenericMathOp(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2, FString TransformName)
{	
	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_Type>(TransformName);

	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_BLOB(Operand1, "Operand1"))
		->AddArg(ARG_BLOB(Operand2, "Operand2"))
		;

	const FString Name = FString::Printf(TEXT("BasicMathOp_%s"), *TransformName);

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}

TiledBlobPtr T_Maths_TwoInputs::CreateAdd(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2)
{
	return CreateGenericMathOp<FSH_Add>(Cycle, DesiredOutputDesc, TargetId, Operand1, Operand2, "T_Add");
}

TiledBlobPtr T_Maths_TwoInputs::CreateSubtract(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2)
{
	return CreateGenericMathOp<FSH_Subtract>(Cycle, DesiredOutputDesc, TargetId, Operand1, Operand2, "T_Subtract");
}

TiledBlobPtr T_Maths_TwoInputs::CreateMultiply(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2)
{
	return CreateGenericMathOp<FSH_Multiply>(Cycle, DesiredOutputDesc, TargetId, Operand1, Operand2, "T_Multiply");
}

TiledBlobPtr T_Maths_TwoInputs::CreateDivide(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2)
{
	return CreateGenericMathOp<FSH_Divide>(Cycle, DesiredOutputDesc, TargetId, Operand1, Operand2, "T_Divide");
}

TiledBlobPtr T_Maths_TwoInputs::CreateDot(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2)
{
	return CreateGenericMathOp<FSH_Dot>(Cycle, DesiredOutputDesc, TargetId, Operand1, Operand2, "T_Dot");
}

TiledBlobPtr T_Maths_TwoInputs::CreateCross(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2)
{
	return CreateGenericMathOp<FSH_Cross>(Cycle, DesiredOutputDesc, TargetId, Operand1, Operand2, "T_Cross");
}

TiledBlobPtr T_Maths_TwoInputs::CreatePow(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2)
{
	return CreateGenericMathOp<FSH_Pow>(Cycle, DesiredOutputDesc, TargetId, Operand1, Operand2, "T_Pow");
}

TiledBlobPtr T_Maths_TwoInputs::CreateStep(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2)
{
	return CreateGenericMathOp<FSH_Step>(Cycle, DesiredOutputDesc, TargetId, Operand1, Operand2, "T_Step");
}

RenderMaterial_FXPtr GetIfThenElseMaterial(EIfThenElseOperator Operator, EIfThenElseType Type)
{
	switch (Operator)
	{
	case EIfThenElseOperator::GT:
		if (Type == EIfThenElseType::IndividualComponent)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_GT_Component>("FSH_IfThenElse_GT_Component");
		else if (Type == EIfThenElseType::AllComponents)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_GT_All>("FSH_IfThenElse_GT_All");
		else if (Type == EIfThenElseType::Grayscale)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_GT_Grayscale>("FSH_IfThenElse_GT_Grayscale");
		break;

	case EIfThenElseOperator::GTE:
		if (Type == EIfThenElseType::IndividualComponent)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_GTE_Component>("FSH_IfThenElse_GTE_Component");
		else if (Type == EIfThenElseType::AllComponents)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_GTE_All>("FSH_IfThenElse_GTE_All");
		else if (Type == EIfThenElseType::Grayscale)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_GTE_Grayscale>("FSH_IfThenElse_GTE_Grayscale");
		break;

	case EIfThenElseOperator::LT:
		if (Type == EIfThenElseType::IndividualComponent)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_LT_Component>("FSH_IfThenElse_LT_Component");
		else if (Type == EIfThenElseType::AllComponents)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_LT_All>("FSH_IfThenElse_LT_All");
		else if (Type == EIfThenElseType::Grayscale)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_LT_Grayscale>("FSH_IfThenElse_LT_Grayscale");
		break;
		
	case EIfThenElseOperator::LTE:
		if (Type == EIfThenElseType::IndividualComponent)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_LTE_Component>("FSH_IfThenElse_LTE_Component");
		else if (Type == EIfThenElseType::AllComponents)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_LTE_All>("FSH_IfThenElse_LTE_All");
		else if (Type == EIfThenElseType::Grayscale)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_LTE_Grayscale>("FSH_IfThenElse_LTE_Grayscale");
		break;

	case EIfThenElseOperator::EQ:
		if (Type == EIfThenElseType::IndividualComponent)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_EQ_Component>("FSH_IfThenElse_EQ_Component");
		else if (Type == EIfThenElseType::AllComponents)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_EQ_All>("FSH_IfThenElse_EQ_All");
		else if (Type == EIfThenElseType::Grayscale)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_EQ_Grayscale>("FSH_IfThenElse_EQ_Grayscale");
		break;
		
	case EIfThenElseOperator::NEQ:
		if (Type == EIfThenElseType::IndividualComponent)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_NEQ_Component>("FSH_IfThenElse_NEQ_Component");
		else if (Type == EIfThenElseType::AllComponents)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_NEQ_All>("FSH_IfThenElse_NEQ_All");
		else if (Type == EIfThenElseType::Grayscale)
			return TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_IfThenElse_NEQ_Grayscale>("FSH_IfThenElse_NEQ_Grayscale");
		break;
	}

	return nullptr;
}

TiledBlobPtr T_Maths_TwoInputs::CreateIfThenElse(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId,
	TiledBlobPtr LHS, TiledBlobPtr RHS, TiledBlobPtr Then, TiledBlobPtr Else, EIfThenElseOperator Operator, EIfThenElseType Type)
{
	const RenderMaterial_FXPtr RenderMaterial = GetIfThenElseMaterial(Operator, Type);
	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_BLOB(LHS, "LHS"))
		->AddArg(ARG_BLOB(RHS, "RHS"))
		->AddArg(ARG_BLOB(Then, "Then"))
		->AddArg(ARG_BLOB(Else, "Else"))
		;

	const FString Name(TEXT("IfThenElse"));

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}

