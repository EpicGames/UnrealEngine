// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#include "T_Maths_TwoInputs.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Basic math op
//////////////////////////////////////////////////////////////////////////
class FSH_BasicMathOp : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_BasicMathOp, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, Operand1)
		SHADER_PARAMETER_TEXTURE(Texture2D, Operand2)
	END_SHADER_PARAMETER_STRUCT()

	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Multiply, FSH_BasicMathOp);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Divide, FSH_BasicMathOp);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Add, FSH_BasicMathOp);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Subtract, FSH_BasicMathOp);

DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Dot, FSH_BasicMathOp);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Cross, FSH_BasicMathOp);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Pow, FSH_BasicMathOp);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_Step, FSH_BasicMathOp);

//////////////////////////////////////////////////////////////////////////
/// Comparison ops
//////////////////////////////////////////////////////////////////////////
UENUM(BlueprintType)
enum EIfThenElseOperator : int 
{
	GT		UMETA(DisplayName = "Greater Than"),
	GTE		UMETA(DisplayName = "Greater Than Or Equal To"),
	LT		UMETA(DisplayName = "Less Than"),
	LTE		UMETA(DisplayName = "Less Than Or Equal To"),
	EQ		UMETA(DisplayName = "Equal To (Tolerance = 0.00001)"),
	NEQ		UMETA(DisplayName = "Not Equal To (Tolerance = 0.00001)"),
};

UENUM(BlueprintType)
enum EIfThenElseType : int 
{
	IndividualComponent	UMETA(DisplayName = "Individual Components"),
	AllComponents		UMETA(DisplayName = "All Components"),
	Grayscale			UMETA(DisplayName = "Grayscale"),
};

class FSH_IfThenElse : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_IfThenElse, FSH_Base);
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_IfThenElse, UE_API);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, LHS)
		SHADER_PARAMETER_TEXTURE(Texture2D, RHS)
		SHADER_PARAMETER_TEXTURE(Texture2D, Then)
		SHADER_PARAMETER_TEXTURE(Texture2D, Else)
	END_SHADER_PARAMETER_STRUCT()

	static bool						ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
	static void						ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env) {}
};

DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_GT_Component, FSH_IfThenElse);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_GT_All, FSH_IfThenElse);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_GT_Grayscale, FSH_IfThenElse);

DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_GTE_Component, FSH_IfThenElse);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_GTE_All, FSH_IfThenElse);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_GTE_Grayscale, FSH_IfThenElse);

DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_LT_Component, FSH_IfThenElse);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_LT_All, FSH_IfThenElse);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_LT_Grayscale, FSH_IfThenElse);

DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_LTE_Component, FSH_IfThenElse);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_LTE_All, FSH_IfThenElse);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_LTE_Grayscale, FSH_IfThenElse);

DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_EQ_Component, FSH_IfThenElse);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_EQ_All, FSH_IfThenElse);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_EQ_Grayscale, FSH_IfThenElse);

DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_NEQ_Component, FSH_IfThenElse);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_NEQ_All, FSH_IfThenElse);
DECLARE_EMPTY_GLOBAL_SHADER_DERIVED_FROM(FSH_IfThenElse_NEQ_Grayscale, FSH_IfThenElse);

//////////////////////////////////////////////////////////////////////////
/// Helper C++ class
//////////////////////////////////////////////////////////////////////////
class T_Maths_TwoInputs
{
public:
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API TiledBlobPtr				CreateMultiply(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2);
	static UE_API TiledBlobPtr				CreateDivide(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2);
	static UE_API TiledBlobPtr				CreateAdd(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2);
	static UE_API TiledBlobPtr				CreateSubtract(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2);
	static UE_API TiledBlobPtr				CreateDot(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2);
	static UE_API TiledBlobPtr				CreateCross(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2);
	static UE_API TiledBlobPtr				CreatePow(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2);
	static UE_API TiledBlobPtr				CreateStep(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, TiledBlobPtr Operand1, TiledBlobPtr Operand2);
	
	static UE_API TiledBlobPtr				CreateIfThenElse(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId,
		TiledBlobPtr LHS, TiledBlobPtr RHS, TiledBlobPtr Then, TiledBlobPtr Else, EIfThenElseOperator Operator, EIfThenElseType Type);
};

#undef UE_API
