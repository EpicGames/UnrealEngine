// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_Transform.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "Math/Vector.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"
#include "Transform/Utility/T_CombineTiledBlob.h"


IMPLEMENT_GLOBAL_SHADER(FSH_TransformBlit, "/Plugin/TextureGraph/Expressions/Expression_Transform.usf", "FSH_TransformBlit", SF_Pixel);


TiledBlobPtr T_Transform::Create(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Source,
	const TransformParameter& TransformParam, const CellParameter& CellParam, const ColorParameter& ColorParam, int32 TargetId)
{
	RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_TransformBlit>(TEXT("T_Transform"));

	check(RenderMaterial);


	TiledBlobPtr CombinedBlob;
	if (Source)
	{
		CombinedBlob = T_CombineTiledBlob::Create(Cycle, Source->GetDescriptor(), TargetId, Source);
	}
	else
	{
		CombinedBlob = TextureHelper::GBlack;
	}

	//FVector2f Source = { CombinedBlob->GetDescriptor().Width, CombinedBlob->GetDescriptor().Height };



	FTileInfo TileInfo;

	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	RenderJob
		->AddArg(ARG_TILEINFO(TileInfo, "TileInfo"))
		->AddArg(ARG_BLOB(CombinedBlob, "SourceTexture"))
		->AddArg(ARG_LINEAR_COLOR(ColorParam.FillColor, "FillColor"))
		->AddArg(ARG_FLOAT((TransformParam.Coverage.X < 0.005 ? 0.005 : TransformParam.Coverage.X), "CoverageX"))
		->AddArg(ARG_FLOAT((TransformParam.Coverage.Y < 0.005 ? 0.005 : TransformParam.Coverage.Y), "CoverageY"))
		->AddArg(ARG_FLOAT(TransformParam.Translation.X, "TranslationX"))
		->AddArg(ARG_FLOAT(TransformParam.Translation.Y, "TranslationY"))
		->AddArg(ARG_FLOAT(TransformParam.Pivot.X, "PivotX"))
		->AddArg(ARG_FLOAT(TransformParam.Pivot.Y, "PivotY"))
		->AddArg(ARG_FLOAT(cos(TransformParam.RotationXY), "RotationX"))
		->AddArg(ARG_FLOAT(sin(TransformParam.RotationXY), "RotationY"))
		->AddArg(ARG_FLOAT(TransformParam.Scale.X, "ScaleX"))
		->AddArg(ARG_FLOAT(TransformParam.Scale.Y, "ScaleY"))
		->AddArg(ARG_FLOAT(CellParam.Stagger.X, "StaggerX"))
		->AddArg(ARG_FLOAT(CellParam.Stagger.Y, "StaggerY"))
		->AddArg(ARG_FLOAT(CellParam.Stride.X, "StrideX"))
		->AddArg(ARG_FLOAT(CellParam.Stride.Y, "StrideY"))
		->AddArg(ARG_FLOAT(CellParam.Zoom, "Zoom"))
		->AddArg(ARG_FLOAT(CellParam.StretchToFit, "StretchToFit"))
		->AddArg(ARG_FLOAT(CellParam.Spacing.X, "SpacingX"))
		->AddArg(ARG_FLOAT(CellParam.Spacing.Y, "SpacingY"))
		->AddArg(ARG_FLOAT(float(ColorParam.WrapFilterMode), "FilterMode"))
		->AddArg(ARG_FLOAT(float(ColorParam.MirrorX), "MirrorX"))
		->AddArg(ARG_FLOAT(float(ColorParam.MirrorY), "MirrorY"))
		->AddArg(ARG_FLOAT(ColorParam.ShowDebugGrid, "BlendDebugGrid"))
		;

	FString Name = TEXT("T_Transform");

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}
