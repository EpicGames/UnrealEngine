// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Math/Vector2D.h"

#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Transform blit shader
//////////////////////////////////////////////////////////////////////////
class FSH_TransformBlit : public FSH_Base
{
public:
	SHADER_USE_PARAMETER_STRUCT(FSH_TransformBlit, FSH_Base);
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_TransformBlit, UE_API);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER(FLinearColor, FillColor)		
		SHADER_PARAMETER(float, CoverageX)
		SHADER_PARAMETER(float, CoverageY)
		SHADER_PARAMETER(float, TranslationX)
		SHADER_PARAMETER(float, TranslationY)
		SHADER_PARAMETER(float, PivotX)
		SHADER_PARAMETER(float, PivotY)
		SHADER_PARAMETER(float, RotationX)
		SHADER_PARAMETER(float, RotationY)
		SHADER_PARAMETER(float, ScaleX)
		SHADER_PARAMETER(float, ScaleY)
		SHADER_PARAMETER(float, StaggerX)
		SHADER_PARAMETER(float, StaggerY)
		SHADER_PARAMETER(float, StrideX)
		SHADER_PARAMETER(float, StrideY)
		SHADER_PARAMETER(float, Zoom)
		SHADER_PARAMETER(float, StretchToFit)
		SHADER_PARAMETER(float, SpacingX)
		SHADER_PARAMETER(float, SpacingY)
		SHADER_PARAMETER(float, FilterMode)
		SHADER_PARAMETER(float, MirrorX)
		SHADER_PARAMETER(float, MirrorY)
		SHADER_PARAMETER(float, BlendDebugGrid)
	END_SHADER_PARAMETER_STRUCT()

	static bool						ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
	static void						ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env) {}
};

//////////////////////////////////////////////////////////////////////////
class T_Transform
{
public:
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////

	struct TransformParameter
	{
		FVector2f Coverage = { 1.f, 1.f };	// Coverage of the whole domain
		FVector2f Translation = { 0.f, 0.f };	// Translation of the whole domain
		FVector2f Pivot = { 0.5f, 0.5f };		// Pivot coord of the rotation and scaling
		float RotationXY = 0;					// Rotation angle in Radians centered on the pivot coord
		FVector2f Scale = { 1.f, 1.f };			// Scaling aka the Repeatition of the motif
	};

	struct CellParameter
	{
		float	  Zoom = 1.0f;
		float	  StretchToFit = 0.f;
		FVector2f Spacing = { 0.f, 0.f };
		FVector2f Stagger = { 0.f, 0.f };
		FVector2f Stride = { 0.f, 0.f };
	};

	struct ColorParameter
	{
		FLinearColor FillColor = { 0,0,0, 1 };
		bool WrapFilterMode = true;
		bool MirrorX = false;
		bool MirrorY = false;
		float ShowDebugGrid = 0.0f;
	};

	static UE_API TiledBlobPtr		Create(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, TiledBlobPtr Source,
								const TransformParameter& TransformParam, 
								const CellParameter& CellParam,
								const ColorParameter& ColorParam,
								int32 TargetId = 0);

};

#undef UE_API
