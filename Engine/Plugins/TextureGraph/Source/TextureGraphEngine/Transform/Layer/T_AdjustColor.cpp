// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_AdjustColor.h"
#include "FxMat/RenderMaterial_FX.h"
#include "Job/JobArgs.h"
#include "Job/JobBatch.h"
#include "TextureGraphEngine.h"
#include "FxMat/MaterialManager.h"
#include "Model/Mix/MixUpdateCycle.h"

IMPLEMENT_GLOBAL_SHADER(FSH_AdjustColor, "/Plugin/TextureGraph/Layer/AdjustColor.usf", "FSH_AdjustColor", SF_Pixel);
template <>
void SetupDefaultParameters(FSH_AdjustColor::FParameters& params) {
	FStandardSamplerStates_Setup(params.SamplerStates);
}

T_AdjustColor::T_AdjustColor()
{
}

T_AdjustColor::~T_AdjustColor()
{
}

