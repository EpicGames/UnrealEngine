// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "FxMat/FxMaterial.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

class ULayer_Textured;
class MixUpdateCycle;
typedef std::shared_ptr<MixUpdateCycle> MixUpdateCyclePtr;
namespace
{
	// Declare one Shader Permutation Var class per parameters
	class FVar_ClearColor: SHADER_PERMUTATION_BOOL("CLEAR_COLOR");
	class FVar_ShiftAlpha: SHADER_PERMUTATION_BOOL("SHIFT_ALPHA");
	class FVar_Invert: SHADER_PERMUTATION_BOOL("INVERT");
}

class FSH_AdjustColor : public FSH_Base
{
public:

	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_AdjustColor, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_AdjustColor, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTex)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER(float, Contrast)
		SHADER_PARAMETER(FLinearColor, ColorOffset)
		SHADER_PARAMETER(FLinearColor, AverageColor)
		
	END_SHADER_PARAMETER_STRUCT()

		// Permutation domain local 
	using FPermutationDomain = TShaderPermutationDomain<FVar_ClearColor, FVar_ShiftAlpha, FVar_Invert>;

public:

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
};

template <> void SetupDefaultParameters(FSH_AdjustColor::FParameters& params);
//////////////////////////////////////////////////////////////////////////
class T_AdjustColor 
{
public:
									UE_API T_AdjustColor();
	UE_API virtual							~T_AdjustColor();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
