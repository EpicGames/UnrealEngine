// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_AdjustFrequency.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h" 
#include "FxMat/MaterialManager.h"
#include "Job/Scheduler.h"
#include "Profiling/StatGroup.h"
#include "Transform/Utility/T_MipMap.h"

// Concrete implementation of the Shader
IMPLEMENT_GLOBAL_SHADER(FSH_AdjustFrequency, "/Plugin/TextureGraph/Layer/AdjustFrequency.usf", "FSH_AdjustFrequency", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_AdjustFrequencyNormals, "/Plugin/TextureGraph/Layer/AdjustFrequencyNormals.usf", "FSH_AdjustFrequencyNormals", SF_Pixel);

DECLARE_CYCLE_STAT(TEXT("T_AdjustFrequency_CreateTextured"), STAT_T_AdjustFrequency_CreateTextured, STATGROUP_TextureGraphEngine);

template <>
void SetupDefaultParameters(FSH_AdjustFrequency::FParameters& params) {
	FStandardSamplerStates_Setup(params.SamplerStates);
}
template <>
void SetupDefaultParameters(FSH_AdjustFrequencyNormals::FParameters& params) {
	FStandardSamplerStates_Setup(params.SamplerStates);
}

//////////////////////////////////////////////////////////////////////////
T_AdjustFrequency::T_AdjustFrequency()
{
}

T_AdjustFrequency::~T_AdjustFrequency()
{
}

