// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API


class FSH_AdjustFrequency : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_AdjustFrequency, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_AdjustFrequency, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, MainTex)
		SHADER_PARAMETER_TEXTURE_ARRAY(Texture2D, BlurredTex, [9])
		SHADER_PARAMETER(FLinearColor, solidColor)
		SHADER_PARAMETER(float, FreqLow)
		SHADER_PARAMETER(float, Threshold)
		SHADER_PARAMETER(float, FreqHigh)
		END_SHADER_PARAMETER_STRUCT()

public:

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector) {
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment) {
	}
};
template <> void SetupDefaultParameters(FSH_AdjustFrequency::FParameters& params);


namespace {
	class FVar_Renormalize : SHADER_PERMUTATION_BOOL("RENORMALIZE");
}

class FSH_AdjustFrequencyNormals : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_AdjustFrequencyNormals, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_AdjustFrequencyNormals, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, MainTex)
		SHADER_PARAMETER_TEXTURE_ARRAY(Texture2D, BlurredTex, [9])
		SHADER_PARAMETER(float, FreqLow)
		SHADER_PARAMETER(float, Threshold)
		SHADER_PARAMETER(float, FreqHigh)
		END_SHADER_PARAMETER_STRUCT()

		// Permutation domain local 
		using FPermutationDomain = TShaderPermutationDomain<FVar_Renormalize>;

public:

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector) {
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment) {
	}
};
template <> void SetupDefaultParameters(FSH_AdjustFrequencyNormals::FParameters& params);


class ULayer_Textured;

//////////////////////////////////////////////////////////////////////////
/// Height Mask Transformation Function
//////////////////////////////////////////////////////////////////////////
class T_AdjustFrequency
{
public:
	UE_API T_AdjustFrequency();
	UE_API virtual							~T_AdjustFrequency();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
