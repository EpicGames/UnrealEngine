// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_BlendWithHeightMask.h"
#include "Job/JobArgs.h"
#include "2D/TargetTextureSet.h"
#include "FxMat/MaterialManager.h"
#include "TextureGraphEngine.h"
#include "Job/JobBatch.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"

IMPLEMENT_GLOBAL_SHADER(FSH_NormalBlendWithHeightMask, "/Plugin/TextureGraph/Mask/Blend/NormalBlendWithHeightMask.usf", "FSH_NormalBlend", SF_Pixel);


//////////////////////////////////////////////////////////////////////////
template <>
void SetupDefaultParameters(FSH_NormalBlendWithHeightMask::FParameters& params) {
	FStandardSamplerStates_Setup(params.SamplerStates);
}

TiledBlobPtr MaxSizeBlob(const TiledBlobPtr lhs, const TiledBlobPtr rhs)
{
	return	(lhs->GetDescriptor().Size() > rhs->GetDescriptor().Size()) ? lhs : rhs;
}

