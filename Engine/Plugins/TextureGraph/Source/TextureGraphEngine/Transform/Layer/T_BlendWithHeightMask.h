// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
class ULayer_Textured;
namespace
{
}
class FSH_NormalBlendWithHeightMask : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_NormalBlendWithHeightMask, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_NormalBlendWithHeightMask, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, DestinationTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, Mask)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER(float, Opacity)
		SHADER_PARAMETER(float, LerpRGB)
		SHADER_PARAMETER(float, LerpAlpha)
	END_SHADER_PARAMETER_STRUCT()


public:	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
	}
};
template <> void SetupDefaultParameters(FSH_NormalBlendWithHeightMask::FParameters& params);

typedef FxMaterial_Normal<VSH_Simple, FSH_NormalBlendWithHeightMask>	Fx_NormalBlendWithHeightMask;

//////////////////////////////////////////////////////////////////////////
class T_BlendWithHeightMask 
{
public:
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
