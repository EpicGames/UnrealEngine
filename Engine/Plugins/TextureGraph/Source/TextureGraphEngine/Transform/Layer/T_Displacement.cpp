// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_Displacement.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"

IMPLEMENT_GLOBAL_SHADER(FSH_LayerDisplacement, "/Plugin/TextureGraph/Layer/Layer_Displacement.usf", "FSH_Displacement", SF_Pixel);

T_Displacement::T_Displacement()
{
}

T_Displacement::~T_Displacement()
{
}
template <>
void SetupDefaultParameters(FSH_LayerDisplacement::FParameters& params) {
	FStandardSamplerStates_Setup(params.SamplerStates);
}

