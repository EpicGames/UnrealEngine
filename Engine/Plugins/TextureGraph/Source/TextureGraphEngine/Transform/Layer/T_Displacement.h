// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

class ULayer_Textured;

class FSH_LayerDisplacement : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_LayerDisplacement, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_LayerDisplacement, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, DestinationTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, Mask)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, BlurredBase)

		SHADER_PARAMETER(float, BorderThreshold)
		SHADER_PARAMETER(float, ReplaceHeight)
		SHADER_PARAMETER(float, ReplaceHeightMip)
		SHADER_PARAMETER(float, Opacity)
		SHADER_PARAMETER(float, ChannelOpacity)
		SHADER_PARAMETER(float, BorderFade)
		SHADER_PARAMETER(float, FromAbove)
		
	END_SHADER_PARAMETER_STRUCT()

public:	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
		
	}
};
template <> void SetupDefaultParameters(FSH_LayerDisplacement::FParameters& params);

typedef FxMaterial_Normal<VSH_Simple, FSH_LayerDisplacement> Fx_LayerDisplacement;

//////////////////////////////////////////////////////////////////////////
class T_Displacement 
{
public:
									UE_API T_Displacement();
	UE_API virtual							~T_Displacement();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
