// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Model/Mix/MixUpdateCycle.h"
#include "2D/TextureType.h"

class ULayer_Textured;
class UStackComponent;
//////////////////////////////////////////////////////////////////////////
class T_Dynamic 
{
public:

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
};
