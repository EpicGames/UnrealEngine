// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_LayerMask.h"
#include "Job/JobArgs.h"
#include "Model/Mix/MixInterface.h"
#include "TextureGraphEngine.h" 
#include "FxMat/MaterialManager.h"
#include "Job/Scheduler.h"
#include "Profiling/StatGroup.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"

// Concrete implementation of the Shader
IMPLEMENT_GLOBAL_SHADER(FSH_LayerMask, "/Plugin/TextureGraph/Mask/LayerMask.usf", "FSH_LayerMask", SF_Pixel);

template <>
void SetupDefaultParameters(FSH_LayerMask::FParameters& params) {
	FStandardSamplerStates_Setup(params.SamplerStates);
}

DECLARE_CYCLE_STAT(TEXT("T_LayerMask_CreateTextured"), STAT_T_LayerMask_CreateTextured, STATGROUP_TextureGraphEngine);
//////////////////////////////////////////////////////////////////////////
T_LayerMask::T_LayerMask()
{ 
}

T_LayerMask::~T_LayerMask()
{
}

