// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include "ShaderPermutation.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////

namespace
{
	// Declare one Shader Permutation Var class per parameters
	class FVar_HeightMask : SHADER_PERMUTATION_BOOL("HEIGHTMASK");
	class FVar_FromAbove : SHADER_PERMUTATION_BOOL("FROMABOVE");
	class FVar_Opacity : SHADER_PERMUTATION_BOOL("OPACITY");
	class FVar_Details : SHADER_PERMUTATION_BOOL("DETAILS");
	class FVar_MaterialId : SHADER_PERMUTATION_BOOL("MATERIALID");
	class FVar_PaintMask : SHADER_PERMUTATION_BOOL("PAINTMASK");
	class FVar_UVMask : SHADER_PERMUTATION_BOOL("UVMASK");
	class FVar_MaskStack : SHADER_PERMUTATION_BOOL("MASKSTACK");
	
}

// Then define the shader class
class FSH_LayerMask : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_LayerMask, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_LayerMask, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)

		SHADER_PARAMETER_TEXTURE(Texture2D, Displacement)
		SHADER_PARAMETER_TEXTURE(Texture2D, OldDisplacement)

		SHADER_PARAMETER_TEXTURE(Texture2D, Occlusion)
		SHADER_PARAMETER_TEXTURE(Texture2D, OldOcclusion)

		SHADER_PARAMETER_TEXTURE(Texture2D, OpacityChannel)

		SHADER_PARAMETER_TEXTURE(Texture2D, MaskStack)

		SHADER_PARAMETER_TEXTURE(Texture2D, UVMask)
		SHADER_PARAMETER_TEXTURE(Texture2D, PaintMask)

		SHADER_PARAMETER_TEXTURE(Texture2D, MaterialID)

		SHADER_PARAMETER(float, PreserveDetails)

		SHADER_PARAMETER(float, BorderFade)
		SHADER_PARAMETER(float, BorderThreshold)
		SHADER_PARAMETER(float, Opacity)

	END_SHADER_PARAMETER_STRUCT()

	// Permutation domain local 
	using FPermutationDomain = TShaderPermutationDomain<FVar_HeightMask, FVar_FromAbove, FVar_Details, FVar_MaterialId, FVar_PaintMask, FVar_UVMask, FVar_MaskStack, FVar_Opacity>;

public:

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};
template <> void SetupDefaultParameters(FSH_LayerMask::FParameters& params);

class ULayer_Textured;

//////////////////////////////////////////////////////////////////////////
/// Height Mask Transformation Function
//////////////////////////////////////////////////////////////////////////
class T_LayerMask
{
public:
									UE_API T_LayerMask();
	UE_API virtual							~T_LayerMask();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
