// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_Normal.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "../Utility/T_MipMap.h"

IMPLEMENT_GLOBAL_SHADER(FSH_LayerNormal, "/Plugin/TextureGraph/Layer/Layer_Normal.usf", "FSH_Normal", SF_Pixel);
template <>
void SetupDefaultParameters(FSH_LayerNormal::FParameters& params) {
	FStandardSamplerStates_Setup(params.SamplerStates);
}

T_Normal::T_Normal()
{
}

T_Normal::~T_Normal()
{
}

