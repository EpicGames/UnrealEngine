// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

class ULayer_Textured;

class FSH_LayerNormal : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_LayerNormal, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_LayerNormal, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, Destination)
		SHADER_PARAMETER_TEXTURE(Texture2D, Mask)
		SHADER_PARAMETER_TEXTURE(Texture2D, BlurredBase)
		SHADER_PARAMETER_TEXTURE(Texture2D, NormalDiff)

		SHADER_PARAMETER(float, Opacity)
		SHADER_PARAMETER(float, InvertX)
		SHADER_PARAMETER(float, InvertY)
		SHADER_PARAMETER(float, Strength)
		SHADER_PARAMETER(float, ReplaceHeightMip)
		SHADER_PARAMETER(float, ReplaceHeight)
		
	END_SHADER_PARAMETER_STRUCT()

public:	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
		
	}
};
template <> void SetupDefaultParameters(FSH_LayerNormal::FParameters& params);

typedef FxMaterial_Normal<VSH_Simple, FSH_LayerNormal> Fx_LayerNormal;

//////////////////////////////////////////////////////////////////////////
class T_Normal 
{
public:
									UE_API T_Normal();
	UE_API virtual							~T_Normal();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
