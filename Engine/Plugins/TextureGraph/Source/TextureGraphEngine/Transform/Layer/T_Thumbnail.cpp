// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_Thumbnail.h"
#include "Job/JobArgs.h"
#include "Model/Mix/MixInterface.h"
#include "TextureGraphEngine.h"
#include "Device/FX/Device_FX.h"
#include "FxMat/MaterialManager.h"
#include "Job/ThumbnailsService.h"
#include "Job/Scheduler.h"
#include <TextureResource.h>
#include <Engine/Texture2D.h>

////////////////////////////////////////////////////////////////////////// Transform //////////////////////////////////////////////////////////////////////////
JobBatchPtr AddThumbJobToCycle(TiledBlobPtr InBlobToBind, JobUPtr JobPtr, UObject* Model, int32 InTargetId)
{
	ThumbnailsServicePtr SvcThumbnail = TextureGraphEngine::GetScheduler()->GetThumbnailsService().lock();
	if (!SvcThumbnail)
		return nullptr;
	
	UMixInterface* Mix = JobPtr->GetMix();
	check(Mix);
	SvcThumbnail->AddUniqueJobToCycle(Model, Mix, std::move(JobPtr), InTargetId);
	return SvcThumbnail->GetCurrentBatch();
}

TiledBlobRef T_Thumbnail::Bind(UMixInterface* Mix, UObject* Model, TiledBlobPtr InBlobToBind, int32 InTargetId)
{
	if (!TextureGraphEngine::GetBlobber()->IsCacheEnabled())
		return TextureHelper::GetBlack();

	if (InBlobToBind.get() == nullptr)
		return TextureHelper::GetBlack();

	if (InBlobToBind->IsTransient())
		return TextureHelper::GetBlack();

	check(InBlobToBind);

	UE_LOGF(LogJob, VeryVerbose, "T_Thumbnail::Bind [%ls]", *InBlobToBind->Name());
	//RenderMaterial_ThumbPtr MatThumb = TextureGraphEngine::GetMaterialManager()->CreateMaterial_Thumbnail(TEXT("T_Thumbnail"), TEXT("Util/CopyUnlit"));
	FString ThmJobName = FString::Printf(TEXT("Thm [%s]"), *InBlobToBind->GetDescriptor().Name);
	RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterialOfType_FX<Fx_FullScreenCopy>(ThmJobName);

	JobUPtr JobPtr = std::make_unique<Job>(Mix, InTargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	JobPtr
		->AddArg(ARG_BLOB(InBlobToBind, "SourceTexture"))
		->SetTiled(false)
		;

	BufferDescriptor Desc = InBlobToBind->GetDescriptor();

	// Normalize dimensions and remap to expected thumbnail size
	const float MaxDimension = FMath::Max(Desc.Width, Desc.Height);
	if (MaxDimension > 0)
	{
		Desc.Width = RenderMaterial_Thumbnail::GThumbWidth * Desc.Width / MaxDimension;
		Desc.Height = RenderMaterial_Thumbnail::GThumbHeight * Desc.Height / MaxDimension;
	}
	
	FString Name = FString::Printf(TEXT("T_Thumbnail [%s]"), *InBlobToBind->GetDescriptor().Name);
	
	TiledBlobPtr Result = JobPtr->InitResult(Name, &Desc);
	Result->MakeSingleBlob();

	AddThumbJobToCycle(InBlobToBind, std::move(JobPtr), Model, InTargetId);

	return Result;
}
