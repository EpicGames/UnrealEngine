// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include "Device/Mem/Device_Mem.h"
#include "FxMat/MaterialManager.h"
#include "TextureGraphEngine.h"

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
class T_Thumbnail
{
public:
	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API TiledBlobRef				Bind(UMixInterface* Mix, UObject* Model, TiledBlobPtr InBlobToBind, int32 TargetId);
};

#undef UE_API
