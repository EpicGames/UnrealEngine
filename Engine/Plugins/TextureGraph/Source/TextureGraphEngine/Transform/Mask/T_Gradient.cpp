// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_Gradient.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "2D/TargetTextureSet.h"
#include "3D/RenderMesh.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"
#include "T_Gradient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(T_Gradient)

IMPLEMENT_GLOBAL_SHADER(FSH_GradientLinear_1, "/Plugin/TextureGraph/Expressions/Expression_Gradient.usf", "FSH_GradientLinear_1", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_GradientLinear_2, "/Plugin/TextureGraph/Expressions/Expression_Gradient.usf", "FSH_GradientLinear_2", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_GradientRadial, "/Plugin/TextureGraph/Expressions/Expression_Gradient.usf", "FSH_GradientRadial", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_GradientAxial1, "/Plugin/TextureGraph/Expressions/Expression_Gradient.usf", "FSH_GradientAxial1", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_GradientAxial2, "/Plugin/TextureGraph/Expressions/Expression_Gradient.usf", "FSH_GradientAxial2", SF_Pixel);

TiledBlobPtr Create_Linear_1(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, const T_Gradient::Params& InParams, int32 TargetId)
{
	RenderMaterial_FXPtr RenderMaterial;

	FSH_GradientLinear_1::FPermutationDomain PermutationVector;
	PermutationVector.Set<FVar_GradientInterpolation>((int32)InParams.Interpolation);
	PermutationVector.Set<FVar_GradientRotation>(InParams.Rotation);
	RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_GradientLinear_1>(TEXT("T_Gradient_Linear_1"), PermutationVector);

	FString Name = FString::Printf(TEXT("%s-%d.[%llu]"), *RenderMaterial->GetName(), (int32)InParams.Type, Cycle->GetBatch()->GetBatchId());

	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	FTileInfo TileInfo;
	RenderJob
		->AddArg(ARG_TILEINFO(TileInfo, "TileInfo"))
		->AddArg(WithUnbounded(ARG_INT((int32)InParams.Interpolation, "Interpolation")))
		->AddArg(WithUnbounded(ARG_INT((int32)InParams.Rotation, "Rotation")))
		;

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}

TiledBlobPtr Create_Linear_2(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, const T_Gradient::Params& InParams, int32 TargetId)
{
	RenderMaterial_FXPtr RenderMaterial;

	FSH_GradientLinear_2::FPermutationDomain PermutationVector;
	PermutationVector.Set<FVar_GradientInterpolation>((int32)InParams.Interpolation);
	PermutationVector.Set<FVar_GradientRotation>(InParams.Rotation);
	RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_GradientLinear_2>(TEXT("T_Gradient_Linear_2"), PermutationVector);

	FString Name = FString::Printf(TEXT("%s-%d.[%llu]"), *RenderMaterial->GetName(), (int32)InParams.Type, Cycle->GetBatch()->GetBatchId());

	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	FTileInfo TileInfo;
	RenderJob
		->AddArg(ARG_TILEINFO(TileInfo, "TileInfo"))
		->AddArg(WithUnbounded(ARG_INT((int32)InParams.Interpolation, "Interpolation")))
		->AddArg(WithUnbounded(ARG_INT((int32)InParams.Rotation, "Rotation")))
		;

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}

TiledBlobPtr Create_Radial(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, const T_Gradient::Params& InParams, int32 TargetId)
{
	RenderMaterial_FXPtr RenderMaterial;

	FSH_GradientRadial::FPermutationDomain PermutationVector;
	PermutationVector.Set<FVar_GradientInterpolation>((int32)InParams.Interpolation);
	RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_GradientRadial>(TEXT("T_Gradient_Radial"), PermutationVector);

	FString Name = FString::Printf(TEXT("%s-%d.[%llu]"), *RenderMaterial->GetName(), (int32)InParams.Type, Cycle->GetBatch()->GetBatchId());

	check(RenderMaterial);

	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	FTileInfo TileInfo;
	RenderJob
		->AddArg(ARG_TILEINFO(TileInfo, "TileInfo"))
		->AddArg(ARG_VECTOR(FLinearColor(InParams.Center.X, InParams.Center.Y, InParams.Radius, 0), "Center"))
		;

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}

TiledBlobPtr Create_Axial(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, const T_Gradient::Params& InParams, int32 TargetId)
{
	RenderMaterial_FXPtr RenderMaterial;

	FSH_GradientRadial::FPermutationDomain PermutationVector;
	PermutationVector.Set<FVar_GradientInterpolation>((int32)InParams.Interpolation);

	if (InParams.Type == EGradientType::GT_Axial_1)
		RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_GradientAxial1>(TEXT("T_Gradient_Axial1"), PermutationVector);
	else
		RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_GradientAxial2>(TEXT("T_Gradient_Axial2"), PermutationVector);

	FString Name = FString::Printf(TEXT("%s-%d.[%llu]"), *RenderMaterial->GetName(), (int32)InParams.Type, Cycle->GetBatch()->GetBatchId());

	check(RenderMaterial);

	FVector2f Line = InParams.Point2 - InParams.Point1;
	float LineLen = Line.Length();
	FVector2f LineDir = Line;
	LineDir.Normalize();

	/// 0 length vector
	if (abs(LineLen) < 0.0001f)
		return TextureHelper::GetBlack();

	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	FTileInfo TileInfo;
	RenderJob
		->AddArg(ARG_TILEINFO(TileInfo, "TileInfo"))
		->AddArg(ARG_VECTOR(FLinearColor(InParams.Point1.X, InParams.Point1.Y, Line.X, Line.Y), "Line"))
		->AddArg(ARG_VECTOR(FLinearColor(LineDir.X, LineDir.Y, Line.SquaredLength(), Line.Length()), "LineDir"))
		;

	TiledBlobPtr Result = RenderJob->InitResult(Name, &DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}

BufferDescriptor T_Gradient::InitOutputDesc(BufferDescriptor DesiredOutputDesc)
{
	int32 MaxSize = std::max(DesiredOutputDesc.Width, DesiredOutputDesc.Height);
	if (DesiredOutputDesc.Width <= 0 || DesiredOutputDesc.Height <= 0)
	{
		MaxSize = (MaxSize > 0 ? MaxSize : DefaultSize);
		DesiredOutputDesc.Width = DesiredOutputDesc.Height = MaxSize;
	}

	if (DesiredOutputDesc.Format == BufferFormat::Auto)
		DesiredOutputDesc.Format = BufferFormat::Byte;

	if (DesiredOutputDesc.ItemsPerPoint == 0)
		DesiredOutputDesc.ItemsPerPoint = 4;

	DesiredOutputDesc.bIsSRGB = true;
	DesiredOutputDesc.DefaultValue = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);

	return DesiredOutputDesc;
}

TiledBlobPtr T_Gradient::Create(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, const Params& InParams, int32 TargetId)
{
	RenderMaterial_FXPtr RenderMaterial;

	DesiredOutputDesc = InitOutputDesc(DesiredOutputDesc);

	switch (InParams.Type) 
	{
	case EGradientType::GT_Linear_1:
		return Create_Linear_1(Cycle, DesiredOutputDesc, InParams, TargetId);
	case EGradientType::GT_Linear_2:
		return Create_Linear_2(Cycle, DesiredOutputDesc, InParams, TargetId);
	case EGradientType::GT_Radial:
		return Create_Radial(Cycle, DesiredOutputDesc, InParams, TargetId);
	case EGradientType::GT_Axial_1:
	case EGradientType::GT_Axial_2:
		return Create_Axial(Cycle, DesiredOutputDesc, InParams, TargetId);
	}

	return nullptr;
}
