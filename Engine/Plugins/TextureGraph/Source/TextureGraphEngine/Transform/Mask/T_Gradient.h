// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "FxMat/FxMaterial.h"
#include "Job/Job.h"
#include "Model/Mix/MixUpdateCycle.h"
#include "UObject/NoExportTypes.h"
#include <DataDrivenShaderPlatformInfo.h>
#include "T_Gradient.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

UENUM(BlueprintType)
enum class EGradientType : uint8
{
	GT_Linear_1 = 0				UMETA(DisplayName = "Simple Linear"),
	GT_Linear_2 = 1				UMETA(DisplayName = "Linear Centered"),
	GT_Radial = 2				UMETA(DisplayName = "Radial"),
	GT_Axial_1 = 3				UMETA(DisplayName = "Axial Linear"),
	GT_Axial_2 = 4				UMETA(DisplayName = "Axial Centered"),
};

UENUM(BlueprintType)
enum class EGradientInterpolation : uint8
{
	GTI_Linear = 0				UMETA(DisplayName = "Linear"),
	GTI_Exp = 1					UMETA(DisplayName = "Exponential"),
	//GTI_Log = 2					UMETA(DisplayName = "Logarithmic"),
};

UENUM(BlueprintType)
enum class EGradientRotation : uint8
{
	GTR_0 = 0					UMETA(DisplayName = "0"),
	GTR_90 = 1					UMETA(DisplayName = "90"),
	GTR_180 = 2					UMETA(DisplayName = "180"),
	GTR_270 = 3					UMETA(DisplayName = "270"),
};

UENUM(BlueprintType)
enum class EGradientRotationLimited : uint8
{
	GTRL_0 = 0					UMETA(DisplayName = "0"),
	GTRL_90 = 1					UMETA(DisplayName = "90"),
};

namespace
{
	// Declare one Shader Permutation Var class per parameters
	class FVar_GradientInterpolation : SHADER_PERMUTATION_INT("GRADIENT_INTERPOLATION", (int32)EGradientInterpolation::GTI_Exp + 1);
	class FVar_GradientRotation : SHADER_PERMUTATION_INT("GRADIENT_ROTATION", (int32)EGradientRotation::GTR_270 + 1);
}

///////////////////////////////////////////////////////////////////////////////
class FSH_GradientLinear_1 : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_GradientLinear_1, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_GradientLinear_1, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<FVar_GradientInterpolation, FVar_GradientRotation>;
public:
	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

///////////////////////////////////////////////////////////////////////////////
class FSH_GradientLinear_2 : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_GradientLinear_2, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_GradientLinear_2, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<FVar_GradientInterpolation, FVar_GradientRotation>;
public:
	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

///////////////////////////////////////////////////////////////////////////////
class FSH_GradientRadial : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_GradientRadial, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_GradientRadial, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER(FVector4f, Center)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<FVar_GradientInterpolation>;
public:
	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

///////////////////////////////////////////////////////////////////////////////
class FSH_GradientAxial1: public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_GradientAxial1, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_GradientAxial1, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER(FVector4f, Line)
		SHADER_PARAMETER(FVector4f, LineDir)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<FVar_GradientInterpolation>;
public:
	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

///////////////////////////////////////////////////////////////////////////////
class FSH_GradientAxial2: public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_GradientAxial2, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_GradientAxial2, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER(FVector4f, Line)
		SHADER_PARAMETER(FVector4f, LineDir)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<FVar_GradientInterpolation>;
public:
	TEXTURE_ENGINE_DEFAULT_PERMUTATION;
	TEXTUREGRAPH_ENGINE_DEFAULT_COMPILATION_ENV;
};

///////////////////////////////////////////////////////////////////////////////
/**
 * 
 */
class T_Gradient
{
public:
	static constexpr int		DefaultSize = 1024;
	static UE_API BufferDescriptor		InitOutputDesc(BufferDescriptor DesiredOutputDesc);

	struct Params {
		EGradientType			Type = EGradientType::GT_Linear_1;
		EGradientInterpolation	Interpolation = EGradientInterpolation::GTI_Linear;
		int32					Rotation = 0; 
		FVector2f				Center;
		float					Radius;
		FVector2f				Point1;
		FVector2f				Point2;
	};

	static UE_API TiledBlobPtr	Create(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, const Params& InParams, int32 TargetId = 0);
};

#undef UE_API
