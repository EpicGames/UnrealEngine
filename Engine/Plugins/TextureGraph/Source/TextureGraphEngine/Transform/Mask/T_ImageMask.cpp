// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_ImageMask.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Utility/T_MinMax.h"

IMPLEMENT_GLOBAL_SHADER(FSH_ImageMask, "/Plugin/TextureGraph/Mask/ImageMask.usf", "FSH_ImageMask", SF_Pixel);

template <>
void SetupDefaultParameters(FSH_ImageMask::FParameters& params) {
	FStandardSamplerStates_Setup(params.SamplerStates);
}

//////////////////////////////////////////////////////////////////////////
T_ImageMask::T_ImageMask()
{
}

T_ImageMask::~T_ImageMask()
{
}

