// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////
class FSH_ImageMask : public FSH_Base
{
public:
	// Declare one Shader Permutation Var class per parameters
	class FVar_NORMALIZE : SHADER_PERMUTATION_BOOL("NORMALIZE");
	class FVar_INVERT : SHADER_PERMUTATION_BOOL("INVERT");

	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_ImageMask, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_ImageMask, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, MinMax)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER(float, RangeMin)
		SHADER_PARAMETER(float, RangeMax)
	END_SHADER_PARAMETER_STRUCT()
		
	using FPermutationDomain = TShaderPermutationDomain<FVar_NORMALIZE, FVar_INVERT>;

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};
template <> void SetupDefaultParameters(FSH_ImageMask::FParameters& params);

typedef FxMaterial_Normal<VSH_Simple, FSH_ImageMask>	Fx_ImageMask;

class UImageMask;

class T_ImageMask
{
private:


public:
									UE_API T_ImageMask();
	UE_API virtual							~T_ImageMask();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
