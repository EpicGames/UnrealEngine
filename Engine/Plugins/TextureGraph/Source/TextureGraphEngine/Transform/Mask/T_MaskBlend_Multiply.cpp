// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_MaskBlend_Multiply.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"

IMPLEMENT_GLOBAL_SHADER(FSH_MaskBlend_Multiply, "/Plugin/TextureGraph/Mask/Blend/MaskBlend_Multiply.usf", "FSH_MaskBlend_Multiply", SF_Pixel);

template <>
void SetupDefaultParameters(FSH_MaskBlend_Multiply::FParameters& params) {
	FStandardSamplerStates_Setup(params.SamplerStates);
}

//////////////////////////////////////////////////////////////////////////
T_MaskBlend_Multiply::T_MaskBlend_Multiply()
{
}

T_MaskBlend_Multiply::~T_MaskBlend_Multiply()
{
}
