// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////
class FSH_MaskBlend_Multiply : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_MaskBlend_Multiply, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_MaskBlend_Multiply, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, MaskFactor0)
		SHADER_PARAMETER_TEXTURE(Texture2D, Mask0)

		SHADER_PARAMETER(float, MaskFactor1)
		SHADER_PARAMETER_TEXTURE(Texture2D, Mask1)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};
template <> void SetupDefaultParameters(FSH_MaskBlend_Multiply::FParameters& params);

typedef FxMaterial_Normal<VSH_Simple, FSH_MaskBlend_Multiply>	Fx_BlendMode_Multiply;

class UImageMask;

class T_MaskBlend_Multiply
{
private:


public:
									UE_API T_MaskBlend_Multiply();
	UE_API virtual							~T_MaskBlend_Multiply();
};

#undef UE_API
