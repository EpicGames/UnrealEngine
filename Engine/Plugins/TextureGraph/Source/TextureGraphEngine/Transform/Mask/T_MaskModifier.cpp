// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_MaskModifier.h"
#include "Job/JobBatch.h"
#include "Transform/Utility/T_MinMax.h"

IMPLEMENT_GLOBAL_SHADER(FSH_BrightnessMaskModifier, "/Plugin/TextureGraph/Mask/MaskModifier.usf", "FSH_MaskModifier", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_ClampMaskModifier, "/Plugin/TextureGraph/Mask/MaskModifier.usf", "FSH_MaskModifier", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_InvertMaskModifier, "/Plugin/TextureGraph/Mask/MaskModifier.usf", "FSH_MaskModifier", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_NormalizeMaskModifier, "/Plugin/TextureGraph/Mask/MaskModifier.usf", "FSH_MaskModifier", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_GradientRemapMaskModifier, "/Plugin/TextureGraph/Mask/MaskModifier.usf", "FSH_MaskModifier", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_PosterizeMaskModifier, "/Plugin/TextureGraph/Mask/MaskModifier.usf", "FSH_MaskModifier", SF_Pixel);



T_MaskModifier::T_MaskModifier()
{
}

T_MaskModifier::~T_MaskModifier()
{
}

