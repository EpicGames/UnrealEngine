// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////

namespace
{
	class FVar_INVERT : SHADER_PERMUTATION_BOOL("INVERT");
}

class FSH_BrightnessMaskModifier : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_BrightnessMaskModifier, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_BrightnessMaskModifier, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER(float, Brightness)
		SHADER_PARAMETER(float, Contrast)
		SHADER_PARAMETER_TEXTURE(Texture2D, MinMaxTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
		env.SetDefine(TEXT("BRIGHTNESS_MODIFIER"), 1);
	}
};

class FSH_ClampMaskModifier : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_ClampMaskModifier, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_ClampMaskModifier, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER(float, ClampMin)
		SHADER_PARAMETER(float, ClampMax)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
		env.SetDefine(TEXT("CLAMP_MODIFIER"), 1);
	}
};

class FSH_InvertMaskModifier : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_InvertMaskModifier, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_InvertMaskModifier, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
	END_SHADER_PARAMETER_STRUCT()

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
		env.SetDefine(TEXT("INVERT_MODIFIER"), 1);
	}
};

class FSH_NormalizeMaskModifier : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_NormalizeMaskModifier, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_NormalizeMaskModifier, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, MinMaxTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
		env.SetDefine(TEXT("NORMALIZE_MODIFIER"), 1);
	}
};

class FSH_GradientRemapMaskModifier : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_GradientRemapMaskModifier, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_GradientRemapMaskModifier, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER(float, RangeMin)
		SHADER_PARAMETER(float, RangeMax)
		SHADER_PARAMETER(int, Repeat)
		SHADER_PARAMETER(float, Curve)
		SHADER_PARAMETER(int, Mirror)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
		env.SetDefine(TEXT("GRADIENT_REMAP_MODIFIER"), 1);
	}
};

class FSH_PosterizeMaskModifier : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_PosterizeMaskModifier, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_PosterizeMaskModifier, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER(int, StepCount)
		END_SHADER_PARAMETER_STRUCT()

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
		env.SetDefine(TEXT("POSTERIZE_MODIFIER"), 1);
	}
};

class UMaskModifier;

/**
 * 
 */
class T_MaskModifier
{
public:
	UE_API T_MaskModifier();
	UE_API ~T_MaskModifier();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
