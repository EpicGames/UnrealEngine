// Copyright Epic Games, Inc. All Rights Reserved.

#include "T_NoiseMask.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"

#include <iostream>
#include <string>

#include UE_INLINE_GENERATED_CPP_BY_NAME(T_NoiseMask)
IMPLEMENT_GLOBAL_SHADER(FSH_NoiseMask, "/Plugin/TextureGraph/Mask/NoiseMask.usf", "FSH_NoiseMask", SF_Pixel);

T_NoiseMask::T_NoiseMask()
{
}

T_NoiseMask::~T_NoiseMask()
{
}

