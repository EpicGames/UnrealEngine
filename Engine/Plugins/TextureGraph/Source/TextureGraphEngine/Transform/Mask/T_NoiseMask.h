// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>
#include "T_NoiseMask.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

UENUM()
enum class NoiseType
{
	Simplex = 0			UMETA(DisplayName = "Simplex"),
	Perlin				UMETA(DisplayName = "Perlin"),
	Worley1				UMETA(DisplayName = "Worley1"),
	Worley2				UMETA(DisplayName = "Worley2"),
	Worley3				UMETA(DisplayName = "Worley3"),
	Count				UMETA(DisplayName = "For fallback purpose"),
};

//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////
namespace
{
	// Declare one Shader Permutation Var class per parameters
	class FVar_NoiseType : SHADER_PERMUTATION_INT("NOISE_TYPE", (int32)NoiseType::Count);
}

class FSH_NoiseMask : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_NoiseMask, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_NoiseMask, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER(int32, NoiseType)
		SHADER_PARAMETER(float, Seed)
		SHADER_PARAMETER(float, Amplitude)
		SHADER_PARAMETER(float, Frequency)
		SHADER_PARAMETER(int32, Octaves)
		SHADER_PARAMETER(float, Lacunarity)
		SHADER_PARAMETER(float, Persistance)
		SHADER_PARAMETER(float, Invert)
	END_SHADER_PARAMETER_STRUCT()

	// Permutation domain local 
	using FPermutationDomain = TShaderPermutationDomain<FVar_NoiseType>;


public:
	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
};

typedef FxMaterial_Normal<VSH_Simple, FSH_NoiseMask>	Fx_NoiseMask;

class UNoiseMask;

/**
 * 
 */
class T_NoiseMask
{
public:
	UE_API T_NoiseMask();
	UE_API ~T_NoiseMask();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
