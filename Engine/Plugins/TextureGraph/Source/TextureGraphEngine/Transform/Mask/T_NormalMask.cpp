// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_NormalMask.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"

IMPLEMENT_GLOBAL_SHADER(FSH_NormalMask, "/Plugin/TextureGraph/Mask/NormalMask.usf", "FSH_NormalMask", SF_Pixel);
template <>
void SetupDefaultParameters(FSH_NormalMask::FParameters& params) {
	FStandardSamplerStates_Setup(params.SamplerStates);
}

T_NormalMask::T_NormalMask()
{
}

T_NormalMask::~T_NormalMask()
{
}

