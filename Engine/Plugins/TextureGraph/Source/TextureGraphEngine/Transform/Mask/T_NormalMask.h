// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

namespace
{
	// Declare one Shader Permutation Var class per parameters
	class FVar_ThreeDimensions : SHADER_PERMUTATION_BOOL("THREE_DIMESIONS");
}
//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////
class FSH_NormalMask : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_NormalMask, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_NormalMask, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTex)
		SHADER_PARAMETER_TEXTURE(Texture2D, MeshNormals)
		SHADER_PARAMETER_TEXTURE(Texture2D, MeshTangents)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER(float, Range)
		SHADER_PARAMETER(float, Radius)
		SHADER_PARAMETER(float, DirectionX)
		SHADER_PARAMETER(float, DirectionY)
		SHADER_PARAMETER(float, DirectionZ)
		SHADER_PARAMETER(int32, ObjectSpace)
		SHADER_PARAMETER(FMatrix44f, LocalToWorldMatrix)
	END_SHADER_PARAMETER_STRUCT()

	// Permutation domain local 
	using FPermutationDomain = TShaderPermutationDomain<FVar_ThreeDimensions>;

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};
template <> void SetupDefaultParameters(FSH_NormalMask::FParameters& params);

typedef FxMaterial_Normal<VSH_Simple, FSH_NormalMask>	Fx_NormalMask;

class UNormalMask;

/**
 * 
 */
class T_NormalMask
{
public:
	UE_API T_NormalMask();
	UE_API ~T_NormalMask();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
