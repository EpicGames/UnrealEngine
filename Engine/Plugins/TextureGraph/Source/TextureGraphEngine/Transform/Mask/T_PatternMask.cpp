// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_PatternMask.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "2D/TargetTextureSet.h"
#include "3D/RenderMesh.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(T_PatternMask)

IMPLEMENT_GLOBAL_SHADER(FSH_PatternMask, "/Plugin/TextureGraph/Mask/PatternMask.usf", "FSH_PatternMask", SF_Pixel);

T_PatternMask::T_PatternMask()
{
}

T_PatternMask::~T_PatternMask()
{
}

