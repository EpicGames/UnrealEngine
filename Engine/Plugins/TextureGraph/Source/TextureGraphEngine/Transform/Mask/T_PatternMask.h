// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "FxMat/FxMaterial.h"
#include "Job/Job.h"
#include "Model/Mix/MixUpdateCycle.h"
#include "UObject/NoExportTypes.h"
#include <DataDrivenShaderPlatformInfo.h>
#include "T_PatternMask.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

UENUM()
enum class PatternType
{
	Square = 0			UMETA(DisplayName = "Square"),
	Circle				UMETA(DisplayName = "Circle"),
	Checker				UMETA(DisplayName = "Checker"),
	Gradient			UMETA(DisplayName = "Gradient"),
	Count				UMETA(DisplayName = "For fallback purpose"),
};


//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////
namespace
{
	// Declare one Shader Permutation Var class per parameters
	class FVar_PatternType : SHADER_PERMUTATION_INT("PATTERN_TYPE", (int32) PatternType::Count);
}

class FSH_PatternMask : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_PatternMask, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_PatternMask, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER(FVector2f, Repeat)
		SHADER_PARAMETER(FVector2f, Spacing)
		SHADER_PARAMETER(FVector2f, Offset)
		SHADER_PARAMETER(float, Invert)
		SHADER_PARAMETER(int32, PatternType)
		SHADER_PARAMETER(float, Bevel)
		SHADER_PARAMETER(float, BevelCurve)
		SHADER_PARAMETER(float, JS_Amount)
		SHADER_PARAMETER(float, JS_Threshold)
		SHADER_PARAMETER(int32, JS_Seed)
		SHADER_PARAMETER(float, JB_Amount)
		SHADER_PARAMETER(float, JB_Threshold)
		SHADER_PARAMETER(int32, JB_Seed)
		SHADER_PARAMETER(FVector2f, JT_Amount)
		SHADER_PARAMETER(float, JT_Threshold)
		SHADER_PARAMETER(int32, JT_Seed)
		SHADER_PARAMETER(FVector2f, JA_Amount)
		SHADER_PARAMETER(int32, JA_Seed)
		SHADER_PARAMETER(float, CO_Threshold)
		SHADER_PARAMETER(int32, CO_Seed)
		SHADER_PARAMETER(FVector2f, GradientDir)

	END_SHADER_PARAMETER_STRUCT()

	// Permutation domain local 
	using FPermutationDomain = TShaderPermutationDomain<FVar_PatternType>;


public:
	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
};

typedef FxMaterial_Normal<VSH_Simple, FSH_PatternMask>	Fx_PatternMask;

class UPatternMask;

/**
 * 
 */
class T_PatternMask
{
public:
	UE_API T_PatternMask();
	UE_API ~T_PatternMask();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
};

#undef UE_API
