// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_PositionGradientMask.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Transform/Utility/T_MinMax.h"

IMPLEMENT_GLOBAL_SHADER(FSH_PositionGradientMask, "/Plugin/TextureGraph/Mask/PositionGradientMask.usf", "FSH_PositionGradientMask", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_CombineDisplacementAndWorldPos, "/Plugin/TextureGraph/Mask/CombineDisplacementAndWorldPos.usf", "FSH_CombineDisplacementAndWorldPos", SF_Pixel);

T_PositionGradientMask::T_PositionGradientMask()
{
}

T_PositionGradientMask::~T_PositionGradientMask()
{
}
template <>
void SetupDefaultParameters(FSH_PositionGradientMask::FParameters& params) {
	FStandardSamplerStates_Setup(params.SamplerStates);
}
template <>
void SetupDefaultParameters(FSH_CombineDisplacementAndWorldPos::FParameters& params) {
	FStandardSamplerStates_Setup(params.SamplerStates);
}


