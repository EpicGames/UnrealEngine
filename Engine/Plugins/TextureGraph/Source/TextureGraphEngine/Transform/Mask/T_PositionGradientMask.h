// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

namespace PositionGradientMask
{
	// Declare one Shader Permutation Var class per parameters
	class FVar_ThreeDimensions : SHADER_PERMUTATION_BOOL("THREE_DIMESIONS");
}
//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////
class FSH_PositionGradientMask : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_PositionGradientMask, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_PositionGradientMask, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTex)
		SHADER_PARAMETER_TEXTURE(Texture2D, MinMaxTex)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER(float, StartingOffset)
		SHADER_PARAMETER(float, Falloff)
		SHADER_PARAMETER(float, DirectionX)
		SHADER_PARAMETER(float, DirectionY)
		SHADER_PARAMETER(float, DirectionZ)
		SHADER_PARAMETER(FLinearColor, RelativeBoundsMin)
		SHADER_PARAMETER(FLinearColor, RelativeBoundsMax)
	END_SHADER_PARAMETER_STRUCT()

	// Permutation domain local 
	using FPermutationDomain = TShaderPermutationDomain<PositionGradientMask::FVar_ThreeDimensions>;

public:
	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
};
template <> void SetupDefaultParameters(FSH_PositionGradientMask::FParameters& params);

typedef FxMaterial_Normal<VSH_Simple, FSH_PositionGradientMask>	Fx_PositionGradientMask;

//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////
class FSH_CombineDisplacementAndWorldPos : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_CombineDisplacementAndWorldPos, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_CombineDisplacementAndWorldPos, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )		
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceDisplacement)
		SHADER_PARAMETER_TEXTURE(Texture2D, WorldPos)
		SHADER_PARAMETER_TEXTURE(Texture2D, WorldNormals)
		SHADER_PARAMETER_TEXTURE(Texture2D, WorldUVMask)
		SHADER_PARAMETER_TEXTURE(Texture2D, MinMaxTex)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER(FLinearColor, MeshBoundsMin)
		SHADER_PARAMETER(FLinearColor, MeshBoundsSize)
		SHADER_PARAMETER(float, MidPoint)
		SHADER_PARAMETER(float, StandardHeight)
		END_SHADER_PARAMETER_STRUCT()
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
};
template <> void SetupDefaultParameters(FSH_CombineDisplacementAndWorldPos::FParameters& params);

typedef FxMaterial_Normal<VSH_Simple, FSH_CombineDisplacementAndWorldPos>	Fx_CombineDisplacementAndWorldPos;

class UPositionGradientMask;

/**
 * 
 */
class T_PositionGradientMask
{
public:
	UE_API T_PositionGradientMask();
	UE_API ~T_PositionGradientMask();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////=
};

#undef UE_API
