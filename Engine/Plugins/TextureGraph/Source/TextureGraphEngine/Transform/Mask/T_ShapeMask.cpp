// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_ShapeMask.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "2D/TargetTextureSet.h"
#include "3D/RenderMesh.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(T_ShapeMask)

IMPLEMENT_GLOBAL_SHADER(FSH_ShapeMask, "/Plugin/TextureGraph/Expressions/Expression_Shape.usf", "FSH_DrawShapeMask", SF_Pixel);

class RegularPolygon
{
public:
	// The angle at the center of a sector of the polygon
	constexpr static float SectorAngle(int32 N) { return UE_TWO_PI / double(N); }
	static float SectorAngleCos(int32 N) { return cos(SectorAngle(N)); }
	static float SectorAngleSin(int32 N) { return sin(SectorAngle(N)); }

	// The angle at a vertex between the radius and the side
	constexpr static float VertexAngle(int32 N) { return (UE_PI - SectorAngle(N)) * 0.5; }
	static float VertexAngleCos(int32 N) { return cos(VertexAngle(N)); }
	static float VertexAngleSin(int32 N) { return sin(VertexAngle(N)); }
};




TiledBlobPtr T_ShapeMask::Create(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc,
	EShapeMaskType ShapeType, const FParams& Params, int32 TargetId)
{
	// Adjust some of the parameters depending of n the shape type
	auto ShaderParams = Params;
	int32 ShaderShapeType = (int32)ShapeType;
	switch (ShapeType)
	{
	case EShapeMaskType::Circle:
		ShaderParams.Size.X = std::max(0.0001f, ShaderParams.Size.X);
		ShaderParams.Size.Y = ShaderParams.Size.X;
		ShaderParams.Rounding = 0;
		break;

	case EShapeMaskType::Ellipse:
		ShaderParams.Size.X = std::max(0.0001f, ShaderParams.Size.X);
		ShaderParams.Size.Y = std::max(0.0001f, ShaderParams.Size.Y);

		// Under the enum Ellipse we also potentially do Circle, the shader path is different
		if (ShaderParams.Size.X == ShaderParams.Size.Y)
		{
			ShaderShapeType = (int32)EShapeMaskType::Circle;
		}
		ShaderParams.Rounding = 0;
		break;

	case EShapeMaskType::Segment:
		ShaderParams.Rounding = std::max(0.001f, Params.Rounding);
		break;

	case EShapeMaskType::Rect:
		break;

	case EShapeMaskType::Triangle:
		ShaderParams.Rounding = std::min(0.98f, Params.Rounding);
		break;

	case EShapeMaskType::Pentagon:
		ShaderParams.Rounding = std::min(0.98f, Params.Rounding);
		break;
	case EShapeMaskType::Hexagon:
		ShaderParams.Rounding = std::min(0.98f, Params.Rounding);
		break;
	case EShapeMaskType::RegularPolygon_7:
	// And all polygons...
	default:
		ShaderParams.Rounding = std::min(0.98f, Params.Rounding);
		break;
	}

	FSH_ShapeMask::FPermutationDomain PermutationVector;
	PermutationVector.Set<FVar_ShapeType>(ShaderShapeType);

	const RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_ShapeMask>(TEXT("T_ShapeMask"), PermutationVector);
	check(RenderMaterial);


	JobUPtr RenderJob = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	FTileInfo TileInfo;
	RenderJob
		->AddArg(ARG_TILEINFO(TileInfo, "TileInfo"))
		->AddArg(ARG_INT(ShaderShapeType, "ShapeType")) /// Adding the shape type arg to hash the job result
		->AddArg(ARG_FLOAT(cos(ShaderParams.Rotation), "RotateX"))
		->AddArg(ARG_FLOAT(sin(ShaderParams.Rotation), "RotateY"))
		->AddArg(ARG_FLOAT(ShaderParams.Size.X, "ParamX"))
		->AddArg(ARG_FLOAT(ShaderParams.Size.Y, "ParamY"))
		->AddArg(ARG_FLOAT(ShaderParams.Rounding, "Rounding"))
		->AddArg(ARG_FLOAT(ShaderParams.BevelWidth, "BevelWidth"))
		->AddArg(ARG_FLOAT(ShaderParams.BevelCurve, "BevelCurve"))
		->AddArg(ARG_FLOAT(ShaderParams.BlendSDF, "BlendSDF"))

		->AddArg(std::make_shared<JobArg_ForceTiling>()) /// Force hashing individual tiles differently
		;

	FString Name = FString::Printf(TEXT("ShapeMask.[%llu]"), Cycle->GetBatch()->GetBatchId());

	// Pick the default size or square if only one size (width or height)
	int32 MaxSize = std::max(DesiredOutputDesc.Width, DesiredOutputDesc.Height);
	if (DesiredOutputDesc.Width <= 0 || DesiredOutputDesc.Height <= 0)
	{
		MaxSize = (MaxSize > 0 ? MaxSize : DefaultSize);
		DesiredOutputDesc.Width = DesiredOutputDesc.Height = MaxSize;
	}

	if (DesiredOutputDesc.Format == BufferFormat::Auto)
		DesiredOutputDesc.Format = BufferFormat::Byte;

	if (DesiredOutputDesc.ItemsPerPoint == 0)
		DesiredOutputDesc.ItemsPerPoint = 1;

	DesiredOutputDesc.DefaultValue = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);

	TiledBlobPtr Result = RenderJob->InitResult(Name,&DesiredOutputDesc);
	Cycle->AddJob(TargetId, std::move(RenderJob));

	return Result;
}
