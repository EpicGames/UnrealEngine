// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "FxMat/FxMaterial.h"
#include "Job/Job.h"
#include "Model/Mix/MixUpdateCycle.h"
#include "UObject/NoExportTypes.h"
#include <DataDrivenShaderPlatformInfo.h>
#include "T_ShapeMask.generated.h"

#define UE_API TEXTUREGRAPHENGINE_API

UENUM(BlueprintType)
enum class EShapeMaskType : uint8
{
	Circle = 0			UMETA(DisplayName = "Circle"),
	Segment				UMETA(DisplayName = "Segment"),
	Rect				UMETA(DisplayName = "Rect"),
	Triangle			UMETA(DisplayName = "Triangle"),
	Ellipse				UMETA(DisplayName = "Ellipse"),
	Pentagon			UMETA(DisplayName = "Pentagon"),
	Hexagon				UMETA(DisplayName = "Hexagon"),
	RegularPolygon_7	UMETA(DisplayName = " 7-gon"),
	RegularPolygon_8	UMETA(DisplayName = " 8-gon"),
	RegularPolygon_9	UMETA(DisplayName = " 9-gon"),
	RegularPolygon_10	UMETA(DisplayName = "10-gon"),
	RegularPolygon_11	UMETA(DisplayName = "11-gon"),
	RegularPolygon_12	UMETA(DisplayName = "12-gon"),
	RegularPolygon_13	UMETA(DisplayName = "13-gon"),
	RegularPolygon_14	UMETA(DisplayName = "14-gon"),
	RegularPolygon_15	UMETA(DisplayName = "15-gon"),
	RegularPolygon_16	UMETA(DisplayName = "16-gon"),
	RegularPolygon_17	UMETA(DisplayName = "17-gon"),
	RegularPolygon_18	UMETA(DisplayName = "18-gon"),
	RegularPolygon_19	UMETA(DisplayName = "19-gon"),
	RegularPolygon_20	UMETA(DisplayName = "20-gon"),
	RegularPolygon_21	UMETA(DisplayName = "21-gon"),
	RegularPolygon_22	UMETA(DisplayName = "22-gon"),
	RegularPolygon_23	UMETA(DisplayName = "23-gon"),
	RegularPolygon_24	UMETA(DisplayName = "24-gon"),
	RegularPolygon_25	UMETA(DisplayName = "25-gon"),
	RegularPolygon_26	UMETA(DisplayName = "26-gon"),
	RegularPolygon_27	UMETA(DisplayName = "27-gon"),
	RegularPolygon_28	UMETA(DisplayName = "28-gon"),
	RegularPolygon_29	UMETA(DisplayName = "29-gon"),
	RegularPolygon_30	UMETA(DisplayName = "30-gon"),
	RegularPolygon_31	UMETA(DisplayName = "31-gon"),
	RegularPolygon_32	UMETA(DisplayName = "32-gon"),

	Count				UMETA(DisplayName = "For fallback purpose"),
};


//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////
namespace
{
	// Declare one Shader Permutation Var class per parameters
	class FVar_ShapeType : SHADER_PERMUTATION_INT("SHAPE_TYPE", (int32)EShapeMaskType::Count);
}

class FSH_ShapeMask : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_ShapeMask, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_ShapeMask, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER(int32, ShapeType)
		SHADER_PARAMETER(float, RotateX)
		SHADER_PARAMETER(float, RotateY)
		SHADER_PARAMETER(float, ParamX)
		SHADER_PARAMETER(float, ParamY)
		SHADER_PARAMETER(float, Rounding)
		SHADER_PARAMETER(float, BevelWidth)
		SHADER_PARAMETER(float, BevelCurve)
		SHADER_PARAMETER(float, BlendSDF)
	END_SHADER_PARAMETER_STRUCT()

	// Permutation domain local 
	using FPermutationDomain = TShaderPermutationDomain<FVar_ShapeType>;


public:
	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
};

typedef FxMaterial_Normal<VSH_Simple, FSH_ShapeMask>	Fx_ShapeMask;

/**
 * 
 */
class T_ShapeMask
{
public:
	static constexpr int				DefaultSize = 1024;

	struct FParams
	{
		float Rotation = 0.f;
		FVector2f Size = { 1.0f, 1.0f };
		float Rounding = 0.0f;
		float BevelWidth = 0.0f;
		float BevelCurve = 0.0f;
		float BlendSDF = 0.0f;
	};

	static UE_API TiledBlobPtr	Create(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc,
		EShapeMaskType ShapeMaskType, const FParams& Params, int32 TargetId = 0);
};

#undef UE_API
