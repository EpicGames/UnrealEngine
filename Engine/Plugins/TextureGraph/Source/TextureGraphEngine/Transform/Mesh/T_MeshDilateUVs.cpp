// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_MeshDilateUVs.h"
#include "2D/TargetTextureSet.h"
#include "Data/TiledBlob.h"
#include "FxMat/MaterialManager.h"
#include "Helper/GraphicsUtil.h"
#include "Job/JobArgs.h"
#include "Job/JobBatch.h"
#include "TextureGraphEngine.h"

IMPLEMENT_GLOBAL_SHADER(FSH_DilateMeshTexture, "/Plugin/TextureGraph/Mesh/Mesh_Dilate.usf", "FSH_Main", SF_Pixel);

//////////////////////////////////////////////////////////////////////////
T_MeshDilateUVs::T_MeshDilateUVs()
{
}

T_MeshDilateUVs::~T_MeshDilateUVs()
{
}
template <>
void SetupDefaultParameters(FSH_DilateMeshTexture::FParameters& Params) {
	FStandardSamplerStates_Setup(Params.SamplerStates);
}

TiledBlobPtr T_MeshDilateUVs::Create(MixUpdateCyclePtr Cycle, int32 TargetId, TiledBlobPtr SourceTexture)
{
	RenderMaterial_FXPtr transform = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Simple, FSH_DilateMeshTexture>(TEXT("T_MeshDilateUVs"));
	check(transform);

	JobUPtr JobP = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(transform));

	JobP
		->AddArg(ARG_BLOB(SourceTexture, "SourceTexture"))
		->AddArg(ARG_FLOAT(1.0f / (float)SourceTexture->GetWidth(), "InvSourceWidth"))
		->AddArg(ARG_FLOAT(1.0f / (float)SourceTexture->GetHeight(), "InvSourceHeight"))
		->AddArg(ARG_FLOAT(5.0f, "Steps"))
		;

	JobP->SetTiled(false);

	FString name = FString::Printf(TEXT("[%s].[TargetId-%d] Dilated"), *SourceTexture->Name(), TargetId);
	TiledBlobPtr Result;

	// Heightmask maps need mips
	Result = JobP->InitResult(name, &SourceTexture->GetDescriptor());
	Cycle->AddJob(TargetId, std::move(JobP));

	/// This should only ever exists as a single blob
	Result->MakeSingleBlob();

	Result->GetBufferRef()->SetDeviceTransferChain({ DeviceType::FX, DeviceType::MemCompressed, DeviceType::Disk });

	return Result;
}
