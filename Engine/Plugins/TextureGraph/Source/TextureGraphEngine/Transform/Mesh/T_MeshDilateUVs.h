// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

class ULayer_Textured;

//////////////////////////////////////////////////////////////////////////
class FSH_DilateMeshTexture : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_DilateMeshTexture, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_DilateMeshTexture, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER(float, InvSourceWidth)
		SHADER_PARAMETER(float, InvSourceHeight)
		SHADER_PARAMETER(float, Steps)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
	}
};
template <> void SetupDefaultParameters(FSH_DilateMeshTexture::FParameters& params);

//////////////////////////////////////////////////////////////////////////
typedef FxMaterial_Normal<VSH_Simple, FSH_DilateMeshTexture> Fx_DilateMeshTexture;

//////////////////////////////////////////////////////////////////////////
class T_MeshDilateUVs
{
public:
									UE_API T_MeshDilateUVs();
	UE_API virtual							~T_MeshDilateUVs();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API TiledBlobPtr				Create(MixUpdateCyclePtr cycle, int32 targetId, TiledBlobPtr sourceTexture);
};

#undef UE_API
