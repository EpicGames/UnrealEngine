// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_MeshEncodedAsTexture.h"
#include "Job/JobArgs.h"
#include "2D/TargetTextureSet.h"
#include "FxMat/MaterialManager.h"
#include "TextureGraphEngine.h"
#include "Job/JobBatch.h"
#include "Helper/GraphicsUtil.h"
#include "Model/Mix/MixInterface.h"
#include "Model/Mix/MixSettings.h"

IMPLEMENT_GLOBAL_SHADER(VSH_MeshTexture, "/Plugin/TextureGraph/Mesh/Mesh_TextureVSH.usf", "VSH_MeshTexture", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(VSH_MeshTexture_WorldPos, "/Plugin/TextureGraph/Mesh/Mesh_TextureVSH.usf", "VSH_MeshTexture_WorldPos", SF_Vertex);

IMPLEMENT_GLOBAL_SHADER(FSH_MeshTexture_WorldPos, "/Plugin/TextureGraph/Mesh/Mesh_TextureEncoding.usf", "FSH_WorldPos", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_MeshTexture_WorldNormals, "/Plugin/TextureGraph/Mesh/Mesh_TextureEncoding.usf", "FSH_WorldNormals", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_MeshTexture_WorldTangents, "/Plugin/TextureGraph/Mesh/Mesh_TextureEncoding.usf", "FSH_WorldTangents", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FSH_MeshTexture_WorldUVMask, "/Plugin/TextureGraph/Mesh/Mesh_TextureEncoding.usf", "FSH_WorldUVMask", SF_Pixel);

int32 T_MeshEncodedAsTexture::s_minMeshmapRes = 4096;

//////////////////////////////////////////////////////////////////////////
T_MeshEncodedAsTexture::T_MeshEncodedAsTexture()
{
}

T_MeshEncodedAsTexture::~T_MeshEncodedAsTexture()
{
}

template <typename VSH_Type, typename FSH_Type>
static JobUPtr CreateGenericJob(MixUpdateCyclePtr Cycle, int32 TargetId, FString Suffix, FLinearColor ClearColor = FLinearColor::Black, int ItemsPerPoint = 4)
{
	FString materialName = FString::Printf(TEXT("T_Mesh_%s"), *Suffix);
	RenderMaterial_FXPtr transform = TextureGraphEngine::GetMaterialManager()->CreateMaterial_FX<VSH_Type, FSH_Type>(materialName);
	check(transform);

	RenderMeshPtr RMesh = Cycle->GetMix()->GetSettings()->Target(TargetId)->GetMesh();

	JobUPtr JobP = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(transform));

	JobP
		->AddArg(ARG_MESH(RMesh, TargetId));

	JobP->SetTiled(false);

	FString Name = FString::Printf(TEXT("[%s].[%d] %s"), *RMesh->Name(), TargetId, *Suffix);

	int32 RequiredWidth = FMath::Max(T_MeshEncodedAsTexture::s_minMeshmapRes, Cycle->GetMix()->Width());
	int32 RequiredHeight = FMath::Max(T_MeshEncodedAsTexture::s_minMeshmapRes, Cycle->GetMix()->Height());

	BufferDescriptor Desc;
	Desc.Width = RequiredWidth;
	Desc.Height = RequiredHeight;
	Desc.Format = BufferFormat::Float;
	Desc.ItemsPerPoint = ItemsPerPoint;
	Desc.DefaultValue = ClearColor;
	//Desc.mipmaps = true;

	// Heightmask maps need mips
	TiledBlobPtr Result = JobP->InitResult(Name, &Desc);

	/// This should only ever exists as a single blob
	Result->MakeSingleBlob();

	/// Don't really need this for a long time. Once the dilated textures are created
	/// this can be safely deleted 
	Result->GetBufferRef()->SetDeviceTransferChain(DeviceBuffer::FXOnlyTransferChain);

	return JobP;
}

TiledBlobPtr T_MeshEncodedAsTexture::Create_WorldPos(MixUpdateCyclePtr Cycle, int32 TargetId)
{
	JobUPtr JobP = CreateGenericJob<VSH_MeshTexture_WorldPos, FSH_MeshTexture_WorldPos>(Cycle, TargetId, TEXT("WorldPos"), FLinearColor(-2, -2, -2, 0));

	RenderMeshPtr RMesh = Cycle->GetMix()->GetSettings()->Target(TargetId)->GetMesh();

	JobP
		->AddArg(ARG_VECTOR3(RMesh->OriginalBounds().Min, "BoundsMin"))
		->AddArg(ARG_VECTOR3(RMesh->InvOriginalBoundsDiameter(), "InvBoundsDiameter"))
		;

	JobPtrW JobW = Cycle->AddJob(TargetId, std::move(JobP));

	return JobW.lock()->GetResult();
}

TiledBlobPtr T_MeshEncodedAsTexture::Create_WorldNormals(MixUpdateCyclePtr Cycle, int32 TargetId)
{
	auto JobP = CreateGenericJob<VSH_MeshTexture, FSH_MeshTexture_WorldNormals>(Cycle, TargetId, TEXT("WorldNormals"), FLinearColor(-2, -2, -2, 0));
	JobPtrW JobW = Cycle->AddJob(TargetId, std::move(JobP));
	return JobW.lock()->GetResult();
}

TiledBlobPtr T_MeshEncodedAsTexture::Create_WorldTangents(MixUpdateCyclePtr Cycle, int32 TargetId)
{
	auto JobP = CreateGenericJob<VSH_MeshTexture, FSH_MeshTexture_WorldTangents>(Cycle, TargetId, TEXT("WorldTangents"), FLinearColor(-2, -2, -2, 0));
	JobPtrW JobW = Cycle->AddJob(TargetId, std::move(JobP));
	return JobW.lock()->GetResult();

}

TiledBlobPtr T_MeshEncodedAsTexture::Create_WorldUVMask(MixUpdateCyclePtr Cycle, int32 TargetId)
{
	auto JobP = CreateGenericJob<VSH_MeshTexture, FSH_MeshTexture_WorldUVMask>(Cycle, TargetId, TEXT("WorldUVMask"), FLinearColor::Black, 1); // Single channel map
	JobPtrW JobW = Cycle->AddJob(TargetId, std::move(JobP));
	return JobW.lock()->GetResult();

}
