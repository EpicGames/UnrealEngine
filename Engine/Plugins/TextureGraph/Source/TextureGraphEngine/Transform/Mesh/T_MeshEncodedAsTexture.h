// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "FxMat/FxMaterial.h"
#include "Job/Job.h"
#include "Model/Mix/MixUpdateCycle.h"
#include "UObject/NoExportTypes.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

class ULayer_Textured;

////////////////////////////////////////////////////////////////////////////
class VSH_MeshTexture : public VSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(VSH_MeshTexture, UE_API);
	SHADER_USE_PARAMETER_STRUCT(VSH_MeshTexture, VSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
};

////////////////////////////////////////////////////////////////////////////
class VSH_MeshTexture_WorldPos : public VSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(VSH_MeshTexture_WorldPos, UE_API);
	SHADER_USE_PARAMETER_STRUCT(VSH_MeshTexture_WorldPos, VSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector3f, BoundsMin)
		SHADER_PARAMETER(FVector3f, InvBoundsDiameter)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
};

//////////////////////////////////////////////////////////////////////////
class FSH_MeshTexture_WorldPos : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_MeshTexture_WorldPos, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_MeshTexture_WorldPos, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
	}
};

//////////////////////////////////////////////////////////////////////////
class FSH_MeshTexture_WorldNormals : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_MeshTexture_WorldNormals, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_MeshTexture_WorldNormals, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
	}
};

//////////////////////////////////////////////////////////////////////////
class FSH_MeshTexture_WorldTangents : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_MeshTexture_WorldTangents, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_MeshTexture_WorldTangents, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
	}
};

//////////////////////////////////////////////////////////////////////////
class FSH_MeshTexture_WorldUVMask : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_MeshTexture_WorldUVMask, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_MeshTexture_WorldUVMask, FSH_Base);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& params, FShaderCompilerEnvironment& env)
	{
	}
};

//////////////////////////////////////////////////////////////////////////
class T_MeshEncodedAsTexture 
{
public:
	static UE_API int32					s_minMeshmapRes;			// We need to create mesh maps in 4k. Otherwise we get artifacts when using in curvature and painting. 
									UE_API T_MeshEncodedAsTexture();
	UE_API virtual							~T_MeshEncodedAsTexture();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API TiledBlobPtr				Create_WorldPos(MixUpdateCyclePtr cycle, int32 targetId);
	static UE_API TiledBlobPtr				Create_WorldNormals(MixUpdateCyclePtr cycle, int32 targetId);
	static UE_API TiledBlobPtr				Create_WorldTangents(MixUpdateCyclePtr cycle, int32 targetId);
	static UE_API TiledBlobPtr				Create_WorldUVMask(MixUpdateCyclePtr cycle, int32 targetId);
};

#undef UE_API
