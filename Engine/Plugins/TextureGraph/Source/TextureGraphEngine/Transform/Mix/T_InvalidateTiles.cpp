// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_InvalidateTiles.h"
#include "Device/FX/Device_FX.h"
#include "TextureGraphEngine.h"
#include "Job/JobBatch.h"

T_InvalidateTiles::T_InvalidateTiles() : BlobTransform(TEXT("T_InvalidateTiles"))
{
}

T_InvalidateTiles::~T_InvalidateTiles()
{
}

Device* T_InvalidateTiles::TargetDevice(size_t index) const
{
	return Device_FX::Get();
}

AsyncTransformResultPtr T_InvalidateTiles::Exec(const TransformArgs& args)
{
	/// Device::Use should have ensured that we we're in the rendering thread
	/// by the time we get to this point!
	check(IsInRenderingThread());

	/// We don't need a target, so it shouldn't be there. The job needs to 
	/// handle this correctly
	check(args.Target.expired());
	check(args.Cycle);

	MixUpdateCyclePtr cycle = args.Cycle;
	cycle->GetTarget(args.TargetId)->InvalidateAllTiles();

	return cti::make_ready_continuable(std::make_shared<TransformResult>());
}

JobUPtr T_InvalidateTiles::CreateJob(MixUpdateCyclePtr cycle, int32 targetId)
{
	BlobTransformPtr transform = std::static_pointer_cast<BlobTransform>(std::make_shared<T_InvalidateTiles>());
	JobUPtr job = std::make_unique<Job>(cycle->GetMix(), targetId, transform);
	return job;
}

void T_InvalidateTiles::Create(MixUpdateCyclePtr cycle, int32 targetId)
{
	JobUPtr job = CreateJob(cycle, targetId);
	job->SetPriority((int32)E_Priority::kHighest, false);
	cycle->AddJob(targetId, std::move(job));
}
