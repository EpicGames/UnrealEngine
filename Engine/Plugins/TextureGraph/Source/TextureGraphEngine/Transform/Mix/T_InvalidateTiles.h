// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Transform/BlobTransform.h"
#include "Job/Job.h"
#include "Model/Mix/MixUpdateCycle.h"

#define UE_API TEXTUREGRAPHENGINE_API

class T_InvalidateTiles : public BlobTransform
{
public:
									UE_API T_InvalidateTiles();
	UE_API virtual							~T_InvalidateTiles() override;

	UE_API virtual Device*					TargetDevice(size_t index) const override;
	UE_API virtual AsyncTransformResultPtr	Exec(const TransformArgs& args) override;

	virtual bool					GeneratesData() const override { return false; }

	////////////////////////////////////////////////////////////////////////// 
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API JobUPtr					CreateJob(MixUpdateCyclePtr cycle, int32 targetId);
	static UE_API void						Create(MixUpdateCyclePtr cycle, int32 targetId);
};

#undef UE_API
