// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_UpdateTargets.h"
#include "Device/FX/Device_FX.h"
#include "TextureGraphEngine.h"
#include "2D/TargetTextureSet.h"
#include "Profiling/StatGroup.h"
#include "Job/JobBatch.h"
#include "Model/Mix/MixSettings.h"
#include "Model/Mix/MixInterface.h"

using namespace TextureGraphEditor;

DECLARE_CYCLE_STAT(TEXT("T_UpdateTargets_PrepareResources"), STAT_UpdateTargets_PrepareResources, STATGROUP_TextureGraphEngine);
DECLARE_CYCLE_STAT(TEXT("T_UpdateTargets_Create"), STAT_UpdateTargets_Create, STATGROUP_TextureGraphEngine);

int T_UpdateTargets::_gpuTesselationFactor = 1;

T_UpdateTargets::T_UpdateTargets() : BlobTransform(TEXT("T_UpdateTargets"))
{
}

T_UpdateTargets::~T_UpdateTargets()
{
}

Device* T_UpdateTargets::TargetDevice(size_t) const
{
	return Device_FX::Get();
}

AsyncTransformResultPtr T_UpdateTargets::Exec(const TransformArgs& Args)
{
	/// Device::Use should have ensured that we we're in the rendering thread
	/// by the time we get to this point!
	check(IsInRenderingThread());

	/// We don't need a Target, so it shouldn't be there. The job needs to 
	/// handle this correctly
	check(Args.Target.expired());
	check(Args.Cycle);

	//ULayerComponent* finalLayer = static_cast<ULayerComponent*>(*prev);
	MixUpdateCyclePtr Cycle = Args.Cycle;
	TargetTextureSetPtr& Target = Cycle->GetMix()->GetSettings()->Target(Args.TargetId);
	check(Target);

	TextureSet& LastRender = Cycle->GetTarget(Args.TargetId)->GetLastRender();

	// TArray<FName> MaterialTargets = Cycle->Mix()->Settings()->GetViewportSettings().MaterialMappingInfos;
	for (const FName& TextureName : LastRender.GetTextureList())
	{
		if (LastRender.ContainsTexture(TextureName))
		{
			Target->SetTexture(TextureName, LastRender.GetTexture(TextureName));
		}
	}

	return cti::make_ready_continuable(std::make_shared<TransformResult>());
}

//////////////////////////////////////////////////////////////////////////
JobUPtr T_UpdateTargets::CreateJob(MixUpdateCyclePtr Cycle, int32 TargetId, bool ShouldUpdate)
{
	std::shared_ptr<T_UpdateTargets> UpdateTargetsTransform = std::make_shared<T_UpdateTargets>();
	UpdateTargetsTransform->SetShouldUpdate(ShouldUpdate);
	BlobTransformPtr Transform = std::static_pointer_cast<BlobTransform>(UpdateTargetsTransform);
	JobUPtr job = std::make_unique<Job>(Cycle->GetMix(), TargetId, Transform);
	return job;
}

void T_UpdateTargets::Create(MixUpdateCyclePtr Cycle, int32 TargetId, bool ShouldUpdate)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateTargets_Create)
		JobUPtr job = CreateJob(Cycle, TargetId, ShouldUpdate);
	Cycle->AddJob(TargetId, std::move(job));
}

