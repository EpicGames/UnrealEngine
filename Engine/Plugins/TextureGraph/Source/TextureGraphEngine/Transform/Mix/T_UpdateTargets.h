// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Transform/BlobTransform.h"
#include "Job/Job.h"
#include "Model/Mix/MixUpdateCycle.h"

#define UE_API TEXTUREGRAPHENGINE_API

class T_UpdateTargets : public BlobTransform
{
private:
	bool							_shouldUpdate = false;				/// This will ensure default texture is used when there is no child in layer set
	static UE_API int						_gpuTesselationFactor;
public:
									UE_API T_UpdateTargets();
	UE_API virtual							~T_UpdateTargets() override;
	UE_API virtual Device*					TargetDevice(size_t index) const override;
	UE_API virtual AsyncTransformResultPtr	Exec(const TransformArgs& args) override;

	virtual bool					GeneratesData() const override { return false; }
	virtual void					SetShouldUpdate(bool shouldUpdate) { _shouldUpdate = shouldUpdate; }
	////////////////////////////////////////////////////////////////////////// 
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static void						SetGPUTesselationFactor(int tesselationFactor) { _gpuTesselationFactor = tesselationFactor; }
	static UE_API JobUPtr					CreateJob(MixUpdateCyclePtr cycle, int32 targetId, bool shouldUpdate);
	static UE_API void						Create(MixUpdateCyclePtr cycle, int32 targetId, bool shouldUpdate);
};

#undef UE_API
