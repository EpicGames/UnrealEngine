// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_CombineTiledBlob.h"
#include "3D/RenderMesh.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include "Model/Mix/MixInterface.h"

TiledBlobPtr T_CombineTiledBlob::Create(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, 
	TiledBlobPtr SourceTex, JobUPtr JobToUse, const CombineSettings* InSettings)
{
	if (SourceTex && SourceTex->IsTiled())
	{
		JobUPtr JobObj;

		if (!JobToUse)
		{
			FString Name = TEXT("T_CombineTiledBlob"); //FString::Printf(TEXT("[%s].[%d].[%llu] CombineTiledBlob"), *SourceTex->Name(), TargetId, Cycle->GetBatch()->GetFrameId());
			std::shared_ptr<CombineTiledBlob_Transform> CombinedTiledBlobMat = std::make_shared<CombineTiledBlob_Transform>(Name, SourceTex, InSettings);
			JobObj = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(CombinedTiledBlobMat));
		}
		else
			JobObj = std::move(JobToUse);

		JobObj->AddArg(WithUnbounded(ARG_BLOB(SourceTex, "Source")));

		if (InSettings)
		{
			JobObj->AddArg(WithUnbounded(ARG_LINEAR_COLOR(InSettings->BackgroundColor, "BackgroundColor")));
			JobObj->AddArg(WithUnbounded(ARG_BOOL(InSettings->bMaintainAspectRatio, "MaintainAspectRatio")));
		}

		// Express the dependency of the new job on the job delivering the SourceTex
		auto PrevJob = SourceTex->Job();
		if (!PrevJob.expired())
			JobObj->AddPrev(std::static_pointer_cast<DeviceNativeTask>(PrevJob.lock()));

		TiledBlobPtr Result = JobObj->InitResult(JobObj->GetName(), &DesiredOutputDesc, 1, 1);

		Cycle->AddJob(TargetId, std::move(JobObj));
		Result->MakeSingleBlob();

		return Result;
	}

	return SourceTex;
#if 0
	RenderMaterial_FXPtr RenderMaterial = TextureGraphEngine::GetMaterialManager()->CreateMaterialOfType_FX<Fx_FullScreenCopy>(TEXT("T_CombineTiledBlob"));
	JobUPtr JobObj = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(RenderMaterial));
	JobObj
		->AddArg(ARG_BLOB(SourceTex, "SourceTexture"))
		->SetTiled(false)
		;

	TiledBlobPtr Result = JobObj->InitResult(JobObj->GetName(), &DesiredOutputDesc, 1, 1);
	Cycle->AddJob(TargetId, std::move(JobObj));
	Result->MakeSingleBlob();

	return Result;
#endif
}


//////////////////////////////////////////////////////////////////////////
CombineTiledBlob_Transform::CombineTiledBlob_Transform(FString InName, TiledBlobPtr InSource, const CombineSettings* InSettings)
	: BlobTransform(InName)
	, Source(InSource)
{
	if (InSettings)
		Settings = *InSettings;
}

Device* CombineTiledBlob_Transform::TargetDevice(size_t DevIndex) const
{
	return Device_FX::Get();
}

bool CombineTiledBlob_Transform::GeneratesData() const 
{
	return true;
}
bool CombineTiledBlob_Transform::CanHandleTiles() const
{ 
	return false;
}

void CombineTiledBlob_Transform::InitDrawSettingsWithoutAspectRatio(BlobPtr Target, T_Tiles<DeviceBufferRef>& Tiles)
{
	uint32 TargetWidth = Target->GetWidth();
	uint32 TargetHeight = Target->GetHeight();
	uint32 SourceWidth = Source->GetWidth();
	uint32 SourceHeight = Source->GetHeight();
	float WidthRatio = (float)TargetWidth / (float)SourceWidth;
	float HeightRatio = (float)TargetHeight / (float)SourceHeight;

	DrawSettings.Position.Resize(Source->Rows(), Source->Cols());
	DrawSettings.Size.Resize(Source->Rows(), Source->Cols());

	uint32 YOffset = 0;
	for (int32 RowId = 0; RowId < Source->Rows(); RowId++)
	{
		uint32 XOffset = 0;
		uint32 MaxRowHeight = 0;

		for (int32 ColId = 0; ColId < Source->Cols(); ColId++)
		{
			check(Source->GetTile(RowId, ColId));

			BlobPtr Tile = Source->GetTile(RowId, ColId);
			check(Tile->GetBufferRef());

			Tiles[RowId][ColId] = Tile->GetBufferRef();

			uint32 Width = (uint32)(float)Tile->GetWidth() * WidthRatio;
			uint32 Height = (uint32)(float)Tile->GetHeight() * HeightRatio;

			/// account for rounding errors in the aspect ratio adjustments
			if (ColId == Source->Cols() - 1)
				Width = TargetWidth - XOffset;
			if (RowId == Source->Rows() - 1)
				Height = TargetHeight - YOffset;

			MaxRowHeight = FMath::Max(Height, MaxRowHeight);

			DrawSettings.Position[RowId][ColId] = FIntPoint(XOffset, YOffset);
			DrawSettings.Size[RowId][ColId] = FIntPoint(Width, Height);

			XOffset += Width;
		}

		YOffset += MaxRowHeight;
	}
}

void CombineTiledBlob_Transform::InitDrawSettingsWithAspectRatio(BlobPtr Target, T_Tiles<DeviceBufferRef>& Tiles)
{
	uint32 TargetWidth = Target->GetWidth();
	uint32 TargetHeight = Target->GetHeight();
	uint32 SourceWidth = Source->GetWidth();
	uint32 SourceHeight = Source->GetHeight();
	float TargetAspectRatio = (float)TargetWidth / (float)TargetHeight;
	float SourceAspectRatio = (float)SourceWidth / (float)SourceHeight;
	uint32 NewTargetWidth = 0;
	uint32 NewTargetHeight = 0;

	if (TargetAspectRatio > SourceAspectRatio)
	{
		NewTargetWidth = TargetWidth;
		NewTargetHeight = (uint32)((float)TargetWidth / SourceAspectRatio);
	}
	else
	{
		NewTargetWidth = (uint32)((float)TargetHeight * SourceAspectRatio);
		NewTargetHeight = TargetHeight;
	}

	if (NewTargetHeight > TargetHeight)
	{
		NewTargetHeight = TargetHeight;
		NewTargetWidth = (uint32)((float)NewTargetHeight * SourceAspectRatio);
	}
	else if (NewTargetWidth > TargetWidth)
	{
		NewTargetWidth = TargetWidth;
		NewTargetHeight = (uint32)((float)TargetWidth / SourceAspectRatio);
	}

	float WidthRatio = (float)NewTargetWidth / (float)SourceWidth;
	float HeightRatio = (float)NewTargetHeight / (float)SourceHeight;

	check(NewTargetWidth <= TargetWidth && NewTargetHeight <= TargetHeight);
	uint32 XMargin = (TargetWidth - NewTargetWidth) >> 1;
	uint32 YMargin = (TargetHeight - NewTargetHeight) >> 1;

	DrawSettings.Position.Resize(Source->Rows(), Source->Cols());
	DrawSettings.Size.Resize(Source->Rows(), Source->Cols());

	uint32 YOffset = YMargin;
	for (int32 RowId = 0; RowId < Source->Rows(); RowId++)
	{
		uint32 XOffset = XMargin;
		uint32 MaxRowHeight = 0;

		for (int32 ColId = 0; ColId < Source->Cols(); ColId++)
		{
			check(Source->GetTile(RowId, ColId));

			BlobPtr Tile = Source->GetTile(RowId, ColId);
			check(Tile->GetBufferRef());

			Tiles[RowId][ColId] = Tile->GetBufferRef();

			uint32 Width = (uint32)(float)Tile->GetWidth() * WidthRatio;
			uint32 Height = (uint32)(float)Tile->GetHeight() * HeightRatio;

			MaxRowHeight = FMath::Max(Height, MaxRowHeight);

			DrawSettings.Position[RowId][ColId] = FIntPoint(XOffset, YOffset);
			DrawSettings.Size[RowId][ColId] = FIntPoint(Width, Height);

			XOffset += Width;
		}

		YOffset += MaxRowHeight;
	}
}

void CombineTiledBlob_Transform::InitDrawSettings(BlobPtr Target, T_Tiles<DeviceBufferRef>& Tiles)
{
	uint32 TargetWidth = Target->GetWidth();
	uint32 TargetHeight = Target->GetHeight();
	uint32 SourceWidth = Source->GetWidth();
	uint32 SourceHeight = Source->GetHeight();
	bool bUseDefault = true;

	if (TargetWidth != SourceWidth || TargetHeight != SourceHeight)
	{
		if (!Settings.bMaintainAspectRatio)
		{
			InitDrawSettingsWithoutAspectRatio(Target, Tiles);
			bUseDefault = false;
		}
		else if (Settings.bMaintainAspectRatio)
		{
			InitDrawSettingsWithAspectRatio(Target, Tiles);
			bUseDefault = false;
		}
	}

	if (bUseDefault)
	{
		for (int32 RowId = 0; RowId < Source->Rows(); RowId++)
		{
			for (int32 ColId = 0; ColId < Source->Cols(); ColId++)
			{
				check(Source->GetTile(RowId, ColId));

				BlobPtr Tile = Source->GetTile(RowId, ColId);
				check(Tile->GetBufferRef());

				Tiles[RowId][ColId] = Tile->GetBufferRef();
			}
		}
	}
}

AsyncTransformResultPtr	CombineTiledBlob_Transform::Exec(const TransformArgs& Args)
{
	BlobPtr Target = Args.Target.lock();
	check(Target);

	check(Target->GetBufferRef().GetDeviceType() == DeviceType::FX);
	DeviceBuffer_FX* DevBuffer = static_cast<DeviceBuffer_FX*>(Target->GetBufferRef().get());

	T_Tiles<DeviceBufferRef> Tiles(Source->Rows(), Source->Cols());
	DrawTilesSettings* DrawSettingsToUse = nullptr;
	InitDrawSettings(Target, Tiles);

	FLinearColor* ClearColor = &Settings.BackgroundColor;

	if (DrawSettings.Position.Rows() == Tiles.Rows())
	{
		DrawSettingsToUse = &DrawSettings;
		ClearColor = &Settings.BackgroundColor;
	}

	Device_FX* FXDevice = Device_FX::Get();
	return FXDevice->DrawTilesToBuffer_Deferred(Target->GetBufferRef(), Tiles, DrawSettingsToUse, ClearColor)
		.then([this, Target](DeviceBufferRef)
		{
				TransformResultPtr Result = std::make_shared<TransformResult>();
				Result->Target = Target;
				return Result;
		});
}

std::shared_ptr<BlobTransform> CombineTiledBlob_Transform::DuplicateInstance(FString NewName)
{
	return std::make_shared<CombineTiledBlob_Transform>(NewName, Source, &Settings);
}

AsyncBufferResultPtr CombineTiledBlob_Transform::Bind(BlobPtr BlobObj, const ResourceBindInfo& BindInfo)
{
	return cti::make_ready_continuable<BufferResultPtr>(std::make_shared<BufferResult>());
}

AsyncBufferResultPtr CombineTiledBlob_Transform::Unbind(BlobPtr BlobObj, const ResourceBindInfo& BindInfo)
{
	return cti::make_ready_continuable<BufferResultPtr>(std::make_shared<BufferResult>());
}
