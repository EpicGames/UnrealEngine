// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ShaderCompilerCore.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>
#include "Device/FX/Device_FX.h"

#define UE_API TEXTUREGRAPHENGINE_API

//UENUM(BlueprintType)
//enum class ResizeSettings : uint8
//{
//	Stretch = 0			UMETA(DisplayName = "Stretch (Default)"),
//	 = 1		UMETA(DisplayName = "Directional"),
//	Radial = 2			UMETA(DisplayName = "Radial")
//};

struct CombineSettings
{
	bool bFixed = false;
	bool bMaintainAspectRatio = false;
	FLinearColor BackgroundColor = FLinearColor::Transparent;
};

/**
 * A dedicated BlobTransform creating a Combined version of a TiledBlob
 */
class CombineTiledBlob_Transform : public BlobTransform
{
private:
	TiledBlobPtr					Source;				/// The source blob that we are combining
	CombineSettings					Settings;			/// Combine settings
	DrawTilesSettings				DrawSettings;		/// FX device specific draw settings

	UE_API void							InitDrawSettings(BlobPtr Target, T_Tiles<DeviceBufferRef>& Tiles);
	UE_API void							InitDrawSettingsWithoutAspectRatio(BlobPtr Target, T_Tiles<DeviceBufferRef>& Tiles);
	UE_API void							InitDrawSettingsWithAspectRatio(BlobPtr Target, T_Tiles<DeviceBufferRef>& Tiles);

public:
	UE_API CombineTiledBlob_Transform(FString InName, TiledBlobPtr InSource, const CombineSettings* InSettings = nullptr);
	UE_API virtual AsyncBufferResultPtr	Bind(BlobPtr Value, const ResourceBindInfo& BindInfo) override;
	UE_API virtual AsyncBufferResultPtr	Unbind(BlobPtr BlobObj, const ResourceBindInfo& BindInfo) override;
	UE_API virtual Device*					TargetDevice(size_t DevIndex) const override;
	UE_API virtual std::shared_ptr<BlobTransform> DuplicateInstance(FString NewName) override;
	UE_API virtual bool					GeneratesData() const override;
	UE_API virtual bool					CanHandleTiles() const override;

	UE_API virtual AsyncTransformResultPtr	Exec(const TransformArgs& Args) override;
};

/**
 * Tiles To Combined Transform
 */
class T_CombineTiledBlob
{
public:

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	
	static UE_API TiledBlobPtr				Create(MixUpdateCyclePtr Cycle, BufferDescriptor DesiredOutputDesc, int32 TargetId, 
									TiledBlobPtr SourceTex, JobUPtr JobToUse = nullptr, const CombineSettings* InSettings = nullptr);
};

#undef UE_API
