// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_FinaliseBlob.h"
#include "Device/FX/Device_FX.h"
#include "Job/JobBatch.h"
#include "Profiling/RenderDoc/RenderDocManager.h" 
#include "Device/Mem/Device_Mem.h"
#include "Job/Scheduler.h"
#include "Data/TiledBlob.h"

//////////////////////////////////////////////////////////////////////////
class TEXTUREGRAPHENGINE_API Job_Finalise : public Job
{
private:
	JobPtr							OriginalJobObj;				/// The job that we wanna finalise in the end

public:
	Job_Finalise(UMixInterface*, int32 InTargetId, JobPtr InOriginalJobObj, UObject* InErrorOWner = nullptr, uint16 InPriority = (uint16)E_Priority::kNormal, uint64 InId = 0)
		: Job(InTargetId, 
			std::make_shared<Null_Transform>(InOriginalJobObj ? InOriginalJobObj->GetTargetDevice() : Device_Mem::Get(), FString::Printf(TEXT("T_Finalise [%s]"), *InOriginalJobObj->GetName()), true, false),
			InErrorOWner,
			InPriority)
		, OriginalJobObj(InOriginalJobObj)
	{
	}

protected:
	virtual AsyncPrepareResult		PrepareTargets(JobBatch* Batch) override { return cti::make_ready_continuable(0); };

	virtual void					GetDependencies(JobPtrVec&, JobPtrVec&, JobRunInfo InRunInfo) override { RunInfo = InRunInfo; }
	virtual int32					Exec() override { return 0; }

	virtual cti::continuable<int32> BeginNative(JobRunInfo InRunInfo) override 
	{
		RunInfo = InRunInfo;
		Stats.BeginNativeTime = Util::Time();
		return cti::make_ready_continuable(0);
	}

	virtual AsyncJobResultPtr		EndNative() override
	{
		MarkJobDone();
		return cti::make_ready_continuable(FinalJobResult);
	}

	virtual cti::continuable<int32>	ExecAsync(ENamedThreads::Type ExecThread, ENamedThreads::Type ReturnThread) override
	{
		check(!OriginalJobObj->IsCulled());
		return cti::make_ready_continuable(0);
	}

	virtual cti::continuable<int32>	PreExecAsync(ENamedThreads::Type ExecThread, ENamedThreads::Type ReturnThread) override
	{
		check(!OriginalJobObj->IsCulled());

		UE_LOGF(LogJob, VeryVerbose, "ExecAsync::%ls", *GetName());

		ThreadId = Util::GetCurrentThreadId();
		Stats.BeginRunTime = Util::Time();

		/// This can be in any thread
		BeginNative(RunInfo);

		/// EndNative
		return OriginalJobObj->EndNative()
			.then([this](JobResultPtr) mutable
			{
				TiledBlobPtr JobResult = OriginalJobObj->GetResult();

				if (JobResult)
				{
					if (!JobResult->IsFinalised())
						return JobResult->Finalise(true, nullptr);
				}

				return static_cast<AsyncBufferResultPtr>(cti::make_ready_continuable<BufferResultPtr>(std::make_shared<BufferResult>()));
			})
			.then([this](BufferResultPtr) mutable
			{
				BlobRef JobResult = OriginalJobObj->GetResultRef();

				if (JobResult)
				{
					OriginalJobObj->AddResultToBlobber();
					TransformResultPtr TResult = std::make_shared<TransformResult>();
					TResult->Target = JobResult;
				}

				EndNative();

				SetPromise(0);
				return 0;
			});
	}
};

//////////////////////////////////////////////////////////////////////////
JobPtr T_FinaliseBlob::Create(MixUpdateCyclePtr Cycle, JobPtr JobObj)
{
	return std::make_shared<Job_Finalise>(Cycle->GetMix(), - 1, JobObj);
}
