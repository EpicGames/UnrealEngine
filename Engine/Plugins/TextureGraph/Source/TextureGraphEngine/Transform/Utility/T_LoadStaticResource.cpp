// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_LoadStaticResource.h"
#include "Device/FX/Device_FX.h"
#include "Job/JobBatch.h"
#include "Profiling/RenderDoc/RenderDocManager.h" 
#include "Device/Mem/Device_Mem.h"
#include "Job/Scheduler.h"
#include "Model/StaticImageResource.h"

Job_LoadStaticImageResource::Job_LoadStaticImageResource(UMixInterface* InMix, UStaticImageResource* InSource,
	int32 TargetId, UObject* InErrorOwner /*= nullptr*/, uint16 Priority /*= (uint16)E_Priority::kHigh*/)
	: Job(InMix, TargetId, std::make_shared<Null_Transform>(Device_Mem::Get(), TEXT("T_LoadStaticImageResource"), true, false), InErrorOwner, Priority)
	, Source(InSource)
{
	check(InSource);
	Name = FString::Printf(TEXT("T_LoadStaticImageResource_%s"), *InSource->GetAssetUUID());
}

FString Job_LoadStaticImageResource::GetReferencerName() const
{
	if (Source)
		return Source->GetAssetUUID();
	return TEXT("Job_LoadStaticImageResource");
}

void Job_LoadStaticImageResource::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (Source)
		Collector.AddReferencedObject(Source);
}

cti::continuable<int32> Job_LoadStaticImageResource::PreExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type returnThread)
{
	auto JobHash = Job::Hash();
	BlobRef CachedResult = TextureGraphEngine::GetBlobber()->Find(JobHash->Value());

	if (CachedResult)
	{
		UE_LOGF(LogData, Log, "[ChannelSource - %ls] Found cached Result!", *Source->GetAssetUUID());

		check(Source->BlobObj == Result.get());
		Source->BlobObj = std::static_pointer_cast<TiledBlob>(CachedResult.get());
		Result->CopyResolveLateBound(Source->BlobObj);
		TileInvalidationMatrix.Resize(Result->Rows(), Result->Cols()); // Populate the tileInvalidation according to the Cached Result.
		TileInvalidationMatrix.ForEach([this](size_t tx, size_t ty) { TileInvalidationMatrix[tx][ty] = false; });

		bIsCulled = true;
		bIsDone = true;
		MarkJobDone();

		return cti::make_ready_continuable(0);
	}

	return cti::make_ready_continuable(0);
}

cti::continuable<int32> Job_LoadStaticImageResource::ExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type returnThread)
{
	UE_LOGF(LogData, Log, "Loading ChannelSource: %ls [UUID: %ls]", *Source->GetAssetUUID(), *Source->GetAssetUUID());

	if (IsDone())
	{
		MarkJobDone();
		return cti::make_ready_continuable(0);
	}

	return Job::BeginNative(RunInfo)
		.then([this]()
		{
			UE_LOGF(LogData, Log, "[ChannelSource - %ls] Calling source load ...", *Source->GetAssetUUID());
			return Source->Load(RunInfo.Cycle);
		})
		.then([this](TiledBlobRef LoadedSource) mutable
		{
			UE_LOGF(LogData, Log, "[ChannelSource - %ls] Source loaded. Setting up the asset!", *Source->GetAssetUUID());
			
			Source->BlobObj = LoadedSource.get();
			Result->CopyResolveLateBound(Source->BlobObj);

			/// Add the Result to blobber
			AddResultToBlobber();

			/// We add this to the mipmap service and generate mipmaps for it. This is useful to
			/// switch the LOD of the entire system down once we have the user tweaking and
			/// requiring more feedback from the system
#if 0 /// TODO
			const auto MipMapService = TextureGraphEngine::Scheduler()->MipMap_Service();

			if (!MipMapService.expired())
				MipMapService.lock()->Add(blob);
#endif

			/// This should have resolved by now
			check(Result->GetDescriptor().Width > 0 && Result->GetDescriptor().Height > 0);

			TileInvalidationMatrix.Resize(Result->Rows(), Result->Cols());
			TileInvalidationMatrix.ForEach([this](size_t tx, size_t ty) { TileInvalidationMatrix[tx][ty] = true; });

			// Record the loaded blob under the job JobHash so it is cached
			auto JobHash = Job::Hash();
			TextureGraphEngine::GetBlobber()->UpdateBlobHash(JobHash->Value(), BlobRef(std::static_pointer_cast<Blob>(Source->BlobObj)));

			EndNative();
			SetPromise(0);

			return 0;
		})
		.fail([this](std::exception_ptr e) mutable
		{
			UE_LOGF(LogData, Log, "[ChannelSource - %ls] Promise failure!", *Source->GetAssetUUID());
			EndNative();
			return -1;
		});
}