// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Job/Job.h"
#include "Model/Mix/MixUpdateCycle.h"

#define UE_API TEXTUREGRAPHENGINE_API

class UStaticImageResource;

class Job_LoadStaticImageResource : public Job, public FGCObject
{
	TObjectPtr<UStaticImageResource> Source;			/// The source for the static image resource

public:
									UE_API Job_LoadStaticImageResource(UMixInterface* InMix, UStaticImageResource* InSource, int32 TargetId, UObject* InErrorOwner = nullptr, uint16 Priority = (uint16)E_Priority::kHigh);
	virtual							~Job_LoadStaticImageResource() override = default;

	//////////////////////////////////////////////////////////////////////////
	/// FGCObject
	//////////////////////////////////////////////////////////////////////////
	UE_API virtual void					AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual FString					GetReferencerName() const override;

protected:
	virtual void					GetDependencies(JobPtrVec& prior, JobPtrVec& after, JobRunInfo runInfo) override { RunInfo = runInfo; Prev.clear(); }
	virtual AsyncPrepareResult		PrepareTargets(JobBatch* batch) override { return cti::make_ready_continuable(0); }
	virtual AsyncPrepareResult		PrepareResources(JobBatch* batch) override { return cti::make_ready_continuable(0); }

	virtual int32					Exec() override { return 0; }
	virtual cti::continuable<int32>	BindArgs_All(JobRunInfo runInfo) override { return cti::make_ready_continuable(0); }
	virtual cti::continuable<int32>	UnbindArgs_All(JobRunInfo runInfo) override { return cti::make_ready_continuable(0); }
	virtual bool					CanHandleTiles() const override { return false; }

	UE_API virtual cti::continuable<int32>	PreExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type returnThread) override;
	UE_API virtual cti::continuable<int32>	ExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type returnThread) override;
	virtual bool					IsAsync() const override { return true; }
};

//////////////////////////////////////////////////////////////////////////
/// File based
////////////////////////////////////////////////////////////////////////// 

#undef UE_API
