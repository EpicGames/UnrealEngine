// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_MinMax.h"
#include "TextureGraphEngine.h"
#include "Job/JobArgs.h"
#include "2D/TargetTextureSet.h"
#include "3D/RenderMesh.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "FxMat/RenderMaterial_FX_MinMax.h"
#include "Model/Mix/MixSettings.h"
#include "Model/Mix/MixInterface.h"

IMPLEMENT_GLOBAL_SHADER(FSH_MinMaxDownsample, "/Plugin/TextureGraph/Utils/MinMaxDownsample.usf", "FSH_MinMaxDownsample", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(CSH_MinMaxDownsample, "/Plugin/TextureGraph/Utils/MinMaxDownsample_comp.usf", "CSH_MinMaxDownsample", SF_Compute);

template <>
void SetupDefaultParameters(FSH_MinMaxDownsample::FParameters& params) {
	FStandardSamplerStates_Setup(params.SamplerStates);
}


T_MinMax::T_MinMax()
{
}

T_MinMax::~T_MinMax()
{
}

template <typename VSH_Type, typename FSH_Type>
RenderMaterial_FX_MinMaxPtr	CreateRenderMaterial_FX_MinMaxPtr(FString name, const typename FSH_Type::FPermutationDomain& fshPermutationDomain, const typename FSH_Type::FPermutationDomain& secondPassFshPermutationDomain)
{
	std::shared_ptr<FxMaterial_Normal<VSH_Type, FSH_Type>> Material = std::make_shared<FxMaterial_Normal<VSH_Type, FSH_Type>>(typename VSH_Type::FPermutationDomain(), fshPermutationDomain);
	std::shared_ptr<FxMaterial_Normal<VSH_Type, FSH_Type>> SecondPassMaterial = std::make_shared<FxMaterial_Normal<VSH_Type, FSH_Type>>(typename VSH_Type::FPermutationDomain(), secondPassFshPermutationDomain);
	return std::make_shared<RenderMaterial_FX_MinMax>(name, std::static_pointer_cast<FxMaterial>(Material), std::static_pointer_cast<FxMaterial>(SecondPassMaterial));
}

TiledBlobPtr CreateMinMaxDownsample(MixUpdateCyclePtr Cycle, TiledBlobPtr SourceTex, int32 TargetId, bool bUseMeshUVMask)
{
	FString Name = FString::Printf(TEXT("[%s].[%d].[%llu] MinMaxDownsample"), *SourceTex->DisplayName(), TargetId, Cycle->GetBatch()->GetFrameId());		

	FSH_MinMaxDownsample::FPermutationDomain PermutationVector;
	PermutationVector.Set<MinMax::FVar_UVMask>(bUseMeshUVMask);
	PermutationVector.Set<MinMax::FVar_FirstPass>(true);	
	
	FSH_MinMaxDownsample::FPermutationDomain secondPassPermutationVector;
	secondPassPermutationVector.Set<MinMax::FVar_UVMask>(false);	
	secondPassPermutationVector.Set<MinMax::FVar_FirstPass>(false);	

	TargetTextureSetPtr& Target = Cycle->GetMix()->GetSettings()->Target(TargetId);
	check(Target);

	TiledBlobPtr MeshUVMask;
	if (bUseMeshUVMask)
	{
		RenderMeshPtr Mesh = Target->GetMesh();
		check(Mesh);
		MeshUVMask = Target->GetMesh()->WorldUVMaskTexture(Cycle, TargetId);
	}

	RenderMaterial_FX_MinMaxPtr Material = CreateRenderMaterial_FX_MinMaxPtr<VSH_Simple, FSH_MinMaxDownsample>(Name, PermutationVector, secondPassPermutationVector);
	Material->SetDescriptor(SourceTex->GetDescriptor());

	JobUPtr JobObj = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(Material));
	JobObj
		->AddArg(ARG_BLOB(SourceTex, "SourceTexture"))		
		->AddArg(ARG_FLOAT(1 / (float)SourceTex->GetWidth(), "DX"))
		->AddArg(ARG_FLOAT(1 / (float)SourceTex->GetHeight(), "DY"));

	if (bUseMeshUVMask)
	{
		JobObj->AddArg(ARG_BLOB(MeshUVMask, "WorldUVMask"));	
	}

	BufferDescriptor Desc;
	Desc.Width = SourceTex->Cols();
	Desc.Height = SourceTex->Rows();
	Desc.Format = BufferFormat::Float;
	Desc.ItemsPerPoint = 2;
	
	TiledBlobPtr result = JobObj->InitResult(Name, &Desc);
	result->MakeSingleBlob();

	JobObj->SetTiled(false);


	Cycle->AddJob(TargetId, std::move(JobObj));
	
	return result;
}

TiledBlobPtr CreateMinMaxDownsampleCompute(MixUpdateCyclePtr Cycle, const FString& srcName, TiledBlobPtr sourceTex, int32 TargetId, bool SourceToMinMaxPass = false)
{
	FString Name = FString::Printf(TEXT("[%s].[%d].[%llu] MinMaxDownsample"), *srcName, TargetId, Cycle->GetBatch()->GetFrameId());

	CSH_MinMaxDownsample::FPermutationDomain PermutationVector;
	PermutationVector.Set<MinMax::FVar_SourceToMinMaxPass>(SourceToMinMaxPass);


	int32 DstWidth = FMath::Max((int32)sourceTex->GetWidth() >> 1, 1);
	int32 DstHeight = FMath::Max((int32)sourceTex->GetHeight() >> 1, 1);

	int32 SrcNumCols = (int32)sourceTex->GetTiles().Cols();
	int32 SrcNumRows = (int32)sourceTex->GetTiles().Rows();

	int32 DstNumCols = FMath::Max(SrcNumCols >> 1, 1);
	int32 DstNumRows = FMath::Max(SrcNumRows >> 1, 1);

	FIntVector4 SrcDimensions(DstWidth / DstNumCols, DstHeight / DstNumRows, SrcNumCols, SrcNumRows);

	auto SrcTiles = std::make_shared<JobArg_Blob>(sourceTex, "SourceTiles");
	SrcTiles->WithDownsampled4To1();

	FString JobName = (SourceToMinMaxPass ? TEXT("T_SourceToMinMaxDownsample"): TEXT("T_MinMaxDownsample"));

	RenderMaterial_FXPtr Transform = TextureGraphEngine::GetMaterialManager()->CreateMaterial_CmpFX<CSH_MinMaxDownsample>(
		JobName, TEXT("Result"), PermutationVector, SrcDimensions.X, SrcDimensions.Y, 1);

	JobUPtr JobObj = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(Transform));	
	JobObj
		->AddArg(std::make_shared<JobArg_SimpleType<FIntVector4>>(SrcDimensions, "TilingDimensions"))
		->AddArg(SrcTiles);

	BufferDescriptor Desc;
	Desc.Width = DstWidth;
	Desc.Height = DstHeight;
	Desc.Format = BufferFormat::Float;
	Desc.ItemsPerPoint = 2;
	Desc.AllowUAV();

	TiledBlobPtr Result = JobObj->InitResult(Name, &Desc);
	if (DstWidth == 1 && DstHeight == 1)
	{
		Result->MakeSingleBlob();
		JobObj->SetTiled(true);
	}

	Cycle->AddJob(TargetId, std::move(JobObj));

	return Result;
}

TiledBlobPtr CreateMinMaxDownsampleComputeChain(MixUpdateCyclePtr Cycle, TiledBlobPtr SourceTex, int32 TargetId)
{
	FString SrcName = SourceTex->DisplayName();
	TiledBlobPtr Result = CreateMinMaxDownsampleCompute(Cycle, SrcName, SourceTex, TargetId, true);
	while (Result->GetWidth() > 1 || Result->GetHeight() > 1)
	{
		Result = CreateMinMaxDownsampleCompute(Cycle, SrcName, Result, TargetId);
	}

	return Result;
}

TiledBlobPtr T_MinMax::Create(MixUpdateCyclePtr Cycle, TiledBlobPtr SourceTex, int32 TargetId, bool bUseMeshUVMask /* = false */)
{
	TiledBlobPtr Result;

	if (bUseMeshUVMask)
	{
		Result = CreateMinMaxDownsample(Cycle, SourceTex, TargetId, bUseMeshUVMask);
	}
	else
	{
		Result = CreateMinMaxDownsampleComputeChain(Cycle, SourceTex, TargetId);
	}

	SourceTex->SetMinMax(Result);

	return Result;
}
