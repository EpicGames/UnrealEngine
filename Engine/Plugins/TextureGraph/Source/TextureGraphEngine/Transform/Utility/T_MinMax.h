// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ShaderCompilerCore.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API


namespace MinMax
{
	// Declare one Shader Permutation Var class per parameters
	class FVar_UVMask : SHADER_PERMUTATION_BOOL("CHECK_UVMASK");
	class FVar_FirstPass : SHADER_PERMUTATION_BOOL("FIRST_PASS");
	class FVar_SourceToMinMaxPass : SHADER_PERMUTATION_BOOL("SOURCE_TO_MIN_MAX_PASS");
}
//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////
class FSH_MinMaxDownsample : public FSH_Base
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FSH_MinMaxDownsample, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FSH_MinMaxDownsample, FSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FStandardSamplerStates, SamplerStates)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, WorldUVMask)
		SHADER_PARAMETER(float, DX)
		SHADER_PARAMETER(float, DY)
	END_SHADER_PARAMETER_STRUCT()

	// Permutation domain local 
	using FPermutationDomain = TShaderPermutationDomain<MinMax::FVar_UVMask, MinMax::FVar_FirstPass>;

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);		
	}
};
template <> void SetupDefaultParameters(FSH_MinMaxDownsample::FParameters& params);

typedef FxMaterial_Normal<VSH_Simple, FSH_MinMaxDownsample>	Fx_MinMax;


class CSH_MinMaxDownsample : public CmpSH_Base<1, 1, 1>
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(CSH_MinMaxDownsample, UE_API);
	SHADER_USE_PARAMETER_STRUCT(CSH_MinMaxDownsample, CmpSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector4, TilingDimensions)
		SHADER_PARAMETER_TEXTURE_ARRAY(Texture2D, SourceTiles, [4])
		SHADER_PARAMETER_UAV(RWTexture2D<float2>, Result)
	END_SHADER_PARAMETER_STRUCT()

	// Permutation domain local 
	using FPermutationDomain = TShaderPermutationDomain<MinMax::FVar_SourceToMinMaxPass>;

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData)
			&& IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

/**
 * 
 */
class T_MinMax
{
public:
	UE_API T_MinMax();
	UE_API ~T_MinMax();

	//////////////////////////////////////////////////////////////////////////
/// Static functions
//////////////////////////////////////////////////////////////////////////=
	static UE_API TiledBlobPtr				Create(MixUpdateCyclePtr Cycle, TiledBlobPtr SourceTex, int32 TargetId, bool bUseMeshUVMask = false);
};

#undef UE_API
