// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_MipMap.h"
#include "TextureGraphEngine.h"
#include "Job/JobArgs.h"
#include "3D/RenderMesh.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"

IMPLEMENT_GLOBAL_SHADER(CSH_MipMapDownsample, "/Plugin/TextureGraph/Utils/MipMapDownsample_comp.usf", "CSH_MipMapDownsample", SF_Compute);

T_MipMap::T_MipMap()
{
}

T_MipMap::~T_MipMap()
{
}

TiledBlobPtr CreateMipMapDownsampleCompute(MixUpdateCyclePtr cycle, const FString& SrcName, TiledBlobPtr SourceTex, int32 TargetId, TiledBlobPtr OrgSource, int32 MipLevel)
{
	FString Name = FString::Printf(TEXT("[%s].[%d].[%llu] MipMapDownsample"), *SrcName, TargetId, cycle->GetBatch()->GetFrameId());

	CSH_MipMapDownsample::FPermutationDomain PermutationVector;

	auto componentFormat = SourceTex->GetDescriptor().Format;
	auto numComponents = SourceTex->GetDescriptor().ItemsPerPoint;

	int32 SrcWidth = SourceTex->GetWidth();
	int32 SrcHeight = SourceTex->GetHeight();

	int32 DstWidth = FMath::Max(SrcWidth >> 1, 1);
	int32 DstHeight = FMath::Max(SrcHeight >> 1, 1);

	int32 SrcNumCols = (int32)SourceTex->GetTiles().Cols();
	int32 SrcNumRows = (int32)SourceTex->GetTiles().Rows();

	int32 DstNumCols = SrcNumCols;
	int32 DstNumRows = SrcNumRows;

	if (SrcWidth == 1 && SrcHeight == 1 && SrcNumCols > 1 && SrcNumRows > 1)
	{
		// this is when the src tiles are at their deepest level
		// we cannot reduce them any further individually
		// we need a different transform for this case @TODO
		return SourceTex;
	}

	FIntVector4 SrcDimensions(DstWidth / DstNumCols, DstHeight / DstNumRows, SrcNumCols, SrcNumRows);

	auto SourceTiles = std::make_shared<JobArg_Blob>(SourceTex, "SourceTiles");

	/// Don't change thes
	FString MaterialName = Blob::LODTransformName;

	RenderMaterial_FXPtr transform = TextureGraphEngine::GetMaterialManager()->CreateMaterial_CmpFX<CSH_MipMapDownsample>(
		MaterialName, TEXT("Result"), PermutationVector, SrcDimensions.X, SrcDimensions.Y, 1);

	JobUPtr JobObj = std::make_unique<Job>(cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(transform));
	JobObj
		->AddArg(WithIgnoreHash(SourceTiles))				/// We don't want this to be used in hash calculation
		->AddArg(WithUnbounded(ARG_BLOB(OrgSource, "")))	/// [Unbounded] We want this to be used in hash calculation ONLY
		->AddArg(WithUnbounded(ARG_FLOAT(DstWidth, "")))	/// [Unbounded] We want this to be used in hash calculation ONLY
		->AddArg(WithUnbounded(ARG_FLOAT(DstHeight, "")))	/// [Unbounded] We want this to be used in hash calculation ONLY
		;

	BufferDescriptor Desc;
	Desc.Width = DstWidth;
	Desc.Height = DstHeight;
	Desc.Format = componentFormat;
	Desc.ItemsPerPoint = numComponents;
	Desc.Name = FString::Printf(TEXT("%s - L%d"), *OrgSource->Name(), MipLevel);
	Desc.AllowUAV();

	TiledBlobPtr Result = JobObj->InitResult(Name, &Desc, DstNumCols, DstNumRows);
	if (DstWidth == 1 && DstHeight == 1)
	{
		Result->MakeSingleBlob();
		JobObj->SetTiled(true);
	}

	cycle->AddJob(TargetId, std::move(JobObj));

	return Result;
}

TiledBlobPtr CreateMipMapDownsampleComputeChain(MixUpdateCyclePtr Cycle, TiledBlobPtr SourceTex, int32 TargetId, int32 MipLevel)
{
	// if we want level 0 then no-op
	if (MipLevel == 0)
		return SourceTex;

	/// else evaluate how many mips available, we do a -1 here because we don't need to include level-0
	/// which is SourceTex itself
	int32 numMips = TextureHelper::CalcNumMipLevels(std::max(SourceTex->GetWidth(), SourceTex->GetHeight())) - 1;

	if (MipLevel < 0)
		MipLevel = numMips;

	int32 numPasses = std::min(numMips, MipLevel);

	FString srcName = SourceTex->DisplayName();

	TiledBlobPtr Parent;
	TiledBlobPtr Result = SourceTex;

	for (int32 mipNum = 0; mipNum < numPasses; ++mipNum)
	{
		Parent = Result;
		Result = CreateMipMapDownsampleCompute(Cycle, srcName, Parent, TargetId, SourceTex, mipNum + 1);
		SourceTex->SetLODLevel(mipNum + 1, Result, Result, SourceTex, !Cycle->NoCache());
	}

	return Result;
}

TiledBlobPtr T_MipMap::Create(MixUpdateCyclePtr Cycle, TiledBlobPtr SourceTex, int32 TargetId, int32 GenMipLevel)
{
	/// Cannot generate. Just return the source texture
	if (!TextureHelper::IsPowerOf2(SourceTex->GetWidth()) || !TextureHelper::IsPowerOf2(SourceTex->GetHeight()))
		return SourceTex;

	return CreateMipMapDownsampleComputeChain(Cycle, SourceTex, TargetId, GenMipLevel);
}
