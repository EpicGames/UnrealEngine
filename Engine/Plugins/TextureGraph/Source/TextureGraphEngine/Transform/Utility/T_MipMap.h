// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ShaderCompilerCore.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API


//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////

class CSH_MipMapDownsample : public CmpSH_Base<1, 1, 1>
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(CSH_MipMapDownsample, UE_API);
	SHADER_USE_PARAMETER_STRUCT(CSH_MipMapDownsample, CmpSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTiles)
		SHADER_PARAMETER_UAV(RWTexture2D<float4>, Result)
	END_SHADER_PARAMETER_STRUCT()


public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) {
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && 
			IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment) {
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

/**
 * MipMap Transform
 */
class T_MipMap
{
public:
	UE_API T_MipMap();
	UE_API ~T_MipMap();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////

	// Generate the mip map raster a the specified level
	// Source must be power of 2 for this to work correctly
	// THe layout in tile grid is matching the source tile grid
	// thus if the required level is higher than the actual deepest tile can produce ( aka log2(size of the tile) )
	// then the transform is returning the deepest level possibly generated with the same tile grid as sourceTex.
	// @TODO SG: Fix that behavior!
	static UE_API TiledBlobPtr				Create(MixUpdateCyclePtr Cycle, TiledBlobPtr SourceTex, int32 TargetId, int32 GenMipLevel = -1);
};

#undef UE_API
