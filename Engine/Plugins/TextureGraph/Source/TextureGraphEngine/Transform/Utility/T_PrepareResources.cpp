// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_PrepareResources.h"
#include "Device/FX/Device_FX.h"
#include "Job/JobBatch.h"
#include "Profiling/RenderDoc/RenderDocManager.h" 
#include "Device/Mem/Device_Mem.h"
#include "Job/Scheduler.h"

//////////////////////////////////////////////////////////////////////////
//////////////////////////////////////////////////////////////////////////
class TEXTUREGRAPHENGINE_API Job_Prepare : public Job
{
private:
	JobPtr							OriginalJobObj;				/// The job that we wanna finalise in the end

public:
	Job_Prepare(UMixInterface*, int32 InTargetId, JobPtr InJobObj, UObject* InErrorOWner = nullptr, uint16 InPriority = (uint16)E_Priority::kNormal, uint64 InId = 0)
		: Job(InTargetId, 
			std::make_shared<Null_Transform>(InJobObj ? InJobObj->GetTargetDevice() : Device_Mem::Get(), FString::Printf(TEXT("T_PrepareResources [%s]"), *InJobObj->GetName()), true, false),
			InErrorOWner,
			InPriority)
		, OriginalJobObj(InJobObj)
	{
	}

protected:
	virtual AsyncPrepareResult		PrepareTargets(JobBatch* batch) override { return cti::make_ready_continuable(0); };

	virtual void					GetDependencies(JobPtrVec& prior, JobPtrVec& after, JobRunInfo InRunInfo) override { RunInfo = InRunInfo; }
	virtual int32					Exec() override { return 0; }

	virtual cti::continuable<int32> BeginNative(JobRunInfo InRunInfo) override 
	{ 
		RunInfo = InRunInfo;
		Stats.BeginNativeTime = Util::Time();
		return cti::make_ready_continuable(0);
	}

	virtual AsyncJobResultPtr		EndNative() override 
	{ 
		MarkJobDone();
		return cti::make_ready_continuable(FinalJobResult);
	}

	virtual cti::continuable<int32>	PreExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type returnThread) override
	{
		check(IsInGameThread());

		BeginNative(RunInfo);

		JobRunInfo InRunInfo = OriginalJobObj->GetRunInfo();
		InRunInfo.Dev = OriginalJobObj->GetTransform()->TargetDevice(0);
		InRunInfo.ThisJob = OriginalJobObj;

		/// EndNative
		return OriginalJobObj->BeginNative(InRunInfo)
			.then([this](int32 JobResult) mutable 
			{
				EndNative();
				SetPromise(JobResult);
				return JobResult;
			})
			.fail([this]() mutable 
			{
				EndNative();
				SetPromise(-1);
				return -1;
			});
	}

	virtual void					PostExec() override
	{
		/// Do nothing here!
	}

	virtual cti::continuable<int32>	ExecAsync(ENamedThreads::Type execThread, ENamedThreads::Type returnThread) override
	{
		check(!OriginalJobObj->IsCulled());
		return cti::make_ready_continuable(0);
	}
};

//////////////////////////////////////////////////////////////////////////
JobPtr T_PrepareResources::Create(MixUpdateCyclePtr Cycle, JobPtr JobObj)
{
	return std::make_shared<Job_Prepare>(Cycle->GetMix(), - 1, JobObj);
}

