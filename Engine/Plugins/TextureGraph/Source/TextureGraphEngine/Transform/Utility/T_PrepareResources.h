// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Transform/BlobTransform.h"
#include "Job/Job.h"
#include "Model/Mix/MixUpdateCycle.h"

#define UE_API TEXTUREGRAPHENGINE_API

class T_PrepareResources 
{
public:
	////////////////////////////////////////////////////////////////////////// 
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	static UE_API JobPtr					Create(MixUpdateCyclePtr Cycle, JobPtr JobObj);
};

#undef UE_API
