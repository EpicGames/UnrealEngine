// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_RenderCapture.h"
#include "Device/FX/Device_FX.h"
#include "TextureGraphEngine.h"
#include "Job/JobBatch.h"
#include "Profiling/RenderDoc/RenderDocManager.h"


T_BeginRenderCapture::T_BeginRenderCapture() : BlobTransform(TEXT("T_BeginRenderCapture"))
{
}

T_BeginRenderCapture::~T_BeginRenderCapture()
{
}

Device* T_BeginRenderCapture::TargetDevice(size_t index) const
{
	return Device_FX::Get();
}

AsyncTransformResultPtr T_BeginRenderCapture::Exec(const TransformArgs& args)
{
	/// Device::Use should have ensured that we we're in the rendering thread
	/// by the time we get to this point!
	check(IsInRenderingThread());

	if (args.Cycle->GetBatch()->IsCaptureRenderDoc())
		TextureGraphEngine::GetRenderDocManager()->BeginCapture();
	/// We don't need a target, so it shouldn't be there. The job needs to 
/// handle this correctly
	check(args.Target.expired());
	check(args.Cycle);

	MixUpdateCyclePtr cycle = args.Cycle;
	cycle->GetTarget(args.TargetId)->InvalidateAllTiles();

	return cti::make_ready_continuable(std::make_shared<TransformResult>());
}

JobUPtr T_BeginRenderCapture::CreateJob(MixUpdateCyclePtr cycle, int32 targetId)
{
	BlobTransformPtr transform = std::static_pointer_cast<BlobTransform>(std::make_shared<T_BeginRenderCapture>());
	JobUPtr job = std::make_unique<Job>(cycle->GetMix(), targetId, transform);
	job->SetPriority((int32)E_Priority::kHighest, false);
	return job;
}



T_EndRenderCapture::T_EndRenderCapture() : BlobTransform(TEXT("T_EndRenderCapture"))
{
}

T_EndRenderCapture::~T_EndRenderCapture()
{
}

Device* T_EndRenderCapture::TargetDevice(size_t index) const
{
	return Device_FX::Get();
}

AsyncTransformResultPtr T_EndRenderCapture::Exec(const TransformArgs& args)
{
	/// Device::Use should have ensured that we we're in the rendering thread
	/// by the time we get to this point!
	check(IsInRenderingThread());

	if (args.Cycle->GetBatch() && args.Cycle->GetBatch()->IsCaptureRenderDoc())
		TextureGraphEngine::GetRenderDocManager()->EndCapture();

	return cti::make_ready_continuable(std::make_shared<TransformResult>());
}

JobUPtr T_EndRenderCapture::CreateJob(MixUpdateCyclePtr cycle, int32 targetId)
{
	BlobTransformPtr transform = std::static_pointer_cast<BlobTransform>(std::make_shared<T_EndRenderCapture>());
	JobUPtr job = std::make_unique<Job>(cycle->GetMix(), targetId, transform);
//	job->SetPriority((int32)E_Priority::kHighest, false);
	return job;
}

