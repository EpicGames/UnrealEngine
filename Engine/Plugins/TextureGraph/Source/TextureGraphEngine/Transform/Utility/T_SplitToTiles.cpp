// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_SplitToTiles.h"
#include "TextureGraphEngine.h"
#include "Job/JobArgs.h"
#include "3D/RenderMesh.h"
#include "FxMat/MaterialManager.h"
#include "Job/JobBatch.h"
#include "Model/Mix/MixInterface.h"

IMPLEMENT_GLOBAL_SHADER(CSH_SplitToTiles, "/Plugin/TextureGraph/Utils/SplitToTiles_comp.usf", "CSH_SplitToTiles", SF_Compute);

T_SplitToTiles::T_SplitToTiles()
{
}

T_SplitToTiles::~T_SplitToTiles()
{
}

TiledBlobPtr CreateSplitToTilesCompute(MixUpdateCyclePtr Cycle, TiledBlobPtr SourceTex, int32 TargetId)
{
	CSH_SplitToTiles::FPermutationDomain PermutationVector;

	FString Name = FString::Printf(TEXT("[%s]_SplitToTiles"), *SourceTex->Name());

	int32 DstNumCols = Cycle->GetMix()->GetNumXTiles();
	int32 DstNumRows = Cycle->GetMix()->GetNumYTiles();

	/// This is the correct check to see if we can actually produce the correct number of rows and columns
	FTileInfo TileInfo;

	std::shared_ptr<FxMaterial_SplitToTiles> Mat = std::make_shared<FxMaterial_SplitToTiles>(TEXT("Result"), DstNumCols, DstNumRows, SourceTex, &PermutationVector);
	RenderMaterial_FXPtr Transform = std::make_shared<RenderMaterial_FX>(Name, Mat);

	JobUPtr JobObj = std::make_unique<Job>(Cycle->GetMix(), TargetId, std::static_pointer_cast<BlobTransform>(Transform));
	JobObj
		->AddArg(ARG_BLOB(SourceTex, "SourceTexture"))
		->AddArg(ARG_TILEINFO(TileInfo, "TileInfo"))
		;

	BufferDescriptor Desc = SourceTex->GetDescriptor();
	Desc.Name = Name;
	Desc.AllowUAV();

	TiledBlobPtr Result = JobObj->InitResult(Name, &Desc);

	Cycle->AddJob(TargetId, std::move(JobObj));

	check(Desc.Width == SourceTex->GetWidth() && Desc.Height == SourceTex->GetHeight())

	return Result;
}

TiledBlobPtr T_SplitToTiles::Create(MixUpdateCyclePtr Cycle, int32 TargetId, TiledBlobPtr SourceTex)
{
	/// Cannot generate. Just return the source texture
	return CreateSplitToTilesCompute(Cycle, SourceTex, TargetId);
}
