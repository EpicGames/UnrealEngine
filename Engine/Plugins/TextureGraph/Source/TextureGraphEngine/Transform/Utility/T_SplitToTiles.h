// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ShaderCompilerCore.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API

//////////////////////////////////////////////////////////////////////////
/// Shader
//////////////////////////////////////////////////////////////////////////

class CSH_SplitToTiles : public CmpSH_Base<1, 1, 1>
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(CSH_SplitToTiles, UE_API);
	SHADER_USE_PARAMETER_STRUCT(CSH_SplitToTiles, CmpSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FTileInfo, TileInfo)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER_UAV(RWTexture2D<float4>, Result)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) {
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && 
			IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

class FxMaterial_SplitToTiles : public FxMaterial_Compute<CSH_SplitToTiles>
{
private:
	int32 NumTilesX;
	int32 NumTilesY;
	TiledBlobPtr RefBlob;

public:
	explicit FxMaterial_SplitToTiles(FString InOutputId, int32 InNumTilesX, int32 InNumTilesY, TiledBlobPtr InRefBlob, 
		const CmpSHPermutationDomain* InPermDomain = nullptr, FUnorderedAccessViewRHIRef InUnorderedAccessView = nullptr) 
		: FxMaterial_Compute<CSH_SplitToTiles>(InOutputId, InPermDomain, 0, 0, 1, InUnorderedAccessView)
		, NumTilesX(InNumTilesX)
		, NumTilesY(InNumTilesY)
		, RefBlob(InRefBlob)
	{
	}

	virtual void Blit(FRHICommandListImmediate& RHI, FRHITexture* Target, const RenderMesh* MeshObj, int32 TargetId, FGraphicsPipelineStateInitializer* PSO = nullptr) override
	{
		if (NumThreadsX == 0 || NumThreadsY == 0)
		{
			NumThreadsX = FMath::Max((int32)RefBlob->GetWidth() / NumTilesX, 1);
			NumThreadsY = FMath::Max((int32)RefBlob->GetHeight() / NumTilesY, 1);
		}

		check(NumThreadsX > 0 && NumThreadsY > 0);

		return FxMaterial_Compute<CSH_SplitToTiles>::Blit(RHI, Target, MeshObj, TargetId, PSO);
	}

	virtual std::shared_ptr<FxMaterial> Clone() override
	{
		std::shared_ptr<FxMaterial_SplitToTiles> Mat = std::make_shared<FxMaterial_SplitToTiles>(OutputId, NumTilesX, NumTilesY, RefBlob, &PermutationDomain, UnorderedAccessView);
		return std::static_pointer_cast<FxMaterial>(Mat);
	}
};

/**
 * SplitToTiles Transform
 */
class T_SplitToTiles
{
public:
	UE_API T_SplitToTiles();
	UE_API ~T_SplitToTiles();

	//////////////////////////////////////////////////////////////////////////
	/// Static functions
	//////////////////////////////////////////////////////////////////////////
	
	static UE_API TiledBlobPtr				Create(MixUpdateCyclePtr Cycle, int32 TargetId, TiledBlobPtr SourceTex);
};

#undef UE_API
