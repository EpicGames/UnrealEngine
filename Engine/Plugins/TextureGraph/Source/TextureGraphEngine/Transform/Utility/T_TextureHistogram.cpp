// Copyright Epic Games, Inc. All Rights Reserved.
#include "T_TextureHistogram.h"

#include "TextureGraphEngine.h"
#include "Job/JobArgs.h"
#include "TextureGraphEngine.h"
#include "Job/HistogramService.h"
#include "Job/JobBatch.h"
#include "Job/Scheduler.h"
#include "Helper/MathUtils.h"
#include "Helper/GraphicsUtil.h"
#include "Device/FX/DeviceBuffer_FX.h"
#include "Device/FX/Device_FX.h"

#include "Engine/TextureRenderTarget2D.h"
#include "Engine/Texture2D.h"
#include "Device/DeviceManager.h"
#include "Device/FX/Device_FX.h"
#include "Device/Mem/Device_Mem.h"
#include "Data/Blobber.h"
#include "2D/Tex.h"
#include "TextureResource.h"
#include "FxMat/MaterialManager.h"
#include "T_MinMax.h"


static constexpr int NumBins = 256;

class CSH_HistogramPerTile : public CmpSH_Base<16, 16, 1>
{
public:
	DECLARE_GLOBAL_SHADER(CSH_HistogramPerTile);
	SHADER_USE_PARAMETER_STRUCT(CSH_HistogramPerTile, CmpSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_UAV(RWTexture2D<float4>, Result)   // Unused but still there for compatibility
		SHADER_PARAMETER_UAV(RWStructuredBuffer<uint4>, TilesHistogramUAV)
		SHADER_PARAMETER_TEXTURE(Texture2D, SourceTiles)
		SHADER_PARAMETER(FIntVector4, InvocationDim)
		END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData)
			&& IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

class CSH_Histogram : public CmpSH_Base<256, 1, 1>
{
public:
	DECLARE_GLOBAL_SHADER(CSH_Histogram);
	SHADER_USE_PARAMETER_STRUCT(CSH_Histogram, CmpSH_Base);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_UAV(RWTexture2D<float4>, Result)
		SHADER_PARAMETER_SRV(StructuredBuffer<uint4>, TilesHistogramSRV)
		SHADER_PARAMETER(FIntVector4, InvocationDim)
		END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsPCPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData)
			&& IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};


IMPLEMENT_GLOBAL_SHADER(CSH_HistogramPerTile, "/Plugin/TextureGraph/Utils/Histogram_comp.usf", "CSH_HistogramPerTile", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(CSH_Histogram, "/Plugin/TextureGraph/Utils/Histogram_comp.usf", "CSH_Histogram", SF_Compute);



/////////////////////////////////////////////////////////////////////////

//template <typename CmpSH_Type>
class FxMaterial_Histogram : public FxMaterial_Compute<CSH_HistogramPerTile>
{
	FRWBufferStructured TilesHistogramBuffer;
	FTextureRWBuffer TilesHistogramTexture;
public:
	// type Name for the permutation domain
	using CmpSHPermutationDomain = typename CSH_HistogramPerTile::FPermutationDomain;

	FxMaterial_Histogram(FString outputId, const CmpSHPermutationDomain* permDomain = nullptr,
		int numThreadsX = FxMaterial_Compute<CSH_HistogramPerTile>::GDefaultNumThreadsXY, int numThreadsY = FxMaterial_Compute<CSH_HistogramPerTile>::GDefaultNumThreadsXY, int numThreadsZ = 1,
		FUnorderedAccessViewRHIRef unorderedAccessView = nullptr)
		: FxMaterial_Compute<CSH_HistogramPerTile>(outputId, permDomain, numThreadsX, numThreadsY, numThreadsZ, unorderedAccessView)
	{

	}

	virtual void Blit(FRHICommandListImmediate& RHI, FRHITexture* Target, const RenderMesh* MeshObj, int32 TargetId, FGraphicsPipelineStateInitializer* PSO = nullptr) override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_ShaderPlugin_ComputeShader); // Used to gather CPU profiling data for the UE4 session frontend
		SCOPED_DRAW_EVENT(RHI, ShaderPlugin_Compute); // Used to profile GPU activity and add metadata to be consumed by for example RenderDoc

		CSH_HistogramPerTile::FParameters& PerTileParams = Params;
		CSH_Histogram::FParameters FinalParams;

		// Collect the Tile textures contributing to the whole Source texture
		TArray<FRHITexture*> TextureTiles;
		int32 TilesCount = 0;
		if (Textures.size() == 1 && Textures[0].tiles.size())
		{
			TilesCount = Textures[0].tiles.size();
			TextureTiles.Reserve(TilesCount);

			int i = 0;
			for (const auto& ti : Textures[0].tiles)
			{
				FRHITexture* TextureRHI = ti->GetResource()->TextureRHI;
				TextureTiles.Add(TextureRHI);
				i++;
			}
		}

		if (TilesCount == 0)
		{
			TilesCount = 1;
		}

		// Allocate and bind the buffer used by the shader to store intermediate results of each pass / tile 
		{
			TilesHistogramBuffer.Initialize(RHI, TEXT("HistogramBuffer"), sizeof(uint32) * 4, NumBins * TilesCount);
			RHI.ClearUAVUint(TilesHistogramBuffer.UAV, FUintVector4(0, 0, 0, 0));

			PerTileParams.TilesHistogramUAV = TilesHistogramBuffer.UAV;
			FinalParams.TilesHistogramSRV = TilesHistogramBuffer.SRV;
		}

		// Allocate and set the result UAV only used in the final pass / param 
		if (!OutputId.IsEmpty())
		{
			FUnorderedAccessViewRHIRef RenderTargetUAV = UnorderedAccessView;

			if (!RenderTargetUAV)
			{
				RenderTargetUAV = RHI.CreateUnorderedAccessView(Target, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(Target));
				RHI.Transition(FRHITransitionInfo(Target, ERHIAccess::Unknown, ERHIAccess::UAVMask));
			}

			FinalParams.Result = RenderTargetUAV;
		}

		// Same as the standard FxMaterial_Compute<> blit but adjusted so we do as many  passes as tiles
		{
			TShaderMapRef<CSH_HistogramPerTile> computeShaderPerTile(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationDomain);
			FIntVector groupSize = computeShaderPerTile->ThreadGroupSize();
			
			PerTileParams.InvocationDim.X = TextureTiles.Num();
			PerTileParams.InvocationDim.Y = 0;
			PerTileParams.InvocationDim.Z = 0;
			PerTileParams.InvocationDim.W = 0;

			int NumTiles = TextureTiles.Num();
			for (int i = 0; i < NumTiles; ++i)
			{
				// bind the pass's tile and increment the tile index for each pass
				PerTileParams.SourceTiles = TextureTiles[i];
				PerTileParams.InvocationDim.Y = i; 
				PerTileParams.InvocationDim.W = (i + 1 == TextureTiles.Num());

				FIntPoint TileDim = TextureTiles[i]->GetSizeXY();

				FComputeShaderUtils::Dispatch(RHI, computeShaderPerTile, PerTileParams,
					FIntVector(
						FMath::DivideAndRoundUp(TileDim.X, groupSize.X),
						FMath::DivideAndRoundUp(TileDim.Y, groupSize.Y),
						FMath::DivideAndRoundUp(1, groupSize.Z)
					)
				);

				RHI.Transition(FRHITransitionInfo(TilesHistogramBuffer.UAV, ERHIAccess::UAVMask, ERHIAccess::UAVMask));
			}

			// All the per tiles histograms have been collected
			// transition to srv read
			RHI.Transition(FRHITransitionInfo(TilesHistogramBuffer.UAV, ERHIAccess::UAVMask, ERHIAccess::SRVMask));

			// Last pass
			TShaderMapRef<CSH_Histogram> computeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), PermutationDomain);
			groupSize = computeShader->ThreadGroupSize();

			FinalParams.InvocationDim.X = TextureTiles.Num();
			FinalParams.InvocationDim.Y = 0;
			FinalParams.InvocationDim.Z = 0;
			FinalParams.InvocationDim.W = 0;

			// Dispatch ONE group 256 threads
			FComputeShaderUtils::Dispatch(RHI, computeShader, FinalParams, FIntVector(1, 1, 1));


		}
		// UAV target has been rendered, transition to the default SRV state for read
		RHI.Transition(FRHITransitionInfo(Target, ERHIAccess::UAVMask, ERHIAccess::SRVMask));
	}
};

T_TextureHistogram::T_TextureHistogram()
{
}

T_TextureHistogram::~T_TextureHistogram()
{
}

TiledBlobPtr T_TextureHistogram::CreateJobAndResult(JobUPtr& OutJob, MixUpdateCyclePtr InCycle, TiledBlobPtr SourceTexture, int32 InTargetId)
{
	check(!SourceTexture->HasHistogram())

	CSH_Histogram::FPermutationDomain PermutationVector;

	FIntVector4 SrcDimensions(SourceTexture->GetWidth(), SourceTexture->GetHeight(), 1, 1);

	FString Name = FString::Printf(TEXT("[%s].[%d] Histogram"), *SourceTexture->DisplayName(), InTargetId);

	// Regular RendermaterialFX using our custom FXMaterial for histogram
	std::shared_ptr<FxMaterial_Histogram> FxMat = std::make_shared<FxMaterial_Histogram>(TEXT("Result"), &PermutationVector, SrcDimensions.X, SrcDimensions.Y, 1);
	RenderMaterial_FXPtr Transform = std::make_shared<RenderMaterial_FX>(TEXT("T_Histogram"), std::static_pointer_cast<FxMaterial>(FxMat));
	
	OutJob = std::make_unique<Job>(InCycle->GetMix(), InTargetId, std::static_pointer_cast<BlobTransform>(Transform));
	auto SourceArg = ARG_BLOB(SourceTexture, "SourceTiles");
	SourceArg->WithArrayOfTiles(); // THe source tiles will be bound as an array of tiles and the FXMat will explicitely bind the correct tile for the various passes
	OutJob->AddArg(SourceArg);

	BufferDescriptor Desc;
	Desc.Width = NumBins;
	Desc.Height = 2;
	Desc.Format = BufferFormat::Float;// BufferFormat::Int;
	Desc.ItemsPerPoint = 4;
	Desc.Name = FString::Printf(TEXT("Histogram - %s"), *SourceTexture->Name());
	Desc.AllowUAV();

	OutJob->SetTiled(false); // Run single tile for the texture and in the transform::exec, unroll the loop for each tiles of the source

	TiledBlobPtr Result = OutJob->InitResult(Name, &Desc, 1, 1);
	Result->MakeSingleBlob();

	if (!SourceTexture->HasHistogram())
	{
		//setting it as the histogram of source so it is retained untill the life cycle of source blob.
		SourceTexture->SetHistogram(Result);
	}
	return Result;
}

TiledBlobPtr T_TextureHistogram::Create(MixUpdateCyclePtr Cycle, TiledBlobPtr SourceTex, int32 TargetId)
{
	if (SourceTex->HasHistogram())
	{
		TiledBlobPtr SourceHistogram = std::static_pointer_cast<TiledBlob>(SourceTex->GetHistogram());
		return SourceHistogram;
	}

	JobUPtr JobObj;
	TiledBlobPtr Result = CreateJobAndResult(JobObj, Cycle, SourceTex, TargetId);

	Cycle->AddJob(TargetId, std::move(JobObj));

	return Result;
}

TiledBlobPtr T_TextureHistogram::CreateOnService(UMixInterface* InMix, TiledBlobPtr SourceTex, int32 TargetId)
{
	if (SourceTex->HasHistogram())
	{
		TiledBlobPtr SourceHistogram = std::static_pointer_cast<TiledBlob>(SourceTex->GetHistogram());
		return SourceHistogram;
	}

	check(InMix);
	check(SourceTex);

	/// If the source texture turns out to be transient at this point, we just return a black histogram. 
	/// We don't want to calculate anything for transient buffers
	if (SourceTex->IsTransient())
		return TextureHelper::GetBlack();

	HistogramServicePtr Service = TextureGraphEngine::GetScheduler()->GetHistogramService().lock();
	check(Service);

	JobBatchPtr Batch = Service->GetOrCreateNewBatch(InMix);

	JobUPtr JobObj;
	TiledBlobPtr Result = CreateJobAndResult(JobObj, Batch->GetCycle(), SourceTex, TargetId);

	//Add the job using histogram idle service
	//TODO: get rid of mix here and use Null Mix instead
	Service->AddHistogramJob(Batch->GetCycle(), std::move(JobObj), TargetId, InMix);

	return Result;
}
