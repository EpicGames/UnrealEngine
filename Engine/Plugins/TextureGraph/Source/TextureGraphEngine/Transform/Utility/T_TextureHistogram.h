// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ShaderCompilerCore.h"
#include "Job/Job.h"
#include "FxMat/FxMaterial.h"
#include "Model/Mix/MixUpdateCycle.h"
#include <DataDrivenShaderPlatformInfo.h>

#define UE_API TEXTUREGRAPHENGINE_API


/**
 * T_TextureHistogram Transform
 */
class T_TextureHistogram
{
public:
	UE_API T_TextureHistogram();
	UE_API ~T_TextureHistogram();

	static UE_API TiledBlobPtr	Create(MixUpdateCyclePtr Cycle, TiledBlobPtr SourceTex, int32 TargetId);

	static UE_API TiledBlobPtr	CreateOnService(UMixInterface* InMix, TiledBlobPtr SourceTex, int32 TargetId);

private:
	static UE_API TiledBlobPtr			CreateJobAndResult(JobUPtr& OutJob, MixUpdateCyclePtr Cycle, TiledBlobPtr Histogram, int32 TargetId);
};

#undef UE_API
