// Copyright Epic Games, Inc. All Rights Reserved.
#include "Model/TextureGraphInsightObserver.h"

#include "TextureGraphInsight.h"
#include "Model/TextureGraphInsightSession.h"

#include <Device/DeviceManager.h>

//DEFINE_LOG_CATEGORY(LogTextureGraphInsight);


TextureGraphInsightDeviceObserver::TextureGraphInsightDeviceObserver()
{
}
TextureGraphInsightDeviceObserver::TextureGraphInsightDeviceObserver(DeviceType InDevType) : DevType(InDevType)
{
}
TextureGraphInsightDeviceObserver::~TextureGraphInsightDeviceObserver()
{
}
void TextureGraphInsightDeviceObserver::DeviceBuffersUpdated(HashNDescArray&& AddedBuffers, HashArray&& RemovedBuffers)
{
	TextureGraphInsight::Instance()->GetSession()->DeviceBuffersUpdated(DevType, std::move(AddedBuffers), std::move(RemovedBuffers));

}

TextureGraphInsightBlobberObserver::TextureGraphInsightBlobberObserver()
{
}
TextureGraphInsightBlobberObserver::~TextureGraphInsightBlobberObserver()
{
}
void TextureGraphInsightBlobberObserver::BlobberUpdated(HashArray&& AddedHashes, HashArray&& RemappedHashes)
{
	TextureGraphInsight::Instance()->GetSession()->BlobberHashesUpdated(std::move(AddedHashes), std::move(RemappedHashes));
}


TextureGraphInsightSchedulerObserver::TextureGraphInsightSchedulerObserver()
{
}
TextureGraphInsightSchedulerObserver::~TextureGraphInsightSchedulerObserver()
{
}

void TextureGraphInsightSchedulerObserver::Start()
{
	UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightSchedulerObserver::Start");
}

void TextureGraphInsightSchedulerObserver::UpdateIdle()
{
	if (TextureGraphEngine::IsTestMode())
		return;

	TextureGraphInsight::Instance()->GetSession()->UpdateIdle();
}

void TextureGraphInsightSchedulerObserver::Stop()
{
	UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightSchedulerObserver::Stop");
}

void TextureGraphInsightSchedulerObserver::BatchAdded(JobBatchPtr Batch)
{
	if (TextureGraphEngine::IsTestMode())
		return;

	TextureGraphInsight::Instance()->GetSession()->BatchAdded(Batch);
}

void TextureGraphInsightSchedulerObserver::BatchDone(JobBatchPtr Batch)
{
	if (TextureGraphEngine::IsTestMode())
		return;

	TextureGraphInsight::Instance()->GetSession()->BatchDone(Batch);
}

void TextureGraphInsightSchedulerObserver::BatchJobsDone(JobBatchPtr Batch)
{
	if (TextureGraphEngine::IsTestMode())
		return;

	TextureGraphInsight::Instance()->GetSession()->BatchJobsDone(Batch);
}

TextureGraphInsightEngineObserver::TextureGraphInsightEngineObserver()
{
	UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightEngineObserver::Constructor");

	for (int i = 0; i < (uint32)DeviceType::Count; ++i)
	{
		_deviceObservers[i] = std::make_shared<TextureGraphInsightDeviceObserver>((DeviceType)i);
	}
	BlobberObserver = std::make_shared<TextureGraphInsightBlobberObserver>();
	SchedulerObserver = std::make_shared<TextureGraphInsightSchedulerObserver>();
}

TextureGraphInsightEngineObserver::~TextureGraphInsightEngineObserver()
{
	UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightEngineObserver::Destructor");
}

void TextureGraphInsightEngineObserver::Created()
{
	if (TextureGraphEngine::IsTestMode())
		return;

	UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightEngineObserver::Created");

	/// Engine is created when notified, this should be true
	if (TextureGraphEngine::GetInstance())
	{
		for (int i = 0; i < (uint32)DeviceType::Count; ++i)
		{
			auto Dev = TextureGraphEngine::GetInstance()->GetDeviceManager()->GetDevice(i);
			if (Dev)
				Dev->RegisterObserverSource(_deviceObservers[i]);
		}
		TextureGraphEngine::GetInstance()->GetBlobber()->RegisterObserverSource(BlobberObserver);
		TextureGraphEngine::GetInstance()->GetScheduler()->RegisterObserverSource(SchedulerObserver);
	}

	/// This is a brand new session engine
	TextureGraphInsight::Instance()->GetSession()->EngineCreated();
}


void TextureGraphInsightEngineObserver::Destroyed()
{
	if (TextureGraphEngine::IsTestMode())
		return;

	UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightEngineObserver::Destroyed");

	/// Engine is already destroyed when notified
	if (TextureGraphEngine::GetInstance())
	{
		for (int i = 0; i < (uint32)DeviceType::Count; ++i)
		{
			auto Dev = TextureGraphEngine::GetInstance()->GetDeviceManager()->GetDevice(i);
			if (Dev)
				Dev->RegisterObserverSource(nullptr);
		}
		TextureGraphEngine::GetInstance()->GetBlobber()->RegisterObserverSource(nullptr); /// remove blobber observer from the engine
		TextureGraphEngine::GetInstance()->GetScheduler()->RegisterObserverSource(nullptr); /// remove scheduler observer from the engine
	}

	TextureGraphInsight::Instance()->GetSession()->EngineDestroyed();
}
