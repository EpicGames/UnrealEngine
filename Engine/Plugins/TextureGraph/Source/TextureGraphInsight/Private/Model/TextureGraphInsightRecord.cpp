// Copyright Epic Games, Inc. All Rights Reserved.
#include "Model/TextureGraphInsightRecord.h"

const DeviceBufferRecord DeviceBufferRecord::null;
const BlobRecord BlobRecord::null;
const JobRecord JobRecord::GNull;
const BatchRecord BatchRecord::null;
const ActionRecord ActionRecord::null;
const MixRecord MixRecord::null;

///
/// Batch
/// 

RecordID SessionRecord::FindBatchRecord(BatchID InBatchID) const
{
	auto found = RecordFromBatchID.find(InBatchID);
	if (found == RecordFromBatchID.end())
	{
		return RecordID();
	} else
	{
		return found->second;
	}
}

RecordID SessionRecord::NewBatchRecord(BatchID InBatchID, BatchRecordConstructor Constructor)
{
	// Unless batchID already exists, we will push the batch as the next element in the array
	auto found = RecordFromBatchID.find(InBatchID);
	if (found == RecordFromBatchID.end())
	{
		RecordID recordID = RecordID::fromBatch( Batches.size() );
		RecordFromBatchID.insert({ InBatchID, recordID });
		Batches.emplace_back(Constructor(recordID));

		auto& newBatchRecord = Batches.back();
		newBatchRecord.SelfID = recordID;
		int j = 0;
		for (auto& jr : newBatchRecord.Jobs)
		{
			jr.SelfID = RecordID::fromBatchJob(recordID.Batch(), j);
			j++;
		}

		return recordID;
	}
	else
	{
		return RecordID();
	}
}


bool SessionRecord::EditBatchRecord(RecordID RecId, BatchRecordEditor Editor)
{
	if (RecId.IsBatch())
	{
		return Editor(Batches[RecId.Batch()]);
	}
	return false;
}


std::tuple<RecordID,bool> SessionRecord::NewDeviceBufferRecord(DeviceType DevType, DeviceBufferID BufferID, const DeviceBufferRecord& Record)
{
	// Unless blobID already exists, we will push the blob as the next element in the array
	auto found = BufferFromDeviceBufferIDActive[(int32) DevType].find(BufferID);
	if (found == BufferFromDeviceBufferIDActive[(int32) DevType].end())
	{
		// Define the RecordID for this new buffer
		RecordID recordID = RecordID::fromBuffer(Record.DevType, DeviceBuffers[(uint32) Record.DevType].size());
		// Add it in the device active buffer map
		BufferFromDeviceBufferIDActive[(int32) DevType].insert({ BufferID, recordID });
		// Add it at the expected location in the per defice array
		DeviceBuffers[(uint32)Record.DevType].emplace_back(std::move(Record));
		// And add it in the global buffer from hash map
		BufferFromHash.insert({ Record.HashValue, recordID });

		return { recordID, true };
	}
	else
	{
		auto& r = DeviceBuffers[(uint32)Record.DevType][found->second.Buffer()];
		if (r.HashValue != Record.HashValue)
		{
			// Take note of the new hash assigned to this device buffer
			r.PrevHashValue = r.HashValue;
			r.HashValue = Record.HashValue;
			r.bLeaked = false;

			// And add it in the global buffer from hash map
			BufferFromHash.insert({ r.HashValue, found->second });

			return  { found->second, false } ;
		}
		else
		{
			return { RecordID(), false};
		}
	}
}

RecordID SessionRecord::EraseDeviceBufferRecord(DeviceType DevType, DeviceBufferID BufferID)
{
	// Expect to find it in the current map
	auto found = BufferFromDeviceBufferIDActive[(int32)DevType].find(BufferID);
	if (found == BufferFromDeviceBufferIDActive[(int32)DevType].end())
	{
		// Should not happen, the removed entry should have been seen before
		return RecordID();
	}
	else
	{
		// good we found the record in the active map
		auto RecId = found->second;
		// let's remove it and add it in the removed map
		BufferFromDeviceBufferIDActive[(int32)DevType].erase(found);
		// Add it to the removed map
		BufferFromDeviceBufferIDErased[BufferID] = RecId;

		// tag the record
		DeviceBuffers[RecId.Buffer_DeviceType()][RecId.Buffer()].bErased = true;
		
		// And return its recordID
		return RecId;
	}
}

RecordIDArray SessionRecord::FindActiveDeviceBufferRecords(DeviceType DevType, const DeviceBufferIDArray& BufferIDs, RecordIDArray& LeakedBuffers)
{
	RecordIDArray results;
	results.reserve(BufferIDs.size());

	auto map = BufferFromDeviceBufferIDActive[(int32) DevType]; // copy the current active map

	for (auto bid : BufferIDs)
	{
		auto found = map.find(bid);
		if (found == map.end())
		{
			results.emplace_back(RecordID());
		}
		else
		{
			results.emplace_back(found->second);
			map.erase(found);
		}
	}

	for (auto lid : map)
	{
		LeakedBuffers.emplace_back(lid.second);
	}

	return results;
}


RecordID SessionRecord::FindDeviceBufferRecord(DeviceBufferID BufferID) const
{
	RecordFromDeviceBufferIDMap::const_iterator found;

	// Look in each device's map
	for (int32 i = 0; i < (int32) DeviceType::Count; ++i)
	{
		found = BufferFromDeviceBufferIDActive[i].find(BufferID);
		if (found != BufferFromDeviceBufferIDActive[i].end())
		{
			return found->second;
		}
	}

	// try again with erased map
	found = BufferFromDeviceBufferIDErased.find(BufferID);
	if (found != BufferFromDeviceBufferIDErased.end())
	{
		return found->second;
		
	}
	
	// nope
	return RecordID();
}

RecordIDArray SessionRecord::FindDeviceBufferRecords(const DeviceBufferIDArray& BufferIDs) const
{
	RecordIDArray results;
	results.reserve(BufferIDs.size());
	for (auto b : BufferIDs)
	{
		results.emplace_back(FindDeviceBufferRecord(b));
	}
	return results;
}

RecordIDArray SessionRecord::FetchActiveDeviceBufferIDs(DeviceType DevType) const
{
	RecordIDArray results;
	results.reserve(BufferFromDeviceBufferIDActive[(int32) DevType].size());
	for (auto b : BufferFromDeviceBufferIDActive[(int32)DevType])
	{
		results.emplace_back(b.second);
	}
	return results;
}

RecordID SessionRecord::FindDeviceBufferRecordFromHash(HashType InHash) const
{
	// try to find the hash in the buffer from hash map
	auto found = BufferFromHash.find(InHash);
	if (found != BufferFromHash.end())
	{
		return found->second;
	}
	// nope
	return RecordID();
}


RecordID SessionRecord::FindBlobRecord(BlobID InBlobID) const
{
	auto found = BlobFromBlobID.find(InBlobID);
	if (found == BlobFromBlobID.end())
	{
		return RecordID();
	}
	else
	{
		return found->second;
	}
}

RecordID SessionRecord::NewBlobRecord(BlobID InBlobID, BlobRecordConstructor Constructor)
{
	// Unless blobID already exists, we will push the blob as the next element in the array
	auto found = BlobFromBlobID.find(InBlobID);
	if (found == BlobFromBlobID.end())
	{
		RecordID recordID = RecordID::fromBlob(Blobs.size());
		BlobFromBlobID.insert({ InBlobID, recordID });
		Blobs.emplace_back(Constructor(recordID));

		return recordID;
	} else
	{
		return RecordID();
	}
}

bool SessionRecord::EditBlobRecord(RecordID RecId, BlobRecordEditor Editor)
{
	if (RecId.IsBlob())
	{
		return Editor(Blobs[RecId.Blob()]);
	}
	return false;
}



RecordID SessionRecord::NewActionRecord(ActionID InActionID, const ActionRecord& action)
{
	// Unless actionID already exists, we will push the blob as the next element in the array
	auto found = ActionFromActionID.find(InActionID);
	if (found == ActionFromActionID.end())
	{
		RecordID recordID = RecordID::fromAction(Actions.size());
		ActionFromActionID.insert({ InActionID, recordID });
		Actions.emplace_back(std::move(action));

		return recordID;
	}
	else
	{
		return RecordID();
	}
}

RecordID SessionRecord::FindActionRecord(ActionID InActionID) const
{
	auto found = ActionFromActionID.find(InActionID);
	if (found == ActionFromActionID.end())
	{
		return RecordID();
	}
	else
	{
		return found->second;
	}
}


///
/// Mix
/// 

RecordID SessionRecord::FindMixRecord(MixID MixID) const
{
	auto found = RecordFromMixID.find(MixID);
	if (found == RecordFromMixID.end())
	{
		return RecordID();
	}
	else
	{
		return found->second;
	}
}

RecordID SessionRecord::NewMixRecord(MixID mixID, MixRecordConstructor constructor)
{
	// Unless mixID already exists, we will push the blob as the next element in the array
	auto found = RecordFromMixID.find(mixID);
	if (found == RecordFromMixID.end())
	{
		RecordID recordID = RecordID::fromMix(Mixes.size());
		RecordFromMixID.insert({ mixID, recordID });
		Mixes.emplace_back(constructor(recordID));

		auto& newRecord = Mixes.back();
		newRecord.SelfID = recordID;

		return recordID;
	}
	else
	{
		return RecordID();
	}
}

bool SessionRecord::EditMixRecord(RecordID RecId, MixRecordEditor editor)
{
	if (RecId.IsMix())
	{
		return editor(Mixes[RecId.Mix()]);
	}
	return false;
}

