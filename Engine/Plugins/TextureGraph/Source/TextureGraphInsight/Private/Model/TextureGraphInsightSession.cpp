// Copyright Epic Games, Inc. All Rights Reserved.
#include "Model/TextureGraphInsightSession.h"

#include "Job/Job.h"
#include "Data/Blob.h"

#include "TextureGraphEngine.h"
#include "Data/Blobber.h"
#include "Device/DeviceManager.h"
#include "Device/Device.h"

#include "Model/Mix/Mix.h"
#include "TextureGraphInsight.h"

#include "Transform/Mix/T_UpdateTargets.h"


TextureGraphInsightSession::TextureGraphInsightSession() :
	Record(new SessionRecord()),
	Cache(new LiveCache(this))
{
}
TextureGraphInsightSession::~TextureGraphInsightSession()
{
}

void TextureGraphInsightSession::LiveCache::AddBlob(RecordID RecId, BlobPtr BlobObj)
{
	if (RecId.IsBlob())
	{
		check(RecId.Blob() == Blobs.size());
		Blobs.emplace_back(BlobEntry(BlobObj->Hash()->Value()));
	}
}

BlobPtr TextureGraphInsightSession::LiveCache::GetBlob(RecordID RecId) const
{
	if (RecId.IsBlob() && RecId.Blob() < Blobs.size())
	{
		const auto& blobEntry = Blobs[RecId.Blob()];
		return TextureGraphEngine::GetInstance()->GetBlobber()->Find<Blob>(blobEntry).get();
	}
	return nullptr;
}

void TextureGraphInsightSession::LiveCache::AddBatch(RecordID RecId, JobBatchPtr Batch)
{
	if (RecId.IsBatch())
	{
		check(RecId.Batch() == Batches.size());
		Batches.emplace_back(Batch);
	}
}

JobBatchPtr TextureGraphInsightSession::LiveCache::GetBatch(RecordID RecId) const
{
	if ((RecId.IsBatch() || RecId.IsJob()) && RecId.Batch() < Batches.size())
	{
		return Batches[RecId.Batch()].lock();
	}
	return nullptr;
}

void TextureGraphInsightSession::LiveCache::AddMix(RecordID RecId, MixPtr MixObj)
{
	if (RecId.IsMix())
	{
		check(RecId.Mix() == Mixes.size());
		Mixes.emplace_back(MixObj);
	}
}

MixPtr TextureGraphInsightSession::LiveCache::GetMix(RecordID RecId) const
{
	if (RecId.IsMix() && RecId.Mix() < Mixes.size())
	{
		//return _mixes[RecId.Mix()].lock();
		return Mixes[RecId.Mix()];
	}
	return nullptr;
}


DeviceBufferPtr TextureGraphInsightSession::LiveCache::GetDeviceBuffer(RecordID RecId) const
{
	if (RecId.IsBuffer())
	{
		auto BufferRecord = Session->GetRecord().GetBuffer(RecId);
		auto SourceBlob = BufferRecord.SourceID;
		auto SourceBlobPtr = GetBlob(SourceBlob);
		auto Hash = BufferRecord.HashValue;
		if (SourceBlobPtr)
			Hash = SourceBlobPtr->Hash()->Value();

		// Try to find the buffer in the device under the  first hash
		auto Dev = TextureGraphEngine::GetInstance()->GetDeviceManager()->GetDevice(RecId.Buffer_DeviceType());
		auto Buffer = Dev->Find(Hash, false);

		// Couldn't find with what we think is final hash, try with prev hash if exists ?
		if (!Buffer) {
			if (BufferRecord.PrevHashValue != 0)
			{
				Buffer = Dev->Find(BufferRecord.PrevHashValue, false);
			}
		}

		for (int32 i = RecId.Buffer_DeviceType() + 1; i < (int32) DeviceType::Count; i++)
		{
			Dev = TextureGraphEngine::GetInstance()->GetDeviceManager()->GetDevice(i);
			if (Dev && !Buffer) {
				Buffer = Dev->Find(Hash, false);
				if (Buffer)
					break;
			}
		}

		// Couldn't find in the device cache... then look up in the blobber
		if (!Buffer) 
		{
			auto BlobPtr = TextureGraphEngine::GetInstance()->GetBlobber()->Find(BufferRecord.HashValue);
			if (BlobPtr) {
				Buffer = BlobPtr->GetBufferRef();
			}
			else
			{
				UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightActionManagerObserver::GetDeviceBuffer find the buffer from hash but no buffer");
			}
		}
		// Couldn't find in the blobber under final hash ? try the prev hash
		if (!Buffer) {
			if (BufferRecord.PrevHashValue != 0)
			{
				auto blobPtr = TextureGraphEngine::GetInstance()->GetBlobber()->Find(BufferRecord.PrevHashValue);
				if (blobPtr) {
					Buffer = blobPtr->GetBufferRef();
				}
				else
				{
					UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightActionManagerObserver::GetDeviceBuffer find the buffer from prevHash but no buffer");
				}
			}
			else
			{
				UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightActionManagerObserver::GetDeviceBuffer cannot find the buffer in the cache, missing second hash");
			}
		}

		return Buffer;
	}
	return nullptr;
}

void TextureGraphInsightSession::EngineCreated()
{
	if (TextureGraphEngine::IsTestMode())
		return;

	UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightActionManagerObserver::EngineCreated");

	// Record and Cache should be defined, empty and ready to be used
	RecordIDArray addedRecordIDs;
	for (int32 i = 0; i < (int32) DeviceType::Count; ++i)
		ParseDeviceBuffersCache((DeviceType)i, addedRecordIDs);

	EmitOnEngineReset(true);
}

void TextureGraphInsightSession::EngineDestroyed()
{
	if (TextureGraphEngine::IsTestMode())
		return;

	UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightActionManagerObserver::EngineDestroyed");

	// Need to reset record and cache
	{
		std::lock_guard<std::mutex> Lock(RecordMutex);
		Record = std::make_shared<SessionRecord>();
		Cache = std::make_shared<LiveCache>(this);
	}

	EmitOnEngineReset(false);
}

void TextureGraphInsightSession::EmitOnEngineReset(bool createOrDestroy)
{
	int32 PrevEngineID = EngineID;
	if (createOrDestroy)
	{
		EngineID = abs(PrevEngineID) + 1;
	}
	else
	{
		EngineID = -abs(PrevEngineID);
	}

	OnEngineResetEvent.Broadcast(EngineID);
}

uint32 TextureGraphInsightSession::ParseDeviceBuffersCache(DeviceType DevType, RecordIDArray& AddedBufferIDs)
{
	uint32 NumNewBuffers = 0;

	auto DevIndex = (int32) DevType;
	auto Dev = TextureGraphEngine::GetInstance()->GetDeviceManager()->GetDevice(DevType);
	if (Dev)
	{
		auto Version = Dev->GetObserverSource()->GetVersion();
		if (Version > GetCache().DeviceCacheVersions[DevIndex])
		{
			GetCache().DeviceCacheVersions[DevIndex] = Version;

			auto NumBuffers = Dev->GetNumDeviceBuffersUsed();
			DeviceObserverSource::HashArray buffers(NumBuffers, 0);
			DeviceObserverSource::HashNDescArray hashNDescs(NumBuffers);
			Dev->TraverseBufferCache([this,&buffers, &hashNDescs](const DeviceBufferPtr& buffer, uint32 index)
			{
				auto& Entry = hashNDescs[index];
				Entry.Raw = (uint64)buffer.get();
				Entry.First = buffer->Hash()->Value();
				Entry.Second = buffer->Descriptor();

				buffers[index] = Entry.Raw;
			});

			if (buffers.size())
			{
				// now compare against what we know
				RecordIDArray LeakedRecIds;

				auto BufferRecordIDs = GetRecord().FindActiveDeviceBufferRecords(DevType, buffers, LeakedRecIds);


				for (auto i = 0; i < BufferRecordIDs.size(); ++i)
				{
					auto& RecId = BufferRecordIDs[i];
					if (RecId.IsBuffer() && (RecId.Buffer_DeviceType() == DevIndex))
					{
						// everything happened as expected, all good, update the rank		
						auto& BuffRecord = GetRecord().GetBuffer(RecId);
						BuffRecord.Rank = i;
						BuffRecord.bLeaked = false;
					}
					else {
						// We just discovered a buffer, let's create it in the records
						DeviceBufferRecord DevBufferRec;
						DevBufferRec.DevType = DevType;
						DevBufferRec.ID = hashNDescs[i].Raw;
						DevBufferRec.HashValue = hashNDescs[i].First;
						DevBufferRec.Descriptor = hashNDescs[i].Second;

						DevBufferRec.bLeaked = true;
						DevBufferRec.Rank = i;
						bool bAdded = false;
						std::tie(RecId, bAdded) = GetRecord().NewDeviceBufferRecord(DevType, DevBufferRec.ID, DevBufferRec);

						if (bAdded)
							AddedBufferIDs.emplace_back(RecId);

						++NumNewBuffers;
					}
				}

				for (auto lid : LeakedRecIds)
				{
					auto& BufferRecord = GetRecord().GetBuffer(lid);
					BufferRecord.bLeaked = true;
					BufferRecord.Rank = -1;
				}
			}
		}
	}

	if (NumNewBuffers)
	{
		auto t = Device::DeviceType_Names(DevType);
		UE_LOGF(LogTextureGraphInsight, VeryVerbose, "TextureGraphInsightSession::ParseDeviceBuffersCache[%ls]: %d new unknown buffers", *t, NumNewBuffers);
	}
	
	return NumNewBuffers;
}

void TextureGraphInsightSession::DeviceBuffersUpdated(DeviceType DevType, TextureGraphInsightDeviceObserver::HashNDescArray&& AddedBuffers, TextureGraphInsightDeviceObserver::HashArray&& RemovedBuffers)
{
	RecordIDArray AddedRecIds;
	for (auto& AddedBuffer : AddedBuffers)
	{
		DeviceBufferRecord DevBuffRec;
		DevBuffRec.DevType = DevType;
		DevBuffRec.ID = AddedBuffer.Raw;
		DevBuffRec.HashValue = AddedBuffer.First;
		DevBuffRec.Descriptor = AddedBuffer.Second;
	
		auto key = AddedBuffer.Raw;

		UE_LOGF(LogTextureGraphInsight, VeryVerbose, "TextureGraphInsightSession::DeviceBuffersAdded : ptr 0x%llx - hash %llu", key, AddedBuffer.First);

		RecordID RecId;
		bool bAdded;
		std::tie(RecId, bAdded) = GetRecord().NewDeviceBufferRecord(DevType, key, DevBuffRec);
		if (RecId.IsValid())
		{
			if (bAdded)
			{
				UE_LOGF(LogTextureGraphInsight, VeryVerbose, "TextureGraphInsightSession::DeviceBuffersAdded : %d - hash %llu", RecId.Buffer(), DevBuffRec.HashValue);
				AddedRecIds.emplace_back(RecId);
			}
			else
			{
				UE_LOGF(LogTextureGraphInsight, VeryVerbose, "TextureGraphInsightSession::DeviceBuffers Updated hash : %d - hash %llu", RecId.Buffer(), DevBuffRec.HashValue);
			}
		}
		else
		{
			UE_LOGF(LogTextureGraphInsight, VeryVerbose, "TextureGraphInsightSession::DeviceBuffersAdded ERROR invalid buffer: ptr 0x%llx - hash %llu", DevBuffRec.ID, DevBuffRec.HashValue);
		}
	}

	RecordIDArray RemovedRecIds;
	for (int i = 0; i < RemovedBuffers.size(); ++i)
	{
		auto bp = RemovedBuffers[i];
		++i;
		auto h = RemovedBuffers[i];
		++i;
		auto ph = RemovedBuffers[i];

		UE_LOGF(LogTextureGraphInsight, VeryVerbose, "TextureGraphInsightSession::DeviceBuffersRemoved : ptr 0x%llx - hash %llu - prevhash %llu", bp, h, ph);
		auto RecId = GetRecord().EraseDeviceBufferRecord(DevType, bp);
		if (RecId.IsValid())
		{
			UE_LOGF(LogTextureGraphInsight, VeryVerbose, "TextureGraphInsightSession::DeviceBuffersRemoved : %d", RecId.Buffer());
			RemovedRecIds.emplace_back(RecId);
		}
		else
		{
			UE_LOGF(LogTextureGraphInsight, VeryVerbose, "TextureGraphInsightSession::DeviceBuffersAdded ERROR invalid buffer ?");
		}
	}

	// There was some changes, let's parse and eventually collect some new buffers that we would have missed ?
	ParseDeviceBuffersCache(DevType, AddedRecIds);

	OnDeviceBufferAddedEvent.Broadcast(AddedRecIds);
	OnDeviceBufferRemovedEvent.Broadcast(RemovedRecIds);
}

void TextureGraphInsightSession::BlobberHashesUpdated(TextureGraphInsightBlobberObserver::HashArray&& AddedHashes, TextureGraphInsightBlobberObserver::HashArray&& RemappedHashes)
{
	//for (auto a : addedHashes)
	for (auto it = AddedHashes.rbegin(); it != AddedHashes.rend(); ++it)
	{
		auto a = (*it);

//		UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightSession::BlobberHashesUpdated::Added hash %llu", a);
		// Try to find the hash BlobObj

//		auto n = FindOrCreateBlobRecordFromHash(a);
	}
	
	int32 numRemapped = RemappedHashes.size() >> 1;
	for (int i = 0; i < numRemapped; ++i)
	{
		UE_LOGF(LogTextureGraphInsight, VeryVerbose, "TextureGraphInsightSession::BlobberHashesUpdated::Remapped hash old %llu ==> new %llu", RemappedHashes[2 * i], RemappedHashes[2 * i + 1]);
		auto o = RemappedHashes[2 * i];
		auto n = RemappedHashes[2 * i + 1];
		if (o != n)
		{
			auto RecId = RemapBlobHash(o, n);
		}
	}

	if (numRemapped)
		EmitOnMappedBlob();
}



void TextureGraphInsightSession::UpdateIdle()
{
	OnUpdateIdleEvent.Broadcast();
}

void TextureGraphInsightSession::BatchAdded(JobBatchPtr Batch)
{
	UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightSchedulerObserver::BatchAdded");
}

void TextureGraphInsightSession::BatchDone(JobBatchPtr Batch)
{
	UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightSchedulerObserver::BatchDone");

	auto recordID = Record->NewBatchRecord(Batch->GetBatchId(), 
		[this, Batch](RecordID RecId) -> BatchRecord
		{
			return MakeBatchRecord(RecId, Batch);
		});

	if (recordID.IsValid())
	{
		// Cache the live job bath ptr
		GetCache().AddBatch(recordID, Batch);

		OnBatchAddedEvent.Broadcast(recordID);
	}

	if (recordID.IsValid())
	{
		OnBatchDoneEvent.Broadcast(recordID);
	}
}

void TextureGraphInsightSession::BatchJobsDone(JobBatchPtr Batch)
{
	UE_LOGF(LogTextureGraphInsight, Log, "TextureGraphInsightSchedulerObserver::BatchDone");

	auto recordID = GetRecord().FindBatchRecord(Batch->GetBatchId());

	GetRecord().EditBatchRecord(recordID,
		[this, Batch](BatchRecord& BufferRecord)
		{
			UpdateBatchRecord(Batch, BufferRecord);
			BufferRecord.bIsJobsDone = true;
			return true;
		});

	if (recordID.IsValid())
	{
		OnBatchJobsDoneEvent.Broadcast(recordID);
	}
}


void TextureGraphInsightSession::SendToInspector(RecordID recordID) const
{
	if (recordID.IsValid())
	{
		if (recordID.IsBatch())
		{
			OnBatchInspectedEvent.Broadcast(recordID);
		}
		else if (recordID.IsJob())
		{
			OnJobInspectedEvent.Broadcast(recordID);
		}
		else if (recordID.IsBlob())
		{
			OnBlobInspectedEvent.Broadcast(recordID);
		}
		else if (recordID.IsBuffer())
		{
			OnBufferInspectedEvent.Broadcast(recordID);
		}
	}
}


BatchRecord TextureGraphInsightSession::MakeBatchRecord(RecordID RecId, JobBatchPtr Batch, bool isDone)
{
	auto MixObj = Batch->GetCycle()->GetMix();
	auto mixRid = AssociateBatchToMix(RecId, MixObj);

	BatchRecord record;
	record.SelfID = RecId;
	record.BatchID = Batch->GetBatchId();
	record.FrameID = Batch->GetFrameId();
	record.BeginTimeMS = Batch->GetStartTime();
	record.ReplayCount = Batch->GetReplayCount();
	record.MixID = mixRid;
	record.bIsDone = isDone;

	record.bIsNoCache = Batch->GetCycle()->NoCache();
	record.bIsFromIdle = Batch->IsIdle();


	size_t numJobs = Batch->NumJobs();
	record.Jobs.resize(numJobs);
	std::vector<Job*> jobPtrs(numJobs, nullptr); // an array of the job pointers to retreive dependencies

	size_t jobIndex = 0;

	for (size_t j = 0; j < numJobs; ++j)
	{
		JobPtrW job = Batch->GetJobAt(j);
		jobPtrs[j] = job.lock().get();

		Job* generatorJobPtr = jobPtrs[j]->GeneratorJob().lock().get();

		if (generatorJobPtr && generatorJobPtr != jobPtrs[j])
		{
			// find the generator job in the list of knwon jobs
			auto genJobIdx = std::find(jobPtrs.begin(), jobPtrs.begin() + j, generatorJobPtr);
			if (genJobIdx == jobPtrs.end())
				check(false); /// Should never happen

			record.Jobs[jobIndex++] = MakeJobRecord(job, (genJobIdx - jobPtrs.begin()), -1);
		}
		else
		{
			record.Jobs[jobIndex++] = MakeJobRecord(job, j);
		}
	}

	/// RVO should handle this
	return record;
}

void TextureGraphInsightSession::UpdateBatchRecord(JobBatchPtr Batch, BatchRecord& record) {
	record.BeginTimeMS = Batch->GetStartTime();
	record.EndTimeMS = Batch->GetEndTime();
	record.ReplayCount = Batch->GetReplayCount();

	auto cycle = Batch->GetCycle();

	auto numJobs = Batch->NumJobs();
	for (uint64_t j = 0; j < numJobs; ++j)
	{
		JobPtrW job = Batch->GetJobAt(j);

		auto& rj = record.Jobs[j];
		UpdateJobRecord(job, rj, record.BeginTimeMS, record.EndTimeMS);
		rj.bIsDone = record.bIsDone;

		record.NumInvalidatedTiles += rj.GetNumInvalidated();
		record.NumTiles += rj.GetNumTiles();
	}


	// Capture the result target blobs
	record.ResultBlobs.resize((int32)TextureType::Count);
	if (cycle->GetNumTargets() && cycle->GetTarget(0))
	{
		TextureSet& lastRender = cycle->GetTarget(0)->GetLastRender();
		for (int32 tti = 0; tti < (int32)TextureType::Count; tti++)
		{
			auto BlobObj = lastRender.GetTexture(tti);
			if (BlobObj)
			{
				//auto bid = FindOrCreateBlobRecord(BlobObj, record._selfID);
				//record._resultBlobs[tti] = bid;
			}
		}
	}

	EmitOnNewBlob();
}

JobRecord TextureGraphInsightSession::MakeJobRecord(JobPtrW job, uint32 idx, int32 phaseIdx)
{
	JobRecord record;
	JobPtr sjob = job.lock();

	// record._jobID = job->ID();
	record.JobIdx = idx;
	record.PhaseIdx = phaseIdx;

	record.Priority = sjob->GetPriority();
	record.ReplayCount = sjob->GetReplayCount();

	if (sjob->GetTiled()) {
		record.Tiles.Resize(1, 1);
	}

	auto r = sjob->GetResult();
	if (r) 
	{
		record.TexWidth = r->GetWidth();
		record.TexHeight = r->GetHeight();

		record.Tiles.Resize(r->Rows(), r->Cols());
	}

	auto stats = sjob->GetStats();
	record.BeginTimeMS = stats.BeginNativeTime;
	record.BeginRunTimeMS = stats.BeginRunTime;
	record.EndTimeMS = stats.EndNativeTime;

	record.TransformName = sjob->GetTransform()->GetName();

	return record;
}

void TextureGraphInsightSession::UpdateJobRecord(JobPtrW job, JobRecord& record, double batchBeginTime_ms, double batchEndTime_ms)
{
	JobPtr sjob = job.lock();

	if (sjob->IsCulled())
		return;

	record.JobHash = sjob->Hash()->Value();

	auto stats = sjob->GetStats();

	if (stats.BeginNativeTime < batchBeginTime_ms)
		record.BeginTimeMS = batchBeginTime_ms;
	else
		record.BeginTimeMS = stats.BeginNativeTime;

	if (stats.BeginRunTime == 0)
		record.BeginRunTimeMS = 0; // record._beginTime_ms; // If the "runtime" stat is invalid then use the begin time
	else
		record.BeginRunTimeMS = stats.BeginRunTime;

	if (stats.EndNativeTime < record.BeginRunTimeMS)
		record.EndTimeMS = record.BeginRunTimeMS;
	else if (stats.EndNativeTime > batchEndTime_ms)
		record.EndTimeMS = batchEndTime_ms;
	else
		record.EndTimeMS = stats.EndNativeTime;

	record.ReplayCount = sjob->GetReplayCount();

	// Only on first replay capture all the inputs
	if (record.ReplayCount == 0)
	{
		record.InputBlobIds.clear();
		for (uint32_t i = 0; i < sjob->NumArgs(); ++i)
		{
			auto arg = sjob->GetArg(i);
			if (arg->IsBlob())
			{
				auto blobArg = std::static_pointer_cast<JobArg_Blob>(arg);
				auto inputBlob = blobArg->GetBlob();

				if (inputBlob) 
				{
					record.InputBlobIds.emplace_back(FindOrCreateBlobRecord(inputBlob.get()));
					record.InputArgNames.emplace_back(blobArg->Target());
				}
			}
		}
	}

	// Deal with output
	{
		auto r = sjob->GetResult();
		// In case of a replay update, all the inputs should be the same, 
		// the generated blobs are replayed too, let's imply call for an update
		if (record.ReplayCount && record.ResultBlobId.IsValid())
		{
			UpdateBlobRecord(record.ResultBlobId, std::static_pointer_cast<Blob>(r.get()));
		}
		// Not a replay but the first time the job is over, generate the resulting BlobObj(s)
		else if (r)
		{
			record.ResultBlobId = FindOrCreateBlobRecord(std::static_pointer_cast<Blob>(r.get()), record.SelfID);
			record.TexWidth = r->GetWidth();
			record.TexHeight = r->GetHeight();

			record.Tiles.Resize(r->Rows(), r->Cols());

			record.Tiles.ForEach([&](uint8_t tx, uint8_t ty) 
			{
				record.Tiles(tx, ty) = sjob->GetTileInvalidation(tx, ty);
			});
		}
	}
}

RecordID TextureGraphInsightSession::FindOrCreateBlobRecordFromHash(HashType hash)
{
	//auto RecId = Record().FindBlobRecord((uint64_t)BlobObj.get());
	auto RecId = GetRecord().FindBlobRecord(hash);
	if (!RecId.IsValid())
	{
		auto BlobObj = TextureGraphEngine::GetInstance()->GetBlobber()->Find(hash);
		if (BlobObj)
		{
			return FindOrCreateBlobRecord(BlobObj.get());
		}
		return RecordID();
	}
	else
		return RecId;
}

RecordID TextureGraphInsightSession::FindOrCreateBlobRecord(BlobPtr BlobObj, RecordID sourceID)
{
	if (BlobObj)
	{
		auto hash = BlobObj->Hash()->Value();
		auto hashIsFinal = BlobObj->Hash()->IsFinal();

		//auto RecId = Record().FindBlobRecord((uint64_t)BlobObj.get());
		auto RecId = GetRecord().FindBlobRecord(hash);
		if (!RecId.IsValid())
		{
			// Record the BlobObj right away (without tiling information)
			//RecId = Record().NewBlobRecord((uint64_t)BlobObj.get(), MakeBlobRecord(BlobObj, sourceID));
			RecId = GetRecord().NewBlobRecord(hash, 
				[this, BlobObj, sourceID](RecordID srid)
				{
					return MakeBlobRecord(srid, BlobObj, sourceID);
				});

			// Cache the live pointer to the BlobObj 
			GetCache().AddBlob(RecId, BlobObj);

			// Remember the recordID of the new BlobObj for notification (from the batchRecord handler)
			NewBlobsToNotify.emplace_back(RecId);

			// Parse the sub tiles next (probably creating the BlobObj records)
			if (BlobObj->IsTiled())
			{
				TiledBlobPtr tiledBlob = std::static_pointer_cast<TiledBlob>(BlobObj);

				RecordIDTiles  subTiles(tiledBlob->Rows(), tiledBlob->Cols());
				subTiles.ForEach([&](uint8_t tx, uint8_t ty) {
					subTiles(tx, ty) = FindOrCreateBlobRecord(tiledBlob->GetTile(tx, ty).get(), RecId);
				});

				// And then update the tiled BlobObj record of the parent
				GetRecord().EditBlobRecord(RecId, [&](BlobRecord& rd)
				{
					EditTiledBlobRecord(rd, std::move(subTiles));
					return true;
				});
			}
		}
		return RecId;
	}

	return RecordID();
}

RecordID TextureGraphInsightSession::RemapBlobHash(HashType oldHash, HashType newHash)
{
	auto old_rid = GetRecord().FindBlobRecord(oldHash);
	auto new_rid = GetRecord().FindBlobRecord(newHash);
	if (new_rid.IsValid() && old_rid.IsValid())
	{
		GetRecord().EditBlobRecord(old_rid, [=](BlobRecord& BufferRecord) { BufferRecord.MappedID = new_rid; return true; });
		GetRecord().EditBlobRecord(new_rid, [=](BlobRecord& BufferRecord) { BufferRecord.NumMapped++; return true; });

		MappedBlobsToNotify.emplace_back(old_rid);
	}

	return old_rid;
}


BlobRecord TextureGraphInsightSession::MakeBlobRecord(RecordID RecId, BlobPtr BlobObj, RecordID sourceID)
{
	BlobRecord record;
	record.SourceID = sourceID;
	record.Name = BlobObj->Name();
	record.HashValue = BlobObj->Hash()->Value();
	record.TexWidth = BlobObj->GetWidth();
	record.TexHeight = BlobObj->GetHeight();

	return record;
}


void TextureGraphInsightSession::EditTiledBlobRecord(BlobRecord& rd, const RecordIDTiles& subTiles)
{
	rd.TileIdxs.Resize(subTiles.Grid());
	for (int i = 0; i < subTiles.Grid().NumTiles(); ++i) {
		auto f = std::find(rd.UniqueBlobs.begin(), rd.UniqueBlobs.end(), subTiles._tiles[i].Blob());
		if (f == rd.UniqueBlobs.end())
		{
			rd.TileIdxs._tiles[i] = (uint8_t)rd.UniqueBlobs.size();
			rd.UniqueBlobs.emplace_back(subTiles._tiles[i].Blob());
		}
		else
		{
			rd.TileIdxs._tiles[i] = (uint8_t)(f - rd.UniqueBlobs.begin());
		}
	}
}

void  TextureGraphInsightSession::UpdateBlobRecord(RecordID RecId, BlobPtr BlobObj)
{
	if (BlobObj && RecId.IsBlob())
	{
		GetRecord().EditBlobRecord(RecId, [this, BlobObj](BlobRecord& BufferRecord)
		{
			if (!BufferRecord.IsTiled())
				BufferRecord.RawBufferMemSize = BlobObj->GetBufferRef()->MemSize();

			// Simply update the replay count, should be the only thing changed
			BufferRecord.ReplayCount = BlobObj->GetReplayCount();

			// Blob is really tiled, then dispatch updates to sub tiles
			if (BlobObj->IsTiled() && BufferRecord.IsTiled())
			{
				TiledBlobPtr tiledBlob = std::static_pointer_cast<TiledBlob>(BlobObj);

				BufferRecord.TileIdxs._grid.ForEach([&](uint32 tx, uint32 ty) {
					UpdateBlobRecord(BufferRecord.GetTileBlob(tx, ty), tiledBlob->GetTile(tx, ty).get());
				});
			}

			return true;
		});
	}
}

void TextureGraphInsightSession::EmitOnNewBlob()
{
	auto addedBlobs = std::move(NewBlobsToNotify);
	NewBlobsToNotify.clear();

	OnBlobAddedEvent.Broadcast(addedBlobs);
}

void TextureGraphInsightSession::EmitOnMappedBlob()
{
	auto mappedBlobs = std::move(MappedBlobsToNotify);
	MappedBlobsToNotify.clear();

	OnBlobMappedEvent.Broadcast(mappedBlobs);
}


bool TextureGraphInsightSession::ReplayBatch(RecordID batchRecordID, bool captureRenderDoc)
{
	// Get to the Batch ptr if still alive:
	auto batchPtr = GetCache().GetBatch(batchRecordID);
	if (batchPtr)
	{
		// configure the replay and spawn it
		batchPtr->ResetForReplay();
		batchPtr->SetCaptureRenderDoc(false);
		if (captureRenderDoc)
			TextureGraphEngine::GetInstance()->GetScheduler()->SetCaptureRenderDocNextBatch(true);
		TextureGraphEngine::GetInstance()->GetScheduler()->AddBatch(batchPtr);
		return true;
	}
	return false;
}


bool TextureGraphInsightSession::ReplayJob(RecordID jobRecordID, bool captureRenderDoc)
{
	// Get to the Batch ptr if still alive:
	auto batchPtr = GetCache().GetBatch(jobRecordID);
	if (batchPtr)
	{
		// configure the replay and spawn it
		batchPtr->ResetForReplay(jobRecordID.Job());
		batchPtr->SetCaptureRenderDoc(false);
		if(captureRenderDoc)
			TextureGraphEngine::GetInstance()->GetScheduler()->SetCaptureRenderDocNextBatch(true);
		TextureGraphEngine::GetInstance()->GetScheduler()->AddBatch(batchPtr);

		return true;
	}
	return false;
}


RecordID TextureGraphInsightSession::AssociateBatchToMix(RecordID Batch, MixPtr MixObj)
{
	auto mixID = FindOrCreateMixRecord(MixObj);

	if (mixID.IsValid())
	{
		if (GetRecord().EditMixRecord(mixID,
			[Batch](MixRecord& mr)
			{
				mr.Batches.emplace_back(Batch);
				return true;
			}))
		{
			OnMixUpdatedEvent.Broadcast(mixID);
		}
	}

	return mixID;
}

RecordID TextureGraphInsightSession::FindOrCreateMixRecord(MixPtr MixObj)
{
	if (MixObj)
	{
		//auto hash = MixObj->Hash()->Value();
		auto hash = (uint64)MixObj;
		
		auto RecId = GetRecord().FindMixRecord(hash);
		if (!RecId.IsValid())
		{
			UMix* mAsMix = dynamic_cast<UMix*>(MixObj);

			// if an Instance, find or allocate the parent MixObj
			RecordID		parentID;

			// Record the MixObj
			RecId = GetRecord().NewMixRecord(hash,
				[=](RecordID srid)
				{
					MixRecord record;
					record.Name = MixObj->GetName();

					record.ParentMixID = parentID;

					return record;
				});

			// Cache the live pointer to the MixObj
			GetCache().AddMix(RecId, MixObj);

			OnMixAddedEvent.Broadcast(RecId);

			// If parent ID is valid, add this knowledge to parent record and notify for update
			if (parentID.IsValid())
			{
				GetRecord().EditMixRecord(parentID,
					[=](MixRecord& mr)
					{
						mr.InstanceMixIDs.emplace_back(RecId);
						return true;
					});

				OnMixUpdatedEvent.Broadcast(parentID);
			}
		}

		// Return the RecId already existing or the brand new one
		return RecId;
	}

	return RecordID();
}
