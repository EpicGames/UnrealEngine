// Copyright Epic Games, Inc. All Rights Reserved.
#include "TextureGraphInsight.h"
#include "Modules/ModuleManager.h"
#include "TextureGraphEngine.h"

#include "Model/TextureGraphInsightObserver.h"
#include "Model/TextureGraphInsightSession.h"
#include "Shader.h"

IMPLEMENT_MODULE(FTextureGraphInsightModule, TextureGraphInsight);
DEFINE_LOG_CATEGORY(LogTextureGraphInsight);

void FTextureGraphInsightModule::StartupModule()
{
	FDefaultGameModuleImpl::StartupModule();
}

//////////////////////////////////////////////////////////////////////////

TextureGraphInsight* TextureGraphInsight::GInstance = nullptr;

TextureGraphInsight::TextureGraphInsight()
{
	UE_LOGF(LogTextureGraphInsight, Log, "Initialising the TextureGraphInsight!");

	Session = std::make_shared<TextureGraphInsightSession>();
}

TextureGraphInsight::~TextureGraphInsight()
{
	UE_LOGF(LogTextureGraphInsight, Log, "Destroying the TextureGraphInsight!");
}

bool TextureGraphInsight::Create()
{
	/// Cannot create a new instance before destroying the old one
	if (!GInstance)
	{
		GInstance = new TextureGraphInsight();

		/// Need the engine observer to watch what is happening
		auto EngineObserver = std::make_shared<TextureGraphInsightEngineObserver>();
		TextureGraphEngine::RegisterObserverSource(EngineObserver); // will also install other observers

		return true;
	}
	return false;
}

bool TextureGraphInsight::Destroy()
{
	/// Destroy an existing instance, no op otherwise
	if (GInstance)
	{
		/// Remove Engine observer
		TextureGraphEngine::RegisterObserverSource(nullptr);

		delete GInstance;
		GInstance = nullptr;

		return true;
	}

	return false;
}
