// Copyright Epic Games, Inc. All Rights Reserved.
#include "View/STextureGraphInsightActionView.h"

#include "TextureGraphInsight.h"
#include "Model/TextureGraphInsightSession.h"


class STextureGraphInsightActionViewRow : public SMultiColumnTableRow<STextureGraphInsightActionView::FItem>
{
public:
	using FItem = STextureGraphInsightActionView::FItem;

	SLATE_BEGIN_ARGS(STextureGraphInsightActionViewRow) {}
		SLATE_ARGUMENT(FItem, Item)
	SLATE_END_ARGS()

public:

	enum Column {
		Main = 0,
		Name,

		NUM_COLUMNS,
	};
	using ColumnInfo = std::pair<FName, float>;
	static ColumnInfo s_columnNames[NUM_COLUMNS];
	static Column NameToColumn(const FName& name) {
		for (int i = 0; i < NUM_COLUMNS; ++i)
		{
			if (name == s_columnNames[i].first)
				return Column(i);
		}
		return NUM_COLUMNS;
	}

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		_recordID = InArgs._Item->_recordID;
		SMultiColumnTableRow<FItem >::Construct(FSuperRowType::FArguments(), InOwnerTableView);	
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& columnName) override
	{
		if (_recordID.IsValid()) {
			Column column = NameToColumn(columnName);
			if (column == Main) {
				// Rows in a TreeView need an expander button and some indentation
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Fill)
					[
						SNew(SExpanderArrow, SharedThis(this))
						.StyleSet(ExpanderStyleSet)
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1)
					.Padding(FMargin(4.0f, 0.0f))
					.VAlign(VAlign_Center)
					[
						SAssignNew(_textBoxes[column], STextBlock)
						.Text(GetTextForColumn(column))
					];
			} else {
				return SNew(SBox)
					.Padding(FMargin(4.0f, 0.0f))
					.VAlign(VAlign_Center)
					[
						SAssignNew(_textBoxes[column], STextBlock)
						.Text(GetTextForColumn(column))
						.OnDoubleClicked(this, &STextureGraphInsightActionViewRow::OnDoubleClickedResultColumn)
					];
			}
		}
		// default to null widget if property cannot be found
		return SNullWidget::NullWidget;
	}

	FText GetTextForColumn(Column column) const
	{
		const auto& ar = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetAction(_recordID);

		FString s;
		switch (column)
		{
		case Main:
				s = FString::FromInt(_recordID.Action()) + " " + ar.Name;
			break;
		case Name:
				if (ar.SubActions.empty())
					s = ar.Meta;
			break;
		}

		return FText::FromString(s);
	}

	void Refresh()
	{
		_textBoxes[Main]->SetText(GetTextForColumn(Main));
		_textBoxes[Name]->SetText(GetTextForColumn(Name));
	}

	FReply OnDoubleClickedResultColumn(/** The geometry of the widget*/
		const FGeometry&,
		/** The Mouse Event that we are processing */
		const FPointerEvent&)
	{
		return FReply::Handled();
	}

protected:
	RecordID _recordID;
	TSharedPtr<STextBlock> _textBoxes[NUM_COLUMNS];
};

STextureGraphInsightActionViewRow::ColumnInfo STextureGraphInsightActionViewRow::s_columnNames[] = {
	{ FName(TEXT("Action")), 0.1},    // main
	{ FName(TEXT("Name")), 0.3},
};

void STextureGraphInsightActionView::Construct(const FArguments& Args)
{

	TSharedPtr<SHeaderRow> headerRow = SNew(SHeaderRow);

	for (int i = 0; i < STextureGraphInsightActionViewRow::NUM_COLUMNS; i++)
	{
		headerRow->AddColumn(
			SHeaderRow::Column(STextureGraphInsightActionViewRow::s_columnNames[i].first)
			.DefaultLabel(FText::FromString(STextureGraphInsightActionViewRow::s_columnNames[i].first.ToString()))
			.FillWidth(STextureGraphInsightActionViewRow::s_columnNames[i].second)
		);
	}

	ChildSlot
	[
		SAssignNew(_tableView, SItemTableView)
		.TreeItemsSource(&_rootItems)
		.OnGenerateRow(this, &STextureGraphInsightActionView::OnGenerateRowForView)
		.OnGetChildren(this, &STextureGraphInsightActionView::OnGetChildrenForView)
		.OnMouseButtonDoubleClick(this, &STextureGraphInsightActionView::OnDoubleClickItemForView)
		.OnMouseButtonClick(this, &STextureGraphInsightActionView::OnClickItemForView)
		.HeaderRow(headerRow)
	];

	// install the observer notifications
	auto sr = StaticCastSharedRef<STextureGraphInsightActionView>(this->AsShared());
	TextureGraphInsight::Instance()->GetSession()->OnEngineReset().AddSP(sr, &STextureGraphInsightActionView::OnEngineReset);
}

TSharedRef<ITableRow> STextureGraphInsightActionView::OnGenerateRowForView(FItem item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SAssignNew(item->_widget, STextureGraphInsightActionViewRow, OwnerTable).Item(item);
}

void STextureGraphInsightActionView::OnClickItemForView(FItem item)
{
	// Item clicked, let's send it to the inspector
	auto record = item->_recordID;
	//TextureGraphInsight::Instance()->Session()->SendToInspector(record);
}
void STextureGraphInsightActionView::OnDoubleClickItemForView(FItem item)
{
	// Item double clicked, let's send it to the inspector
	auto record = item->_recordID;
	//TextureGraphInsight::Instance()->Session()->SendToInspector(record);
}

void STextureGraphInsightActionView::OnActionNew(RecordID rid)
{
	// Fetch the actual data from the record and add job's items
	const auto& ar = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetAction(rid);

	// Do not worry about actions with parent, we will get notified for the actual parent and
	// will populate the sub ations consequently
	if (ar.ParentAction.IsValid())
		return;

	FItem item = MakeShareable(new FItemData(rid));

	for (const auto& sa : ar.SubActions)
	{
		item->_children.Add(MakeShareable(new FItemData(sa)));
	}

	_rootItems.Add(item);

	if (item->_children.Num())
		_tableView->SetItemExpansion(item, true);

	_tableView->RequestTreeRefresh();
}

void STextureGraphInsightActionView::OnEngineReset(int id)
{
	_rootItems.Empty();
	_tableView->RequestTreeRefresh();
}

