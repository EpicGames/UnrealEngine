// Copyright Epic Games, Inc. All Rights Reserved.
#include "View/STextureGraphInsightBlobView.h"

#include "2D/Tex.h"
#include "TextureGraphInsight.h"
#include "Model/TextureGraphInsightSession.h"

#include "Device/FX/DeviceBuffer_FX.h"

#include "Widgets/Layout/SScaleBox.h"
#include "Materials/Material.h"

#include "View/STextureGraphInsightDeviceBufferView.h"
#include <Widgets/Layout/SUniformGridPanel.h>
#include <Materials/MaterialInstanceDynamic.h>
#include <Brushes/SlateColorBrush.h>
#include <Widgets/SBoxPanel.h>
#include <Widgets/Text/STextBlock.h>
#include <Widgets/SOverlay.h>


void STextureGraphInsightBlobView::Construct(const FArguments& InArgs)
{
	RecordID recordID = InArgs._recordID;
	bool withHighlight = InArgs._withHighlight;
	BoolTiles tilesMask = InArgs._tilesMask;

	_recordID = recordID;

	// Check for a valid blob id before anything
	if (!(recordID.IsValid() && recordID.IsBlob()))
	{
		return;
	}

	uint32_t SIMAGE_WIDTH = 512;
	uint32_t SIMAGE_HEIGHT = 512;

	const float TEXT_PADDING = 3.0;
	const float BLOB_PADDING = 0.5;

	bool addTouchedBorder = withHighlight;

	_borderBrush = MakeShareable(new FSlateColorBrush(FColor(255, 37, 18)));


	// Describe the blob, identify if it is tiled or not
	const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBlob(recordID);

	if (br.IsTiled()) {
		_inspectedID = _recordID; // Default inspected is self if tiled.

		FString s = br.Name;


		if (br.TexWidth || br.TexHeight)
			s += "\n" + FString::FromInt(br.TexWidth) + " x " + FString::FromInt(br.TexHeight);

		{ // Add some meta data specific to a tiled blob
			if (br.NumUniqueBlobs() < br.NumTiles())
			{
				s += "    " + FString::FromInt(br.NumUniqueBlobs()) + "unique tiles / ";
				s += (br.Grid().IsUnique() ? "1" : FString::FromInt(br.Grid().Rows()) + " x " + FString::FromInt(br.Grid().Cols()));
			}
		}
		if (br.ReplayCount > 0)
			s += " replay #" + FString::FromInt(br.ReplayCount);

		TSharedPtr<SUniformGridPanel> gridPanel;
		ChildSlot
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(TEXT_PADDING)
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(FText::FromString(s))
					.AutoWrapText(true)
				]
				+ SVerticalBox::Slot()
				[
					SAssignNew(gridPanel, SUniformGridPanel)
				]
			];

		bool isSingleTiledBlob = br.Grid().IsUnique();

		br.Grid().ForEach([&](TiledGrid::Size tx, TiledGrid::Size ty)
		{
			bool highlight = false;
			if (tilesMask.Grid().IsValidTile(tx, ty))
			{
				highlight = tilesMask(tx, ty) != 0;
			}

			auto subTileRid = br.GetTileBlob(tx, ty);

			if (isSingleTiledBlob || (subTileRid.id == _recordID.id)) /// In some case, the sub tile is the same as the parent tile, capture this case
			{
				RecordID bufferID = TextureGraphInsight::Instance()->GetSession()->GetRecord().FindDeviceBufferRecordFromHash(br.HashValue);
				bool blobContentInBrush = false;
				_inspectedID = bufferID;

				gridPanel->AddSlot(tx, ty)
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					[
						SNew(STextureGraphInsightDeviceBufferView)
						.recordID(bufferID)
						.blobID(_recordID)
						.withToolTip(true)
					]
				];
			}
			else
			{
				gridPanel->AddSlot(tx, ty)
				[
					SNew(STextureGraphInsightBlobView)
					.recordID(subTileRid)
					.withHighlight(highlight)
				];
			}
		});
			}
	else
	{
		RecordID bufferID = TextureGraphInsight::Instance()->GetSession()->GetRecord().FindDeviceBufferRecordFromHash(br.HashValue);

		bool blobContentInBrush = false;
		FString s = br.Name;
		if (br.TexWidth || br.TexHeight)
			s += "\n" + FString::FromInt(br.TexWidth) + " x " + FString::FromInt(br.TexHeight);
		if (br.ReplayCount > 0)
			s += " replay #" + FString::FromInt(br.ReplayCount);
		
		_inspectedID = bufferID;

		ChildSlot
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.Padding(BLOB_PADDING )
				[
					SNew(STextureGraphInsightDeviceBufferView)
					.recordID(bufferID)
					.blobID(_recordID)
					.withToolTip(true)
				]
			];
	}

	if (addTouchedBorder)
	{
		SetBorderImage(_borderBrush.Get());
	}
};

FReply STextureGraphInsightBlobView::OnInspect()
{
	TextureGraphInsight::Instance()->GetSession()->SendToInspector(_inspectedID);

	return FReply::Handled();
}


FReply STextureGraphInsightBlobView::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	//return OnInspect();
	return FReply::Unhandled();
}

FReply STextureGraphInsightBlobView::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return OnInspect();
}
