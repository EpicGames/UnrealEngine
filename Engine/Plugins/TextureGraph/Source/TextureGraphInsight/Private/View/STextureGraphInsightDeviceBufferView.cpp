// Copyright Epic Games, Inc. All Rights Reserved.
#include "View/STextureGraphInsightDeviceBufferView.h"

#include "2D/Tex.h"
#include "TextureGraphInsight.h"
#include "Model/TextureGraphInsightSession.h"

#include "TextureGraphEngine.h"
#include "Device/DeviceManager.h"
#include "Device/FX/DeviceBuffer_FX.h"

#include "Widgets/Layout/SScaleBox.h"
#include "Materials/Material.h"
#include <Materials/MaterialInstanceDynamic.h>
#include <Styling/SlateBrush.h>
#include <Widgets/SOverlay.h>
#include <Widgets/Images/SImage.h>
#include <Widgets/Text/STextBlock.h>
#include <Widgets/Layout/SBox.h>
#include <Widgets/SBoxPanel.h>
#include <Brushes/SlateDynamicImageBrush.h>
void STextureGraphInsightDeviceBufferView::Construct(const FArguments& InArgs)
{
	RecordID recordID = InArgs._recordID;
	RecordID blobID = InArgs._blobID;
	bool withToolTip = InArgs._withToolTip;
	bool withDescription = InArgs._withDescription;

	_recordID = recordID;
	_blobID = blobID;

	// allocate the material brush where we can assign the texture (or a copy of ) async, or just reflect the status
	MakeWidgetMaterial();

	// Build the basic widget eventually showing the checker board if no buffer found
	TSharedPtr<SOverlay> overlay;
	ChildSlot
		[
			SAssignNew(overlay, SOverlay)
			+ SOverlay::Slot()
		[
			SNew(SImage)
			.Image(_brush.Get())
		]
		];

	// Check for a valid buffer id before anything
	bool useBlobId = false;
	//if (!(recordID.IsValid() && recordID.IsBuffer() && (recordID.Buffer() != INVALID_INDEX)))
	{
		useBlobId = true;
		// if the Buffer ID is invalid, try to the Blob Id
		if (!(blobID.IsValid() && blobID.IsBlob() && (blobID.Blob() != INVALID_INDEX)))
		{
			UE_LOGF(LogTemp, Warning, "BlobTexture Invalid recordID for the buffer ?");
			return;
		}	
	}

	// Now let's go find the true live DeviceBuffer ptr
	BlobPtr blob;
	if (useBlobId)
	{
		blob = TextureGraphInsight::Instance()->GetSession()->GetCache().GetBlob(blobID);
	}
	DeviceBufferPtr buffer;
	if (blob)
	{
		buffer = blob->GetBufferRef().GetPtr();
	} else
	{
		buffer = TextureGraphInsight::Instance()->GetSession()->GetCache().GetDeviceBuffer(recordID);
	}

	// Hopefully found a device buffer if not early exit
	if (!buffer)
		return;

	uint32_t SIMAGE_WIDTH = 512;
	uint32_t SIMAGE_HEIGHT = 512;

	const float TEXT_PADDING = 3.0;
	const float BLOB_PADDING = 2.0;

	bool blobContentInBrush = false;

	const auto& d = buffer->Descriptor();
	FString s = d.Name;
	if (d.Width || d.Height)
		s += "\n" + FString::FromInt(d.Width) + " x " + FString::FromInt(d.Height);

	s += "\n" + FString::FromInt(d.ItemsPerPoint) + " x " + FString(BufferDescriptor::FormatToString(d.Format));
	s += "\n" + HashToFString(buffer->Hash()->Value());
	//if (buffer->Hash()->HasTempDependency())
	//	s += "\n" + HashToFString(buffer->PrevHashValue());


	_imageWidth = std::min(d.Width, SIMAGE_WIDTH);
	_imageHeight = std::min(d.Height, SIMAGE_HEIGHT);
	_imageName = d.Name;

	 // Got the description, now need to retreive the true live buffer and the texture to configure the brush
	 {
		// Hopefully found a device buffer
		if (buffer)
		{
			// Let's create a texture from the raw data
			auto tex = std::make_shared<Tex>();

			if (!buffer->IsNull())
			{
				buffer->Raw()
					.then([this, tex](RawBufferPtr raw) mutable
						{
							return tex->LoadRaw(raw);
						})
					.then([this, tex](auto result) mutable
						{
							if (result->IsOK())
							{
								check(tex && tex->GetTexture());
								tex->SetFilter(TextureFilter::TF_Nearest);
								UTexture* blobTexture = tex->GetTexture();
								CreateWidgetBrush(blobTexture);
							}
							else
							{
								UE_LOGF(LogTemp, Warning, "BlobTexture failed to create the widget material");
							}
						})
					.fail([this, tex]() mutable
						{
							UE_LOGF(LogTemp, Warning, "BlobTexture failed to create the widget material");
						});
			}
			else
			{

			}
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "BlobTexture failed to find the buffer ?");
			// no buffer ?
		}
	}

	if (withToolTip)
	{
		overlay->SetToolTipText(FText::FromString(s));
	}
	if (withDescription)
	{
		overlay->AddSlot()
			.Padding(TEXT_PADDING)
			.VAlign(EVerticalAlignment::VAlign_Bottom)
			[
				SNew(STextBlock)
				.Text(FText::FromString(s))
			.AutoWrapText(true)
			];
	}
};

void STextureGraphInsightDeviceBufferView::CreateWidgetBrush(UTexture* blobTexture) {
	if (blobTexture && _brush && _brushMaterial) {
		_brushMaterial->SetTextureParameterValue("BlobTex", blobTexture);
		_brushMaterial->SetScalarParameterValue("BlobTexIsValid", 1.0);
	}
}

void STextureGraphInsightDeviceBufferView::MakeWidgetMaterial() {
	_brush = MakeShareable(
		new FSlateDynamicImageBrush(
			(UTexture2D*) nullptr,
			FVector2D(_imageWidth, _imageHeight),
			//desiredSize,
			FName(_imageName),
			FLinearColor(1.0f, 1.0f, 1.0f, 1.0f),
			ESlateBrushTileType::NoTile,
			ESlateBrushImageType::Linear
		)
	);
	UMaterial* material = LoadObject<UMaterial>(nullptr, TEXT("/TextureGraph/UI/Materials/BlobView"));
	_brushMaterial = (UMaterialInstanceDynamic::Create(material, nullptr));
	_brushMaterial->SetScalarParameterValue("BlobTexIsValid", 0.0);
	_brush->SetResourceObject(_brushMaterial);
}
