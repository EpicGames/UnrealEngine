// Copyright Epic Games, Inc. All Rights Reserved.
#include "View/STextureGraphInsightDeviceView.h"

#include "TextureGraphInsight.h"
#include "Model/TextureGraphInsightSession.h"

#include "Device/DeviceManager.h"
#include "Data/Blobber.h"
#include "TextureGraphEngine.h"
#include <Widgets/Views/ITableRow.h>
#include <Widgets/Views/STableViewBase.h>
#include <Widgets/Text/STextBlock.h>

class STextureGraphInsightDeviceListViewRow : public SMultiColumnTableRow<STextureGraphInsightDeviceListView::FItem>
{
public:
	using FItem = STextureGraphInsightDeviceListView::FItem;
	using FItemW = STextureGraphInsightDeviceListView::FItemW;

	SLATE_BEGIN_ARGS(STextureGraphInsightDeviceListViewRow) {}
	SLATE_ARGUMENT(FItem, Item)
	SLATE_END_ARGS()

public:

	enum Column {
		Main = 0,
		Count,
		Hash,
		Hash1,
		Hash2,
		MemSize,
		Pixels,

		NUM_COLUMNS,
	};
	using ColumnInfo = std::pair<FName, float>;
	static ColumnInfo s_columnNames[NUM_COLUMNS];
	static Column NameToColumn(const FName& name) {
		for (int i = 0; i < NUM_COLUMNS; ++i)
		{
			if (name == s_columnNames[i].first)
				return Column(i);
		}
		return NUM_COLUMNS;
	}

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		_item = InArgs._Item;
		_recordID = InArgs._Item->_recordID;
		SMultiColumnTableRow<FItem >::Construct(FSuperRowType::FArguments(), InOwnerTableView);

		Refresh();
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& columnName) override
	{
		Column column = NameToColumn(columnName);

		if (column == Main) {
			// Rows in a TreeView need an expander button and some indentation
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Fill)
				[
					SNew(SExpanderArrow, SharedThis(this))
					.StyleSet(ExpanderStyleSet)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SAssignNew(_textBoxes[column], STextBlock)
					.Text(GetTextForColumn(column))
				];
		}
		else
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SAssignNew(_textBoxes[column], STextBlock)
					.Text(GetTextForColumn(column))
					.OnDoubleClicked(this, &STextureGraphInsightDeviceListViewRow::OnDoubleClickedResultColumn)
				];
		}
	}

	FText GetTextForColumn(Column column) const
	{
		bool isDevice = (_recordID.IsDevice());

		const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBuffer(_recordID);
		auto memSize = br.RawBufferMemSize;

		FString s;
		switch (column)
		{
		case Main:
			if (isDevice)
			{
				auto item = _item.Pin();
				if (item->_metaItem == 1)
					s = FString(TEXT("erased"));
				else
					s = (Device::DeviceType_Names((DeviceType) _recordID.Buffer_DeviceType()));
			}
			else
				s = br.Descriptor.Name;

			//	s = FString::FromInt(_recordID.Buffer());
			break;
		case Count:
			if (isDevice)
			{
				auto item = _item.Pin();
				s = FString::FromInt(item->_children.Num());
			}
			else
			{
				s = FString::FromInt(br.Rank);
			}
			break;
		case Hash:
			if (isDevice)
			{
			}
			else
			{
				auto& h = br.ID; // the key used to find this buffer (ptr)
				s = HashToFString(h);
			}
			break;
		case Hash1:
			if (isDevice)
			{
			}
			else
			{
				auto& h = br.HashValue;
				s = HashToFString(h);
			}
			break;
		case Hash2:
			if (isDevice)
			{
			}
			else
			{
				auto& h = br.PrevHashValue;
				s = HashToFString(h);
			}
			break;
		case MemSize:
			if (isDevice)
			{
			}
			else
			{
				s = FString::FromInt(memSize / 1024) + " kb";
			}
			break;
		case Pixels:
			if (isDevice)
			{
			}
			else
			{
				if (br.Descriptor.Width || br.Descriptor.Height)
				{
					s = FString::FromInt(br.Descriptor.Width) + " x " + FString::FromInt(br.Descriptor.Height);
				}
			}
			break;
		}

		return FText::FromString(s);
	}

	void Refresh()
	{
		if (_recordID.IsDevice())
		{
			if (!TextureGraphEngine::GetInstance())
			{
				SetColorAndOpacity(FLinearColor(0.3f, 0.3f, 0.3f));
			}
			else
			{
				auto item = _item.Pin();
				auto device = TextureGraphEngine::GetInstance()->GetDeviceManager()->GetDevice(_recordID.Buffer_DeviceType());
				if (!device)
				{
					SetColorAndOpacity(FLinearColor(0.2f, 0.2f, 0.2f));
				}
				else if (item->_metaItem == 1)
				{
					SetColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f));
					_textBoxes[Count]->SetText(FText::FromString(FString::FromInt(item->_children.Num())));
				}
				else
				{ // This is a true device item, update the data
					auto count = device->GetNumDeviceBuffersUsed();
					if (count == 0)
						SetColorAndOpacity(FLinearColor(0.7f, 0.7f, 0.7f));
					if (count != item->_children.Num())
						SetColorAndOpacity(FLinearColor(1.0f, 0.5f, 0.5f));
					else
						SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f));

					_textBoxes[Count]->SetText(FText::FromString(FString::FromInt(item->_children.Num())));
					_textBoxes[Hash]->SetText(FText::FromString(FString::FromInt(count)));
				}
			}
		}
		else
		{
			const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBuffer(_recordID);
			if (br.bLeaked)
				SetColorAndOpacity(FLinearColor(1.0f, 0.8f, 0.5f)); 
			else if (br.bErased)
				SetColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f));
			else
				SetColorAndOpacity(FLinearColor(0.5f, 1.0f, 1.0f));

			_textBoxes[Count]->SetText(GetTextForColumn(Count));
		}
	}

	FReply OnDoubleClickedResultColumn(/** The geometry of the widget*/
		const FGeometry&,
		/** The Mouse Event that we are processing */
		const FPointerEvent&)
	{
		TextureGraphInsight::Instance()->GetSession()->SendToInspector(_recordID);
		return FReply::Handled();
	}

protected:
	FItemW					_item;
	RecordID				_recordID;
	TSharedPtr<STextBlock>	_textBoxes[NUM_COLUMNS];
};

STextureGraphInsightDeviceListViewRow::ColumnInfo STextureGraphInsightDeviceListViewRow::s_columnNames[] = {
	{ FName(TEXT("Name")), 0.5},    // main
	{ FName(TEXT("Rank")), 0.1},
	{ FName(TEXT("Ptr")), 0.1},
	{ FName(TEXT("Hash")), 0.1},
	{ FName(TEXT("HashPrev")), 0.1},
	{ FName(TEXT("Size")), 0.05},
	{ FName(TEXT("Pixels")), 0.05}
};

void STextureGraphInsightDeviceListView::Construct(const FArguments& Args)
{
	_inspectOnSimpleClick = Args._inspectOnSimpleClick;
	_inspectDevices = Args._inspectDevices;

	TSharedPtr<SHeaderRow> headerRow = SNew(SHeaderRow);

	for (int i = 0; i < STextureGraphInsightDeviceListViewRow::NUM_COLUMNS; i++)
	{
		headerRow->AddColumn(
			SHeaderRow::Column(STextureGraphInsightDeviceListViewRow::s_columnNames[i].first)
			.DefaultLabel(FText::FromString(STextureGraphInsightDeviceListViewRow::s_columnNames[i].first.ToString()))
			.FillWidth(STextureGraphInsightDeviceListViewRow::s_columnNames[i].second)
		);
	}

	ChildSlot
	[
		SAssignNew(_tableView, SItemTableView)
		.TreeItemsSource(&_rootItems)
		.OnGenerateRow(this, &STextureGraphInsightDeviceListView::OnGenerateRowForView)
		.OnGetChildren(this, &STextureGraphInsightDeviceListView::OnGetChildrenForView)

		.OnMouseButtonDoubleClick(this, &STextureGraphInsightDeviceListView::OnDoubleClickItemForView)
		.OnMouseButtonClick(this, &STextureGraphInsightDeviceListView::OnClickItemForView)
		.HeaderRow(headerRow)
	];

	// If a valid recordID was specified, let's use it as the root device type,
	// meaning we only care about that device type
	// the value of the buffer index doesn't matter
	if (Args._recordID.IsBuffer())
		_rootDevice = Args._recordID;

	// Allocate the root items
	for (int i = 0; i < (int)DeviceType::Count; ++i)
	{
		auto deviceType = (DeviceType)i;
		auto rid = RecordID::fromBuffer(deviceType, INVALID_INDEX);
		FItem item = MakeShareable(new FItemData(rid));
		_rootItems.Add(item);
	}

	// install the observer notifications:
	auto sr = StaticCastSharedRef<STextureGraphInsightDeviceListView>(this->AsShared());
	TextureGraphInsight::Instance()->GetSession()->OnDeviceBufferAdded().AddSP(sr, &STextureGraphInsightDeviceListView::OnDeviceBufferAdded);
	TextureGraphInsight::Instance()->GetSession()->OnDeviceBufferRemoved().AddSP(sr, &STextureGraphInsightDeviceListView::OnDeviceBufferRemoved);
	TextureGraphInsight::Instance()->GetSession()->OnEngineReset().AddSP(sr, &STextureGraphInsightDeviceListView::OnEngineReset);
}

void STextureGraphInsightDeviceListView::Reset()
{
	for (int i = 0; i < (int)DeviceType::Count; ++i)
	{
		auto deviceType = (DeviceType)i;
		_rootItems[i]->_children.Empty();
		_rootItems[i]->_removedRoot.Reset();
		if (_rootItems[i]->_widget) _rootItems[i]->_widget->Refresh();

		if (TextureGraphInsight::Instance()->GetSession())
		{
			auto deviceBufferIDs = TextureGraphInsight::Instance()->GetSession()->GetRecord().FetchActiveDeviceBufferIDs((DeviceType)i);
			for (auto rid : deviceBufferIDs)
			{
				_rootItems[i]->_children.Add(MakeShareable(new FItemData(rid)));
			}
		}
	}

	if (_rootDevice.IsDevice())
	{
		_tableView->SetItemExpansion(_rootItems[_rootDevice.Buffer_DeviceType()], true);
	}

	_tableView->RequestListRefresh();

	RefreshRootItems();
}

TSharedRef<ITableRow> STextureGraphInsightDeviceListView::OnGenerateRowForView(FItem item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SAssignNew(item->_widget, STextureGraphInsightDeviceListViewRow, OwnerTable).Item(item);
}

void STextureGraphInsightDeviceListView::OnClickItemForView(FItem item)
{
	if (_inspectOnSimpleClick)
		OnInspect(item);
}
void STextureGraphInsightDeviceListView::OnDoubleClickItemForView(FItem item)
{
	OnInspect(item);
}

void STextureGraphInsightDeviceListView::OnInspect(FItem item)
{
	if (item->_recordID.IsDevice())
	{
		if (item->_metaItem > 0)
			return;
		if (!_inspectDevices)
			return;
	}

	auto record = item->_recordID;
	TextureGraphInsight::Instance()->GetSession()->SendToInspector(record);
}

void STextureGraphInsightDeviceListView::OnDeviceBufferAdded(const RecordIDArray& rids)
{
	for (const auto& rid : rids)
	{
		if (!rid.IsBuffer() || (rid.Buffer() == INVALID_INDEX))
			continue;

		auto i = rid.Buffer_DeviceType();
		FItem item = MakeShareable(new FItemData(rid));
		_rootItems[i]->_children.Add(item);
	}


	RefreshRootItems();

	_tableView->RequestListRefresh();
}

void STextureGraphInsightDeviceListView::OnDeviceBufferRemoved(const RecordIDArray& rids)
{
	for (const auto& rid : rids)
	{
		if (!rid.IsBuffer() || (rid.Buffer() == INVALID_INDEX))
			continue;

		// First let's find the position of the record in the existing model:
		auto& root = _rootItems[rid.Buffer_DeviceType()];
		auto& children = root->_children;

		auto index = children.IndexOfByKey(rid);
		if (index == INDEX_NONE)
			continue; /// This should not happen

		// Then let's move the removed item to the sub group in removedRoot
		// First make sure the removedRoot exists
		if (!root->_removedRoot)
		{
			root->_removedRoot = MakeShareable(new FItemData(root->_recordID, 1));
			//_tableView->SetItemExpansion(root->_removedRoot, true);
		}

		// Then move the item there
		root->_removedRoot->_children.Add(children[index]);
		children.RemoveAt(index);
	}


	RefreshRootItems();

	_tableView->RequestListRefresh();

}

void STextureGraphInsightDeviceListView::OnEngineReset(int id)
{
	Reset();
}

bool operator <(const STextureGraphInsightDeviceListView::FItem& a, const  STextureGraphInsightDeviceListView::FItem& b)
{
	return ((uint32) a->_rank) < ((uint32) b->_rank);
}

void STextureGraphInsightDeviceListView::RefreshRootItems()
{
	for (auto& i: _rootItems)
	{

		for (auto& c : i->_children)
		{
			const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBuffer(c->_recordID);
			c->_rank = br.Rank;
		}
		i->_children.Sort();
			
		if (i->_widget)
		{
			i->_widget->Refresh();

			for (auto& c : i->_children)
			{
				if (c->_widget)
				{
					c->_widget->Refresh();
				}
			}
		}
		if (i->_removedRoot && i->_removedRoot->_widget)
		{
			i->_removedRoot->_widget->Refresh();
		}
	}
}

void STextureGraphInsightDeviceView::Construct(const FArguments& Args)
{
	_deviceType = (DeviceType) Args._deviceType;
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Text(FText::FromString(Device::DeviceType_Names(_deviceType)))
			]
			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Text(FText::FromString("Threads"))
				.Justification(ETextJustify::Right)
			]
			+ SHorizontalBox::Slot()
			[
				SAssignNew(_threadCount, STextBlock)
				.Text(FText::FromString("__"))
				.AutoWrapText(true)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Text(FText::FromString("Memory"))
			]
			+ SHorizontalBox::Slot()
			[
				SAssignNew(_mem_percent, STextBlock)
				.Text(FText::FromString("___%"))
				.AutoWrapText(true)
				.Justification(ETextJustify::Right)
			]
			+ SHorizontalBox::Slot()
			[
				SAssignNew(_mem_number, STextBlock)
				.Text(FText::FromString("______ Kb"))
				.AutoWrapText(true)
				.Justification(ETextJustify::Right)
			]
			+ SHorizontalBox::Slot()
			[
				SAssignNew(_mem_budget, STextBlock)
				.Text(FText::FromString("______ Kb"))
				.AutoWrapText(true)
				.Justification(ETextJustify::Right)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Text(FText::FromString("Buffers"))
			]
			+ SHorizontalBox::Slot()
			[
				SAssignNew(_buffer_percent, STextBlock)
				.Text(FText::FromString("___%"))
			.AutoWrapText(true)
			.Justification(ETextJustify::Right)
			]
			+ SHorizontalBox::Slot()
			[
				SAssignNew(_buffer_number, STextBlock)
				.Text(FText::FromString("______"))
			.AutoWrapText(true)
			.Justification(ETextJustify::Right)
			]
			+ SHorizontalBox::Slot()
			[
				SAssignNew(_buffer_budget, STextBlock)
				.Text(FText::FromString("______"))
			.AutoWrapText(true)
			.Justification(ETextJustify::Right)
			]
		]
	];
		
}

void STextureGraphInsightDeviceView::Refresh()
{
	auto device = TextureGraphEngine::GetInstance()->GetDeviceManager()->GetDevice(_deviceType);
	_threadCount->SetText(FText::FromString(FString::FromInt(device->GetMaxThreads())));

	auto memMax = device->GetMaxMemory();
	auto memUsed = device->GetMemUsed();
	auto memPercent = 0;
	if (memMax > 0)
		memPercent = (memUsed * 100) / memMax;

	auto bufferMax = device->GetNumDeviceBuffersMax();
	auto bufferUsed = device->GetNumDeviceBuffersUsed();
	auto bufferPercent = 0;
	if (bufferMax > 0)
		bufferPercent = (bufferUsed * 100) / bufferMax;


	_mem_percent->SetText(FText::FromString(FString::FromInt(memPercent)));
	_mem_number->SetText(FText::FromString(FString::FromInt(memUsed / (1024 * 1024)) + " Mb"));
	_mem_budget->SetText(FText::FromString(FString::FromInt(memMax / (1024 * 1024)) + " Mb"));

	_buffer_percent->SetText(FText::FromString(FString::FromInt(bufferPercent)));
	_buffer_number->SetText(FText::FromString(FString::FromInt(bufferUsed)));
	_buffer_budget->SetText(FText::FromString(FString::FromInt(bufferMax)));

	
}



void STextureGraphInsightDeviceManagerView::Construct(const FArguments& Args)
{
	// install the observer notifications
	auto sr = StaticCastSharedRef<STextureGraphInsightDeviceManagerView>(this->AsShared());
	TextureGraphInsight::Instance()->GetSession()->OnUpdateIdle().AddSP(sr, &STextureGraphInsightDeviceManagerView::OnUpdate);
	TextureGraphInsight::Instance()->GetSession()->OnEngineReset().AddSP(sr, &STextureGraphInsightDeviceManagerView::OnEngineReset);
}

void STextureGraphInsightDeviceManagerView::Reset()
{
	_deviceViews.clear();

	TSharedPtr<SVerticalBox> vbox;
	ChildSlot
	[
		SAssignNew(vbox, SVerticalBox)
	]; 

	if (TextureGraphEngine::GetInstance())
	{
		for (int i = 0; i < (int)DeviceType::Count; ++i)
		{
			//if (i == (int)DeviceType::Null)
			//	continue;

			auto deviceType = (DeviceType)i;
			auto device = TextureGraphEngine::GetInstance()->GetDeviceManager()->GetDevice(deviceType);
			if (device)
			{
				TSharedPtr< STextureGraphInsightDeviceView> deviceView;
				vbox->AddSlot()
					[
						SAssignNew(deviceView, STextureGraphInsightDeviceView)
						.deviceType(deviceType)
					];

				_deviceViews.push_back(deviceView);
			}
		}
	}
}

void STextureGraphInsightDeviceManagerView::OnUpdate()
{
	for (auto& d : _deviceViews)
	{
		d->Refresh();
	}
}

void STextureGraphInsightDeviceManagerView::OnEngineReset(int32 id)
{
	Reset();
}