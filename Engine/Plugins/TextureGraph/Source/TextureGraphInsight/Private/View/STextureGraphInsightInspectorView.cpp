// Copyright Epic Games, Inc. All Rights Reserved.
#include "View/STextureGraphInsightInspectorView.h"

#include "2D/Tex.h"
#include "TextureGraphInsight.h"
#include "Model/TextureGraphInsightSession.h"
#include "View/STextureGraphInsightBlobView.h"
#include "View/STextureGraphInsightDeviceBufferView.h"
#include "View/STextureGraphInsightDeviceView.h"
#include "View/STextureGraphInsightRecordTrailView.h"
#include "View/STextureGraphInsightSessionView.h"
#include "Widgets/SNullWidget.h"

#include "Device/FX/DeviceBuffer_FX.h"
#include <Widgets/Layout/SHeader.h>


void STextureGraphInsightBatchInspectorView::Construct(const FArguments& Args)
{
	_rootItems.Empty(0);

	InspectBatch(Args._recordID);
};

TSharedRef<ITableRow> STextureGraphInsightBatchInspectorView::OnGenerateTileForView(FItem item, const TSharedRef<STableViewBase>& OwnerTable)
{
	const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBatch(item->_batchID);

	RecordID targetRID = br.ResultBlobs[item->_targetIndex];
	FString targetTypeName = TextureHelper::TextureTypeToString((TextureType) item->_targetIndex);
	FString label = "Target " + targetTypeName + " : " + FString::FromInt(targetRID.Blob());
	return SNew(STableRow<FItem>, OwnerTable)
		.Padding(4)
		.Content()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHeader)
				[
					SNew(STextBlock)
					.Text(FText::FromString(label))
				]
			]
			+ SVerticalBox::Slot()
			[
				SNew(STextureGraphInsightBlobView)
				.recordID(targetRID)
			]
		];
}

void STextureGraphInsightBatchInspectorView::InspectBatch(RecordID batchID)
{
	if (!batchID.IsBatch())
	{
		batchID = RecordID();
	}

	_rootItems.Empty();

	const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBatch(batchID);

	int32 resultIdx = 0;
	for (auto inputBlobId : br.ResultBlobs)
	{
		if (inputBlobId.IsValid())
			_rootItems.Add(MakeShareable(new FItemData(batchID, resultIdx)));
		resultIdx++;
	}

	ChildSlot
		[
			SNew(SSplitter)
			.Orientation(Orient_Vertical)
		+ SSplitter::Slot()
		.Value(3)
		[

			SAssignNew(_view, SItemTileView)
			.ListItemsSource(&_rootItems)
		.ItemWidth(256)
		.ItemHeight(256)
		.OnGenerateTile(this, &STextureGraphInsightBatchInspectorView::OnGenerateTileForView)
		]
		];


	_view->RequestListRefresh();
}


void STextureGraphInsightJobInspectorView::Construct(const FArguments& Args)
{
	_rootItems.Empty(0);

	InspectJob(Args._recordID);
};

TSharedRef<ITableRow> STextureGraphInsightJobInspectorView::OnGenerateTileForView(FItem item, const TSharedRef<STableViewBase>& OwnerTable)
{
	const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBatch(item->_jobID);
	const auto& jr = br.GetJob(item->_jobID);

	RecordID inputRID = jr.InputBlobIds[item->_inputIndex];

	FString label = jr.InputArgNames[item->_inputIndex] + " : " + FString::FromInt(inputRID.Blob());
	return SNew(STableRow<FItem>, OwnerTable)
		.Padding(4)
		.Content()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHeader)
				[
					SNew(STextBlock)
					.Text(FText::FromString(label))
				]
			]
			+ SVerticalBox::Slot()
			[
				SNew(STextureGraphInsightBlobView)
				.recordID(inputRID)
				.tilesMask(item->_tilesMask)
			]
		];
}

void STextureGraphInsightJobInspectorView::InspectJob(RecordID jobID)
{
	if (!jobID.IsJob())
	{
		jobID = RecordID();
	}

	_rootItems.Empty();

	const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBatch(jobID);
	const auto& jr = br.GetJob(jobID);

	int32 inputIdx = 0;
	for (auto inputBlobId : jr.InputBlobIds)
	{
		if (inputBlobId.IsValid())
			_rootItems.Add(MakeShareable(new FItemData(jobID, inputIdx)));
		inputIdx++;
	}

	ChildSlot
		[
		SNew(SSplitter)
		.Orientation(Orient_Vertical)
		+ SSplitter::Slot()
		.Value(3)
		[
			SAssignNew(_outBlobView, STextureGraphInsightBlobView)
			.recordID(jr.ResultBlobId)
			.tilesMask(jr.Tiles)
		]
		+ SSplitter::Slot()
		[
			SAssignNew(_view, SItemTileView)
			.ListItemsSource(&_rootItems)
			.ItemWidth(256)
			.ItemHeight(256)
			.OnGenerateTile(this, &STextureGraphInsightJobInspectorView::OnGenerateTileForView)
		]
		];


	_view->RequestListRefresh();
}


void STextureGraphInsightBlobInspectorView::Construct(const FArguments& Args)
{
	InspectBlob(Args._recordID);
};


void STextureGraphInsightBlobInspectorView::InspectBlob(RecordID blobID)
{
	ChildSlot
		[
			SAssignNew(_blobView, STextureGraphInsightBlobView)
				.recordID(blobID)
		];
}

void STextureGraphInsightDeviceBufferInspectorView::Construct(const FArguments& Args)
{
	InspectBuffer(Args._recordID);
};


void STextureGraphInsightDeviceBufferInspectorView::InspectBuffer(RecordID rid)
{
	if (rid.IsDevice())
	{
		ChildSlot
		[
			SNew(STextureGraphInsightDeviceView)
			.deviceType((DeviceType) rid.Buffer_DeviceType())
		];
	}
	else
	{
		ChildSlot
		[
			SNew(STextureGraphInsightDeviceBufferView)
			.recordID(rid)
			.withDescription(true)
		];
	}
}


void STextureGraphInsightInspectorView::Construct(const FArguments& Args)
{
	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(_recordTrail, STextureGraphInsightRecordTrailView)
			]
			+ SVerticalBox::Slot()
			[
				SAssignNew(_stack, SOverlay)
			]
		];

	// install the observer notifications
	auto sr = StaticCastSharedRef<STextureGraphInsightInspectorView>(this->AsShared());
	TextureGraphInsight::Instance()->GetSession()->OnBatchInspected().AddSP(sr, &STextureGraphInsightInspectorView::OnInspected);
	TextureGraphInsight::Instance()->GetSession()->OnJobInspected().AddSP(sr, &STextureGraphInsightInspectorView::OnInspected);
	TextureGraphInsight::Instance()->GetSession()->OnBlobInspected().AddSP(sr, &STextureGraphInsightInspectorView::OnInspected);
	TextureGraphInsight::Instance()->GetSession()->OnBufferInspected().AddSP(sr, &STextureGraphInsightInspectorView::OnInspected);

	TextureGraphInsight::Instance()->GetSession()->OnEngineReset().AddSP(sr, &STextureGraphInsightInspectorView::OnEngineReset);
};


void STextureGraphInsightInspectorView::OnInspected(RecordID rid)
{
	_recordTrail->Refresh(rid);
	_stack->ClearChildren();

	if (rid.IsBuffer())
	{
		_stack->AddSlot()
			[
				SNew(STextureGraphInsightDeviceBufferInspectorView)
				.recordID(rid)
			];
	}
	else if (rid.IsBlob())
	{
		_stack->AddSlot()
			[
				SNew(STextureGraphInsightBlobInspectorView)
				.recordID(rid)
			];
	}
	else if (rid.IsJob())
	{
		_stack->AddSlot()
			[
				SNew(STextureGraphInsightJobInspectorView)
				.recordID(rid)
			];
	}
	else if (rid.IsBatch())
	{
		_stack->AddSlot()
			[
				SNew(STextureGraphInsightBatchInspectorView)
				.recordID(rid)
			];
	}
}


void STextureGraphInsightInspectorView::OnEngineReset(int32 id)
{
	// clear the inspector _trail and _stack when engine is destroyed
	_recordTrail->Refresh(RecordID());
	_stack->ClearChildren();
}
