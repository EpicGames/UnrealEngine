// Copyright Epic Games, Inc. All Rights Reserved.
#include "View/STextureGraphInsightMixView.h"

#include "TextureGraphInsight.h"
#include "Model/TextureGraphInsightSession.h"

#include "Device/DeviceManager.h"
#include "Data/Blobber.h"
#include "TextureGraphEngine.h"


class STextureGraphInsightMixListViewRow : public SMultiColumnTableRow<STextureGraphInsightMixListView::FItem>
{
public:
	using FItem = STextureGraphInsightMixListView::FItem;

	SLATE_BEGIN_ARGS(STextureGraphInsightMixListViewRow) {}
	SLATE_ARGUMENT(FItem, Item)
		SLATE_END_ARGS()

public:

	enum Column {
		Main = 0,
		Name,
		Parent,
		Instances,

		NUM_COLUMNS,
	};
	using ColumnInfo = std::pair<FName, float>;
	static ColumnInfo s_columnNames[NUM_COLUMNS];
	static Column NameToColumn(const FName& name) {
		for (int i = 0; i < NUM_COLUMNS; ++i)
		{
			if (name == s_columnNames[i].first)
				return Column(i);
		}
		return NUM_COLUMNS;
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& columnName) override
	{
		if (_recordID.IsValid()) {

			Column column = NameToColumn(columnName);

			if (column == Main) {
				// Rows in a TreeView need an expander button and some indentation
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Fill)
					[
						SNew(SExpanderArrow, SharedThis(this))
						.StyleSet(ExpanderStyleSet)
					]
				+ SHorizontalBox::Slot()
					.FillWidth(1)
					.Padding(FMargin(4.0f, 0.0f))
					.VAlign(VAlign_Center)
					[
						SAssignNew(_textBoxes[column], STextBlock)
						.Text(GetTextForColumn(column))
					];
			} else
			{
				return SNew(SBox)
					.Padding(FMargin(4.0f, 0.0f))
					.VAlign(VAlign_Center)
					[
						SAssignNew(_textBoxes[column], STextBlock)
						.Text(GetTextForColumn(column))
					.OnDoubleClicked(this, &STextureGraphInsightMixListViewRow::OnDoubleClickedResultColumn)
					];
			}
		}
		// default to null widget if property cannot be found
		return SNullWidget::NullWidget;
	}

	FText GetTextForColumn(Column column) const
	{
		FString s;
		if (_recordID.IsMix())
		{
			const auto& mr = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetMix(_recordID);

			switch (column)
			{
			case Main:
				s = "Mix " + FString::FromInt(_recordID.Mix());
				break;

			case Name:
				s = mr.Name;
				break;

			case Parent:
				if (mr.IsInstanceMix())
				{
					s = "Mix " + FString::FromInt(mr.ParentMixID.Mix());
				}
				break;

			case Instances:
				if (!mr.IsInstanceMix())
				{
					s = FString::FromInt(mr.InstanceMixIDs.size());
				}
				break;
			}
		}
		else if (_recordID.IsBatch())
		{
			const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBatch(_recordID);
			switch (column)
			{
			case Main:
				s = "Batch " + FString::FromInt(_recordID.Batch());
				break;

			case Name:
				s = br.Action;
				break;

			case Parent:
				break;
			case Instances:
				break;
			}
		}
		return FText::FromString(s);
	}


	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		_recordID = InArgs._Item->_recordID;
		SMultiColumnTableRow<FItem >::Construct(FSuperRowType::FArguments(), InOwnerTableView);

		Refresh();
	}

	void Refresh()
	{
		const auto& mr = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetMix(_recordID);	
		_textBoxes[Name]->SetText(GetTextForColumn(Name));
	}

	FReply OnDoubleClickedResultColumn(/** The geometry of the widget*/
		const FGeometry&,
		/** The Mouse Event that we are processing */
		const FPointerEvent&)
	{
		TextureGraphInsight::Instance()->GetSession()->SendToInspector(_recordID);
		return FReply::Handled();
	}

protected:
	RecordID _recordID;
	TSharedPtr<STextBlock> _textBoxes[NUM_COLUMNS];
};

STextureGraphInsightMixListViewRow::ColumnInfo STextureGraphInsightMixListViewRow::s_columnNames[] = {
	{ FName(TEXT("Id")), 0.1},    // main
	{ FName(TEXT("Name")), 0.5},
	{ FName(TEXT("Parent")), 0.2},
	{ FName(TEXT("Instances")), 0.2},
};

void STextureGraphInsightMixListView::Construct(const FArguments& Args)
{

	TSharedPtr<SHeaderRow> headerRow = SNew(SHeaderRow);

	for (int i = 0; i < STextureGraphInsightMixListViewRow::NUM_COLUMNS; i++)
	{
		headerRow->AddColumn(
			SHeaderRow::Column(STextureGraphInsightMixListViewRow::s_columnNames[i].first)
			.DefaultLabel(FText::FromString(STextureGraphInsightMixListViewRow::s_columnNames[i].first.ToString()))
			.FillWidth(STextureGraphInsightMixListViewRow::s_columnNames[i].second)
		);
	}

	ChildSlot
		[
			SAssignNew(_tableView, SItemTableView)
		//.ListItemsSource(&_rootItems)
		.TreeItemsSource(&_rootItems)
		.OnGenerateRow(this, &STextureGraphInsightMixListView::OnGenerateRowForView)
		.OnGetChildren(this, &STextureGraphInsightMixListView::OnGetChildrenForView)

		.OnMouseButtonDoubleClick(this, &STextureGraphInsightMixListView::OnDoubleClickItemForView)
		.OnMouseButtonClick(this, &STextureGraphInsightMixListView::OnClickItemForView)
		.HeaderRow(headerRow)
		];

	// install the observer notifications
	auto sr = StaticCastSharedRef<STextureGraphInsightMixListView>(this->AsShared());
	TextureGraphInsight::Instance()->GetSession()->OnMixAdded().AddSP(sr, &STextureGraphInsightMixListView::OnMixNew);
	TextureGraphInsight::Instance()->GetSession()->OnMixUpdated().AddSP(sr, &STextureGraphInsightMixListView::OnMixUpdate);
	TextureGraphInsight::Instance()->GetSession()->OnEngineReset().AddSP(sr, &STextureGraphInsightMixListView::OnEngineReset);

}

TSharedRef<ITableRow> STextureGraphInsightMixListView::OnGenerateRowForView(FItem item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SAssignNew(item->_widget, STextureGraphInsightMixListViewRow, OwnerTable).Item(item);
}

void STextureGraphInsightMixListView::OnClickItemForView(FItem item)
{
	// Item clicked, let's send it to the inspector
	auto record = item->_recordID;
	TextureGraphInsight::Instance()->GetSession()->SendToInspector(record);
}
void STextureGraphInsightMixListView::OnDoubleClickItemForView(FItem item)
{
	// Item double clicked, let's send it to the inspector
	auto record = item->_recordID;
	TextureGraphInsight::Instance()->GetSession()->SendToInspector(record);
}

void STextureGraphInsightMixListView::OnMixNew(RecordID rid)
{
	const auto& mr = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetMix(rid);

	FItem item = MakeShareable(new FItemData(rid));

	_rootItems.Add(item);

	UpdateItemData(*_rootItems.Last());

	_tableView->RequestListRefresh();
}

bool operator ==(const STextureGraphInsightMixListView::FItem& a, const  RecordID& i)
{
	return a->_recordID.id == i.id;
}

void STextureGraphInsightMixListView::RefreshRootItems()
{
	for (auto& i : _rootItems)
	{
		if (i->_widget)
		{
			i->_widget->Refresh();

			for (auto& c : i->_children)
			{
				if (c->_widget)
				{
					c->_widget->Refresh();
				}
			}
		}
	}
}

void STextureGraphInsightMixListView::OnMixUpdate(RecordID rid)
{
	auto itemPtr = _rootItems.FindByKey(rid);
	if (itemPtr)
	{
		UpdateItemData(*(* itemPtr));
	}

	_tableView->RequestListRefresh();
}

void STextureGraphInsightMixListView::UpdateItemData(FItemData& item)
{
	if (item._recordID.IsMix())
	{
		const auto& mr = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetMix(item._recordID);

		for (int i = 0; i < mr.Batches.size(); ++i)
		{
			auto bid = mr.Batches[i];
			if (i < item._children.Num() && item._children[i]->_recordID.id == bid.id)
			{
			}
			else
			{
				if (i < item._children.Num())
				{
					// erase the existing array starting at i on the first not equal item
					item._children.SetNum(i);
				}

				auto newItem = MakeShareable(new FItemData(bid));
				item._children.Push(newItem);
				UpdateItemData(*item._children.Last());
			}
		}

		if (item._widget)
		{
			item._widget->Refresh();
		}
	}
}

void STextureGraphInsightMixListView::OnEngineReset(int id)
{
	_rootItems.Empty();
	_tableView->RequestListRefresh();
}

