// Copyright Epic Games, Inc. All Rights Reserved.
#include "View/STextureGraphInsightRecordTrailView.h"

#include "2D/Tex.h"
#include "TextureGraphInsight.h"
#include "Model/TextureGraphInsightSession.h"
#include "Widgets/SNullWidget.h"

void STextureGraphInsightRecordTrailView::Construct(const FArguments& Args)
{
	ChildSlot
	[
		SAssignNew(_trail, SBreadcrumbTrail<RecordID>)
		.OnCrumbClicked(this, &STextureGraphInsightRecordTrailView::OnCrumbClicked)
	];

	Refresh(Args._recordID);
}


RecordID STextureGraphInsightRecordTrailView::FindRootRecord(RecordID rid)
{
	if (rid.IsBuffer())
	{
		const auto& bufferRecord = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBuffer(rid);

		return TextureGraphInsight::Instance()->GetSession()->GetRecord().FindBlobRecord(bufferRecord.HashValue);
	}
	else if (rid.IsBlob())
	{
		const auto& blobRecord = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBlob(rid);
		return blobRecord.SourceID;
	}
	else if (rid.IsJob())
	{
		return RecordID::fromBatch(rid.Batch());
	}
	else if (rid.IsBatch())
	{
		const auto& batchRecord = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBatch(rid);
	}
	else if (rid.IsAction())
	{
	}

	return RecordID();
}

void STextureGraphInsightRecordTrailView::Refresh(RecordID rid)
{
	_trail->ClearCrumbs(true);

	//build a trail of rid navigating the records
	RecordIDArray trail;
	do
	{
		trail.emplace_back(rid);
		rid = FindRootRecord(rid);
	} while (rid.IsValid());

	// then create the crumbs as needed
	while (!trail.empty())
	{
		auto id = trail.back();
		trail.pop_back();

		if (id.IsBuffer())
		{
			const auto& bufferRecord = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBuffer(id);
			_trail->PushCrumb(FText::FromString("Buffer " + HashToFString(bufferRecord.HashValue)), id);
		}
		else if (id.IsBlob())
		{
			const auto& blobRecord = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBlob(id);
			_trail->PushCrumb(FText::FromString("Blob " + HashToFString(blobRecord.HashValue)), id);
		}
		else if (id.IsJob())
		{
			const auto& jobRecord = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetJob(id);
			_trail->PushCrumb(FText::FromString(jobRecord.TransformName), id);
		}
		else if (id.IsBatch())
		{
			const auto& batchRecord = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBatch(id);
			_trail->PushCrumb(FText::FromString("Batch " + FString::FromInt(batchRecord.BatchID)), id);
		}
		else if (id.IsAction())
		{
			const auto& actionRecord = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetAction(id);
			_trail->PushCrumb(FText::FromString("Action " + actionRecord.Name), id);
		}
		else // Invalid
		{
			_trail->PushCrumb(FText::FromString(FString("Null")), RecordID());
		}
	}
}

void STextureGraphInsightRecordTrailView::OnCrumbClicked(const RecordID& rid)
{
	TextureGraphInsight::Instance()->GetSession()->SendToInspector(rid);
}
