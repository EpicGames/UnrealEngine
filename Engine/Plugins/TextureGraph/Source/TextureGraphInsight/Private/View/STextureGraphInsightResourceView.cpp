// Copyright Epic Games, Inc. All Rights Reserved.
#include "View/STextureGraphInsightResourceView.h"

#include "TextureGraphInsight.h"
#include "Model/TextureGraphInsightSession.h"

#include "Device/DeviceManager.h"
#include "Data/Blobber.h"
#include "TextureGraphEngine.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/SBoxPanel.h"


class STextureGraphInsightResourceViewRow : public SMultiColumnTableRow<STextureGraphInsightResourceView::FItem>
{
public:
	using FItem = STextureGraphInsightResourceView::FItem;

	SLATE_BEGIN_ARGS(STextureGraphInsightResourceViewRow) {}
	SLATE_ARGUMENT(FItem, Item)
		SLATE_END_ARGS()

public:

	enum Column {
		Main = 0,
		Name,
		Hash,
		RefCount,
		NumMapped,
		Mapped,
		Tiles,
		Pixels,

		NUM_COLUMNS,
	};
	using ColumnInfo = std::pair<FName, float>;
	static ColumnInfo s_columnNames[NUM_COLUMNS];
	static Column NameToColumn(const FName& name) {
		for (int i = 0; i < NUM_COLUMNS; ++i)
		{
			if (name == s_columnNames[i].first)
				return Column(i);
		}
		return NUM_COLUMNS;
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& columnName) override
	{
		if (_recordID.IsValid()) {

			Column column = NameToColumn(columnName);

			if (column == Main) {
				// Rows in a TreeView need an expander button and some indentation
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Fill)
					[
						SNew(SExpanderArrow, SharedThis(this))
						.StyleSet(ExpanderStyleSet)
					]
				+ SHorizontalBox::Slot()
					.FillWidth(1)
					.Padding(FMargin(4.0f, 0.0f))
					.VAlign(VAlign_Center)
					[
						SAssignNew(_textBoxes[column], STextBlock)
						.Text(GetTextForColumn(column))
					];
			}
			else
			{
				return SNew(SBox)
					.Padding(FMargin(4.0f, 0.0f))
					.VAlign(VAlign_Center)
					[
						SAssignNew(_textBoxes[column], STextBlock)
						.Text(GetTextForColumn(column))
					.OnDoubleClicked(this, &STextureGraphInsightResourceViewRow::OnDoubleClickedResultColumn)
					];
			}
		}
		// default to null widget if property cannot be found
		return SNullWidget::NullWidget;
	}

	FText GetTextForColumn(Column column) const
	{
		const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBlob(_recordID);

		FString s;
		switch (column)
		{
		case Main:
				s = FString::FromInt(_recordID.Blob());
				if (br.ReplayCount > 0)
					s += " ...#" + FString::FromInt(br.ReplayCount);
			break;
		case Hash:
			s = HashToFString(br.HashValue);
			break;
		case Name:
				s = br.Name;
			break;
		case RefCount:
			{
				auto refCount = TextureGraphInsight::Instance()->GetSession()->GetCache().GetBlob(_recordID).use_count();
				s = FString::FromInt(refCount);
			}
			break;
		case NumMapped:
			if (br.NumMapped > 0)
				s = FString::FromInt(br.NumMapped);
			break;
		case Mapped:
			if (br.MappedID.IsValid())
				s = FString::FromInt(br.MappedID.Blob());
			break;
		case Tiles:
			if (br.NumTiles())
			{
				s = (br.Grid().IsUnique() ? "1" : FString::FromInt(br.Grid().Rows()) + " x " + FString::FromInt(br.Grid().Cols()));
			}
			break;
		case Pixels:
			if (br.TexWidth || br.TexHeight)
			{
				s = FString::FromInt(br.TexWidth) + " x " + FString::FromInt(br.TexHeight);
			}
			break;
		}

		return FText::FromString(s);
	}


	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		_recordID = InArgs._Item->_recordID;
		SMultiColumnTableRow<FItem >::Construct(FSuperRowType::FArguments(), InOwnerTableView);

		Refresh();
	}

	void Refresh()
	{
		const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBlob(_recordID);
		if (br.ReplayCount > 0)
			SetColorAndOpacity(FLinearColor(1, 0, 0));
		if (br.MappedID.IsValid())
			SetColorAndOpacity(FLinearColor(0.5, 0.5, 0.5));
		if (br.NumMapped > 0)
			SetColorAndOpacity(FLinearColor(0.5, 1.0, 1.0));

		_textBoxes[Main]->SetText(GetTextForColumn(Main));
		_textBoxes[RefCount]->SetText(GetTextForColumn(RefCount));
		_textBoxes[NumMapped]->SetText(GetTextForColumn(NumMapped));
		_textBoxes[Mapped]->SetText(GetTextForColumn(Mapped));
		_textBoxes[Tiles]->SetText(GetTextForColumn(Tiles));
		_textBoxes[Pixels]->SetText(GetTextForColumn(Pixels));
	}

	FReply OnDoubleClickedResultColumn(/** The geometry of the widget*/
		const FGeometry&,
		/** The Mouse Event that we are processing */
		const FPointerEvent&)
	{
		TextureGraphInsight::Instance()->GetSession()->SendToInspector(_recordID);
		return FReply::Handled();
	}

protected:
	RecordID _recordID;
	TSharedPtr<STextBlock> _textBoxes[NUM_COLUMNS];
};

STextureGraphInsightResourceViewRow::ColumnInfo STextureGraphInsightResourceViewRow::s_columnNames[] = {
	{ FName(TEXT("Blob Id")), 0.05},    // main
	{ FName(TEXT("Name")), 0.3},
	{ FName(TEXT("Hash")), 0.1},
	{ FName(TEXT("RefCount")), 0.05},
	{ FName(TEXT("NumMapped")), 0.05},
	{ FName(TEXT("Mapped")), 0.05},
	{ FName(TEXT("Tiles")), 0.05},
	{ FName(TEXT("Pixels")), 0.05},
};

void STextureGraphInsightResourceView::Construct(const FArguments& Args)
{

	TSharedPtr<SHeaderRow> headerRow = SNew(SHeaderRow);

	for (int i = 0; i < STextureGraphInsightResourceViewRow::NUM_COLUMNS; i++)
	{
		headerRow->AddColumn(
			SHeaderRow::Column(STextureGraphInsightResourceViewRow::s_columnNames[i].first)
			.DefaultLabel(FText::FromString(STextureGraphInsightResourceViewRow::s_columnNames[i].first.ToString()))
			.FillWidth(STextureGraphInsightResourceViewRow::s_columnNames[i].second)
		);
	}

	ChildSlot
	[
		SAssignNew(_tableView, SItemTableView)
		//.ListItemsSource(&_rootItems)
		.TreeItemsSource(&_rootItems)
		.OnGenerateRow(this, &STextureGraphInsightResourceView::OnGenerateRowForView)
		.OnGetChildren(this, &STextureGraphInsightResourceView::OnGetChildrenForView)

		.OnMouseButtonDoubleClick(this, &STextureGraphInsightResourceView::OnDoubleClickItemForView)
		.OnMouseButtonClick(this, &STextureGraphInsightResourceView::OnClickItemForView)
		.HeaderRow(headerRow)
	];

	// install the observer notifications
	auto sr = StaticCastSharedRef<STextureGraphInsightResourceView>(this->AsShared());
	TextureGraphInsight::Instance()->GetSession()->OnBlobAdded().AddSP(sr, &STextureGraphInsightResourceView::OnBlobNew);
	TextureGraphInsight::Instance()->GetSession()->OnBlobMapped().AddSP(sr, &STextureGraphInsightResourceView::OnBlobMapped);
	TextureGraphInsight::Instance()->GetSession()->OnEngineReset().AddSP(sr, &STextureGraphInsightResourceView::OnEngineReset);

}

TSharedRef<ITableRow> STextureGraphInsightResourceView::OnGenerateRowForView(FItem item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SAssignNew(item->_widget, STextureGraphInsightResourceViewRow, OwnerTable).Item(item);
}

void STextureGraphInsightResourceView::OnClickItemForView(FItem item)
{
	// Item clicked, let's send it to the inspector
	auto record = item->_recordID;
	TextureGraphInsight::Instance()->GetSession()->SendToInspector(record);
}
void STextureGraphInsightResourceView::OnDoubleClickItemForView(FItem item)
{
	// Item double clicked, let's send it to the inspector
	auto record = item->_recordID;
	TextureGraphInsight::Instance()->GetSession()->SendToInspector(record);
}

void STextureGraphInsightResourceView::OnBlobNew(const RecordIDArray& rids)
{
	for (auto& rid : rids)
	{
		const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBlob(rid);

		if (br.SourceID.IsJob() || !br.SourceID.IsValid())
		{
			FItem item = MakeShareable(new FItemData(rid));
			_rootItems.Add(item);

			if (br.IsTiled())
			{
				for (int i = 0; i < br.NumUniqueBlobs(); ++i)
				{
					item->_children.Add(MakeShareable(new FItemData(br.GetUniqueBlob(i))));
				}
			}
		}
	}

	_tableView->RequestListRefresh();
}


bool operator ==(const STextureGraphInsightResourceView::FItem& a, const  RecordID& i)
{
	return a->_recordID.id == i.id;
}

void STextureGraphInsightResourceView::OnBlobMapped(const RecordIDArray& rids)
{
	RefreshRootItems();
}

		
void STextureGraphInsightResourceView::RefreshRootItems()
{
	for (auto& i : _rootItems)
	{
		if (i->_widget)
		{
			i->_widget->Refresh();

			for (auto& c : i->_children)
			{
				if (c->_widget)
				{
					c->_widget->Refresh();
				}
			}
		}
	}
}

void STextureGraphInsightResourceView::OnEngineReset(int id)
{
	_rootItems.Empty();
	_tableView->RequestListRefresh();
}

