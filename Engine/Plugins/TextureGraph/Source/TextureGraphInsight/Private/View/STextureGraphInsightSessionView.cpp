// Copyright Epic Games, Inc. All Rights Reserved.
#include "View/STextureGraphInsightSessionView.h"

#include "TextureGraphInsight.h"
#include "Model/TextureGraphInsightSession.h"
#include <Widgets/Views/STableViewBase.h>
#include <Widgets/Views/ITableRow.h>
#include <Brushes/SlateColorBrush.h>
#include <Framework/MultiBox/MultiBoxBuilder.h>
#include <Framework/Commands/UIAction.h>


class STextureGraphInsightBatchJobViewRow : public SMultiColumnTableRow<STextureGraphInsightBatchJobView::FItem>
{
public:
	using FItem = STextureGraphInsightBatchJobView::FItem;

	SLATE_BEGIN_ARGS(STextureGraphInsightBatchJobViewRow) {}
	SLATE_ARGUMENT(FItem, Item)
	SLATE_END_ARGS()

public:

	enum Column {
		Main = 0,
		Name,
		Hash,
		Tiles,
		Pixels,
		Fillrate,
		Timeline,

		NUM_COLUMNS,
	};
	using ColumnInfo = std::tuple<FName, float, ETextJustify::Type>;
	static ColumnInfo s_columnNames[NUM_COLUMNS];
	static Column NameToColumn(const FName& name) {
		for (int i = 0; i < NUM_COLUMNS; ++i)
		{
			if (name == std::get<0>(s_columnNames[i]))
				return Column(i);
		}
		return NUM_COLUMNS;
	}

	static FLinearColor GetColorForBatchState(const BatchRecord& br)
	{
		auto color = FLinearColor(1, 0, 0);
		if (br.bIsDone || br.bIsJobsDone)
		{
			if (br.bIsJobsDone)
			{
				color = FLinearColor(1, 1, 1); // Back to normal
			}
			else
			{
				color = FLinearColor(1, 0.5, 0); // Waiting on jobs
			}
		}

		if (br.bIsNoCache)
			color *= 0.6f;

		return color;
	}

	static FLinearColor GetColorForJobState(bool isDone, bool isNoOp, bool isMainPhase)
	{
		if (!isMainPhase)
		{
			return FLinearColor(0.5f, 0.5f, 0.8f); // Not main phase
		}
		else if (isNoOp)
		{
			return FLinearColor(0.5f, 0.8f, 0.5f); // No op
		}
		else
		{
			return FLinearColor(1, 1, 1); // Back to normal
		}
	}

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		_recordID = InArgs._Item->_recordID;
		_isMainPhase = InArgs._Item->_isMainPhase;

		const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBatch(_recordID);
		if (_recordID.IsBatch())
		{
			_accentColor = GetColorForBatchState(br);
		}

		if (_recordID.IsJob())
		{
			const auto& jr = br.GetJob(_recordID);
			_accentColor = GetColorForJobState(jr.bIsDone, jr.IsNoOp(), jr.IsMainPhase());
		}

		SMultiColumnTableRow<FItem >::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& columnName) override
	{
		static FSlateBrush* s_timelineBrush = new FSlateColorBrush(FColor(125, 125, 255));
		static FSlateBrush* s_timelineBrush2 = new FSlateColorBrush(FColor(125,  255, 125));
		if (_recordID.IsValid()) {


			Column column = NameToColumn(columnName);
			if (column == Main) {
				// Rows in a TreeView need an expander button and some indentation
				return SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Fill)
						[
							SNew(SExpanderArrow, SharedThis(this))
							.StyleSet(ExpanderStyleSet)
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1)
						.Padding(FMargin(4.0f, 0.0f))
						.VAlign(VAlign_Center)
						[
							SAssignNew(_textBoxes[column], STextBlock)
							.Text(GetTextForColumn(column))
							.ColorAndOpacity(_accentColor)
						];
			}
			else if (column == Timeline)
			{

				auto& sr = TextureGraphInsight::Instance()->GetSession()->GetRecord();
				const auto& br = sr.GetBatch(_recordID);
				const auto& jr = br.GetJob(_recordID);

				if (_recordID.IsJob())
				{
					auto positionPre = FVector2D((jr.GetBeginTimeMS() - br.BeginTimeMS) / br.ScopeTime_ms(), 0.7);
					auto sizePre = FVector2D(jr.GetPreRunTimeMS() / br.ScopeTime_ms(), 0.2);

					auto position = FVector2D((jr.GetBeginRunTimeMS() - br.BeginTimeMS) / br.ScopeTime_ms(), 0.1);
					auto sizeRun = FVector2D(jr.GetRunTimeMS() / br.ScopeTime_ms(), 0.7);
				
					return	
						SNew(SVerticalBox)

						// First row
						// Display time top, justified aright at the end of the runtime
						+SVerticalBox::Slot()
						.FillHeight(sizeRun.Y)
						
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.FillWidth(position.X)

							+ SHorizontalBox::Slot()
							.FillWidth(1 - position.X)
							.Padding(FMargin(2.0f, 0.0f))
							.VAlign(EVerticalAlignment::VAlign_Top)
							[
								SAssignNew(_textBoxes[column], STextBlock)
								.Text(GetTextForColumn(column))
								//.Justification(ETextJustify::Right)
								.Justification(ETextJustify::Left)
								.ColorAndOpacity(_accentColor)
							]
						
						]

						// Second row
						// Display rectangle for pretime, then for runtime
						+SVerticalBox::Slot()
						.FillHeight(sizePre.Y)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.FillWidth(positionPre.X)
						
							+ SHorizontalBox::Slot()
							.FillWidth(sizePre.X)
							[
								SNew(SBorder)
								.BorderImage(s_timelineBrush2)
							]

							+ SHorizontalBox::Slot()
							.FillWidth(sizeRun.X)
							[
								SNew(SBorder)
								.BorderImage(s_timelineBrush)
							]

							+ SHorizontalBox::Slot()
							.FillWidth(1.0 - position.X - sizeRun.X)
						]

						+ SVerticalBox::Slot()
						.FillHeight(1.0 - sizeRun.Y - sizePre.Y)
						;
				} else {
					auto sizeX = 0.5;
					auto sizeY = 0.1;
					auto positionY = 0.45;

					return	SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(sizeX)
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.FillHeight(positionY)
							+ SVerticalBox::Slot()
							.FillHeight(sizeY)
							[
								SNew(SBorder)
								.BorderImage(s_timelineBrush)
							]
							+ SVerticalBox::Slot()
							.FillHeight(1.0 - positionY - sizeY)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.0)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.HAlign(HAlign_Center)
						[
							SAssignNew(_textBoxes[column], STextBlock)
							.Text(GetTextForColumn(Timeline))
							.ColorAndOpacity(_accentColor)
						]
						+ SHorizontalBox::Slot()
						.FillWidth(0.5)
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.FillHeight(positionY)
							+ SVerticalBox::Slot()
							.FillHeight(sizeY)
							[
								SNew(SBorder)
								.BorderImage(s_timelineBrush)
							]
							+ SVerticalBox::Slot()
							.FillHeight(1.0 - positionY - sizeY)
						];
				}
			}
			else if (column == Name && _recordID.IsBatch())
			{
				return 	SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(2.0)
					.FillWidth(1)
					[
						SAssignNew(_textBoxes[column], STextBlock)
						.Text(GetTextForColumn(column))
						.Justification(ETextJustify::Left)
						.ColorAndOpacity(_accentColor)
						.OnDoubleClicked(this, &STextureGraphInsightBatchJobViewRow::OnDoubleClickedResultColumn)
					];
			}
			else
			{
				return 	SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(2.0)
					.FillWidth(1)
					[	
						SAssignNew(_textBoxes[column], STextBlock)
						.Text(GetTextForColumn(column))
						.Justification(std::get<2>(STextureGraphInsightBatchJobViewRow::s_columnNames[column]))
						.ColorAndOpacity(_accentColor)
						.OnDoubleClicked(this, &STextureGraphInsightBatchJobViewRow::OnDoubleClickedResultColumn)
					];
			}
		}
		// default to null widget if property cannot be found
		return SNullWidget::NullWidget;
	}

	FText GetTextForColumn(Column column) const
	{
		auto& sr = TextureGraphInsight::Instance()->GetSession()->GetRecord();
		const auto& br = sr.GetBatch(_recordID);
		const auto& jr = br.GetJob(_recordID);
		const auto& mr = sr.GetMix(br.MixID);

		FString s;
		switch (column)
		{
		case Main:
			if (_recordID.IsBatch())
			{
				s = (_isMainPhase ? "Batch " : "     Sub batch ") + FString::FromInt(br.BatchID);
				if (br.bIsNoCache)
					s += " - NO CACHE"; 
				if (br.bIsFromIdle)
					s += " - IDLE";
				if (br.ReplayCount > 0)
					s += " ...#" + FString::FromInt(br.ReplayCount);
			}
			else
			{
				s = "Job " + FString::FromInt(jr.JobIdx);
				if (jr.PhaseIdx)
					s += " : " + FString::FromInt(jr.PhaseIdx);
				if (br.ReplayCount > 0)
					s += " ...#" + FString::FromInt(br.ReplayCount);
			}
			break;
		case Name:
			if (_recordID.IsBatch())
			{
				s = mr.Name;
				if (br.Action.Len())
					s += " - " + br.Action;
			}
			else
			{
				s = jr.TransformName;
			}
			break;
		case Hash:
			if (_recordID.IsBatch())
			{
			} else
			{
				s = HashToFString(jr.JobHash);
			}
			break;
		case Tiles:
			if (_recordID.IsBatch())
			{
				s = FString::Printf(TEXT("%8d"), br.NumInvalidatedTiles) + " / " + FString::FromInt(br.NumTiles);
			}
			else
			{
				if (jr.GetNumTiles())
					s = FString::Printf(TEXT("%8d"), jr.GetNumInvalidated()) + " / " + (jr.GetGrid().IsUnique() ? "1" : FString::FromInt(jr.GetGrid().Rows()) + " x " + FString::FromInt(jr.GetGrid().Cols()));
			}
			break;
		case Pixels:
			if (_recordID.IsBatch())
			{
			}
			else
			{
				if (jr.TexWidth || jr.TexHeight)
					s = FString::FromInt(jr.TexWidth) + " x " + FString::FromInt(jr.TexHeight);
			}
			break;
		case Fillrate:
			if (_recordID.IsBatch())
			{
			}
			else
			{
				if (jr.TexWidth || jr.TexHeight)
					s = FString::FromInt(jr.GetFillRate() * 0.000001); /// Fillrate expressed in Million pix per sec
			}
		break;
		case Timeline:
			if (_recordID.IsBatch())
			{
				//s = FString::FromInt(br.ScopeTime_ms());
				s = FString::Printf(TEXT("%.1f"), br.ScopeTime_ms());
			}
			else
			{
				//s = FString::FromInt(jr.RunTime_ms());
				s = FString::Printf(TEXT("%.1f"), jr.GetRunTimeMS());
			}
			break;
		}

		return FText::FromString(s);
	}

	void OnUpdate()
	{
		const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBatch(_recordID);
		SetColorAndOpacity(GetColorForBatchState(br));

		if (_textBoxes[Main])
		{
			_textBoxes[Main]->SetText(GetTextForColumn(Main));
			if (_recordID.IsBatch())
				_textBoxes[Name]->SetText(GetTextForColumn(Name));
			_textBoxes[Tiles]->SetText(GetTextForColumn(Tiles));
			_textBoxes[Pixels]->SetText(GetTextForColumn(Pixels));
			_textBoxes[Fillrate]->SetText(GetTextForColumn(Fillrate));
			_textBoxes[Timeline]->SetText(GetTextForColumn(Timeline));
		}
	}

	FReply OnDoubleClickedResultColumn(/** The geometry of the widget*/
		const FGeometry&,
		/** The Mouse Event that we are processing */
		const FPointerEvent&)
	{
		TextureGraphInsight::Instance()->GetSession()->SendToInspector(_recordID);
		return FReply::Handled();
	}

protected:
	RecordID _recordID;
	TSharedPtr<STextBlock> _textBoxes[NUM_COLUMNS];
	FLinearColor _accentColor = FLinearColor(1, 1, 1);
	bool _isMainPhase = true;
};

STextureGraphInsightBatchJobViewRow::ColumnInfo STextureGraphInsightBatchJobViewRow::s_columnNames[] = {
	{ FName(TEXT("Id")), 0.1, ETextJustify::Left},    // main
	{ FName(TEXT("Name")), 0.15, ETextJustify::Right},
	{ FName(TEXT("Hash")), 0.1, ETextJustify::Center},
	{ FName(TEXT("Tiles")), 0.08, ETextJustify::Left},
	{ FName(TEXT("Pixels")), 0.08, ETextJustify::Center},
	{ FName(TEXT("Fillrate [Mpx/s]")), 0.08, ETextJustify::Right},
	{ FName(TEXT("Timeline [ms]")), 0.3, ETextJustify::Center},
};

void STextureGraphInsightBatchJobView::Construct(const FArguments& Args)
{
	_inspectOnSimpleClick = Args._inspectOnSimpleClick;

	TSharedPtr<SHeaderRow> headerRow = SNew(SHeaderRow);

	for (int i = 0; i < STextureGraphInsightBatchJobViewRow::NUM_COLUMNS; i++)
	{
		headerRow->AddColumn(
			SHeaderRow::Column(std::get<0>(STextureGraphInsightBatchJobViewRow::s_columnNames[i]))
			.DefaultLabel(FText::FromString(std::get<0>(STextureGraphInsightBatchJobViewRow::s_columnNames[i]).ToString()))
			.HAlignHeader(EHorizontalAlignment::HAlign_Center)
			.HAlignCell(EHorizontalAlignment::HAlign_Fill)
			.FillWidth(std::get<1>(STextureGraphInsightBatchJobViewRow::s_columnNames[i]))
		);
	}

	ChildSlot
	[
		
	SAssignNew(_treeView, SItemTreeView)
		.TreeItemsSource(&_rootItems)
		.OnGenerateRow(this, &STextureGraphInsightBatchJobView::OnGenerateRowForTree)
		.OnGetChildren(this, &STextureGraphInsightBatchJobView::OnGetChildrenForView)
		.OnMouseButtonDoubleClick(this, &STextureGraphInsightBatchJobView::OnDoubleClickItemForTree)
		.OnMouseButtonClick(this, &STextureGraphInsightBatchJobView::OnClickItemForTree)
		.OnContextMenuOpening(this, &STextureGraphInsightBatchJobView::OnContextMenuOpeningForTree)
		.HeaderRow(headerRow)
	];

	OnBatchNew(Args._recordID);

	if (Args._recordID.IsBatch())
	{
		_treeView->SetItemExpansion(_rootItems[0], true);
	}
}

TSharedRef<ITableRow> STextureGraphInsightBatchJobView::OnGenerateRowForTree(FItem item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SAssignNew(item->_widget, STextureGraphInsightBatchJobViewRow, OwnerTable).Item(item);
}

void STextureGraphInsightBatchJobView::OnClickItemForTree(FItem item)
{
	// Item clicked, let's send it to the inspector
	auto record = item->_recordID;
	if (_inspectOnSimpleClick)
		TextureGraphInsight::Instance()->GetSession()->SendToInspector(record);
}
void STextureGraphInsightBatchJobView::OnDoubleClickItemForTree(FItem item)
{
	// Item double clicked, let's send it to the inspector
	auto record = item->_recordID;
	TextureGraphInsight::Instance()->GetSession()->SendToInspector(record);
}

TSharedPtr<SWidget> STextureGraphInsightBatchJobView::OnContextMenuOpeningForTree()
{
	// Only if a batch is focused then we would show a context menu
	auto selection = _treeView->GetSelectedItems();
	if (selection.Num())
	{
		if (selection[0]->_recordID.IsBatch())
		{
			return OnContextMenuBatch(selection[0]);
		}
		if (selection[0]->_recordID.IsJob())
		{
			return OnContextMenuJob(selection[0]);
		}
	}

	return nullptr;
}
TSharedPtr<SWidget> STextureGraphInsightBatchJobView::OnContextMenuBatch(FItem batchItem)
{
	const bool closeAfterSelection = true;
	FMenuBuilder menuBuilder(closeAfterSelection, NULL);

	// Begin menu section
	menuBuilder.BeginSection("BatchContextMenu");
	{
		menuBuilder.AddWidget(SNew(STextBlock).Text(FText::FromString("Batch " + FString::FromInt(batchItem->_recordID.Batch()) + " ...")), FText(), true);
		// Create action delegate and Add menu entry
		{
			FUIAction action = FUIAction(FExecuteAction::CreateRaw(this, &STextureGraphInsightBatchJobView::OnReplayBatch, batchItem->_recordID, false));
			menuBuilder.AddMenuEntry(FText::FromString(FString(TEXT("Replay Batch"))), FText(), FSlateIcon(), action);
		}
		{
			FUIAction action = FUIAction(FExecuteAction::CreateRaw(this, &STextureGraphInsightBatchJobView::OnReplayBatch, batchItem->_recordID, true));
			menuBuilder.AddMenuEntry(FText::FromString(FString(TEXT("Replay Batch & Capture RenderDoc"))), FText(), FSlateIcon(), action);
		}
	}
	menuBuilder.EndSection();

	return menuBuilder.MakeWidget();
}


TSharedPtr<SWidget> STextureGraphInsightBatchJobView::OnContextMenuJob(FItem jobItem)
{
	const bool closeAfterSelection = true;
	FMenuBuilder menuBuilder(closeAfterSelection, NULL);

	// Begin menu section
	menuBuilder.BeginSection("JobContextMenu");
	{
		menuBuilder.AddWidget(SNew(STextBlock).Text(FText::FromString("Batch " + FString::FromInt(jobItem->_recordID.Batch()) + " - Job " + FString::FromInt(jobItem->_recordID.Job()) + " ...")), FText(), true);
		// Create action delegate and Add menu entry
		{
			FUIAction action = FUIAction(FExecuteAction::CreateRaw(this, &STextureGraphInsightBatchJobView::OnReplayJob, jobItem->_recordID, false));
			menuBuilder.AddMenuEntry(FText::FromString(FString(TEXT("Replay Job"))), FText(), FSlateIcon(), action);
		}
		{
			FUIAction action = FUIAction(FExecuteAction::CreateRaw(this, &STextureGraphInsightBatchJobView::OnReplayJob, jobItem->_recordID, true));
			menuBuilder.AddMenuEntry(FText::FromString(FString(TEXT("Replay Job & Capture RenderDoc"))), FText(), FSlateIcon(), action);
		}
	}
	menuBuilder.EndSection();

	return menuBuilder.MakeWidget();
}


void STextureGraphInsightBatchJobView::OnReplayBatch(RecordID batchId, bool captureRenderDoc)
{
	if (!batchId.IsBatch())
		return;

	// Trigger a replay!
	bool result = TextureGraphInsight::Instance()->GetSession()->ReplayBatch(batchId, captureRenderDoc);

}

void STextureGraphInsightBatchJobView::OnReplayJob(RecordID jobId, bool captureRenderDoc)
{
	if (!jobId.IsJob())
		return;

	// Trigger a replay!
	bool result = TextureGraphInsight::Instance()->GetSession()->ReplayJob(jobId, captureRenderDoc);

}

void STextureGraphInsightBatchJobView::OnBatchNew(RecordID batchRecordID)
{

	// Fetch the actual data from the record and add job's items
	const auto& br = TextureGraphInsight::Instance()->GetSession()->GetRecord().GetBatch(batchRecordID);

	FItem batchItem = MakeShareable(new FItemData(batchRecordID));
	

	FItem batchItemSubs = MakeShareable(new FItemData(batchRecordID, false));

	uint32_t jobIdx = 0;
	for (const auto& j : br.Jobs)
	{
		if (j.IsMainPhase())
		{
			batchItem->_children.Add(MakeShareable(new FItemData(RecordID::fromBatchJob(batchRecordID.Batch(), jobIdx))));
		}
		else
		{
			batchItemSubs->_children.Add(MakeShareable(new FItemData(RecordID::fromBatchJob(batchRecordID.Batch(), jobIdx), false)));
		}

		jobIdx++;
	}

	batchItem->_children.Add(batchItemSubs);

	_rootItems.Add(batchItem);

	_treeView->RequestTreeRefresh();

}

void STextureGraphInsightBatchJobView::OnBatchUpdate(RecordID batchRecordID)
{
	const auto& bri = _rootItems[batchRecordID.Batch()];
	if (bri->_widget) {
		bri->_widget->OnUpdate();
	}
	for (const auto& j : bri->_children)
	{
		if (j->_widget) {
			j->_widget->OnUpdate();
		}
	}

}

void STextureGraphInsightBatchJobView::OnEngineReset(int32 id)
{
	_rootItems.Empty();
	_treeView->RequestTreeRefresh();
}







void STextureGraphInsightSessionView::Construct(const FArguments& Args)
{
	ChildSlot
	[
		SAssignNew(_batchJobView, STextureGraphInsightBatchJobView)
		.inspectOnSimpleClick(true)
	];


	// install the observer notifications
	auto sr = StaticCastSharedRef<STextureGraphInsightBatchJobView>(this->_batchJobView->AsShared());

	TextureGraphInsight::Instance()->GetSession()->OnBatchAdded().AddSP(sr, &STextureGraphInsightBatchJobView::OnBatchNew);
	TextureGraphInsight::Instance()->GetSession()->OnBatchDone().AddSP(sr, &STextureGraphInsightBatchJobView::OnBatchUpdate);
	TextureGraphInsight::Instance()->GetSession()->OnBatchJobsDone().AddSP(sr, &STextureGraphInsightBatchJobView::OnBatchUpdate);
	TextureGraphInsight::Instance()->GetSession()->OnEngineReset().AddSP(sr, &STextureGraphInsightBatchJobView::OnEngineReset);
}

