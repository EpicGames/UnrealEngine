// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "TextureGraphEngine.h"
#include "Device/Device.h"
#include "Device/DeviceObserverSource.h"
#include "Data/Blobber.h"
#include "Job/Scheduler.h"

#define UE_API TEXTUREGRAPHINSIGHT_API

using HashArray = DeviceObserverSource::HashArray;

DECLARE_LOG_CATEGORY_EXTERN(LogTextureGraphInsightObserver, Log, All);

/// Concrete DeviceObserver interface
class TextureGraphInsightDeviceObserver : public DeviceObserverSource
{
protected:
	DeviceType DevType;

	/// Protected interface of emitters called by the device to notify the observers
	UE_API virtual void DeviceBuffersUpdated(HashNDescArray&& AddedBuffers, HashArray&& RemovedBuffers) override;
public:
	UE_API TextureGraphInsightDeviceObserver();
	UE_API explicit TextureGraphInsightDeviceObserver(DeviceType);
	UE_API virtual ~TextureGraphInsightDeviceObserver() override;
};

/// Concrete BlobberObserver interface
class TextureGraphInsightBlobberObserver : public BlobberObserverSource
{
private:
protected:
	/// Protected interface of emitters called by the device to notify the observers
	UE_API virtual void BlobberUpdated(HashArray&& AddedHashes, HashArray&& RemappedHashes) override;

public:
	UE_API TextureGraphInsightBlobberObserver();
	UE_API virtual ~TextureGraphInsightBlobberObserver() override;
};

/// Concrete SchedulerObserver interface
class TextureGraphInsightSchedulerObserver : public SchedulerObserverSource
{
private:
protected:
	/// Protected interface of emitters called by the scheduler to notify the observers
	UE_API virtual void Start() override;
	UE_API virtual void UpdateIdle() override;
	UE_API virtual void Stop() override;
	UE_API virtual void BatchAdded(JobBatchPtr Batch) override;
	UE_API virtual void BatchDone(JobBatchPtr Batch) override;
	UE_API virtual void BatchJobsDone(JobBatchPtr Batch) override;

public:

	UE_API TextureGraphInsightSchedulerObserver();
	UE_API virtual ~TextureGraphInsightSchedulerObserver() override;
};

/// Concrete EngineObserver interface
/// Responsible for:
///	  1/ watching the engine life cycle
///   2/ owning the other system observers, and installing them appropirately when an Engine is active
///	  3/ notifying Insight
class TextureGraphInsightEngineObserver : public EngineObserverSource
{
protected:
	/// Protected interface of emitters called by the engine to notify the observers
	UE_API virtual void Created() override;
	UE_API virtual void Destroyed() override;

public:
	UE_API TextureGraphInsightEngineObserver();
	UE_API virtual ~TextureGraphInsightEngineObserver() override;

	std::shared_ptr<TextureGraphInsightDeviceObserver>			_deviceObservers[(uint32)DeviceType::Count];
	std::shared_ptr<TextureGraphInsightBlobberObserver>		BlobberObserver;
	std::shared_ptr<TextureGraphInsightSchedulerObserver>		SchedulerObserver;
};

#undef UE_API
