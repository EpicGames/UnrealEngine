// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <vector>
#include <unordered_map>

#include "Data/Blob.h"
#include "Device/Device.h"

#define UE_API TEXTUREGRAPHINSIGHT_API

using FStringArray = std::vector<FString>;

template <typename DimType>
class T_TiledGrid
{
public:
	using Size = DimType;
	using Index = int32_t;

	Size							_rows = 0;
	Size							_cols = 0;

	T_TiledGrid() {}
	T_TiledGrid(Size size) : T_TiledGrid(size, size) {}
	T_TiledGrid(Size rows, Size cols) : _rows(rows), _cols(cols) {}

	FORCEINLINE	Index				operator()(Size tx, Size ty) const { check(IsValidTile(tx, ty)); return tx * _cols + ty; }
	FORCEINLINE Size				Rows() const { return _rows; }
	FORCEINLINE Size				Cols() const { return _cols; }
	FORCEINLINE Index				NumTiles() const { return _rows * _cols; }
	FORCEINLINE bool				IsEmpty() const { return (_rows < 1 || _cols < 1); }
	FORCEINLINE bool				IsUnique() const { return ( _rows * _cols == 1);  }
	FORCEINLINE bool				IsTiled() const { return (_rows * _cols > 0); }

	FORCEINLINE bool				IsValidTile(Size tx, Size ty) const { return (tx < _rows && ty < _cols); }

	template<typename Functor>
	FORCEINLINE void				ForEach(Functor functor) const
	{
		for (Size tx = 0; tx < _rows; ++tx)
		{
			for (Size ty = 0; ty < _cols; ++ty)
			{
				functor(tx, ty);
			}
		}
	}
};
using TiledGrid = T_TiledGrid<uint8_t>;

template <typename Type>
class T_Tiled
{
public:
	using Size = typename TiledGrid::Size;
	using TypeArray = std::vector<Type>;


	TiledGrid						_grid;
	TypeArray						_tiles;

	T_Tiled() {}
	T_Tiled(Size size) : T_Tiled(size, size) {}
	T_Tiled(Size rows, Size cols) : _grid(rows, cols) { _tiles.resize(_grid.NumTiles()); }

	FORCEINLINE void	 			Resize(Size size) { Resize(size, size); }
	FORCEINLINE void	 			Resize(Size rows, Size cols) { _grid = TiledGrid(rows, cols); _tiles.resize(_grid.NumTiles()); }
	FORCEINLINE void	 			Resize(TiledGrid grid) { _grid = grid; _tiles.resize(_grid.NumTiles()); }
	FORCEINLINE void				Fill(const Type& value) { for (auto& v : _tiles) v = value; }

	FORCEINLINE const Type&			operator()(Size tx, Size ty) const { return _tiles[_grid(tx, ty)]; }
	FORCEINLINE Type&				operator()(Size tx, Size ty) { return _tiles[_grid(tx, ty)]; }
	FORCEINLINE const TiledGrid&	Grid() const { return _grid; }

	//FORCEINLINE const TypeArray& Tiles() const { return _tiles; }
	//FORCEINLINE TypeArray& Tiles() { return _tiles; }

	template<typename Functor>
	FORCEINLINE void				ForEach(Functor functor) { _grid.ForEach(functor); }
	template<typename Functor>
	FORCEINLINE void				ForEachGetTile(Functor functor) const { _grid.ForEach([&] (Size tx, Size ty) { functor(tx, ty, _tiles[_grid(tx, ty)]); }); 	}
};
using BoolTiles = T_Tiled<uint8_t>;
using Uint8Tiles = T_Tiled<uint8_t>;

template<typename Type>
int32_t TiledSum(const T_Tiled<Type>& tiles)
{
	int32_t s = 0;
	tiles.ForEachGetTile([&s](typename T_Tiled<Type>::Size tx, typename T_Tiled<Type>::Size ty, const Type& v)
	{
		s += v;
	});
	return s;
}

using DeviceBufferID = uint64;
using DeviceBufferIDArray = std::vector<DeviceBufferID>;

using BlobID = uint64;

using BatchID = uint64;
using FrameID = uint64;
using QueueID = uint32;

using ActionID = uint64;

using MixID = uint64;


static const uint16_t INVALID_SHORT = 0xFFFF;
static const uint32_t INVALID_INDEX = 0xFFFFFFFF;
static const uint64_t INVALID_RECORD_ID = 0xFFFFFFFFFFFFFFFF;

struct RecordID
{
	enum Type : uint8
	{
		kInvalid = 0xFF,
		kBatch = 0,
		kJob,
		kBlob,
		kDeviceBuffer,
		kAction,
		kMix,
	};
	union
	{
		struct 
		{
			uint8			type;
			uint32_t		prim;
			uint32_t		second : 24;
		};
		uint64			id = INVALID_RECORD_ID;
	};
	RecordID(uint8 t, uint32_t p, uint32_t s) :type(t), prim(p), second(s) {};
	RecordID(uint64 recordId) :id(recordId) {};
	RecordID() = default;

	FORCEINLINE bool IsValid() const { return id != INVALID_RECORD_ID; }
	FORCEINLINE bool IsBuffer() const { return (type == kDeviceBuffer); }
	FORCEINLINE bool IsDevice() const { return IsBuffer() && (prim == INVALID_INDEX); }
	FORCEINLINE bool IsBlob() const { return (type == kBlob); }
	FORCEINLINE bool IsJob() const { return (type == kJob); }
	FORCEINLINE bool IsBatch() const { return (type == kBatch); }
	FORCEINLINE bool IsAction() const { return (type == kAction); }
	FORCEINLINE bool IsMix() const { return (type == kMix); }

	FORCEINLINE uint32 Buffer() const { return prim; }
	FORCEINLINE uint32 Buffer_DeviceType() const { return second; }
	FORCEINLINE uint32 Blob() const { return prim; }
	FORCEINLINE uint32 Batch() const { return prim; }
	FORCEINLINE uint32 Job() const { return second; }
	FORCEINLINE uint32 Action() const { return prim; }
	FORCEINLINE uint32 Mix() const { return prim; }

	static RecordID fromBuffer(DeviceType t, uint32_t b) { return RecordID(kDeviceBuffer, b, (uint32)t); }
	static RecordID fromBlob(uint32_t b) { return RecordID(kBlob, b, INVALID_INDEX); }
	static RecordID fromBatch(uint32_t b) { return RecordID(kBatch, b, INVALID_INDEX); }
	static RecordID fromBatchJob(uint32_t b, uint32_t j) { return RecordID(kJob, b, j); }
	static RecordID fromAction(uint32_t a) { return RecordID(kAction, a, INVALID_INDEX); }
	static RecordID fromMix(uint32_t a) { return RecordID(kMix, a, INVALID_INDEX); }
};
using RecordIDArray = std::vector<RecordID>;
using RecordIDTiles = T_Tiled<RecordID>;

/// 
/// DeviceBuffer Record
///
class DeviceBufferRecord
{
public:
	// adding a default constructor to enforce this to be a non POD class to successfully initialize
	DeviceBufferRecord() {}
	//RecordID						_selfID;
	RecordID						SourceID;

	static UE_API const DeviceBufferRecord null;
	DeviceType						DevType = DeviceType::Null;
	uint64							ID = 0;
	HashType						HashValue = 0;
	HashType						PrevHashValue = 0;
	BufferDescriptor				Descriptor;

	mutable uint64					RawBufferMemSize = 0; /// The true size as informed
	mutable uint64					Rank = 0;

	mutable bool					bErased = false;
	mutable bool					bLeaked = false;
};
using DeviceBufferRecordArray = std::vector< DeviceBufferRecord >;
using RecordFromDeviceBufferIDMap = std::unordered_map<DeviceBufferID, RecordID>;
using RecordFromHashMap = std::unordered_map<HashType, RecordID>;



/// Blob Record 
class BlobRecord
{
public:
	// adding a default constructor to enforce this to be a non POD class to successfully initialize
	BlobRecord() {}
	RecordID						SelfID;

	static UE_API const BlobRecord			null;
	uint32							ReplayCount = 0;

	FString							Name;

	RecordID						SourceID; // Source job or action or which has generated this Blob

	HashType						HashValue = 0;
	uint32							TexWidth = 0;
	uint32							TexHeight = 0;

	uint32							NumMapped = 0;			/// if this blob is mapped by other blobs, here is the number of mapping to this
	RecordID						MappedID;				/// if this blob hash is remapped, here is the remapped to blob ID
	uint64							RawBufferMemSize = 0;	/// The true size as informed

	/// A tiled blob reference its sub tile blobs in the following fields _tileIdxs and _uniqueBlobs
	/// The tiled information for the blob and the indices of the unique blobs which make the tiles grid 
	Uint8Tiles						TileIdxs;
	std::vector<uint16_t>			UniqueBlobs;

	FORCEINLINE const TiledGrid&	Grid() const { return TileIdxs.Grid(); }
	FORCEINLINE bool				IsTiled() const { return TileIdxs.Grid().IsTiled(); }
	FORCEINLINE int32_t				NumTiles() const { return TileIdxs.Grid().NumTiles(); }
	FORCEINLINE RecordID			GetTileBlob(uint8_t x, uint8_t y) const { return GetUniqueBlob(TileIdxs(x, y)); }

	FORCEINLINE int32_t				NumUniqueBlobs() const { return UniqueBlobs.size(); }
	FORCEINLINE RecordID			GetUniqueBlob(uint8_t i) const { return RecordID::fromBlob(UniqueBlobs[i]); }
};
using BlobRecordArray = std::vector< BlobRecord >;
using RecordFromBlobIDMap = std::unordered_map<BlobID, RecordID>;


/// Individual Job Record is the leaf of the data model
/// It gather meta information to present what the job has done
/// the inputs and output blob
class JobRecord
{
public:
	// adding a default constructor to enforce this to be a non POD class to successfully initialize
	JobRecord() {}
	static UE_API const JobRecord	GNull;

	RecordID				SelfID;

	uint32					JobIdx = -1;	/// Index of the job in the batch
	int32					PhaseIdx = 0;  /// Phase index of the Job within the job
													/// If 0 it is THE job, < 0 a prior job, > 0 an after job

	int						Priority = 0;
	FString					TransformName;

	HashType				JobHash = 0;

	uint32					TexWidth = 0;
	uint32					TexHeight = 0;

	double					BeginTimeMS = 0;
	double					BeginRunTimeMS = 0;
	double					EndTimeMS = 0;

	RecordID				ResultBlobId;
	RecordIDArray			InputBlobIds;
	FStringArray			InputArgNames;

	// Information about the tiled job
	BoolTiles				Tiles;
	
	FORCEINLINE uint64				GetNumPixels() const { return TexWidth * TexHeight; }
	FORCEINLINE uint64				GetNumPixelsPerTile() const { return GetNumPixels() / GetNumTiles(); }

	FORCEINLINE const TiledGrid&	GetGrid() const { return Tiles.Grid(); }
	FORCEINLINE bool				IsTiled() const { return Tiles.Grid().IsTiled(); }
	FORCEINLINE int32				GetNumTiles() const { return Tiles.Grid().NumTiles(); }

	FORCEINLINE bool				IsTileInvalidated(uint8_t x, uint8_t y) const { return Tiles(x, y) != 0; }
	FORCEINLINE int32				GetNumInvalidated() const { return TiledSum(Tiles); }

	FORCEINLINE double				GetBeginTimeMS() const { return BeginTimeMS; }
	FORCEINLINE double				GetBeginRunTimeMS() const { return (BeginRunTimeMS != 0 ? BeginRunTimeMS : BeginTimeMS); } // Ideal begin runtime, use begin TIme if none assigned
	FORCEINLINE double				GetRunTimeMS() const { return (BeginRunTimeMS != 0 ? EndTimeMS - BeginRunTimeMS : EndTimeMS - BeginTimeMS); } // Runtime must be positive, clamp to 0
	FORCEINLINE double				GetPreRunTimeMS() const { return GetBeginRunTimeMS() - BeginTimeMS; }

	FORCEINLINE double				GetFillRate() const { return GetNumPixelsPerTile() * GetNumInvalidated() / (0.001 * GetRunTimeMS()); }

	FORCEINLINE bool				IsNoOp() const { return bIsDone && (GetNumInvalidated() == 0); }

	FORCEINLINE bool				IsMainPhase() const { return PhaseIdx == 0; }


	bool					bIsDone = false;
	uint32					ReplayCount = 0;
};
using JobRecordArray = std::vector< JobRecord >;


/// Batch Record
/// Hold the Job Records
class BatchRecord
{
public:
	// adding a default constructor to enforce this to be a non POD class to successfully initialize
	BatchRecord() {}

	RecordID						SelfID;

	static UE_API const BatchRecord null;

	RecordID						MixID;

	BatchID							BatchID = INVALID_RECORD_ID;
	FrameID							FrameID = INVALID_RECORD_ID;
	JobRecordArray					Jobs;

	RecordIDArray					ResultBlobs;

	FString							Action;

	double							BeginTimeMS = 0;
	double							EndTimeMS = 0;

	uint16_t						NumTiles = 0;
	uint16_t						NumInvalidatedTiles = 0;

	uint32_t						ReplayCount = 0;

	bool							bIsDone = false;
	bool							bIsJobsDone = false;

	bool							bIsNoCache = false;
	bool							bIsFromIdle = false;

	FORCEINLINE const JobRecord&	GetJob(RecordID ID) const { return ((ID.Job() < Jobs.size()) ? Jobs[ID.Job()] : JobRecord::GNull); }
	FORCEINLINE double				ScopeTime_ms() const { return EndTimeMS - BeginTimeMS; }

};
using BatchRecordArray = std::vector< BatchRecord >;
using RecordFromBatchIDMap = std::unordered_map<BatchID, RecordID>;


/// Action Record
/// Hold the Action Records
class ActionRecord
{
public:
	// adding a default constructor to enforce this to be a non POD class to successfully initialize
	ActionRecord() {}
	static UE_API const ActionRecord null;

	FString							Name;
	FString							Meta;

	mutable RecordID				ParentAction;
	RecordIDArray					SubActions;
};
using ActionRecordArray = std::vector< ActionRecord >;
using RecordFromActionIDMap = std::unordered_map<ActionID, RecordID>;


/// Mix Record
/// Hold the Mix Records
class MixRecord
{
public:
	// adding a default constructor to enforce this to be a non POD class to successfully initialize
	MixRecord() {}
	RecordID						SelfID;

	static UE_API const MixRecord null;

	RecordID						ParentMixID; // If mix is an instance, it has a parentMix
	bool							IsInstanceMix() const { return ParentMixID.IsValid(); }

	RecordIDArray					InstanceMixIDs; // Or it may have instances
	bool							HasInstances() const { return InstanceMixIDs.size() > 0; }

	FString							Name;
	FString							AssetName;

	RecordIDArray					Batches;

};
using MixRecordArray = std::vector< MixRecord >;
using RecordFromMixIDMap = std::unordered_map<MixID, RecordID>;



/// Session Record container defining the namespace for all the recordIDs
class SessionRecord
{
private:
	BatchRecordArray				Batches;
	RecordFromBatchIDMap			RecordFromBatchID;

	DeviceBufferRecordArray			DeviceBuffers[(uint32) DeviceType::Count];
	RecordFromDeviceBufferIDMap		BufferFromDeviceBufferIDActive[(uint32)DeviceType::Count];	/// Store the current active (or leaked) device buffer in a per device map
	RecordFromDeviceBufferIDMap		BufferFromDeviceBufferIDErased;  /// Once erased, store all the device buffer in this map
	RecordFromHashMap				BufferFromHash;

	BlobRecordArray					Blobs;
	RecordFromBlobIDMap				BlobFromBlobID;
	
	ActionRecordArray				Actions;
	RecordFromActionIDMap			ActionFromActionID;

	MixRecordArray					Mixes;
	RecordFromMixIDMap				RecordFromMixID;

public:
	double							StartTime_ms() const { return (Batches.empty() ? 0 : Batches.front().BeginTimeMS); }
	
	// Batch records
	UE_API RecordID						FindBatchRecord(BatchID InBatchID) const;
	using BatchRecordConstructor	= std::function<BatchRecord (RecordID RecId)>;
	using BatchRecordEditor			= std::function<bool (BatchRecord& r)>;
	UE_API RecordID						NewBatchRecord(BatchID InBatchID, BatchRecordConstructor Constructor);
	UE_API bool							EditBatchRecord(RecordID RecId, BatchRecordEditor Editor);

	// Device Buffer records
	UE_API std::tuple<RecordID, bool>		NewDeviceBufferRecord(DeviceType DevType, DeviceBufferID BufferID, const DeviceBufferRecord& Record);
	UE_API RecordID						EraseDeviceBufferRecord(DeviceType DevType, DeviceBufferID BufferID); /// Assuming the db is in the current map, then move its key in the erased map and tag it erased

	/// Find the records from the array of bufferIDs & output the leaked buffers
	UE_API RecordIDArray    				FindActiveDeviceBufferRecords(DeviceType DevType, const DeviceBufferIDArray& BufferIDs, RecordIDArray& LeakedBuffers);
	
	UE_API RecordID						FindDeviceBufferRecord(DeviceBufferID BufferID) const;
	UE_API RecordIDArray					FindDeviceBufferRecords(const DeviceBufferIDArray& BufferIDs) const;
	UE_API RecordIDArray					FetchActiveDeviceBufferIDs(DeviceType DevType) const;

	UE_API RecordID						FindDeviceBufferRecordFromHash(HashType InHash) const;

	// Blob records
	UE_API RecordID						FindBlobRecord(BlobID InBlobID) const;
	using BlobRecordConstructor		= std::function<BlobRecord(RecordID RecId)>;
	using BlobRecordEditor			= std::function<bool(BlobRecord& r)>;
	UE_API RecordID						NewBlobRecord(BlobID InBlobID, BlobRecordConstructor Constructor);
	UE_API bool							EditBlobRecord(RecordID RecId, BlobRecordEditor Editor);

	UE_API RecordID						NewActionRecord(ActionID InActionID, const ActionRecord& Record);
	UE_API RecordID						FindActionRecord(ActionID InActionID) const;

	// Mix records
	UE_API RecordID						FindMixRecord(MixID MixID) const;
	using MixRecordConstructor		= std::function<MixRecord(RecordID RecId)>;
	using MixRecordEditor			= std::function<bool(MixRecord& r)>;
	UE_API RecordID						NewMixRecord(BatchID InBatchID, MixRecordConstructor constructor);
	UE_API bool							EditMixRecord(RecordID RecId, MixRecordEditor editor);

	FORCEINLINE const DeviceBufferRecord&	GetBuffer(RecordID RecId) const { return ((RecId.Buffer() < DeviceBuffers[RecId.Buffer_DeviceType()].size()) ? DeviceBuffers[RecId.Buffer_DeviceType()][RecId.Buffer()] : DeviceBufferRecord::null); }
	FORCEINLINE const BlobRecord&			GetBlob(RecordID RecId) const { return ((RecId.Blob() < Blobs.size()) ? Blobs[RecId.Blob()] : BlobRecord::null); }
	FORCEINLINE const BatchRecord&			GetBatch(RecordID RecId) const { return ((RecId.Batch() < Batches.size()) ? Batches[RecId.Batch()] : BatchRecord::null); }
	FORCEINLINE const JobRecord&			GetJob(RecordID RecId) const { return GetBatch(RecId).GetJob(RecId); }
	FORCEINLINE const ActionRecord&			GetAction(RecordID RecId) const { return ((RecId.Action() < Actions.size()) ? Actions[RecId.Action()] : ActionRecord::null); }
	FORCEINLINE const MixRecord&			GetMix(RecordID RecId) const { return ((RecId.Mix() < Mixes.size()) ? Mixes[RecId.Mix()] : MixRecord::null); }

};

using SessionRecordPtr = std::shared_ptr<SessionRecord>;

#undef UE_API
