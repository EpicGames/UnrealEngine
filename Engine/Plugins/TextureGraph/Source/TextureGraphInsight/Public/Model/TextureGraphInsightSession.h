// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "TextureGraphInsightRecord.h"
#include "TextureGraphInsightObserver.h"

#define UE_API TEXTUREGRAPHINSIGHT_API

class UMixInterface;
using MixPtr = UMixInterface*;
using MixPtrW = UMixInterface*;
//using MixPtr = std::shared_ptr<UMixInterface>;
//using MixPtrW = std::weak_ptr<UMixInterface>;

class TextureGraphInsightSession
{
public:
	/// Keep all the pointers to "real" TextureGraph Engine objects in the live runtime cache
	class LiveCache
	{
	public:
		LiveCache(TextureGraphInsightSession* InSession) : Session(InSession) {}
		TextureGraphInsightSession*		Session = nullptr;

		// TODO: Find a better way to keep the Batch pointer alive, for now, only the last one will still be valid
		std::vector<JobBatchPtrW>	Batches; 
		using						BlobEntry= HashType;
		std::vector<BlobEntry>		Blobs;
		std::vector<MixPtrW>		Mixes;

		void						AddBlob(RecordID RecId, BlobPtr BlobObj);
		BlobPtr						GetBlob(RecordID RecId) const;

		void						AddBatch(RecordID RecId, JobBatchPtr Batch);
		JobBatchPtr					GetBatch(RecordID RecId) const;

		uint64						DeviceCacheVersions[(uint32)DeviceType::Count] = { 0, 0, 0, 0, 0, 0 , 0};

		DeviceBufferPtr				GetDeviceBuffer(RecordID RecId) const;

		void						AddMix(RecordID RecId, MixPtr MixObj);
		MixPtr						GetMix(RecordID RecId) const;
	};

private:
protected:
	friend class TextureGraphInsightEngineObserver;
	/// Protected interface of emitters called by the engine observer
	UE_API void							EngineCreated();
	UE_API void							EngineDestroyed();
	UE_API void							EmitOnEngineReset(bool);

	/// manage DeviceBuffer update
	friend class TextureGraphInsightDeviceObserver;
	UE_API uint32							ParseDeviceBuffersCache(DeviceType type, RecordIDArray& addedBufferIds);
	UE_API void							DeviceBuffersUpdated(DeviceType DevType, TextureGraphInsightDeviceObserver::HashNDescArray&& AddedBuffers, TextureGraphInsightDeviceObserver::HashArray&& RemovedBuffers);

	/// Manage Blobber update
	friend class TextureGraphInsightBlobberObserver;
	UE_API void							BlobberHashesUpdated(TextureGraphInsightBlobberObserver::HashArray&& AddedHashes, TextureGraphInsightBlobberObserver::HashArray&& RemappedHashes);

	friend class TextureGraphInsightSchedulerObserver;
	/// Protected interface of emitters called by the scheduler observer
	UE_API void							UpdateIdle();
	UE_API void							BatchAdded(JobBatchPtr Batch);
	UE_API void							BatchDone(JobBatchPtr Batch);
	UE_API void							BatchJobsDone(JobBatchPtr Batch);

	/// manage creation and update of the Batch job records
	UE_API BatchRecord						MakeBatchRecord(RecordID RecId, JobBatchPtr Batch, bool isDone = true);
	UE_API void							UpdateBatchRecord(JobBatchPtr Batch, BatchRecord& record);
	UE_API JobRecord						MakeJobRecord(JobPtrW job, uint32 idx, int32 phaseIdx = 0);
	UE_API void							UpdateJobRecord(JobPtrW job, JobRecord& record, double batchBeginTime_ms, double batchEndTime_ms);


	/// manage creation and update of the BlobObj records
	UE_API RecordID						FindOrCreateBlobRecordFromHash(HashType hash);
	UE_API RecordID						RemapBlobHash(HashType oldHash, HashType newHash);
	UE_API RecordID						FindOrCreateBlobRecord(BlobPtr BlobObj, RecordID sourceID = RecordID());
	UE_API BlobRecord						MakeBlobRecord(RecordID RecId, BlobPtr BlobObj, RecordID sourceID = RecordID());
	UE_API void							EditTiledBlobRecord(BlobRecord& rd, const RecordIDTiles& subTiles);
	UE_API void							UpdateBlobRecord(RecordID RecId, BlobPtr BlobObj);
	RecordIDArray					NewBlobsToNotify;
	RecordIDArray					MappedBlobsToNotify;
	UE_API void							EmitOnNewBlob();
	UE_API void							EmitOnMappedBlob();

	/// manage creation and update of the MixObj records
	UE_API RecordID						AssociateBatchToMix(RecordID Batch, MixPtr MixObj);
	UE_API RecordID						FindOrCreateMixRecord(MixPtr MixObj);

	/// manage creation and update of the buffer records
	RecordIDArray					NewBuffersToNotify;

	/// The session record, collecting all the VALUES
	/// A session record is renewed for each engine
	SessionRecordPtr				Record;
	std::mutex						RecordMutex;

	/// Keep all the pointers to "real" TextureGraph Engine objects in the live cache
	/// A session live cache is renewed for each engine
	std::shared_ptr<LiveCache>		Cache;

	int32							EngineID = -1;
public:

	UE_API TextureGraphInsightSession();
    UE_API ~TextureGraphInsightSession();

	FORCEINLINE SessionRecord&		GetRecord() { std::lock_guard<std::mutex> Lock(RecordMutex); return (*Record); }
	FORCEINLINE LiveCache&			GetCache() { std::lock_guard<std::mutex> Lock(RecordMutex); return (*Cache); }

	/// Send the specified record to be inspected
	UE_API void							SendToInspector(RecordID RecId) const;

	/// Notifier subscription
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEngineReset, int32);
	FOnEngineReset& OnEngineReset()		{ return OnEngineResetEvent; }

	DECLARE_MULTICAST_DELEGATE(FOnUpdateIdle);
	FOnUpdateIdle& OnUpdateIdle()		{ return OnUpdateIdleEvent; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRecordIDArray, const RecordIDArray&);

	FOnRecordIDArray& OnDeviceBufferAdded()		{ return OnDeviceBufferAddedEvent; }
	FOnRecordIDArray& OnDeviceBufferRemoved()	{ return OnDeviceBufferRemovedEvent; }
	FOnRecordIDArray& OnBlobAdded()				{ return OnBlobAddedEvent; }
	FOnRecordIDArray& OnBlobMapped()			{ return OnBlobMappedEvent; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRecordID, RecordID);

	FOnRecordID& OnBatchAdded()			{ return OnBatchAddedEvent; }
	FOnRecordID& OnBatchDone()			{ return OnBatchDoneEvent; }
	FOnRecordID& OnBatchJobsDone()		{ return OnBatchJobsDoneEvent; }

	FOnRecordID& OnMixAdded()			{ return OnMixAddedEvent; }
	FOnRecordID& OnMixUpdated()			{ return OnMixUpdatedEvent; }
	
	FOnRecordID& OnBatchInspected()		{ return OnBatchInspectedEvent; }
	FOnRecordID& OnJobInspected()		{ return OnJobInspectedEvent; }
	FOnRecordID& OnBlobInspected()		{ return OnBlobInspectedEvent; }
	FOnRecordID& OnBufferInspected()	{ return OnBufferInspectedEvent; }


	// Trigger Replay of Batch and jobs
	// this call is async and returns if the scheduling of the replay is successful or not
	UE_API bool							ReplayBatch(RecordID batchRecordID, bool captureRenderDoc);
	UE_API bool							ReplayJob(RecordID jobRecordID, bool captureRenderDoc);

protected:
	FOnEngineReset		OnEngineResetEvent;
	FOnUpdateIdle		OnUpdateIdleEvent;

	FOnRecordIDArray	OnDeviceBufferAddedEvent;
	FOnRecordIDArray	OnDeviceBufferRemovedEvent;

	FOnRecordIDArray	OnBlobAddedEvent;
	FOnRecordIDArray	OnBlobMappedEvent;

	FOnRecordID			OnBatchAddedEvent;
	FOnRecordID			OnBatchDoneEvent;
	FOnRecordID			OnBatchJobsDoneEvent;

	FOnRecordID			OnMixAddedEvent;
	FOnRecordID			OnMixUpdatedEvent;

	FOnRecordID			OnBatchInspectedEvent;
	FOnRecordID			OnJobInspectedEvent;
	FOnRecordID			OnBlobInspectedEvent;
	FOnRecordID			OnBufferInspectedEvent;
};

#undef UE_API
