// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include <memory>
#include "Modules/ModuleManager.h"

#define UE_API TEXTUREGRAPHINSIGHT_API

DECLARE_LOG_CATEGORY_EXTERN(LogTextureGraphInsight, Log, All);

class TextureGraphInsightSchedulerObserver;
class TextureGraphInsightSession;
using TextureGraphInsightSessionPtr = std::shared_ptr<TextureGraphInsightSession>;

class FTextureGraphInsightModule : public FDefaultGameModuleImpl
{
public:
	virtual void StartupModule() override;
};

struct RecordID;

class TextureGraphInsight
{
private:
	static UE_API TextureGraphInsight* GInstance;					/// Static instance. Use Create to create a new instance. Destroy the current instance first
	UE_API TextureGraphInsight();
	UE_API ~TextureGraphInsight();

	TextureGraphInsightSessionPtr					Session;
public:
	static UE_API bool								Create();  /// Create the Insight Instance ONLY if no other currently created AND if a TextureGraphEngine is already created

	/// Destroy the current instance
	static UE_API bool								Destroy(); /// Destroy the instance of Insight if it exists.

	//////////////////////////////////////////////////////////////////////////
	/// Inline functions
	//////////////////////////////////////////////////////////////////////////
	FORCEINLINE static TextureGraphInsight*		Instance() { return GInstance; }

	FORCEINLINE TextureGraphInsightSessionPtr		GetSession() const { return Session; }
};

// Macro to control in one place how we display hash value
//#define HashToFString(h) FString::Printf(TEXT("%20llu"), (h))
#define HashToFString(h) FString::Printf(TEXT("%llX"), (h))

#undef UE_API
