// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//#include "SlateBasics.h"

#include "Model/TextureGraphInsightSession.h"
#include <Widgets/Views/STreeView.h>

#define UE_API TEXTUREGRAPHINSIGHT_API

class STextureGraphInsightActionViewRow; /// Declare the concrete type of widget used for the raws of the view. Defined in the cpp file

class STextureGraphInsightActionView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextureGraphInsightActionView) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args);

	// Item Types
	class FItemData;
	using FItem = TSharedPtr<FItemData>;
	using FItemArray = TArray< FItem >;
	class FItemData
	{
	public:
		FItemData(RecordID rid) : _recordID(rid) {}
		TSharedPtr < STextureGraphInsightActionViewRow > _widget;
		RecordID _recordID;
		FItemArray _children;
	};
	using SItemTableView = STreeView< FItem >;

	// Standard delegates for the view
	UE_API TSharedRef<ITableRow>	OnGenerateRowForView(FItem item, const TSharedRef<STableViewBase>& OwnerTable);
	FORCEINLINE void		OnGetChildrenForView(FItem item, FItemArray& children) { children = item->_children; };
	UE_API void					OnClickItemForView(FItem item);
	UE_API void					OnDoubleClickItemForView(FItem item);

	/// The list of root items
	FItemArray _rootItems;

	// The TreeView widget
	TSharedPtr<SItemTableView> _tableView;

	UE_API void OnActionNew(RecordID rid);
	UE_API void OnEngineReset(int);
};

#undef UE_API
