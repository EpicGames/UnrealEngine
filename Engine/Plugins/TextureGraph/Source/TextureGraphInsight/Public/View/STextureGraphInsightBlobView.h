// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//#include "SlateBasics.h"

#include "Model/TextureGraphInsightRecord.h"
#include <Widgets/Layout/SBorder.h>
#include <UObject/GCObject.h>

#define UE_API TEXTUREGRAPHINSIGHT_API

class STextureGraphInsightDeviceBufferView;
class UMaterialInstanceDynamic;
struct FSlateBrush;
struct FSlateColorBrush;
class STextureGraphInsightBlobView : public SBorder, public FGCObject /// Need FGCObject to control garbage collection of objects
{
public:
	SLATE_BEGIN_ARGS(STextureGraphInsightBlobView) :
		_withHighlight(false) {}
		SLATE_ARGUMENT(RecordID, recordID)
		SLATE_ARGUMENT(bool, withHighlight)

		SLATE_ARGUMENT(BoolTiles, tilesMask)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args);

	RecordID _recordID;
	RecordID _inspectedID;

	// Needs our own brush to display the blob content
	TSharedPtr<FSlateBrush> _brush;
	TObjectPtr<UMaterialInstanceDynamic> _brushMaterial = nullptr; /// 

	// Border brush
	TSharedPtr<FSlateColorBrush> _borderBrush;

	UE_API FReply OnInspect();

	// Capture mouse clicks here
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;


	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(_brushMaterial);
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("STextureGraphInsightBlobView");
	}

};

#undef UE_API
