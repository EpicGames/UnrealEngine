// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//#include "SlateBasics.h"

#include "Model/TextureGraphInsightRecord.h"
#include <Widgets/Layout/SBorder.h>
#include <UObject/GCObject.h>

#define UE_API TEXTUREGRAPHINSIGHT_API

class UMaterialInstanceDynamic;

class STextureGraphInsightDeviceBufferView : public SBorder, public FGCObject /// Need FGCObject to control garbage collection of objects
{
public:
	SLATE_BEGIN_ARGS(STextureGraphInsightDeviceBufferView) :
		_withToolTip(false),
		_withDescription(false) {}
		SLATE_ARGUMENT(RecordID, recordID)
		SLATE_ARGUMENT(RecordID, blobID)
		SLATE_ARGUMENT(bool, withToolTip)
		SLATE_ARGUMENT(bool, withDescription)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args);

	RecordID _recordID;
	RecordID _blobID;
	uint32 _imageWidth = 1;
	uint32 _imageHeight = 1;
	FString _imageName;

	// Needs our own brush to display the blob content
	TSharedPtr<FSlateBrush> _brush;
	TObjectPtr<UMaterialInstanceDynamic> _brushMaterial = nullptr;

	UE_API void CreateWidgetBrush(UTexture* blobTexture);
	UE_API void MakeWidgetMaterial();

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(_brushMaterial);
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("STextureGraphInsightDeviceBufferView");
	}
};

#undef UE_API
