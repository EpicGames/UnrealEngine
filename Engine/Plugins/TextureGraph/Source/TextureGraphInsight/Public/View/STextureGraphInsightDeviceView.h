// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//#include "SlateBasics.h"

#include "Model/TextureGraphInsightSession.h"
#include <Widgets/SCompoundWidget.h>
#include <Widgets/Views/STreeView.h>

#define UE_API TEXTUREGRAPHINSIGHT_API

class STextureGraphInsightDeviceListViewRow; /// Declare the concrete type of widget used for the raws of the view. Defined in the cpp file
class STableViewBase;
class ITableRow;
class STextBlock;

class STextureGraphInsightDeviceListView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextureGraphInsightDeviceListView)
	: _inspectOnSimpleClick(false), _inspectDevices(true) {}
	SLATE_ARGUMENT(RecordID, recordID) /// Can use an empty RecordID with type device and the expected Device type to only care about that device
	SLATE_ARGUMENT(bool, inspectOnSimpleClick)
	SLATE_ARGUMENT(bool, inspectDevices)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args);

	// TreeView Item Types
	class FItemData;
	using FItem = TSharedPtr<FItemData>;
	using FItemW = TWeakPtr<FItemData>;
	using FItemArray = TArray< FItem >;
	class FItemData
	{
	public:
		FItemData(RecordID rid, int32 metaItem = 0) : _recordID(rid), _metaItem(metaItem) {}
		TSharedPtr < STextureGraphInsightDeviceListViewRow > _widget;
		RecordID	_recordID;
		FItemArray	_children;
		FItem		_removedRoot;
		const int32 _metaItem = 0;
		int32		_rank = 0;
	};
	using SItemTableView = STreeView< FItem >;

	// Standard delegates for the view
	UE_API TSharedRef<ITableRow>	OnGenerateRowForView(FItem item, const TSharedRef<STableViewBase>& OwnerTable);
	FORCEINLINE void		OnGetChildrenForView(FItem item, FItemArray& children) { children = item->_children; if (item->_removedRoot) children.Add(item->_removedRoot); };
	UE_API void OnClickItemForView(FItem item);
	UE_API void OnDoubleClickItemForView(FItem item);
	UE_API void OnInspect(FItem item);
	
	RecordID	_rootDevice; /// Can use an empty RecordID parameter with type DeviceBuffer and the expected Device type to only care about that device

	// Inspect on double click always
	// eventually trigger inspect on simple click
	bool _inspectOnSimpleClick = false;
	
	bool _inspectDevices = true;

	/// The list of root items
	FItemArray _rootItems;

	// The TreeView widget
	TSharedPtr<SItemTableView> _tableView;

	UE_API void OnDeviceBufferAdded(const RecordIDArray& rids);
	UE_API void OnDeviceBufferRemoved(const RecordIDArray& rids);
	UE_API void OnEngineReset(int);

	UE_API void Reset();

	UE_API void RefreshRootItems();
};

inline bool operator ==(const STextureGraphInsightDeviceListView::FItem& i,  const RecordID& rid) { return i->_recordID.Buffer() == rid.Buffer(); }

class STextureGraphInsightDeviceView : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(STextureGraphInsightDeviceView) {}
	SLATE_ARGUMENT(DeviceType, deviceType)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args);

	DeviceType _deviceType;
	
	TSharedPtr<STextBlock> _mem_percent;
	TSharedPtr<STextBlock> _mem_number;
	TSharedPtr<STextBlock> _mem_budget;

	TSharedPtr<STextBlock> _buffer_percent;
	TSharedPtr<STextBlock> _buffer_number;
	TSharedPtr<STextBlock> _buffer_budget;

	TSharedPtr<STextBlock> _threadCount;
	
	UE_API void Refresh();
};

class STextureGraphInsightDeviceManagerView : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(STextureGraphInsightDeviceManagerView) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args);

	std::vector < TSharedPtr< STextureGraphInsightDeviceView >> _deviceViews;

	UE_API void Reset(); // Rebuild the full ui of this widget, called by update
	UE_API void OnUpdate();
	UE_API void OnEngineReset(int32);
};

#undef UE_API
