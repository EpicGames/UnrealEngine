// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//#include "SlateBasics.h"
#include "Model/TextureGraphInsightSession.h"
#include <Widgets/Views/STileView.h>

#define UE_API TEXTUREGRAPHINSIGHT_API

class STextureGraphInsightDeviceBufferView;
class STextureGraphInsightBlobView;
class STextureGraphInsightRecordTrailView;

class STextureGraphInsightBatchInspectorView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextureGraphInsightBatchInspectorView) {}
	SLATE_ARGUMENT(RecordID, recordID)
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& Args);

	// ListView Item Types presenting various results of a batch
	class FItemData;
	using FItem = TSharedPtr<FItemData>;
	using FItemArray = TArray< FItem >;
	class FItemData
	{
	public:
		FItemData(RecordID batchID) : _batchID(batchID) {}
		FItemData(RecordID batchID, int32 targetIdx) : _batchID(batchID), _targetIndex(targetIdx) {}

		RecordID	_batchID;
		int32		_targetIndex = -1; // -1 meaning it is not an input
	};
	using SItemTileView = STileView< FItem >;

	// Standard delegates for the tree view
	UE_API TSharedRef<ITableRow> OnGenerateTileForView(FItem item, const TSharedRef<STableViewBase>& OwnerTable);

	/// The list of root items
	FItemArray _rootItems;
	// The TreeView widget
	TSharedPtr<SItemTileView> _view;


	// Inspect the specified batch (Called by Constructor with recordID param)
	UE_API void InspectBatch(RecordID job);
};

class STextureGraphInsightJobInspectorView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextureGraphInsightJobInspectorView) {}
	SLATE_ARGUMENT(RecordID, recordID)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args);

	// ListView Item Types presenting various input of a job
	class FItemData;
	using FItem = TSharedPtr<FItemData>;
	using FItemArray = TArray< FItem >;
	class FItemData
	{
	public:
		FItemData(RecordID jobID) : _jobID(jobID) {}
		FItemData(RecordID jobID, BoolTiles mask) : _jobID(jobID), _tilesMask(mask) {}
		FItemData(RecordID jobID, int32 inputIdx) :  _jobID(jobID), _inputIndex(inputIdx){}
		
		RecordID	_jobID;
		BoolTiles	_tilesMask;
		int32		_inputIndex = -1; // -1 meaning it is not an input
	};
	using SItemTileView = STileView< FItem >;

	// Standard delegates for the tree view
	UE_API TSharedRef<ITableRow> OnGenerateTileForView(FItem item, const TSharedRef<STableViewBase>& OwnerTable);

	// The output blob view generated by the job
	TSharedPtr<STextureGraphInsightBlobView> _outBlobView;

	/// The list of root items
	FItemArray _rootItems;
	// The TreeView widget
	TSharedPtr<SItemTileView> _view;


	// Inspect the specified Job (Called by Constructor with recordID param)
	UE_API void InspectJob(RecordID job);
};


class STextureGraphInsightBlobInspectorView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextureGraphInsightBlobInspectorView) {}
	SLATE_ARGUMENT(RecordID, recordID)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args);

	// The main blob view generated by the job
	TSharedPtr<STextureGraphInsightBlobView> _blobView;

	// Inspect the specified Blob (Called by Constructor with recordID param)
	UE_API void InspectBlob(RecordID blob);
};

class STextureGraphInsightDeviceBufferInspectorView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextureGraphInsightDeviceBufferInspectorView) {}
	SLATE_ARGUMENT(RecordID, recordID)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args);


	// Inspect the specified buffer (Called by Constructor with recordID param)
	UE_API void InspectBuffer(RecordID buffer);
};

class STextureGraphInsightInspectorView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextureGraphInsightInspectorView) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args);

	TSharedPtr<STextureGraphInsightRecordTrailView> _recordTrail;

	TSharedPtr<SOverlay> _stack;



	UE_API void OnInspected(RecordID rid);
	UE_API void OnEngineReset(int32);
};

#undef UE_API
