// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//#include "SlateBasics.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"

#include "Model/TextureGraphInsightSession.h"

#define UE_API TEXTUREGRAPHINSIGHT_API


class STextureGraphInsightRecordTrailView : public SCompoundWidget
{
	TSharedPtr< SBreadcrumbTrail<RecordID> > _trail;
public:
	SLATE_BEGIN_ARGS(STextureGraphInsightRecordTrailView) {}
		SLATE_ARGUMENT(RecordID, recordID)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args);

	UE_API void Refresh(RecordID rid);

	static UE_API RecordID FindRootRecord(RecordID rid);

	UE_API void OnCrumbClicked(const RecordID& blob);
};


#undef UE_API
