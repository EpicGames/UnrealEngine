// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//#include "SlateBasics.h"

#include "Model/TextureGraphInsightSession.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

#define UE_API TEXTUREGRAPHINSIGHT_API
class STextureGraphInsightResourceViewRow; /// Declare the concrete type of widget used for the raws of the view. Defined in the cpp file
class ITableRow;
class STableViewBase;

class STextureGraphInsightResourceView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextureGraphInsightResourceView) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args);

	// TreeView Item Types
	class FItemData;
	using FItem = TSharedPtr<FItemData>;
	using FItemArray = TArray< FItem >;
	class FItemData
	{
	public:
		FItemData(RecordID rid) : _recordID(rid) {}
		TSharedPtr < STextureGraphInsightResourceViewRow > _widget;
		RecordID _recordID;
		FItemArray	_children;
	};
	//using SItemTableView = SListView< FItem >;
	using SItemTableView = STreeView< FItem >;


	// Standard delegates for the view
	UE_API TSharedRef<ITableRow>	OnGenerateRowForView(FItem item, const TSharedRef<STableViewBase>& OwnerTable);
	FORCEINLINE void		OnGetChildrenForView(FItem item, FItemArray& children) { children = item->_children; };
	UE_API void					OnClickItemForView(FItem item);
	UE_API void					OnDoubleClickItemForView(FItem item);

	/// The list of root items
	FItemArray _rootItems;

	// The TreeView widget
	TSharedPtr<SItemTableView> _tableView;

	UE_API void RefreshRootItems();

	UE_API void OnBlobNew(const RecordIDArray& rids);
	UE_API void OnBlobMapped(const RecordIDArray& rids);
	UE_API void OnEngineReset(int id);
};

#undef UE_API
