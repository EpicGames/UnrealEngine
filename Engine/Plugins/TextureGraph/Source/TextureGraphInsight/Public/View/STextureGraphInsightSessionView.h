// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

//#include "SlateBasics.h"

#include "Model/TextureGraphInsightSession.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

#define UE_API TEXTUREGRAPHINSIGHT_API

class STextureGraphInsightBatchJobViewRow; /// Declare the concrete type of widget used for the raws of the tree view. Defined in the cpp file
class ITableRow;
class STableViewBase;

class STextureGraphInsightBatchJobView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextureGraphInsightBatchJobView)
		: _inspectOnSimpleClick(false) {}
	SLATE_ARGUMENT(RecordID, recordID)
	SLATE_ARGUMENT(bool, inspectOnSimpleClick)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args);

	// TreeView Item Types
	class FItemData;
	using FItem = TSharedPtr<FItemData>;
	using FItemArray = TArray< FItem >;
	class FItemData
	{
	public:
		FItemData(RecordID id, bool isMainPhase = true) : _recordID(id), _isMainPhase(isMainPhase) {}
		TSharedPtr < STextureGraphInsightBatchJobViewRow > _widget;
		RecordID _recordID;
		FItemArray _children;
		bool _isMainPhase = true;
	};
	using SItemTreeView = STreeView< FItem >;

	// Standard delegates for the tree view
	UE_API TSharedRef<ITableRow>	OnGenerateRowForTree(FItem item, const TSharedRef<STableViewBase>& OwnerTable);
	FORCEINLINE void		OnGetChildrenForView(FItem item, FItemArray& children) { children = item->_children; };
	UE_API void					OnClickItemForTree(FItem item);
	UE_API void					OnDoubleClickItemForTree(FItem item);
	UE_API TSharedPtr<SWidget>		OnContextMenuOpeningForTree();
	UE_API TSharedPtr<SWidget>		OnContextMenuBatch(FItem batchItem);
	UE_API TSharedPtr<SWidget>		OnContextMenuJob(FItem jobItem);

	// Inspect on double click always
	// eventually trigger inspect on simple click
	bool _inspectOnSimpleClick = false;

	/// The list of root items
	FItemArray _rootItems;

	// The TreeView widget
	TSharedPtr<SItemTreeView> _treeView;


	UE_API void OnReplayBatch(RecordID batchId, bool captureRenderDoc);
	UE_API void OnReplayJob(RecordID batchId, bool captureRenderDoc);

	UE_API void OnBatchNew(RecordID batch);
	UE_API void OnBatchUpdate(RecordID batch);
	UE_API void OnEngineReset(int32);
};


class STextureGraphInsightSessionView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STextureGraphInsightSessionView) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Args);

	TSharedPtr<STextureGraphInsightBatchJobView> _batchJobView;
};

#undef UE_API
