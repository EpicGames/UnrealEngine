// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

public class TextureGraphInsight : ModuleRules
{
	public TextureGraphInsight(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		bEnableExceptions = true;
		bDisableAutoRTFMInstrumentation = true;

		PublicDependencyModuleNames.AddRange(new string[] {
			"Core",
			"CoreUObject",
			"Engine",
			"InputCore",
			"RenderCore",
			"RHI"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Slate",
			"SlateCore",
			"UMG"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"TextureGraphEngine",
			"FreeImage",
			"Function2",
			"Continuable",
		});
	}
}
