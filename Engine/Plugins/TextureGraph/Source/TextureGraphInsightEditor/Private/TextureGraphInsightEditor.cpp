// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureGraphInsightEditor.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/WidgetPath.h"
#include "LevelEditor.h"
#include "TextureGraphEngine.h"
#include "TextureGraphInsight.h"
#include "TextureGraphInsightEditorCommands.h"
#include "TextureGraphInsightEditorStyle.h"
#include "ToolMenus.h"
#include "View/STextureGraphInsightActionView.h"
#include "View/STextureGraphInsightDeviceView.h"
#include "View/STextureGraphInsightInspectorView.h"
#include "View/STextureGraphInsightMixView.h"
#include "View/STextureGraphInsightResourceView.h"
#include "View/STextureGraphInsightSessionView.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"


static const FName TextureGraphInsightEditorTabName("TextureGraphInsightEditor");
static const FName TextureGraphInsightEditorTabName_Mixes("TextureGraphInsightEditor_Mixes");
static const FName TextureGraphInsightEditorTabName_Actions("TextureGraphInsightEditor_Actions");
static const FName TextureGraphInsightEditorTabName_JobBatches("TextureGraphInsightEditor_JobBatches");
static const FName TextureGraphInsightEditorTabName_Resources("TextureGraphInsightEditor_Resources");
static const FName TextureGraphInsightEditorTabName_Devices("TextureGraphInsightEditor_Devices");
static const FName TextureGraphInsightEditorTabName_Inspector("TextureGraphInsightEditor_Inspector");

#define LOCTEXT_NAMESPACE "FTextureGraphInsightEditorModule"


struct FInsightTabCommands : public TCommands<FInsightTabCommands>
{
	FInsightTabCommands()
		: TCommands<FInsightTabCommands>(
			TEXT("TextureGraphInsightTab"), // Context name for fast lookup
			LOCTEXT("TextureGraphInsightTab", "TextureGraphInsightTab Debugger"), // Localized context name for displaying
			NAME_None, // Parent
			FCoreStyle::Get().GetStyleSetName() // Icon Style Set
			)
	{
	}

	// TCommand<> interface
	virtual void RegisterCommands() override;
	// End of TCommand<> interface

	TSharedPtr<FUICommandInfo> ShowMixesTab;
	TSharedPtr<FUICommandInfo> ShowActionsTab;
	TSharedPtr<FUICommandInfo> ShowJobBatchesTab;
	TSharedPtr<FUICommandInfo> ShowResourcesTab;
	TSharedPtr<FUICommandInfo> ShowDevicesTab;
	TSharedPtr<FUICommandInfo> ShowInspectorTab;
};

void FInsightTabCommands::RegisterCommands()
{
	UI_COMMAND(ShowMixesTab, "Mixes", "Toggles visibility of the Mixes tab", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(ShowActionsTab, "Actions", "Toggles visibility of the Actions tab", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(ShowJobBatchesTab, "JobBatches", "Toggles visibility of the JobBatches tab", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(ShowResourcesTab, "Resources", "Toggles visibility of the Resources tab", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(ShowDevicesTab, "Devices", "Toggles visibility of the Devices tab", EUserInterfaceActionType::Check, FInputChord());
	UI_COMMAND(ShowInspectorTab, "Inspector", "Toggles visibility of the Inspector tab", EUserInterfaceActionType::Check, FInputChord());
}

void FTextureGraphInsightEditorModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	
	FTextureGraphInsightEditorStyle::Initialize();
	FTextureGraphInsightEditorStyle::ReloadTextures();

	FTextureGraphInsightEditorCommands::Register();
	
	Commands = MakeShareable(new FUICommandList);

	// Commands->MapAction(
	// 	FTextureGraphInsightEditorCommands::Get().OpenPluginWindow,
	// 	FExecuteAction::CreateRaw(this, &FTextureGraphInsightEditorModule::PluginButtonClicked),
	// 	FCanExecuteAction());

	// UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FTextureGraphInsightEditorModule::RegisterMenus));

	FInsightTabCommands::Register();

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TextureGraphInsightEditorTabName, FOnSpawnTab::CreateRaw(this, &FTextureGraphInsightEditorModule::OnSpawnPluginTab))
		.SetDisplayName(LOCTEXT("FTextureGraphInsightEditorTabTitle", "TextureGraph Insight"))
		.SetMenuType(ETabSpawnerMenuType::Hidden);


	// in StartupModule()
	_tickDelegate = FTickerDelegate::CreateRaw(this, &FTextureGraphInsightEditorModule::Tick);
	_tickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(_tickDelegate);
}

void FTextureGraphInsightEditorModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	FTSTicker::RemoveTicker(_tickDelegateHandle);

	UToolMenus::UnRegisterStartupCallback(this);

	UToolMenus::UnregisterOwner(this);

	TextureGraphInsight::Destroy();

	FTextureGraphInsightEditorStyle::Shutdown();

	FTextureGraphInsightEditorCommands::Unregister();

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(TextureGraphInsightEditorTabName);
}
bool FTextureGraphInsightEditorModule::Tick(float DeltaTime)
{
	return true;
}
TSharedRef<SDockTab> FTextureGraphInsightEditorModule::OnSpawnPluginTab(const FSpawnTabArgs& SpawnTabArgs)
{
	// (re)Instantiate the TextureGraph Insight singleton
	if (TextureGraphInsight::Instance())
	{
		TextureGraphInsight::Destroy();
	}
	TextureGraphInsight::Create();


	const TSharedRef<SDockTab> NomadTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label(NSLOCTEXT("TextureGraphInsight", "TextureGraph Insight", "TextureGraph Insight"));

	if (!TabManager.IsValid())
	{
		TabManager = FGlobalTabmanager::Get()->NewTabManager(NomadTab);
		// on persist layout will handle saving layout if the editor is shut down:
		TabManager->SetOnPersistLayout(
			FTabManager::FOnPersistLayout::CreateStatic(
				[](const TSharedRef<FTabManager::FLayout>& InLayout)
				{
					if (InLayout->GetPrimaryArea().Pin().IsValid())
					{
						FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, InLayout);
					}
				}
			)
		);
	}
	else
	{
		ensure(Layout.IsValid());
	}

	TWeakPtr<FTabManager> tabManagerWeak = TabManager;
	// On tab close will save the layout if the debugging window itself is closed,
	// this handler also cleans up any floating debugging controls. If we don't close
	// all areas we need to add some logic to the tab manager to reuse existing tabs:
	NomadTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateStatic(
		[](TSharedRef<SDockTab> Self, TWeakPtr<FTabManager> InTabManager)
		{
			TSharedPtr<FTabManager> OwningTabManager = InTabManager.Pin();
			if (OwningTabManager.IsValid())
			{
				FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, OwningTabManager->PersistLayout());
				OwningTabManager->CloseAllAreas();
			}
		}
		, tabManagerWeak
	));

	if (!Layout.IsValid())
	{
		TabManager->RegisterTabSpawner(TextureGraphInsightEditorTabName_Mixes, FOnSpawnTab::CreateStatic(
			[](const FSpawnTabArgs&)->TSharedRef<SDockTab>
			{
				return SNew(SDockTab)
					.TabRole(ETabRole::PanelTab)
					.Label(LOCTEXT("FTextureGraphInsightEditorTabTitleMixes", "TextureGraph Assets"))
					[
						SNew(STextureGraphInsightMixListView)
					];
			}));
		TabManager->RegisterTabSpawner(TextureGraphInsightEditorTabName_Actions, FOnSpawnTab::CreateStatic(
			[](const FSpawnTabArgs&)->TSharedRef<SDockTab>
			{
				return SNew(SDockTab)
					.TabRole(ETabRole::PanelTab)
					.Label(LOCTEXT("FTextureGraphInsightEditorTabTitleAction", "TextureGraph Actions"))
					[
						SNew(STextureGraphInsightActionView)
					];
			}));
		TabManager->RegisterTabSpawner(TextureGraphInsightEditorTabName_JobBatches, FOnSpawnTab::CreateStatic(
			[](const FSpawnTabArgs&)->TSharedRef<SDockTab>
			{
				return SNew(SDockTab)
					.TabRole(ETabRole::PanelTab)
					.Label(LOCTEXT("FTextureGraphInsightEditorTabTitleJobBatches", "TextureGraph Jobs & Batches"))
					[
						SNew(STextureGraphInsightSessionView)
					];
			}));
		TabManager->RegisterTabSpawner(TextureGraphInsightEditorTabName_Resources, FOnSpawnTab::CreateStatic(
			[](const FSpawnTabArgs&)->TSharedRef<SDockTab>
			{
				return SNew(SDockTab)
					.TabRole(ETabRole::PanelTab)
					.Label(LOCTEXT("FTextureGraphInsightEditorTabTitleResources", "TextureGraph Resources"))
					[
						SNew(STextureGraphInsightResourceView)
					]; 
			}));
		TabManager->RegisterTabSpawner(TextureGraphInsightEditorTabName_Devices, FOnSpawnTab::CreateStatic(
			[](const FSpawnTabArgs&)->TSharedRef<SDockTab>
			{
				return SNew(SDockTab)
					.TabRole(ETabRole::PanelTab)
					.Label(LOCTEXT("FTextureGraphInsightEditorTabTitleDevices", "TextureGraph Devices"))
					[
						SNew(STextureGraphInsightDeviceListView)
					];
			}));
		TabManager->RegisterTabSpawner(TextureGraphInsightEditorTabName_Inspector, FOnSpawnTab::CreateStatic(
			[](const FSpawnTabArgs&)->TSharedRef<SDockTab>
			{
				return SNew(SDockTab)
					.TabRole(ETabRole::PanelTab)
					.Label(LOCTEXT("FTextureGraphInsightEditorTabTitleInspector", "TextureGraph Inspector"))
					[
						SNew(STextureGraphInsightInspectorView)
					];
			}));

		Layout = FTabManager::NewLayout("Standalone_TextureGraphInsight_Layout_v2")
			->AddArea
			(
				FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(.4f)
					->SetHideTabWell(true)
					->AddTab(TextureGraphInsightEditorTabName_Mixes, ETabState::OpenedTab)
					->AddTab(TextureGraphInsightEditorTabName_Actions, ETabState::OpenedTab)
					->AddTab(TextureGraphInsightEditorTabName_JobBatches, ETabState::OpenedTab)
					->AddTab(TextureGraphInsightEditorTabName_Resources, ETabState::OpenedTab)
					->AddTab(TextureGraphInsightEditorTabName_Devices, ETabState::OpenedTab)
					->SetForegroundTab(TextureGraphInsightEditorTabName_JobBatches)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(.4f)
					->SetHideTabWell(true)
					->AddTab(TextureGraphInsightEditorTabName_Inspector, ETabState::OpenedTab)
					->SetForegroundTab(TextureGraphInsightEditorTabName_Inspector)
				)
			);
	}

	Layout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, Layout.ToSharedRef());

	TSharedRef<SWidget> TabContents = TabManager->RestoreFrom(Layout.ToSharedRef(), TSharedPtr<SWindow>()).ToSharedRef();


	// build command list for tab restoration menu:
	TSharedPtr<FUICommandList> CommandList = MakeShareable(new FUICommandList());

	TWeakPtr<FTabManager> DebuggingToolsManagerWeak = TabManager;

	const auto ToggleTabVisibility = [](TWeakPtr<FTabManager> InTabManagerWeak, FName InTabName)
	{
		TSharedPtr<FTabManager> InDebuggingToolsManager = InTabManagerWeak.Pin();
		if (InDebuggingToolsManager.IsValid())
		{
			TSharedPtr<SDockTab> ExistingTab = InDebuggingToolsManager->FindExistingLiveTab(InTabName);
			if (ExistingTab.IsValid())
			{
				ExistingTab->RequestCloseTab();
			}
			else
			{
				InDebuggingToolsManager->TryInvokeTab(InTabName);
			}
		}
	};

	const auto IsTabVisible = [](TWeakPtr<FTabManager> InTabManagerWeak, FName InTabName)
	{
		TSharedPtr<FTabManager> InDebuggingToolsManager = InTabManagerWeak.Pin();
		if (InDebuggingToolsManager.IsValid())
		{
			return InDebuggingToolsManager->FindExistingLiveTab(InTabName).IsValid();
		}
		return false;
	};

	const auto ActionMapperToCommandList = [&](TSharedPtr<FUICommandInfo> command, const FName name)
	{
		CommandList->MapAction(
			command,
			FExecuteAction::CreateStatic(
				ToggleTabVisibility,
				DebuggingToolsManagerWeak,
				name
			),
			FCanExecuteAction::CreateStatic(
				[]() { return true; }
			),
			FIsActionChecked::CreateStatic(
				IsTabVisible,
				DebuggingToolsManagerWeak,
				name
			)
		);
	};

	ActionMapperToCommandList(FInsightTabCommands::Get().ShowMixesTab, TextureGraphInsightEditorTabName_Mixes);
	ActionMapperToCommandList(FInsightTabCommands::Get().ShowActionsTab, TextureGraphInsightEditorTabName_Actions);
	ActionMapperToCommandList(FInsightTabCommands::Get().ShowJobBatchesTab, TextureGraphInsightEditorTabName_JobBatches);
	ActionMapperToCommandList(FInsightTabCommands::Get().ShowResourcesTab, TextureGraphInsightEditorTabName_Resources);
	ActionMapperToCommandList(FInsightTabCommands::Get().ShowDevicesTab, TextureGraphInsightEditorTabName_Devices);
	ActionMapperToCommandList(FInsightTabCommands::Get().ShowInspectorTab, TextureGraphInsightEditorTabName_Inspector);


	TWeakPtr<SWidget> OwningWidgetWeak = NomadTab;
	TabContents->SetOnMouseButtonUp(
		FPointerEventHandler::CreateStatic(
			[]( /** The geometry of the widget*/
				const FGeometry&,
				/** The Mouse Event that we are processing */
				const FPointerEvent& PointerEvent,
				TWeakPtr<SWidget> InOwnerWeak,
				TSharedPtr<FUICommandList> InCommandList) -> FReply
			{
				if (PointerEvent.GetEffectingButton() == EKeys::RightMouseButton)
				{
					// if the tab manager is still available then make a context window that allows users to
					// show and hide tabs:
					TSharedPtr<SWidget> InOwner = InOwnerWeak.Pin();
					if (InOwner.IsValid())
					{
						FMenuBuilder MenuBuilder(true, InCommandList);

						MenuBuilder.PushCommandList(InCommandList.ToSharedRef());
						{
							MenuBuilder.AddMenuEntry(FInsightTabCommands::Get().ShowMixesTab);
							MenuBuilder.AddMenuEntry(FInsightTabCommands::Get().ShowActionsTab);
							MenuBuilder.AddMenuEntry(FInsightTabCommands::Get().ShowJobBatchesTab);
							MenuBuilder.AddMenuEntry(FInsightTabCommands::Get().ShowResourcesTab);
							MenuBuilder.AddMenuEntry(FInsightTabCommands::Get().ShowDevicesTab);
							MenuBuilder.AddMenuEntry(FInsightTabCommands::Get().ShowInspectorTab);
						}
						MenuBuilder.PopCommandList();

						FWidgetPath WidgetPath = PointerEvent.GetEventPath() != nullptr ? *PointerEvent.GetEventPath() : FWidgetPath();
						FSlateApplication::Get().PushMenu(InOwner.ToSharedRef(), WidgetPath, MenuBuilder.MakeWidget(), PointerEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

						return FReply::Handled();
					}
				}

				return FReply::Unhandled();
			}
			, OwningWidgetWeak
			, CommandList
			)
	);

	NomadTab->SetContent(
		SNew(SBorder)
		[
			TabContents
		]
	);

	return NomadTab;
}

void FTextureGraphInsightEditorModule::PluginButtonClicked()
{
	FGlobalTabmanager::Get()->TryInvokeTab(TextureGraphInsightEditorTabName);
}

void FTextureGraphInsightEditorModule::RegisterMenus()
{
	// // Owner will be used for cleanup in call to UToolMenus::UnregisterOwner
	// FToolMenuOwnerScoped OwnerScoped(this);
	//
	// {
	// 	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Window");
	// 	{
	// 		FToolMenuSection& Section = Menu->FindOrAddSection("WindowLayout");
	// 		Section.AddMenuEntryWithCommandList(FTextureGraphInsightEditorCommands::Get().OpenPluginWindow, Commands);
	// 	}
	// }
	//
	// {
	// 	UToolMenu* ToolbarMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar");
	// 	{
	// 		FToolMenuSection& Section = ToolbarMenu->FindOrAddSection("Settings");
	// 		{
	// 			FToolMenuEntry& Entry = Section.AddEntry(FToolMenuEntry::InitToolBarButton(FTextureGraphInsightEditorCommands::Get().OpenPluginWindow));
	// 			Entry.SetCommandList(Commands);
	// 		}
	// 	}
	// }
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FTextureGraphInsightEditorModule, TextureGraphInsightEditor)
