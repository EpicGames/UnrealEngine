// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureGraphInsightEditorCommands.h"

#define LOCTEXT_NAMESPACE "FTextureGraphInsightEditorModule"

void FTextureGraphInsightEditorCommands::RegisterCommands()
{
	// UI_COMMAND(OpenPluginWindow, "TextureGraph Insight", "Bring up TextureGraph Insight window", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
