// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureGraphInsightEditorStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/SlateGameResources.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleMacros.h"

#define RootToContentDir Style->RootToContentDir

TSharedPtr< FSlateStyleSet > FTextureGraphInsightEditorStyle::StyleInstance = NULL;

void FTextureGraphInsightEditorStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FTextureGraphInsightEditorStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FTextureGraphInsightEditorStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("TextureGraphInsightEditorStyle"));
	return StyleSetName;
}

const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon40x40(40.0f, 40.0f);

TSharedRef< FSlateStyleSet > FTextureGraphInsightEditorStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("TextureGraphInsightEditorStyle"));
	Style->SetContentRoot(IPluginManager::Get().FindPlugin("TextureGraph")->GetBaseDir() / TEXT("Resources"));

	Style->Set("TextureGraphInsightEditor.OpenPluginWindow", new IMAGE_BRUSH(TEXT("texturegraph_insight_icon_40x"), Icon40x40));

	return Style;
}

void FTextureGraphInsightEditorStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	}
}

const ISlateStyle& FTextureGraphInsightEditorStyle::Get()
{
	return *StyleInstance;
}

#undef RootToContentDir
