// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Containers/Ticker.h"
#include "Framework/Docking/TabManager.h"

class FTextureGraphInsightEditorModule : public IModuleInterface
{
	FTickerDelegate _tickDelegate;
	FTSTicker::FDelegateHandle _tickDelegateHandle;
public:

	/** IModuleInterface implementation */
	virtual void	StartupModule() override;
	virtual void	ShutdownModule() override;

	bool Tick(float DeltaTime);
	/** This function will be bound to Command (by default it will bring up plugin window) */
	void PluginButtonClicked();
	
private:
	void RegisterMenus();
	TSharedRef<class SDockTab> OnSpawnPluginTab(const class FSpawnTabArgs& SpawnTabArgs);

private:
	TSharedPtr<class FUICommandList>	Commands;
	TSharedPtr<FTabManager > TabManager;
	TSharedPtr<FTabManager::FLayout > Layout;

};
