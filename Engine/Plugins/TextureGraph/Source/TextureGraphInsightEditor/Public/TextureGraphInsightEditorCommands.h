// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "TextureGraphInsightEditorStyle.h"

class FTextureGraphInsightEditorCommands : public TCommands<FTextureGraphInsightEditorCommands>
{
public:

	FTextureGraphInsightEditorCommands()
		: TCommands<FTextureGraphInsightEditorCommands>(TEXT("TextureGraphInsightEditor"), NSLOCTEXT("Contexts", "TextureGraphInsightEditor", "TextureGraphInsightEditor Plugin"), NAME_None, FTextureGraphInsightEditorStyle::GetStyleSetName())
	{
	}

	// TCommands<> interface
	virtual void RegisterCommands() override;

// public:
	// TSharedPtr< FUICommandInfo > OpenPluginWindow;
};
