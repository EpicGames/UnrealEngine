{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0 Beta",
	"Description": "Texture creation tool using graphs.",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "TextureGraphEngine",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "TextureGraphInsight",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "TextureGraphInsightEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "TextureGraph",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "TextureGraphEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins": [
		{
			"Name": "ProceduralMeshComponent",
			"Enabled": true
		},
		{
			"Name": "Synthesis",
			"Enabled": true
		},
		{
			"Name": "Mutable",
			"Enabled": true
		},
		{
			"Name": "RenderDocPlugin",
			"Enabled": true
		},
		{
			"Name": "ImageWidgets",
			"Enabled": true
		}
	]
}